// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#define INSIGHTS_DECLARE_TOGGLE_COMMAND(CmdName) \
	public: \
		void Map_##CmdName(); /**< Maps UI command info CmdName with the specified UI command list. */ \
		const FUIAction GetAction_##CmdName(); /**< UI action for CmdName command. */ \
	protected: \
		void CmdName##_Execute(); /**< Handles FExecuteAction for CmdName. */ \
		bool CmdName##_CanExecute() const; /**< Handles FCanExecuteAction for CmdName. */ \
		ECheckBoxState CmdName##_GetCheckState() const; /**< Handles FGetActionCheckState for CmdName. */

#define INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(Class, This, CmdName, IsEnabled, SetIsEnabled) \
	void Class::Map_##CmdName() \
	{ \
		This->GetCommandList()->MapAction(This->GetCommands().CmdName, GetAction_##CmdName()); \
	} \
	const FUIAction Class::GetAction_##CmdName() \
	{ \
		FUIAction UIAction; \
		UIAction.ExecuteAction = FExecuteAction::CreateRaw(this, &Class::CmdName##_Execute); \
		UIAction.CanExecuteAction = FCanExecuteAction::CreateRaw(this, &Class::CmdName##_CanExecute); \
		UIAction.GetActionCheckState = FGetActionCheckState::CreateRaw(this, &Class::CmdName##_GetCheckState); \
		return UIAction; \
	} \
	void Class::CmdName##_Execute() \
	{ \
		This->SetIsEnabled(!This->IsEnabled()); \
	} \
	bool Class::CmdName##_CanExecute() const \
	{ \
		return FInsightsManager::Get()->GetSession().IsValid(); \
	} \
	ECheckBoxState Class::CmdName##_GetCheckState() const \
	{ \
		return This->IsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; \
	}
