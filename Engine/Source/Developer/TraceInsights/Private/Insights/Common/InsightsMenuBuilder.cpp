// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsMenuBuilder.h"

#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Testing/SStarshipSuite.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

// TraceInsightsCore
#include "InsightsCore/Common/InsightsCoreStyle.h"
#include "InsightsCore/Common/MiscUtils.h"

// TraceInsights
#include "Insights/ImportTool/TableImportTool.h"
#include "Insights/InsightsStyle.h"
#include "Insights/InsightsManager.h"

#if WITH_LIVE_CODING
#include "ILiveCodingModule.h"
#endif

#define LOCTEXT_NAMESPACE "UE::Insights::MenuBuilder"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FInsightsMenuBuilder
////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsMenuBuilder::FInsightsMenuBuilder()
#if !WITH_EDITOR
	: InsightsToolsGroup(FGlobalTabmanager::Get()->AddLocalWorkspaceMenuCategory(NSLOCTEXT("InsightsMenuTools", "InsightTools", "Insights Tools")))
	, WindowsGroup(FGlobalTabmanager::Get()->AddLocalWorkspaceMenuCategory(NSLOCTEXT("InsightsMenuTools", "InsightWindows", "Windows")))
#endif
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FWorkspaceItem> FInsightsMenuBuilder::GetInsightsToolsGroup()
{
#if !WITH_EDITOR
	return InsightsToolsGroup;
#else
	return WorkspaceMenu::GetMenuStructure().GetDeveloperToolsProfilingCategory();
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FWorkspaceItem> FInsightsMenuBuilder::GetWindowsGroup()
{
#if !WITH_EDITOR
	return WindowsGroup;
#else
	return WorkspaceMenu::GetMenuStructure().GetDeveloperToolsProfilingCategory();
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsMenuBuilder::PopulateMenu(FMenuBuilder& MenuBuilder)
{
#if !WITH_EDITOR
	MenuBuilder.BeginSection("Insights");

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ImportTable", "Import Table..."),
		LOCTEXT("ImportTable_ToolTip", "Import CSV or TSV data from a file to an Insights Table."),
		FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.ImportTable"),
		FUIAction(FExecuteAction::CreateLambda([] { Insights::FTableImportTool::Get()->StartImportProcess(); })));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("DiffTables", "Diff Tables..."),
		LOCTEXT("DiffTables_Tooltip", "Opens two table files in diff mode."),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FolderOpen"),
		FUIAction(FExecuteAction::CreateLambda([] { Insights::FTableImportTool::Get()->StartDiffProcess(); })));

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("OpenUnrealInsightsFrontend", "Unreal Insights Frontend"),
		LOCTEXT("OpenUnrealInsightsFrontend_ToolTip", "Opens the Unreal Insights Frontend window."),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "AppIcon.Small"),
		FUIAction(FExecuteAction::CreateLambda([] { FMiscUtils::OpenUnrealInsights(); })));

	MenuBuilder.AddSubMenu(
		LOCTEXT("OpenTraceFile_SubMenu", "Open Trace File"),
		LOCTEXT("OpenTraceFile_SubMenu_Desc", "Starts analysis for a specified trace file."),
		FNewMenuDelegate::CreateSP(this, &FInsightsMenuBuilder::BuildOpenTraceFileSubMenu),
		false,
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FolderOpen")
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("OpenTraceControl", "Live Trace Control"),
		LOCTEXT("OpenTraceControl_ToolTip", "Opens the Trace Control window."),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TraceControl"),
		FUIAction(FExecuteAction::CreateLambda([] { FInsightsManager::Get()->OpenTraceControlWindow(); })));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AutoOpenLiveTrace", "Auto Open Live Trace"),
		LOCTEXT("AutoOpenLiveTrace_ToolTip", "If enabled, the analysis starts automatically for each new live trace session, replacing the current analysis session."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([]() { FInsightsManager::Get()->ToggleAutoLoadLiveSession(); }),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([]() -> bool { return FInsightsManager::Get()->IsAutoLoadLiveSessionEnabled(); })
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);

	MenuBuilder.EndSection();

	FGlobalTabmanager::Get()->PopulateLocalTabSpawnerMenu(MenuBuilder);

	static FName WidgetReflectorTabId("WidgetReflector");
	bool bAllowDebugTools = FGlobalTabmanager::Get()->HasTabSpawner(WidgetReflectorTabId);
	if (bAllowDebugTools)
	{
		MenuBuilder.BeginSection("WidgetTools");
		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenWidgetReflector", "Widget Reflector"),
			LOCTEXT("OpenWidgetReflector_ToolTip", "Opens the Widget Reflector, a handy tool for diagnosing problems with live widgets."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "WidgetReflector.Icon"),
			FUIAction(FExecuteAction::CreateLambda([] { FGlobalTabmanager::Get()->TryInvokeTab(WidgetReflectorTabId); })));
		MenuBuilder.EndSection();
	}

#if WITH_LIVE_CODING
	if (ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME))
	{
		MenuBuilder.BeginSection("Developer");
		MenuBuilder.AddMenuEntry(
			LOCTEXT("LiveCodingCompile", "Live Coding Compile"),
			LOCTEXT("LiveCodingCompile_Tooltip", "Compiles the Unreal Insights code and patches this executable"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "WidgetReflector.Icon"),
			FUIAction(
				FExecuteAction::CreateLambda([]()
				{
					ILiveCodingModule& LiveCoding = FModuleManager::GetModuleChecked<ILiveCodingModule>(LIVE_CODING_MODULE_NAME);
					LiveCoding.Compile();
				}),
				FCanExecuteAction::CreateLambda([]() -> bool
				{
					ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME);
					return LiveCoding != nullptr && LiveCoding->IsEnabledForSession();
				}))
		);
			
		MenuBuilder.EndSection();
	}
#endif

#if !UE_BUILD_SHIPPING
	// Open Starship Test Suite
	{
		FUIAction OpenStarshipSuiteAction;
		OpenStarshipSuiteAction.ExecuteAction = FExecuteAction::CreateLambda([this]()
			{
				RestoreStarshipSuite();
			});

		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenStarshipSuite", "Starship Test Suite"),
			LOCTEXT("OpenStarshipSuite_ToolTip", "Opens the Starship UX test suite."),
			FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.Test"),
			OpenStarshipSuiteAction,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
#endif // !UE_BUILD_SHIPPING
#endif // !WITH_EDITOR
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsMenuBuilder::BuildOpenTraceFileSubMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("OpenTraceFile1", "Open in New Instance..."),
		LOCTEXT("OpenTraceFile1_ToolTip", "Starts analysis for a specified trace file, in a separate Unreal Insights instance."),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FolderOpen"),
		FUIAction(FExecuteAction::CreateLambda([] { FInsightsManager::Get()->OpenTraceFile(); })));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("OpenTraceFile2", "Open in Same Instance..."),
		LOCTEXT("OpenTraceFile2_ToolTip", "Starts analysis for a specified trace file, replacing the current analysis session."),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FolderOpen"),
		FUIAction(FExecuteAction::CreateLambda([] { FInsightsManager::Get()->LoadTraceFile(); })));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsMenuBuilder::AddMenuEntry(
	FMenuBuilder& InOutMenuBuilder,
	const FUIAction& InAction,
	const TAttribute<FText>& InLabel,
	const TAttribute<FText>& InToolTipText,
	const TAttribute<FText>& InKeybinding,
	const EUserInterfaceActionType InUserInterfaceActionType)
{
	InOutMenuBuilder.AddMenuEntry(
		InAction,
		SNew(SBox)
		.Padding(FMargin(0.0f))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(2.0f, 0.0f))
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "Menu.Label")
				.Text(InLabel)
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(4.0f, 0.0f))
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "Menu.Keybinding")
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text(InKeybinding)
			]
		],
		NAME_None,
		InToolTipText,
		InUserInterfaceActionType
	);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
