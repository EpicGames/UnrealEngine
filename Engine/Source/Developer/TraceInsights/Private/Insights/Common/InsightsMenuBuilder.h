// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Framework/Commands/UICommandInfo.h"

class FWorkspaceItem;
class FMenuBuilder;

struct FUIAction;

namespace UE::Insights
{

class FInsightsMenuBuilder : public TSharedFromThis<FInsightsMenuBuilder>
{
public:
	FInsightsMenuBuilder();

	TSharedRef<FWorkspaceItem> GetInsightsToolsGroup();
	TSharedRef<FWorkspaceItem> GetWindowsGroup();

	void PopulateMenu(FMenuBuilder& InOutMenuBuilder);
	void BuildOpenTraceFileSubMenu(FMenuBuilder& InOutMenuBuilder);

	// Adds a menu entry with a custom key binding text.
	static void AddMenuEntry(
		FMenuBuilder& InOutMenuBuilder,
		const FUIAction& InAction,
		const TAttribute<FText>& InLabel,
		const TAttribute<FText>& InToolTipText,
		const TAttribute<FText>& InKeybinding,
		const EUserInterfaceActionType InUserInterfaceActionType);

private:
#if !WITH_EDITOR
	TSharedRef<FWorkspaceItem> InsightsToolsGroup;
	TSharedRef<FWorkspaceItem> WindowsGroup;
#endif
};

} // namespace UE::Insights
