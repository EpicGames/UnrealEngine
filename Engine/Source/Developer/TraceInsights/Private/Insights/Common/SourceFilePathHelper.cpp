// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceFilePathHelper.h"

#include "Insights/InsightsManager.h"
#include "InsightsCore/Common/Log.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FSourceFilePathHelper"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FSourceFilePathHelper
////////////////////////////////////////////////////////////////////////////////////////////////////

FSourceFilePathHelper::FSourceFilePathHelper()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSourceFilePathHelper::~FSourceFilePathHelper()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSourceFilePathHelper::InitVFSMapping(const FString& VFSPaths)
{
	TArray<FString> Entries;
	VFSPaths.ParseIntoArray(Entries, TEXT(";"));

	for (int32 Index = 0; Index < Entries.Num(); Index = Index + 2)
	{
		if (Index + 1 < Entries.Num())
		{
			VFSMappings.Add(Entries[Index], Entries[Index + 1]);
		}
		else
		{
			UE_LOGF(LogInsights, Warning, "Detected unmatched VFS paths in the session:%ls", *VFSPaths);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FSourceFilePathHelper::GetUsableFilePath(FString InPath, FString& OutPath)
{
	if (InPath.IsEmpty())
	{
		return false;
	}

	FPaths::NormalizeFilename(InPath);
	// The goal of the LastAccessType checks is to minimise the number of calls to FPaths::FileExists.
	if (LastAccessType == ELastAccessType::Direct && FPaths::FileExists(InPath))
	{
		OutPath = InPath;
		LastAccessType = ELastAccessType::Direct;
		return true;
	}

	const FString UserDefinedPath = GetLocalRootDirectoryPath();
	if (!UserDefinedPath.IsEmpty() && LastAccessType == ELastAccessType::UserDefined)
	{
		for (auto& Pair : VFSMappings)
		{
			OutPath = InPath.Replace(*Pair.Value, *UserDefinedPath);
			if (FPaths::FileExists(OutPath))
			{
				LastAccessType = ELastAccessType::UserDefined;
				return true;
			}

			const TCHAR* Keywords[] = { TEXT("/Source/"), TEXT("/Plugins/"), TEXT("/Platforms/"),  TEXT("/Restricted/") };
			for (const TCHAR* Keyword : Keywords)
			{
				// Try to reach the root path
				int32 Index = InPath.Find(Keyword);
				while (Index > 0 && (InPath[Index] == TEXT('/'))) --Index;
				while (Index > 0 && (InPath[Index] != TEXT('/'))) --Index;
				while (Index > 0 && (InPath[Index] == TEXT('/'))) --Index;
				++Index;

				if (Index <= InPath.Len())
				{
					FString RootPath = InPath.Left(Index);
					OutPath = InPath.Replace(*RootPath, *UserDefinedPath);
					if (FPaths::FileExists(OutPath))
					{
						LastAccessType = ELastAccessType::UserDefined;
						return true;
					}
				}
			}
		}
	}

	for (auto& Pair : VFSMappings)
	{
		if (InPath.StartsWith(Pair.Key))
		{
			OutPath = InPath.Replace(*Pair.Key, *Pair.Value);
			if (FPaths::FileExists(OutPath))
			{
				LastAccessType = ELastAccessType::VFS;
				return true;
			}
			OutPath.Empty();
		}
	}

	if (FPaths::FileExists(InPath))
	{
		OutPath = InPath;
		LastAccessType = ELastAccessType::Direct;
		return true;
	}

	if (!UserDefinedPath.IsEmpty())
	{
		for (auto& Pair : VFSMappings)
		{
			OutPath = InPath.Replace(*Pair.Value, *UserDefinedPath);
			if (FPaths::FileExists(OutPath))
			{
				LastAccessType = ELastAccessType::UserDefined;
				return true;
			}
		}

		const TCHAR* Keywords[] = { TEXT("\\Source\\"), TEXT("\\Plugins\\"), TEXT("/Source/"), TEXT("/Plugins/") };
		for (const TCHAR* Keyword : Keywords)
		{
			// Try to reach the root path
			int32 Index = InPath.Find(Keyword);
			while (Index > 0 && (InPath[Index] == TEXT('/') || InPath[Index] == TEXT('\\'))) --Index;
			while (Index > 0 && (InPath[Index] != TEXT('/') && InPath[Index] != TEXT('\\'))) --Index;
			while (Index > 0 && (InPath[Index] == TEXT('/') || InPath[Index] == TEXT('\\'))) --Index;
			++Index;

			if (Index <= InPath.Len())
			{
				FString RootPath = InPath.Left(Index);
				OutPath = InPath.Replace(*RootPath, *UserDefinedPath);
				if (FPaths::FileExists(OutPath))
				{
					LastAccessType = ELastAccessType::UserDefined;
					return true;
				}
			}
		}
	}

	OutPath = InPath;
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSourceFilePathHelper::SetUserDefinedPath(FString InPath)
{
	FPaths::NormalizeFilename(InPath);
	FInsightsManager::Get()->GetSettings().SetAndSaveSourceFilesSearchPath(InPath);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

const FString& FSourceFilePathHelper::GetUserDefinedPath() const
{
	return FInsightsManager::Get()->GetSettings().GetSourceFilesSearchPath();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FString FSourceFilePathHelper::GetLocalRootDirectoryPath() const
{
	FString Path = GetUserDefinedPath();
	if (Path.IsEmpty())
	{
		return FPaths::RootDir();
	}

	return Path;
}

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
