// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"

namespace UE::Insights
{

enum class ELastAccessType
{
	Unknown,
	Direct,
	VFS,
	UserDefined,
};

class FSourceFilePathHelper
{
public:
	FSourceFilePathHelper();
	~FSourceFilePathHelper();

	void InitVFSMapping(const FString& VFSPaths);
	bool GetUsableFilePath(FString InPath, FString& OutPath);
	const FString& GetUserDefinedPath() const;
	void SetUserDefinedPath(FString InPath);
	FString GetLocalRootDirectoryPath() const;

private:
	TMap<FString, FString> VFSMappings;
	ELastAccessType LastAccessType = ELastAccessType::Unknown;
};

} // namespace UE::Insights
