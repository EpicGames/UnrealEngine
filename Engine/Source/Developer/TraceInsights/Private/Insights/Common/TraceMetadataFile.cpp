// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/Common/TraceMetadataFile.h"

#include "HAL/FileManager.h"
#include "Misc/DefaultValueHelper.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceMetadataFile::SetFilePath(const FString& InFilePath)
{
	FilePath = InFilePath;
	bIsLoaded = false;
	Sections.Empty();
	SectionOrder.Empty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FTraceMetadataFile::GetSidecarPath(const FString& TraceFilePath)
{
	// MyTrace.utrace -> MyTrace.utrace.ini
	return TraceFilePath + TEXT(".ini");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTraceMetadataFile::Load()
{
	Sections.Empty();
	SectionOrder.Empty();
	bIsLoaded = false;

	if (FilePath.IsEmpty())
	{
		bIsLoaded = true;
		return true;
	}

	if (!IFileManager::Get().FileExists(*FilePath))
	{
		// No sidecar file yet — valid state, just no data.
		bIsLoaded = true;
		return true;
	}

	TArray<FString> Lines;
	if (!FFileHelper::LoadFileToStringArray(Lines, *FilePath))
	{
		return false;
	}

	FString CurrentSection;

	for (const FString& Line : Lines)
	{
		FString TrimmedLine = Line.TrimStartAndEnd();

		// Skip empty lines and comments
		if (TrimmedLine.IsEmpty() || TrimmedLine.StartsWith(TEXT(";")))
		{
			continue;
		}

		// Section header: [SectionName]
		if (TrimmedLine.StartsWith(TEXT("[")) && TrimmedLine.EndsWith(TEXT("]")))
		{
			CurrentSection = TrimmedLine.Mid(1, TrimmedLine.Len() - 2).TrimStartAndEnd();
			if (!Sections.Contains(CurrentSection))
			{
				Sections.Add(CurrentSection);
				SectionOrder.Add(CurrentSection);
			}
			continue;
		}

		// Key=Value (ignore lines before any section header)
		int32 EqualsIdx;
		if (!CurrentSection.IsEmpty() && TrimmedLine.FindChar(TEXT('='), EqualsIdx))
		{
			FString Key = TrimmedLine.Left(EqualsIdx).TrimEnd();
			FString Value = TrimmedLine.Mid(EqualsIdx + 1).TrimStart();
			Sections.FindOrAdd(CurrentSection).FindOrAdd(Key) = Value;
		}
	}

	bIsLoaded = true;
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTraceMetadataFile::Save()
{
	if (FilePath.IsEmpty())
	{
		return false;
	}

	// Ensure [TraceMetadata] section with version exists
	SetInt(TEXT("TraceMetadata"), TEXT("Version"), CurrentVersion);

	FString Output;

	// Write [TraceMetadata] section first
	if (const TMap<FString, FString>* MetadataSection = Sections.Find(TEXT("TraceMetadata")))
	{
		Output += TEXT("[TraceMetadata]\r\n");
		for (const TPair<FString, FString>& Pair : *MetadataSection)
		{
			Output += FString::Printf(TEXT("%s=%s\r\n"), *Pair.Key, *Pair.Value);
		}
		Output += TEXT("\r\n");
	}

	// Write remaining sections in first-seen order (read order on Load, set order on SetString).
	// Preserved across Load/Save so future order-sensitive consumers get a stable layout.
	for (const FString& SectionName : SectionOrder)
	{
		if (SectionName == TEXT("TraceMetadata"))
		{
			continue; // Already written
		}

		const TMap<FString, FString>* SectionData = Sections.Find(SectionName);
		if (!SectionData)
		{
			continue; // Defensive: stale name in SectionOrder
		}
		Output += FString::Printf(TEXT("[%s]\r\n"), *SectionName);
		for (const TPair<FString, FString>& Pair : *SectionData)
		{
			Output += FString::Printf(TEXT("%s=%s\r\n"), *Pair.Key, *Pair.Value);
		}
		Output += TEXT("\r\n");
	}

	// Ensure the destination directory exists
	IFileManager::Get().MakeDirectory(*FPaths::GetPath(FilePath), true);

	return FFileHelper::SaveStringToFile(Output, *FilePath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTraceMetadataFile::GetString(const TCHAR* Section, const TCHAR* Key, FString& OutValue) const
{
	if (const TMap<FString, FString>* SectionData = Sections.Find(Section))
	{
		if (const FString* Value = SectionData->Find(Key))
		{
			OutValue = *Value;
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceMetadataFile::SetString(const TCHAR* Section, const TCHAR* Key, const FString& Value)
{
	if (!Sections.Contains(Section))
	{
		SectionOrder.Add(Section);
	}
	Sections.FindOrAdd(Section).FindOrAdd(Key) = Value;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTraceMetadataFile::GetInt(const TCHAR* Section, const TCHAR* Key, int32& OutValue) const
{
	FString StringValue;
	if (GetString(Section, Key, StringValue))
	{
		const TCHAR* Start = *StringValue;
		TCHAR* End = nullptr;
		const int32 Parsed = FCString::Strtoi(Start, &End, 10);
		// Reject if nothing was consumed or if trailing non-whitespace remains
		if (End != Start)
		{
			while (FChar::IsWhitespace(*End)) { ++End; }
			if (*End == TEXT('\0'))
			{
				OutValue = Parsed;
				return true;
			}
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceMetadataFile::SetInt(const TCHAR* Section, const TCHAR* Key, int32 Value)
{
	SetString(Section, Key, FString::Printf(TEXT("%d"), Value));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTraceMetadataFile::GetDouble(const TCHAR* Section, const TCHAR* Key, double& OutValue) const
{
	FString StringValue;
	if (GetString(Section, Key, StringValue))
	{
		StringValue.TrimStartAndEndInline();
		if (!StringValue.IsEmpty())
		{
			return FDefaultValueHelper::ParseDouble(StringValue, OutValue);
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceMetadataFile::SetDouble(const TCHAR* Section, const TCHAR* Key, double Value)
{
	SetString(Section, Key, FString::Printf(TEXT("%.10f"), Value));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceMetadataFile::ClearSection(const TCHAR* Section)
{
	Sections.Remove(Section);
	SectionOrder.RemoveSingle(Section);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
