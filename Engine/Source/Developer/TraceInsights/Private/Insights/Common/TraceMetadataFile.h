// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Manages reading and writing an INI sidecar file (.utrace.ini) alongside a trace file.
 * Feature layers (user annotations, tags, etc.) use this to read/write their respective sections.
 */
class FTraceMetadataFile
{
public:
	FTraceMetadataFile() = default;

	void SetFilePath(const FString& InFilePath);
	const FString& GetFilePath() const { return FilePath; }

	/** Derives the sidecar path: "MyTrace.utrace" -> "MyTrace.utrace.ini" */
	static FString GetSidecarPath(const FString& TraceFilePath);

	/** Load from disk. Returns true even if the file does not yet exist. */
	bool Load();

	/** Save all in-memory data to the INI file on disk. */
	bool Save();

	bool IsLoaded() const { return bIsLoaded; }

	// Section-level read/write API
	bool GetString(const TCHAR* Section, const TCHAR* Key, FString& OutValue) const;
	void SetString(const TCHAR* Section, const TCHAR* Key, const FString& Value);
	bool GetInt(const TCHAR* Section, const TCHAR* Key, int32& OutValue) const;
	void SetInt(const TCHAR* Section, const TCHAR* Key, int32 Value);
	bool GetDouble(const TCHAR* Section, const TCHAR* Key, double& OutValue) const;
	void SetDouble(const TCHAR* Section, const TCHAR* Key, double Value);
	void ClearSection(const TCHAR* Section);

	static constexpr int32 CurrentVersion = 1;

private:
	FString FilePath;
	TMap<FString, TMap<FString, FString>> Sections;
	/** Section names in first-seen order. Preserved across Load/Save so future order-sensitive
	 *  consumers see a deterministic layout instead of an alphabetical reshuffle. */
	TArray<FString> SectionOrder;
	bool bIsLoaded = false;
};

} // namespace UE::Insights
