// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utf8FileWriter.h"

#include "Containers/Utf8String.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "String/Find.h"

#include <limits>
#include <cmath>

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FUtf8FileWriter::FUtf8FileWriter(IFileHandle* InFileHandle, bool bIsCSV)
	: FileHandle(InFileHandle)
	, Separator(bIsCSV ? UTF8CHAR(',') : UTF8CHAR('\t'))
{
	check(FileHandle);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUtf8FileWriter::AppendValue(const UTF8CHAR* Str)
{
	if (Str != nullptr)
	{
		AppendValue(FUtf8StringView(Str));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUtf8FileWriter::AppendValue(FUtf8StringView Str)
{
	if (Str.IsEmpty())
	{
		// nothing to append
		return;
	}

	if (IsCSV())
	{
		bool bNeedsQuotes = false;
		if (UE::String::FindFirstChar(Str, Separator, ESearchCase::CaseSensitive) != INDEX_NONE)
		{
			AppendDoubleQuotationMark();
			bNeedsQuotes = true;
		}
		if (UE::String::FindFirstChar(Str, UTF8CHAR('\"'), ESearchCase::CaseSensitive) != INDEX_NONE)
		{
			if (!bNeedsQuotes)
			{
				AppendDoubleQuotationMark();
				bNeedsQuotes = true;
			}
			FUtf8String String{ Str };
			String.ReplaceInline(UTF8TEXT("\""), UTF8TEXT("\"\"")); // CSV escape for " is ""
			StringBuilder.Append(String);
		}
		else
		{
			StringBuilder.Append(Str);
		}
		if (bNeedsQuotes)
		{
			AppendDoubleQuotationMark();
		}
	}
	else // TSV
	{
		FUtf8String String{ Str };
		String.ReplaceCharWithEscapedCharInline();
		StringBuilder.Append(String);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUtf8FileWriter::AppendValue(const TCHAR* Str)
{
	if (Str != nullptr)
	{
		AppendValue(FStringView(Str));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUtf8FileWriter::AppendValue(FStringView Str)
{
	if (Str.IsEmpty())
	{
		// nothing to append
		return;
	}

	if (IsCSV())
	{
		bool bNeedsQuotes = false;
		if (UE::String::FindFirstChar(Str, static_cast<TCHAR>(Separator), ESearchCase::CaseSensitive) != INDEX_NONE)
		{
			AppendDoubleQuotationMark();
			bNeedsQuotes = true;
		}
		if (UE::String::FindFirstChar(Str, TCHAR('\"'), ESearchCase::CaseSensitive) != INDEX_NONE)
		{
			if (!bNeedsQuotes)
			{
				AppendDoubleQuotationMark();
				bNeedsQuotes = true;
			}
			FString String{ Str };
			String.ReplaceInline(TEXT("\""), TEXT("\"\"")); // CSV escape for " is ""
			StringBuilder.Append(String);
		}
		else
		{
			StringBuilder.Append(Str);
		}
		if (bNeedsQuotes)
		{
			AppendDoubleQuotationMark();
		}
	}
	else // TSV
	{
		FString String{ Str };
		String.ReplaceCharWithEscapedCharInline();
		StringBuilder.Append(String);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUtf8FileWriter::AppendValue(float InValue)
{
	if (std::isnan(InValue))
	{
		StringBuilder.Append(UTF8TEXTVIEW("nan"));
	}
	else if (InValue == +std::numeric_limits<float>::infinity() || InValue == +FLT_MAX)
	{
		StringBuilder.Append(UTF8TEXTVIEW("+inf"));
	}
	else if (InValue == -std::numeric_limits<float>::infinity() || InValue == -FLT_MAX)
	{
		StringBuilder.Append(UTF8TEXTVIEW("-inf"));
	}
	else
	{
		StringBuilder.Appendf(UTF8TEXT("%.6g"), InValue);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUtf8FileWriter::AppendValue(double InValue)
{
	if (std::isnan(InValue))
	{
		StringBuilder.Append(UTF8TEXTVIEW("nan"));
	}
	else if (InValue == +std::numeric_limits<double>::infinity() || InValue == +DBL_MAX)
	{
		StringBuilder.Append(UTF8TEXTVIEW("+inf"));
	}
	else if (InValue == -std::numeric_limits<double>::infinity() || InValue == -DBL_MAX)
	{
		StringBuilder.Append(UTF8TEXTVIEW("-inf"));
	}
	else
	{
		StringBuilder.Appendf(UTF8TEXT("%.9g"), InValue);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUtf8FileWriter::AppendValueAsTimestamp(double InValue)
{
	if (std::isnan(InValue))
	{
		StringBuilder.Append(UTF8TEXTVIEW("nan"));
	}
	else if (InValue == +std::numeric_limits<double>::infinity() || InValue == +DBL_MAX)
	{
		StringBuilder.Append(UTF8TEXTVIEW("+inf"));
	}
	else if (InValue == -std::numeric_limits<double>::infinity() || InValue == -DBL_MAX)
	{
		StringBuilder.Append(UTF8TEXTVIEW("-inf"));
	}
	else
	{
		StringBuilder.Appendf(UTF8TEXT("%.9g"), InValue);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUtf8FileWriter::AppendValueAsDuration(double InValue)
{
	if (std::isnan(InValue))
	{
		StringBuilder.Append(UTF8TEXTVIEW("nan"));
	}
	else if (InValue == +std::numeric_limits<double>::infinity() || InValue == +DBL_MAX)
	{
		StringBuilder.Append(UTF8TEXTVIEW("+inf"));
	}
	else if (InValue == -std::numeric_limits<double>::infinity() || InValue == -DBL_MAX)
	{
		StringBuilder.Append(UTF8TEXTVIEW("-inf"));
	}
	else
	{
		StringBuilder.Appendf(UTF8TEXT("%.9f"), InValue);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUtf8FileWriter::WriteStringBuilder(int32 CacheLen)
{
	if (StringBuilder.Len() > CacheLen)
	{
		FileHandle->Write((const uint8*)StringBuilder.ToString(), StringBuilder.Len() * sizeof(UTF8CHAR));
		StringBuilder.Reset();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
