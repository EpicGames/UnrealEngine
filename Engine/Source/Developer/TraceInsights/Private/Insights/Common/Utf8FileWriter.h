// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

class IFileHandle;

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FUtf8FileWriter
{
private:
	static constexpr int BufferSize = 64 * 1024;

public:
	FUtf8FileWriter() = delete;
	FUtf8FileWriter(IFileHandle* InFileHandle, bool bIsCSV);
	~FUtf8FileWriter() = default;

	bool IsCSV() const { return Separator == UTF8CHAR(','); }
	bool IsTSV() const { return Separator == UTF8CHAR('\t'); }

	FUtf8StringBuilderBase& GetStringBuilder() { return StringBuilder; }

	UTF8CHAR GetSeparator() const { return Separator; }
	UTF8CHAR GetQuotationMark() const { return UTF8CHAR('\"'); }
	UTF8CHAR GetLineEnd() const { return UTF8CHAR('\n'); }

	//////////////////////////////////////////////////
	// Writes the content without escaping.

	void AppendSeparator() { StringBuilder.AppendChar(Separator); }

	void AppendSingleQuotationMark() { StringBuilder.AppendChar(UTF8CHAR('\'')); }
	void AppendDoubleQuotationMark() { StringBuilder.AppendChar(UTF8CHAR('\"')); }

	void AppendLineEnd()
	{
		StringBuilder.AppendChar(UTF8CHAR('\n'));
		WriteStringBuilder(BufferSize - 1024);
	}

	template <typename OtherCharType>
	inline void Append(const OtherCharType* const String, const int32 Length)
	{
		StringBuilder.Append(String, Length);
	}

	template <typename CharRangeType>
	inline void Append(CharRangeType&& Range)
	{
		StringBuilder.Append(Range);
	}

	template <typename AppendedCharType>
	inline void AppendChar(AppendedCharType Char)
	{
		StringBuilder.Append(Char);
	}

	template <typename FmtType, typename... Types>
	inline void Appendf(const FmtType& Fmt, Types... Args)
	{
		StringBuilder.Appendf(Fmt, Forward<Types>(Args)...);
	}

	inline void AppendV(const UTF8CHAR* Fmt, va_list Args)
	{
		StringBuilder.AppendV(Fmt, Args);
	}

	//////////////////////////////////////////////////
	// Escapes the value (if needed; see CSV/TSV) before writing it to file.

	void AppendValue(bool InValue)   { StringBuilder.Append(InValue ? UTF8TEXT("True") : UTF8TEXT("False")); }
	void AppendValue(int32 InValue)  { StringBuilder.Appendf(UTF8TEXT("%i"), InValue); }
	void AppendValue(uint32 InValue) { StringBuilder.Appendf(UTF8TEXT("%u"), InValue); }
	void AppendValue(int64 InValue)  { StringBuilder.Appendf(UTF8TEXT("%lli"), InValue); }
	void AppendValue(uint64 InValue) { StringBuilder.Appendf(UTF8TEXT("%llu"), InValue); }

	void AppendValueAsHex32(uint32 InValue) { StringBuilder.Appendf(UTF8TEXT("0x%X"), InValue); }
	void AppendValueAsHex64(uint64 InValue) { StringBuilder.Appendf(UTF8TEXT("0x%llX"), InValue); }

	void AppendValue(float InValue);
	void AppendValue(double InValue);
	void AppendValueAsTimestamp(double InValue);
	void AppendValueAsDuration(double InValue);

	void AppendValue(const UTF8CHAR* InValue);
	void AppendValue(FUtf8StringView InValue);
	void AppendValue(const TCHAR* InValue);
	void AppendValue(FStringView InValue);

	//////////////////////////////////////////////////

	void Flush()
	{
		WriteStringBuilder(0);
	}

private:
	void WriteStringBuilder(int32 CacheLen);

private:
	IFileHandle* FileHandle = nullptr;
	TUtf8StringBuilder<BufferSize> StringBuilder;
	UTF8CHAR Separator;
};

} // namespace UE::Insights
