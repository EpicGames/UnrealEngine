// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextSwitchesProfilerManager.h"

#include "Features/IModularFeatures.h"
#include "Framework/Commands/UICommandList.h"
#include "Modules/ModuleManager.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/ContextSwitches.h"

// TraceInsights
#include "Insights/ContextSwitches/ViewModels/ContextSwitchesSharedState.h"
#include "Insights/ITimingViewExtender.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/Widgets/STimingView.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::ContextSwitches
{

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FContextSwitchesProfilerManager> FContextSwitchesProfilerManager::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FContextSwitchesProfilerManager> FContextSwitchesProfilerManager::Get()
{
	return FContextSwitchesProfilerManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FContextSwitchesProfilerManager> FContextSwitchesProfilerManager::CreateInstance()
{
	ensure(!FContextSwitchesProfilerManager::Instance.IsValid());
	if (FContextSwitchesProfilerManager::Instance.IsValid())
	{
		FContextSwitchesProfilerManager::Instance.Reset();
	}

	FContextSwitchesProfilerManager::Instance = MakeShared<FContextSwitchesProfilerManager>(FInsightsManager::Get()->GetCommandList());

	return FContextSwitchesProfilerManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FContextSwitchesProfilerManager::FContextSwitchesProfilerManager(TSharedRef<FUICommandList> InCommandList)
	: bIsInitialized(false)
	, bIsAvailable(false)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesProfilerManager::Initialize(IUnrealInsightsModule& InsightsModule)
{
	ensure(!bIsInitialized);
	if (bIsInitialized)
	{
		return;
	}
	bIsInitialized = true;

	// Register tick functions.
	OnTick = FTickerDelegate::CreateSP(this, &FContextSwitchesProfilerManager::Tick);
	OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(OnTick, 0.0f);

	FOnRegisterMajorTabExtensions* TimingProfilerLayoutExtension = InsightsModule.FindMajorTabLayoutExtension(FInsightsManagerTabs::TimingProfilerTabId);
	if (TimingProfilerLayoutExtension)
	{
		TimingProfilerLayoutExtension->AddRaw(this, &FContextSwitchesProfilerManager::RegisterTimingProfilerLayoutExtensions);
	}

	FInsightsManager::Get()->GetSessionChangedEvent().AddSP(this, &FContextSwitchesProfilerManager::OnSessionChanged);
	OnSessionChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesProfilerManager::Shutdown()
{
	if (!bIsInitialized)
	{
		return;
	}
	bIsInitialized = false;

	FInsightsManager::Get()->GetSessionChangedEvent().RemoveAll(this);

	// Unregister tick function.
	FTSTicker::RemoveTicker(OnTickHandle);

	FContextSwitchesProfilerManager::Instance.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FContextSwitchesProfilerManager::~FContextSwitchesProfilerManager()
{
	ensure(!bIsInitialized);

	if (ContextSwitchesSharedState.IsValid())
	{
		IModularFeatures::Get().UnregisterModularFeature(Timing::TimingViewExtenderFeatureName, ContextSwitchesSharedState.Get());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesProfilerManager::RegisterMajorTabs(IUnrealInsightsModule& InsightsModule)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesProfilerManager::UnregisterMajorTabs()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FContextSwitchesProfilerManager::Tick(float DeltaTime)
{
	// Check if session has ContextSwitch events (to spawn the tab), but not too often.
	if (!bIsAvailable && AvailabilityCheck.Tick())
	{
		bool bShouldBeAvailable = false;

		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			bool bHasData = false;
			if (const TraceServices::IContextSwitchesProvider* ContextSwitchesProvider = TraceServices::ReadContextSwitchesProvider(*Session.Get()))
			{
				TraceServices::FProviderReadScopeLock ScopedLock(*ContextSwitchesProvider);
				bHasData = ContextSwitchesProvider->HasData();
			}
			if (bHasData)
			{
				using namespace UE::Insights::TimingProfiler;

				TSharedPtr<STimingProfilerWindow> Window = FTimingProfilerManager::Get()->GetProfilerWindow();
				if (!Window.IsValid())
				{
					return true;
				}

				TSharedPtr<STimingView> TimingView = Window->GetTimingView();
				if (!TimingView.IsValid())
				{
					return true;
				}

				bIsAvailable = true;

				if (!ContextSwitchesSharedState.IsValid())
				{
					ContextSwitchesSharedState = MakeShared<FContextSwitchesSharedState>(nullptr);
					IModularFeatures::Get().RegisterModularFeature(Timing::TimingViewExtenderFeatureName, ContextSwitchesSharedState.Get());
					ContextSwitchesSharedState->OnBeginSession(*TimingView.Get());
				}
			}

			if (Session->IsAnalysisComplete())
			{
				// Never check again during this session.
				AvailabilityCheck.Disable();
			}
		}
		else
		{
			// Do not check again until the next session changed event (see OnSessionChanged).
			AvailabilityCheck.Disable();
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesProfilerManager::OnSessionChanged()
{
	bIsAvailable = false;
	if (FInsightsManager::Get()->GetSession().IsValid())
	{
		AvailabilityCheck.Enable(0.5);
	}
	else
	{
		AvailabilityCheck.Disable();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesProfilerManager::RegisterTimingProfilerLayoutExtensions(FInsightsMajorTabExtender& InOutExtender)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ContextSwitches
