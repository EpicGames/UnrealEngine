// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Ticker.h"
#include "Framework/Commands/UICommandList.h"
#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Common/AvailabilityCheck.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"

class FThreadTrackEvent;
class FUICommandList;

namespace UE::Insights::ContextSwitches
{

class FContextSwitchesSharedState;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * This class manages the Context Switches Profiler state and settings.
 */
class FContextSwitchesProfilerManager : public TSharedFromThis<FContextSwitchesProfilerManager>, public IInsightsComponent
{
public:
	/** Creates the Context Switches Profiler manager, only one instance can exist. */
	FContextSwitchesProfilerManager(TSharedRef<FUICommandList> InCommandList);

	/** Virtual destructor. */
	virtual ~FContextSwitchesProfilerManager();

	/** Creates an instance of the Context Switches manager. */
	static TSharedPtr<FContextSwitchesProfilerManager> CreateInstance();

	static TSharedPtr<FContextSwitchesProfilerManager> Get();

	//////////////////////////////////////////////////
	// IInsightsComponent

	virtual void Initialize(IUnrealInsightsModule& InsightsModule) override;
	virtual void Shutdown() override;
	virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) override;
	virtual void UnregisterMajorTabs() override;

	//////////////////////////////////////////////////

	bool IsAvailable() const { return bIsAvailable; }

	void OnSessionChanged();

	TSharedPtr<FContextSwitchesSharedState> GetContextSwitchesSharedState() { return ContextSwitchesSharedState; }

private:
	/** Updates this manager, done through FCoreTicker. */
	bool Tick(float DeltaTime);

	void RegisterTimingProfilerLayoutExtensions(FInsightsMajorTabExtender& InOutExtender);

private:
	bool bIsInitialized;
	bool bIsAvailable;
	FAvailabilityCheck AvailabilityCheck;

	/** The delegate to be invoked when this manager ticks. */
	FTickerDelegate OnTick;

	/** Handle to the registered OnTick. */
	FTSTicker::FDelegateHandle OnTickHandle;

	/** A shared pointer to the global instance of the Context Switches Profiler manager. */
	static TSharedPtr<FContextSwitchesProfilerManager> Instance;

	/** Shared state for Context Switches tracks */
	TSharedPtr<FContextSwitchesSharedState> ContextSwitchesSharedState;
};

} // namespace UE::Insights::ContextSwitches
