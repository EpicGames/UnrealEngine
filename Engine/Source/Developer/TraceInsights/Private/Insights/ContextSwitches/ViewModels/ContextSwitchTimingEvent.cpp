// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextSwitchTimingEvent.h"

namespace UE::Insights::ContextSwitches
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FContextSwitchTimingEvent
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FContextSwitchTimingEvent)

////////////////////////////////////////////////////////////////////////////////////////////////////

FContextSwitchTimingEvent::FContextSwitchTimingEvent(const TSharedRef<const FBaseTimingTrack> InTrack, double InStartTime, double InEndTime, uint32 InDepth)
	: FTimingEvent(InTrack, InStartTime, InEndTime, InDepth)
{}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FCpuCoreTimingEvent
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FCpuCoreTimingEvent)

////////////////////////////////////////////////////////////////////////////////////////////////////

FCpuCoreTimingEvent::FCpuCoreTimingEvent(const TSharedRef<const FBaseTimingTrack> InTrack, double InStartTime, double InEndTime, uint32 InDepth)
	: FTimingEvent(InTrack, InStartTime, InEndTime, InDepth)
{}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ContextSwitches
