// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Templates/SharedPointer.h"

// TraceInsights
#include "Insights/ViewModels/TimingEvent.h"

namespace UE::Insights::ContextSwitches
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FContextSwitchTimingEvent : public FTimingEvent
{
	INSIGHTS_DECLARE_RTTI(FContextSwitchTimingEvent, FTimingEvent)

public:
	FContextSwitchTimingEvent(const TSharedRef<const FBaseTimingTrack> InTrack, double InStartTime, double InEndTime, uint32 InDepth);
	virtual ~FContextSwitchTimingEvent() = default;

	void SetCoreNumber(uint32 InCoreNumber) { SetType((uint64)InCoreNumber); }
	uint32 GetCoreNumber() const { return (uint32)GetType(); }
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FCpuCoreTimingEvent : public FTimingEvent
{
	INSIGHTS_DECLARE_RTTI(FCpuCoreTimingEvent, FTimingEvent)

public:
	FCpuCoreTimingEvent(const TSharedRef<const FBaseTimingTrack> InTrack, double InStartTime, double InEndTime, uint32 InDepth);
	virtual ~FCpuCoreTimingEvent() = default;

	void SetSystemThreadId(uint32 InSystemThreadId) { SetType((uint64)InSystemThreadId); }
	uint32 GetSystemThreadId() const { return (uint32)GetType(); }
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ContextSwitches
