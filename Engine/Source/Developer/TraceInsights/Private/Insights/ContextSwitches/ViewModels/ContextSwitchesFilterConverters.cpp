// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextSwitchesFilterConverters.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/ContextSwitches.h"
#include "TraceServices/Model/Threads.h"

// TraceInsights
#include "Insights/InsightsManager.h"

#define LOCTEXT_NAMESPACE "UE::Insights::ContextSwitches"

namespace UE::Insights::ContextSwitches
{

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FCoreEventNameFilterValueConverter::Convert(const FString& Input, int64& Output, FText& OutError) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	bool bFound = false;
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(*Session.Get());
		ThreadProvider.EnumerateThreads([&Input, &Output, &Session, &bFound](const TraceServices::FThreadInfo& ThreadInfo)
			{
				if (!bFound && FCString::Stricmp(*Input, ThreadInfo.Name) == 0)
				{
					if (const TraceServices::IContextSwitchesProvider* ContextSwitchesProvider = TraceServices::ReadContextSwitchesProvider(*Session.Get()))
					{
						TraceServices::FProviderReadScopeLock ScopedLock(*ContextSwitchesProvider);

						uint32 SystemThreadId;
						if (ContextSwitchesProvider->GetSystemThreadId(ThreadInfo.Id, SystemThreadId))
						{
							Output = SystemThreadId;
							bFound = true;
						}
					}
				}
			});
	}

	if (!bFound)
	{
		OutError = LOCTEXT("NoCoreEventFound", "No core event with this name was found!");
		return false;
	}
	
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FCoreEventNameFilterValueConverter::GetTooltipText() const
{
	return LOCTEXT("FCoreEventNameFilterValueConverterTooltip", "Enter the exact name of the core event.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FCoreEventNameFilterValueConverter::GetHintText() const
{
	// Use the name of a well known event as the hint to show the user what kind of name we are expecting.
	return FText::FromString(TEXT("GameThread"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ContextSwitches

#undef LOCTEXT_NAMESPACE
