// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Internationalization/Text.h"

// TraceInsightsCore
#include "InsightsCore/Filter/ViewModels/Filters.h"

namespace UE::Insights::ContextSwitches
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FCoreEventNameFilterValueConverter : public IFilterValueConverter
{
public:
	virtual bool Convert(const FString& Input, int64& Output, FText& OutError) const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetHintText() const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ContextSwitches
