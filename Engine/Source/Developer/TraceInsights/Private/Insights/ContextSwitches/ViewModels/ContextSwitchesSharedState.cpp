// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextSwitchesSharedState.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/Threads.h"
#include "TraceServices/Model/ContextSwitches.h"

// TraceInsightsCore
#include "InsightsCore/Filter/ViewModels/FilterConfigurator.h"

// TraceInsights
#include "Insights/ContextSwitches/ContextSwitchesProfilerManager.h"
#include "Insights/ContextSwitches/ViewModels/ContextSwitchesFilterConverters.h"
#include "Insights/ContextSwitches/ViewModels/ContextSwitchesTimingTrack.h"
#include "Insights/ContextSwitches/ViewModels/ContextSwitchTimingEvent.h"
#include "Insights/ContextSwitches/ViewModels/CpuCoreTimingTrack.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Tracks/CpuTimingTrack.h"
#include "Insights/TimingProfiler/ViewModels/ThreadTimingSharedState.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::ContextSwitches"

namespace UE::Insights::ContextSwitches
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FContextSwitchesStateCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

FContextSwitchesStateCommands::FContextSwitchesStateCommands()
: TCommands<FContextSwitchesStateCommands>(
	TEXT("ContextSwitchesStateCommands"),
	NSLOCTEXT("Contexts", "ContextSwitchesStateCommands", "Insights - Context Switches"),
	NAME_None,
	FInsightsStyle::GetStyleSetName())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// UI_COMMAND takes long for the compiler to optimize
UE_DISABLE_OPTIMIZATION_SHIP
void FContextSwitchesStateCommands::RegisterCommands()
{
	UI_COMMAND(Command_ShowCpuCoreTracks,
		"CPU Core Tracks",
		"Shows/hides the CPU Core tracks.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EModifierKey::Alt, EKeys::C));

	UI_COMMAND(Command_ShowContextSwitches,
		"Context Switches",
		"Shows/hides the context switches on top of CPU timing tracks.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EModifierKey::Shift, EKeys::C));

	UI_COMMAND(Command_ShowOverlays,
		"Overlays",
		"Extends the visualization of context switches over the CPU timing tracks.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EModifierKey::Shift, EKeys::O));

	UI_COMMAND(Command_ShowExtendedLines,
		"Extended Lines",
		"Shows/hides the extended vertical lines at edges of each context switch event.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EModifierKey::Shift, EKeys::L));

	UI_COMMAND(Command_ShowNonTargetProcessEvents,
		"Non-Target Process Events",
		"Shows/hides the CPU Core events that do not belong to the target process.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(Command_NavigateToCpuThreadEvent,
		"Go To CPU Thread Track",
		"Selects the context switch event in the corresponding CPU Thread track.",
		EUserInterfaceActionType::Button,
		FInputChord(/*EModifierKey::Control, EKeys::Tab*/));

	UI_COMMAND(Command_DockCpuThreadTrackToBottom,
		"Dock CPU Thread Track To Bottom",
		"Docks the corresponding CPU Thread track to the bottom of the Timing view.",
		EUserInterfaceActionType::Button,
		FInputChord());

	UI_COMMAND(Command_NavigateToCpuCoreEvent,
		"Go To CPU Core Track",
		"Selects the timing event in the corresponding CPU Core track.",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Control, EKeys::Tab));

	UI_COMMAND(Command_DockCpuCoreTrackToTop,
		"Dock CPU Core Track To Top",
		"Docks the corresponding CPU Core track to the top of the Timing view.",
		EUserInterfaceActionType::Button,
		FInputChord());
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////
// FContextSwitchesSharedState
////////////////////////////////////////////////////////////////////////////////////////////////////

FContextSwitchesSharedState::FContextSwitchesSharedState(TimingProfiler::STimingView* InTimingView)
	: TimingViewSession(InTimingView)
	, ThreadsSerial(0)
	, CpuCoresSerial(0)
	, bSyncWithProviders(true)
	, TimingViewSettings(MakeShared<FTimingViewLocalSettings>())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::OnBeginSession(Timing::ITimingViewSession& InSession)
{
	if (&InSession != TimingViewSession)
	{
		if (InSession.GetName() == FInsightsManagerTabs::TimingProfilerTabId)
		{
			TimingViewSession = &InSession;
			AddCommands();

			TimingViewSettings = MakeShared<FTimingViewPersistentSettings>(InSession.GetName());
		}
		else
		{
			return;
		}
	}

	CpuCoreTimingTracks.Reset();
	ContextSwitchesTimingTracks.Reset();

	ThreadsSerial = 0;
	CpuCoresSerial = 0;

	bSyncWithProviders = true;

	TargetTimingEvent = nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::OnEndSession(Timing::ITimingViewSession& InSession)
{
	if (&InSession != TimingViewSession)
	{
		return;
	}

	CpuCoreTimingTracks.Reset();
	ContextSwitchesTimingTracks.Reset();

	ThreadsSerial = 0;
	CpuCoresSerial = 0;

	bSyncWithProviders = false;

	TargetTimingEvent = nullptr;
	TimingViewSession = nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::Tick(Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	if (&InSession != TimingViewSession)
	{
		return;
	}

	if (bSyncWithProviders && AreContextSwitchesAvailable())
	{
		if (AreCpuCoreTracksVisible())
		{
			AddCpuCoreTracks();
		}

		if (AreContextSwitchesVisible())
		{
			AddContextSwitchesChildTracks();
		}

		if (FInsightsManager::Get()->IsAnalysisComplete())
		{
			// No need to sync anymore when analysis is completed.
			bSyncWithProviders = false;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::ExtendCpuTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder)
{
	if (&InSession != TimingViewSession)
	{
		return;
	}

	BuildSubMenu(InMenuBuilder);

	//InMenuBuilder.BeginSection("ContextSwitches");
	//{
	//	InMenuBuilder.AddSubMenu(
	//		LOCTEXT("ContextSwitches_SubMenu", "Context Switches"),
	//		LOCTEXT("ContextSwitches_SubMenu_Desc", "Context Switch track options"),
	//		FNewMenuDelegate::CreateSP(this, &FContextSwitchesSharedState::BuildSubMenu),
	//		false,
	//		FSlateIcon()
	//	);
	//}
	//InMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::BuildSubMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("ContextSwitches", LOCTEXT("ContextMenu_Section_ContextSwitches", "Context Switches"));
	{
		InMenuBuilder.AddMenuEntry(FContextSwitchesStateCommands::Get().Command_ShowCpuCoreTracks);
		InMenuBuilder.AddMenuEntry(FContextSwitchesStateCommands::Get().Command_ShowNonTargetProcessEvents);
		InMenuBuilder.AddMenuEntry(FContextSwitchesStateCommands::Get().Command_ShowContextSwitches);
		InMenuBuilder.AddMenuEntry(FContextSwitchesStateCommands::Get().Command_ShowOverlays);
		InMenuBuilder.AddMenuEntry(FContextSwitchesStateCommands::Get().Command_ShowExtendedLines);
	}
	InMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::AddQuickFindFilters(TSharedPtr<FFilterConfigurator> FilterConfigurator)
{
	using namespace UE::Insights;

	TSharedPtr<TArray<TSharedPtr<IFilterOperator>>> CoreEventNameFilterOperators = MakeShared<TArray<TSharedPtr<IFilterOperator>>>();
	CoreEventNameFilterOperators->Add(StaticCastSharedRef<IFilterOperator>(MakeShared<FFilterOperator<int64>>(EFilterOperator::Eq, TEXT("Is"), [](int64 lhs, int64 rhs) { return lhs == rhs; })));

	TSharedRef<FFilterWithSuggestions> TimerNameFilter = MakeShared<FFilterWithSuggestions>(
		static_cast<int32>(EFilterField::CoreEventName),
		LOCTEXT("CoreEventName", "Core Event Name"),
		LOCTEXT("CoreEventName", "Core Event Name"),
		EFilterDataType::StringInt64Pair,
		MakeShared<FCoreEventNameFilterValueConverter>(),
		CoreEventNameFilterOperators);

	TimerNameFilter->SetCallback([this](const FString& Text, TArray<FString>& OutSuggestions)
	{
		this->PopulateCoreEventNameSuggestionList(Text, OutSuggestions);
	});

	FilterConfigurator->Add(TimerNameFilter);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::PopulateCoreEventNameSuggestionList(const FString& Text, TArray<FString>& OutSuggestions)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(*Session.Get());
		ThreadProvider.EnumerateThreads([&Text, &OutSuggestions](const TraceServices::FThreadInfo& ThreadInfo)
			{
				if (FCString::Stristr(ThreadInfo.Name, *Text))
				{
					OutSuggestions.Add(ThreadInfo.Name);
				}
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::AddCommands()
{
	FContextSwitchesStateCommands::Register();

	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();

	if (!TimingView.IsValid())
	{
		return;
	}
	TSharedPtr<FUICommandList> CommandList = TimingView->GetCommandList();
	ensure(CommandList.IsValid());

	CommandList->MapAction(
		FContextSwitchesStateCommands::Get().Command_ShowCpuCoreTracks,
		FExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_ShowCpuCoreTracks_Execute),
		FCanExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_ShowCpuCoreTracks_CanExecute),
		FIsActionChecked::CreateSP(this, &FContextSwitchesSharedState::Command_ShowCpuCoreTracks_IsChecked));

	CommandList->MapAction(
		FContextSwitchesStateCommands::Get().Command_ShowNonTargetProcessEvents,
		FExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_ShowNonTargetProcessEvents_Execute),
		FCanExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_ShowNonTargetProcessEvents_CanExecute),
		FIsActionChecked::CreateSP(this, &FContextSwitchesSharedState::Command_ShowNonTargetProcessEvents_IsChecked));

	CommandList->MapAction(
		FContextSwitchesStateCommands::Get().Command_ShowContextSwitches,
		FExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_ShowContextSwitches_Execute),
		FCanExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_ShowContextSwitches_CanExecute),
		FIsActionChecked::CreateSP(this, &FContextSwitchesSharedState::Command_ShowContextSwitches_IsChecked));

	CommandList->MapAction(
		FContextSwitchesStateCommands::Get().Command_ShowOverlays,
		FExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_ShowOverlays_Execute),
		FCanExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_ShowOverlays_CanExecute),
		FIsActionChecked::CreateSP(this, &FContextSwitchesSharedState::Command_ShowOverlays_IsChecked));

	CommandList->MapAction(
		FContextSwitchesStateCommands::Get().Command_ShowExtendedLines,
		FExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_ShowExtendedLines_Execute),
		FCanExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_ShowExtendedLines_CanExecute),
		FIsActionChecked::CreateSP(this, &FContextSwitchesSharedState::Command_ShowExtendedLines_IsChecked));

	CommandList->MapAction(
		FContextSwitchesStateCommands::Get().Command_NavigateToCpuThreadEvent,
		FExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_NavigateToCpuThreadEvent_Execute),
		FCanExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_NavigateToCpuThreadEvent_CanExecute));

	CommandList->MapAction(
		FContextSwitchesStateCommands::Get().Command_DockCpuThreadTrackToBottom,
		FExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_DockCpuThreadTrackToBottom_Execute),
		FCanExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_DockCpuThreadTrackToBottom_CanExecute));

	CommandList->MapAction(
		FContextSwitchesStateCommands::Get().Command_NavigateToCpuCoreEvent,
		FExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_NavigateToCpuCoreEvent_Execute),
		FCanExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_NavigateToCpuCoreEvent_CanExecute));

	CommandList->MapAction(
		FContextSwitchesStateCommands::Get().Command_DockCpuCoreTrackToTop,
		FExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_DockCpuCoreTrackToTop_Execute),
		FCanExecuteAction::CreateSP(this, &FContextSwitchesSharedState::Command_DockCpuCoreTrackToTop_CanExecute));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FContextSwitchesSharedState::AreContextSwitchesAvailable() const
{
	return FContextSwitchesProfilerManager::Get()->IsAvailable();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::AddCpuCoreTracks()
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();

	if (!TimingView.IsValid())
	{
		return;
	}

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	if (const TraceServices::IContextSwitchesProvider* ContextSwitchesProvider = TraceServices::ReadContextSwitchesProvider(*Session.Get()))
	{
		TraceServices::FProviderReadScopeLock ScopedLock(*ContextSwitchesProvider);

		const uint64 NewCpuCoresSerial = ContextSwitchesProvider->GetCpuCoresSerial();
		if (NewCpuCoresSerial != CpuCoresSerial)
		{
			CpuCoresSerial = NewCpuCoresSerial;

			ContextSwitchesProvider->EnumerateCpuCores(
				[this, TimingView]
				(const TraceServices::FCpuCoreInfo& CpuCoreInfo)
				{
					TSharedPtr<FCpuCoreTimingTrack>* TrackPtrPtr = CpuCoreTimingTracks.Find(CpuCoreInfo.CoreNumber);
					if (TrackPtrPtr == nullptr)
					{
						const FString TrackName = FString::Printf(TEXT("Core %u"), CpuCoreInfo.CoreNumber);
						TSharedPtr<FCpuCoreTimingTrack> Track = MakeShared<FCpuCoreTimingTrack>(*this, TrackName, CpuCoreInfo.CoreNumber);

						const int32 Order = FTimingTrackOrder::Cpu - 1024 + CpuCoreInfo.CoreNumber;
						Track->SetOrder(Order);

						CpuCoreTimingTracks.Add(CpuCoreInfo.CoreNumber, Track);
						TimingView->AddScrollableTrack(Track);
					}
				}
			);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::RemoveCpuCoreTracks()
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	CpuCoresSerial = 0;

	if (TimingView)
	{
		for (const auto& KV : CpuCoreTimingTracks)
		{
			TimingView->RemoveTrack(KV.Value);
		}
	}

	CpuCoreTimingTracks.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::AddContextSwitchesChildTracks()
{
	//TODO: Create "CPU Thread" timing tracks also for threads without CPU timing events (i.e. only with context switch events).

	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return;
	}

	TSharedPtr<TimingProfiler::FThreadTimingSharedState> TimingSharedState = TimingView->GetThreadTimingSharedState();

	if (!TimingSharedState.IsValid())
	{
		return;
	}

	const TMap<uint32, TSharedPtr<TimingProfiler::FCpuTimingTrack>>& CpuTracks = TimingSharedState->GetAllCpuTracks();

	for (const TPair<uint32, TSharedPtr<TimingProfiler::FCpuTimingTrack>>& MapEntry : CpuTracks)
	{
		const TSharedPtr<TimingProfiler::FCpuTimingTrack>& CpuTrack = MapEntry.Value;
		if (CpuTrack.IsValid() && !CpuTrack->FindChildTrackOfType<FContextSwitchesTimingTrack>().IsValid())
		{
			TSharedRef<FContextSwitchesTimingTrack> ContextSwitchesTrack = MakeShared<FContextSwitchesTimingTrack>(*this, TEXT("Context Switches"), CpuTrack->GetTimelineIndex(), CpuTrack->GetThreadId());
			ContextSwitchesTrack->SetParentTrack(CpuTrack);
			CpuTrack->AddChildTrack(ContextSwitchesTrack);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::RemoveContextSwitchesChildTracks()
{
	ThreadsSerial = 0;

	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return;
	}
	TSharedPtr<TimingProfiler::FThreadTimingSharedState> TimingSharedState = TimingView->GetThreadTimingSharedState();

	if (!TimingSharedState.IsValid())
	{
		return;
	}

	const TMap<uint32, TSharedPtr<TimingProfiler::FCpuTimingTrack>>& CpuTracks = TimingSharedState->GetAllCpuTracks();

	for (const TPair<uint32, TSharedPtr<TimingProfiler::FCpuTimingTrack>>& MapEntry : CpuTracks)
	{
		const TSharedPtr<TimingProfiler::FCpuTimingTrack>& CpuTrack = MapEntry.Value;
		if (CpuTrack.IsValid())
		{
			if (TSharedPtr<FContextSwitchesTimingTrack> ChildTrack = CpuTrack->FindChildTrackOfType<FContextSwitchesTimingTrack>())
			{
				CpuTrack->RemoveChildTrack(ChildTrack.ToSharedRef());
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::SetCpuCoreTracksVisible(bool bOnOff)
{
	if (AreCpuCoreTracksVisible() != bOnOff)
	{
		TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowCpuCoreTracks, bOnOff);

		if (bOnOff)
		{
			AddCpuCoreTracks();
		}
		else
		{
			RemoveCpuCoreTracks();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::SetContextSwitchesVisible(bool bOnOff)
{
	if (AreContextSwitchesVisible() != bOnOff)
	{
		TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowContextSwitches, bOnOff);

		if (bOnOff)
		{
			AddContextSwitchesChildTracks();
		}
		else
		{
			RemoveContextSwitchesChildTracks();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::SetOverlaysVisible(bool bOnOff)
{
	TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowContextSwitchOverlays, bOnOff);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::SetExtendedLinesVisible(bool bOnOff)
{
	TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowContextSwitchExtendedLines, bOnOff);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::SetNonTargetProcessEventsVisible(bool bOnOff)
{
	if (AreNonTargetProcessEventsVisible() != bOnOff)
	{
		TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
		const TSharedPtr<ITimingEventFilter> EventFilter = TimingView->GetEventFilter();
		for (auto Track : CpuCoreTimingTracks)
		{
			Track.Value->SetDirtyFlag();
			if (!bOnOff && EventFilter)
			{
				if (EventFilter->FilterTrack(*Track.Value))
				{
					TimingView->ResetEventFilter();
				}
			}
		}

		TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowNonTargetProcessEvents, bOnOff);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FContextSwitchesSharedState::IsValidCpuCoreEventSelected() const
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (TimingView == nullptr)
	{
		return false;
	}
	if (TargetTimingEvent.IsValid())
	{
		return TargetTimingEvent->Is<FCpuCoreTimingEvent>();
	}
	return TimingView->GetSelectedEvent().IsValid() && TimingView->GetSelectedEvent()->Is<FCpuCoreTimingEvent>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FContextSwitchesSharedState::IsValidContextSwitchEventSelected() const
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (TimingView == nullptr)
	{
		return false;
	}
	if (TargetTimingEvent.IsValid())
	{
		return TargetTimingEvent->Is<FContextSwitchTimingEvent>();
	}
	return TimingView->GetSelectedEvent().IsValid() && TimingView->GetSelectedEvent()->Is<FContextSwitchTimingEvent>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::Command_NavigateToCpuThreadEvent_Execute()
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();

	if (TimingView && ensure(IsValidCpuCoreEventSelected()))
	{
		if (!TargetTimingEvent.IsValid())
		{
			TargetTimingEvent = TimingView->GetSelectedEvent();
		}

		if (ensure(TargetTimingEvent.IsValid() && TargetTimingEvent->Is<FCpuCoreTimingEvent>()))
		{
			const FCpuCoreTimingEvent& CpuCoreEvent = TargetTimingEvent->As<FCpuCoreTimingEvent>();
			const uint32 SystemThreadId = CpuCoreEvent.GetSystemThreadId();
			uint32 ThreadId;
			const TCHAR* ThreadName;
			GetThreadInfo(SystemThreadId, ThreadId, ThreadName);
			if (ThreadId != ~0)
			{
				TSharedPtr<TimingProfiler::FThreadTimingTrack> ThreadTimingTrack = GetThreadTimingTrack(ThreadId);
				if (ThreadTimingTrack.IsValid() && ThreadTimingTrack->IsVisible())
				{
					TimingView->SelectTimingTrack(ThreadTimingTrack, true);
					if (TSharedPtr<FContextSwitchesTimingTrack> ContextSwitchesTrack = ThreadTimingTrack->FindChildTrackOfType<FContextSwitchesTimingTrack>())
					{
						TSharedPtr<const ITimingEvent> FoundEvent = ContextSwitchesTrack->SearchEvent(
							FTimingEventSearchParameters(CpuCoreEvent.GetStartTime(), CpuCoreEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch));

						if (FoundEvent.IsValid())
						{
							TimingView->SelectTimingEvent(FoundEvent, true);
						}
					}
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FContextSwitchesSharedState::Command_NavigateToCpuThreadEvent_CanExecute() const
{
	if (!AreContextSwitchesAvailable() || !AreContextSwitchesVisible() || !IsValidCpuCoreEventSelected())
	{
		return false;
	}

	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();

	if (TimingView && IsValidCpuCoreEventSelected())
	{
		if (TargetTimingEvent.IsValid() && TargetTimingEvent->Is<FCpuCoreTimingEvent>())
		{
			const FCpuCoreTimingEvent& CpuCoreEvent = TargetTimingEvent->As<FCpuCoreTimingEvent>();
			const uint32 SystemThreadId = CpuCoreEvent.GetSystemThreadId();
			uint32 ThreadId;
			const TCHAR* ThreadName;
			GetThreadInfo(SystemThreadId, ThreadId, ThreadName);
			if (ThreadId != ~0)
			{
				TSharedPtr<TimingProfiler::FThreadTimingTrack> ThreadTimingTrack = GetThreadTimingTrack(ThreadId);
				if (ThreadTimingTrack.IsValid() && ThreadTimingTrack->IsVisible())
				{
					return true;
				}
			}
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::Command_DockCpuThreadTrackToBottom_Execute()
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();

	if (TimingView && ensure(IsValidCpuCoreEventSelected()))
	{
		if (!TargetTimingEvent.IsValid())
		{
			TargetTimingEvent = TimingView->GetSelectedEvent();
		}

		if (ensure(TargetTimingEvent.IsValid() && TargetTimingEvent->Is<FCpuCoreTimingEvent>()))
		{
			const FCpuCoreTimingEvent& CpuCoreEvent = TargetTimingEvent->As<FCpuCoreTimingEvent>();
			const uint32 SystemThreadId = CpuCoreEvent.GetSystemThreadId();
			uint32 ThreadId;
			const TCHAR* ThreadName;
			GetThreadInfo(SystemThreadId, ThreadId, ThreadName);
			if (ThreadId != ~0)
			{
				TSharedPtr<TimingProfiler::FThreadTimingTrack> ThreadTimingTrack = GetThreadTimingTrack(ThreadId);
				if (ThreadTimingTrack.IsValid())
				{
					TimingView->ChangeTrackLocation(ThreadTimingTrack.ToSharedRef(), ETimingTrackLocation::BottomDocked);
					if (!ThreadTimingTrack->IsVisible())
					{
						ThreadTimingTrack->Show();
						TimingView->HandleTrackVisibilityChanged();
					}
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::Command_NavigateToCpuCoreEvent_Execute()
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();

	if (TimingView && ensure(IsValidContextSwitchEventSelected()))
	{
		if (!TargetTimingEvent.IsValid())
		{
			TargetTimingEvent = TimingView->GetSelectedEvent();
		}

		if (ensure(TargetTimingEvent.IsValid() && TargetTimingEvent->Is<FContextSwitchTimingEvent>()))
		{
			const FContextSwitchTimingEvent& ContextSwitchEvent = TargetTimingEvent->As<FContextSwitchTimingEvent>();
			const uint32 CoreNumber = ContextSwitchEvent.GetCoreNumber();
			if (CoreNumber != ~0)
			{
				TSharedPtr<FCpuCoreTimingTrack> CpuCoreTimingTrack = GetCpuCoreTimingTrack(CoreNumber);
				if (CpuCoreTimingTrack.IsValid() && CpuCoreTimingTrack->IsVisible())
				{
					TimingView->SelectTimingTrack(CpuCoreTimingTrack, true);
					TSharedPtr<const ITimingEvent> FoundEvent = CpuCoreTimingTrack->SearchEvent(
						FTimingEventSearchParameters(ContextSwitchEvent.GetStartTime(), ContextSwitchEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch));

					if (FoundEvent.IsValid())
					{
						TimingView->SelectTimingEvent(FoundEvent, true);
					}
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::Command_DockCpuCoreTrackToTop_Execute()
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();

	if (TimingView && ensure(IsValidContextSwitchEventSelected()))
	{
		if (!TargetTimingEvent.IsValid())
		{
			TargetTimingEvent = TimingView->GetSelectedEvent();
		}

		if (ensure(TargetTimingEvent.IsValid() && TargetTimingEvent->Is<FContextSwitchTimingEvent>()))
		{
			const FContextSwitchTimingEvent& ContextSwitchEvent = TargetTimingEvent->As<FContextSwitchTimingEvent>();
			const uint32 CoreNumber = ContextSwitchEvent.GetCoreNumber();
			if (CoreNumber != ~0)
			{
				TSharedPtr<FCpuCoreTimingTrack> CpuCoreTimingTrack = GetCpuCoreTimingTrack(CoreNumber);
				if (CpuCoreTimingTrack.IsValid())
				{
					if (TimingView->CanChangeTrackLocation(CpuCoreTimingTrack.ToSharedRef(), ETimingTrackLocation::TopDocked))
					{
						TimingView->ChangeTrackLocation(CpuCoreTimingTrack.ToSharedRef(), ETimingTrackLocation::TopDocked);
					}
					if (!CpuCoreTimingTrack->IsVisible())
					{
						CpuCoreTimingTrack->Show();
						TimingView->HandleTrackVisibilityChanged();
					}
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesSharedState::GetThreadInfo(uint32 InSystemThreadId, uint32& OutThreadId, const TCHAR*& OutThreadName) const
{
	OutThreadId = ~0;
	OutThreadName = TEXT("Unknown Thread");

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		uint32 ThreadId = 0;
		bool bIsValidThreadId = false;
		if (const TraceServices::IContextSwitchesProvider* ContextSwitchesProvider = TraceServices::ReadContextSwitchesProvider(*Session.Get()))
		{
			TraceServices::FProviderReadScopeLock ScopedLock(*ContextSwitchesProvider);
			bIsValidThreadId = ContextSwitchesProvider->GetThreadId(InSystemThreadId, ThreadId);
		}
		if (bIsValidThreadId)
		{
			const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(*Session.Get());
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			OutThreadId = ThreadId;
			OutThreadName = ThreadProvider.GetThreadName(ThreadId);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<TimingProfiler::FThreadTimingTrack> FContextSwitchesSharedState::GetThreadTimingTrack(uint32 ThreadId) const
{
	TSharedPtr<TimingProfiler::FThreadTimingTrack> FoundTrack;
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();

	if (TimingView)
	{
		TimingView->EnumerateAllTracks([&FoundTrack, ThreadId](TSharedPtr<FBaseTimingTrack>& Track) -> bool
		{
			if (Track->Is<TimingProfiler::FThreadTimingTrack>() &&
				Track->As<TimingProfiler::FThreadTimingTrack>().GetThreadId() == ThreadId)
			{
				FoundTrack = StaticCastSharedPtr<TimingProfiler::FThreadTimingTrack>(Track);
				return false;
			}
			return true;
		});
	}

	return FoundTrack;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FCpuCoreTimingTrack> FContextSwitchesSharedState::GetCpuCoreTimingTrack(uint32 CoreNumber) const
{
	TSharedPtr<FCpuCoreTimingTrack> FoundTrack;
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();

	if (TimingView)
	{
		TimingView->EnumerateAllTracks([&FoundTrack, CoreNumber](TSharedPtr<FBaseTimingTrack>& Track) -> bool
		{
			if (Track->Is<FCpuCoreTimingTrack>() &&
				Track->As<FCpuCoreTimingTrack>().GetCoreNumber() == CoreNumber)
			{
				FoundTrack = StaticCastSharedPtr<FCpuCoreTimingTrack>(Track);
				return false;
			}
			return true;
		});
	}

	return FoundTrack;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<TimingProfiler::STimingView> FContextSwitchesSharedState::GetTimingView()
{
	using namespace UE::Insights::TimingProfiler;
	TSharedPtr<STimingProfilerWindow> Window = FTimingProfilerManager::Get()->GetProfilerWindow();
	return Window.IsValid() ? Window->GetTimingView() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ContextSwitches

#undef LOCTEXT_NAMESPACE
