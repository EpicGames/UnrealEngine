// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Framework/Commands/Commands.h"

// TraceInsights
#include "Insights/ITimingViewExtender.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsSettings.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMenuBuilder;

namespace TraceServices { class IAnalysisSession; }

class FBaseTimingTrack;
class ITimingEvent;

namespace UE::Insights::Timing { class ITimingViewSession; }

namespace UE::Insights::TimingProfiler
{
	class FThreadTimingTrack;
	class STimingView;
}

namespace UE::Insights::ContextSwitches
{

class FCpuCoreTimingTrack;
class FContextSwitchesTimingTrack;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FContextSwitchesStateCommands : public TCommands<FContextSwitchesStateCommands>
{
public:
	FContextSwitchesStateCommands();
	virtual ~FContextSwitchesStateCommands() = default;
	virtual void RegisterCommands() override;

	// Commands for the Tracks Filter menu.
	TSharedPtr<FUICommandInfo> Command_ShowCpuCoreTracks;
	TSharedPtr<FUICommandInfo> Command_ShowContextSwitches;
	TSharedPtr<FUICommandInfo> Command_ShowOverlays;
	TSharedPtr<FUICommandInfo> Command_ShowExtendedLines;
	TSharedPtr<FUICommandInfo> Command_ShowNonTargetProcessEvents;

	// Commands for a CPU Core track (context menu).
	TSharedPtr<FUICommandInfo> Command_NavigateToCpuThreadEvent;
	TSharedPtr<FUICommandInfo> Command_DockCpuThreadTrackToBottom;

	// Commands for a CPU Thread track (context menu).
	TSharedPtr<FUICommandInfo> Command_NavigateToCpuCoreEvent;
	TSharedPtr<FUICommandInfo> Command_DockCpuCoreTrackToTop;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FContextSwitchesSharedState : public Timing::ITimingViewExtender, public TSharedFromThis<FContextSwitchesSharedState>
{
	class ITimingViewSettings
	{
	public:
		virtual ~ITimingViewSettings() = default;

		virtual bool IsToggleOptionEnabled(const TCHAR* Key) const = 0;
		virtual void SetToggleOption(const TCHAR* Key, bool Value) = 0;
	};

	class FTimingViewPersistentSettings : public ITimingViewSettings
	{
	public:
		FTimingViewPersistentSettings(const FName& InTimingViewName)
			: Settings(GetInsightsSettings().GetOrCreateTimingViewSettings(InTimingViewName))
		{
		}

		virtual ~FTimingViewPersistentSettings() = default;

		const FName& GetName() const
		{
			return Settings.GetName();
		}

		//////////////////////////////////////////////////

		virtual bool IsToggleOptionEnabled(const TCHAR* Key) const override
		{
			return Settings.IsToggleOptionEnabled(Key);
		}
		virtual void SetToggleOption(const TCHAR* Key, bool Value) override
		{
			GetInsightsSettings().SetAndSaveTimingViewToggleOption(GetName(), Key, Value);
		}

	private:
		FInsightsSettings& GetInsightsSettings()
		{
			return UE::Insights::FInsightsManager::Get()->GetSettings();
		}
		const FInsightsSettings& GetInsightsSettings() const
		{
			return UE::Insights::FInsightsManager::Get()->GetSettings();
		}

	private:
		FTimingViewSettings& Settings;
	};
	
	class FTimingViewLocalSettings : public ITimingViewSettings
    {
    public:
    	FTimingViewLocalSettings() {}
    	virtual ~FTimingViewLocalSettings() = default;

    	virtual bool IsToggleOptionEnabled(const TCHAR* Key) const override              { return Settings.IsToggleOptionEnabled(Key); }
    	virtual void SetToggleOption(const TCHAR* Key, bool Value) override              { Settings.SetToggleOption(Key, Value); }

    private:
    	FTimingViewSettings Settings;
    };
	
public:
	FContextSwitchesSharedState(TimingProfiler::STimingView* InTimingView);
	virtual ~FContextSwitchesSharedState() = default;

	//////////////////////////////////////////////////
	// ITimingViewExtender interface

	virtual void OnBeginSession(Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(Timing::ITimingViewSession& InSession) override;
	virtual void Tick(Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;
	virtual void ExtendCpuTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;
	virtual void AddQuickFindFilters(TSharedPtr<FFilterConfigurator> FilterConfigurator) override;

	//////////////////////////////////////////////////

	static TSharedPtr<TimingProfiler::STimingView> GetTimingView();

	bool AreContextSwitchesAvailable() const;

	bool AreCpuCoreTracksVisible() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowCpuCoreTracks); }
	void ShowCpuCoreTracks() { SetCpuCoreTracksVisible(true); }
	void HideCpuCoreTracks() { SetCpuCoreTracksVisible(false); }
	void ToggleCpuCoreTracks() { SetCpuCoreTracksVisible(!AreCpuCoreTracksVisible()); }
	void SetCpuCoreTracksVisible(bool bOnOff);

	bool AreContextSwitchesVisible() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowContextSwitches); }
	void ShowContextSwitches() { SetContextSwitchesVisible(true); }
	void HideContextSwitches() { SetContextSwitchesVisible(false); }
	void ToggleContextSwitches() { SetContextSwitchesVisible(!AreContextSwitchesVisible()); }
	void SetContextSwitchesVisible(bool bOnOff);

	bool AreOverlaysVisible() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowContextSwitchOverlays); }
	void ShowOverlays() { SetOverlaysVisible(true); }
	void HideOverlays() { SetOverlaysVisible(false); }
	void ToggleOverlays() { SetOverlaysVisible(!AreOverlaysVisible()); }
	void SetOverlaysVisible(bool bOnOff);

	bool AreExtendedLinesVisible() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowContextSwitchExtendedLines); }
	void ShowExtendedLines() { SetExtendedLinesVisible(true); }
	void HideExtendedLines() { SetExtendedLinesVisible(false); }
	void ToggleExtendedLines() { SetExtendedLinesVisible(!AreExtendedLinesVisible()); }
	void SetExtendedLinesVisible(bool bOnOff);

	bool AreNonTargetProcessEventsVisible() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowNonTargetProcessEvents); }
	void ShowNonTargetProcessEvents() { SetNonTargetProcessEventsVisible(true); }
	void HideNonTargetProcessEvents() { SetNonTargetProcessEventsVisible(false); }
	void ToggleNonTargetProcessEvents() { SetNonTargetProcessEventsVisible(!AreNonTargetProcessEventsVisible()); }
	void SetNonTargetProcessEventsVisible(bool bOnOff);

	void SetTargetTimingEvent(const TSharedPtr<const ITimingEvent> InEvent)
	{
		TargetTimingEvent = InEvent;
	}

	void AddCommands();

	void GetThreadInfo(uint32 InSystemThreadId, uint32& OutThreadId, const TCHAR*& OutThreadName) const;
	TSharedPtr<TimingProfiler::FThreadTimingTrack> GetThreadTimingTrack(uint32 ThreadId) const;
	TSharedPtr<FCpuCoreTimingTrack> GetCpuCoreTimingTrack(uint32 CoreNumber) const;

private:
	void AddCpuCoreTracks();
	void RemoveCpuCoreTracks();

	void AddContextSwitchesChildTracks();
	void RemoveContextSwitchesChildTracks();

	void BuildSubMenu(FMenuBuilder& InMenuBuilder);

	void Command_ShowCpuCoreTracks_Execute() { ToggleCpuCoreTracks(); }
	bool Command_ShowCpuCoreTracks_CanExecute() const { return AreContextSwitchesAvailable(); }
	bool Command_ShowCpuCoreTracks_IsChecked() const { return AreCpuCoreTracksVisible(); }

	void Command_ShowContextSwitches_Execute() { ToggleContextSwitches(); }
	bool Command_ShowContextSwitches_CanExecute() const { return AreContextSwitchesAvailable(); }
	bool Command_ShowContextSwitches_IsChecked() const { return AreContextSwitchesVisible(); }

	void Command_ShowOverlays_Execute() { ToggleOverlays(); }
	bool Command_ShowOverlays_CanExecute() const { return AreContextSwitchesAvailable() && AreContextSwitchesVisible(); }
	bool Command_ShowOverlays_IsChecked() const { return AreOverlaysVisible(); }

	void Command_ShowExtendedLines_Execute() { ToggleExtendedLines(); }
	bool Command_ShowExtendedLines_CanExecute() const { return AreContextSwitchesAvailable() && AreContextSwitchesVisible(); }
	bool Command_ShowExtendedLines_IsChecked() const { return AreExtendedLinesVisible(); }

	void Command_ShowNonTargetProcessEvents_Execute() { ToggleNonTargetProcessEvents(); }
	bool Command_ShowNonTargetProcessEvents_CanExecute() const { return AreContextSwitchesAvailable() && AreCpuCoreTracksVisible(); }
	bool Command_ShowNonTargetProcessEvents_IsChecked() const { return AreNonTargetProcessEventsVisible(); }

	bool IsValidCpuCoreEventSelected() const;
	bool IsValidContextSwitchEventSelected() const;

	void Command_NavigateToCpuThreadEvent_Execute();
	bool Command_NavigateToCpuThreadEvent_CanExecute() const;

	void Command_DockCpuThreadTrackToBottom_Execute();
	bool Command_DockCpuThreadTrackToBottom_CanExecute() const { return AreContextSwitchesAvailable() && AreContextSwitchesVisible() && IsValidCpuCoreEventSelected(); }

	void Command_NavigateToCpuCoreEvent_Execute();
	bool Command_NavigateToCpuCoreEvent_CanExecute() const { return AreContextSwitchesAvailable() && AreCpuCoreTracksVisible() && IsValidContextSwitchEventSelected(); }

	void Command_DockCpuCoreTrackToTop_Execute();
	bool Command_DockCpuCoreTrackToTop_CanExecute() const { return AreContextSwitchesAvailable() && AreCpuCoreTracksVisible() && IsValidContextSwitchEventSelected(); }

	void PopulateCoreEventNameSuggestionList(const FString& Text, TArray<FString>& OutSuggestions);

private:
	TMap<uint32, TSharedPtr<FCpuCoreTimingTrack>> CpuCoreTimingTracks;
	TMap<uint32, TSharedPtr<FContextSwitchesTimingTrack>> ContextSwitchesTimingTracks;

	Timing::ITimingViewSession* TimingViewSession;

	uint64 ThreadsSerial;
	uint64 CpuCoresSerial;

	bool bSyncWithProviders;

	TSharedPtr<const ITimingEvent> TargetTimingEvent;
	TSharedRef<ITimingViewSettings> TimingViewSettings;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ContextSwitches
