// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextSwitchesTimingTrack.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/ContextSwitches.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/ContextSwitches/ViewModels/ContextSwitchesSharedState.h"
#include "Insights/ContextSwitches/ViewModels/ContextSwitchTimingEvent.h"
#include "Insights/InsightsManager.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingViewDrawHelper.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Insights/Widgets/STimingView.h"

namespace UE::Insights::ContextSwitches
{

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FContextSwitchesTimingTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesTimingTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	if (const TraceServices::IContextSwitchesProvider* ContextSwitchesProvider = TraceServices::ReadContextSwitchesProvider(*Session.Get()))
	{
		TraceServices::FProviderReadScopeLock ScopedLock(*ContextSwitchesProvider);

		const FTimingTrackViewport& Viewport = Context.GetViewport();

		ContextSwitchesProvider->EnumerateContextSwitches(ThreadId, Viewport.GetStartTime(), Viewport.GetEndTime(),
			[&Builder]
			(const TraceServices::FContextSwitch& ContextSwitch)
			{
				Builder.AddEvent(ContextSwitch.Start, ContextSwitch.End, 0, *FString::Printf(TEXT("Core %d"), ContextSwitch.CoreNumber), 0, 0);
				return TraceServices::EContextSwitchEnumerationResult::Continue;
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesTimingTrack::BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	BuildDrawState(Builder, Context);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesTimingTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	DrawLineEvents(Context, 1.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesTimingTrack::DrawLineEvents(const ITimingTrackDrawContext& Context, const float OffsetY) const
{
	const FTimingViewDrawHelper& Helper = *static_cast<const FTimingViewDrawHelper*>(&Context.GetHelper());

	if ((Context.GetEventFilter().IsValid() && Context.GetEventFilter()->FilterTrack(*this)) || HasCustomFilter())
	{
		Helper.DrawFadedLineEvents(GetDrawState(), *this, OffsetY, 0.1f);

		if (UpdateFilteredDrawStateOpacity())
		{
			Helper.DrawLineEvents(GetFilteredDrawState(), *this, OffsetY);
		}
		else
		{
			Helper.DrawFadedLineEvents(GetFilteredDrawState(), *this, OffsetY, GetFilteredDrawStateOpacity());
		}
	}
	else
	{
		Helper.DrawLineEvents(GetDrawState(), *this, OffsetY);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesTimingTrack::PostDraw(const ITimingTrackDrawContext& Context) const
{
	if (SharedState.AreOverlaysVisible() || SharedState.AreExtendedLinesVisible())
	{
		float LineY1 = 0.0f;
		float LineY2 = 0.0f;
		ETimingTrackLocation LocalLocation = ETimingTrackLocation::None;
		TSharedPtr<FBaseTimingTrack> LocalParentTrack = GetParentTrack().Pin();
		if (LocalParentTrack)
		{
			LineY1 = LocalParentTrack->GetPosY();
			LineY2 = LineY1 + LocalParentTrack->GetHeight();
			LocalLocation = LocalParentTrack->GetLocation();
		}
		else
		{
			LineY1 = GetPosY();
			LineY2 = LineY1 + GetHeight();
			LocalLocation = GetLocation();
		}

		const FTimingTrackViewport& Viewport = Context.GetViewport();
		switch (LocalLocation)
		{
			case ETimingTrackLocation::Scrollable:
			{
				const float TopY = Viewport.GetPosY() + Viewport.GetTopOffset();
				if (LineY1 < TopY)
				{
					LineY1 = TopY;
				}
				const float BottomY = Viewport.GetPosY() + Viewport.GetHeight() - Viewport.GetBottomOffset();
				if (LineY2 > BottomY)
				{
					LineY2 = BottomY;
				}
				break;
			}
			case ETimingTrackLocation::TopDocked:
			{
				const float TopY = Viewport.GetPosY();
				if (LineY1 < TopY)
				{
					LineY1 = TopY;
				}
				const float BottomY = Viewport.GetPosY() + Viewport.GetTopOffset();
				if (LineY2 > BottomY)
				{
					LineY2 = BottomY;
				}
				break;
			}
			case ETimingTrackLocation::BottomDocked:
			{
				const float TopY = Viewport.GetPosY() + Viewport.GetHeight() - Viewport.GetBottomOffset();
				if (LineY1 < TopY)
				{
					LineY1 = TopY;
				}
				const float BottomY = Viewport.GetPosY() + Viewport.GetHeight();
				if (LineY2 > BottomY)
				{
					LineY2 = BottomY;
				}
				break;
			}
		}

		const float LineH = LineY2 - LineY1;
		if (LineH > 0.0f)
		{
			const FTimingViewDrawHelper& Helper = *static_cast<const FTimingViewDrawHelper*>(&Context.GetHelper());
			Helper.DrawContextSwitchMarkers(GetDrawState(), LineY1, LineH, 0.25f, SharedState.AreOverlaysVisible(), SharedState.AreExtendedLinesVisible());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FContextSwitchesTimingTrack::GetEvent(double InTime, double SecondsPerPixel, int32 Depth) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return nullptr;
	}

	TraceServices::FContextSwitch BestMatchContextSwitch;
	double Delta = 2 * SecondsPerPixel;

	if (const TraceServices::IContextSwitchesProvider* ContextSwitchesProvider = TraceServices::ReadContextSwitchesProvider(*Session.Get()))
	{
		TraceServices::FProviderReadScopeLock ScopedLock(*ContextSwitchesProvider);

		ContextSwitchesProvider->EnumerateContextSwitches(ThreadId, InTime - Delta, InTime + Delta,
			[InTime, &BestMatchContextSwitch, &Delta]
			(const TraceServices::FContextSwitch& ContextSwitch)
			{
				if (ContextSwitch.Start <= InTime && ContextSwitch.End >= InTime)
				{
					BestMatchContextSwitch = ContextSwitch;
					Delta = 0.0f;
					return TraceServices::EContextSwitchEnumerationResult::Stop;
				}

				double DeltaLeft = InTime - ContextSwitch.End;
				if (DeltaLeft >= 0 && DeltaLeft < Delta)
				{
					Delta = DeltaLeft;
					BestMatchContextSwitch = ContextSwitch;
				}

				double DeltaRight = ContextSwitch.Start - InTime;
				if (DeltaRight >= 0 && DeltaRight < Delta)
				{
					Delta = DeltaRight;
					BestMatchContextSwitch = ContextSwitch;
				}

				return TraceServices::EContextSwitchEnumerationResult::Continue;
			});
	}
	else
	{
		return nullptr;
	}

	if (Delta < 2 * SecondsPerPixel)
	{
		TSharedPtr<FContextSwitchTimingEvent> TimingEvent = MakeShared<FContextSwitchTimingEvent>(SharedThis(this), BestMatchContextSwitch.Start, BestMatchContextSwitch.End, 0);
		TimingEvent->SetCoreNumber(BestMatchContextSwitch.CoreNumber);
		return TimingEvent;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesTimingTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	if (!InTooltipEvent.CheckTrack(this) || !InTooltipEvent.Is<FContextSwitchTimingEvent>())
	{
		return;
	}

	const FContextSwitchTimingEvent& ContextSwitchEvent = InTooltipEvent.As<FContextSwitchTimingEvent>();
	InOutTooltip.AddTitle(FString::Printf(TEXT("Core %d"), ContextSwitchEvent.GetCoreNumber()));

	using namespace UE::Insights;
	InOutTooltip.AddNameValueTextLine(TEXT("Start Time:"), FormatTimeAuto(InTooltipEvent.GetStartTime(), 6));
	InOutTooltip.AddNameValueTextLine(TEXT("End Time:"), FormatTimeAuto(InTooltipEvent.GetEndTime(), 6));
	InOutTooltip.AddNameValueTextLine(TEXT("Duration:"), FormatTimeAuto(InTooltipEvent.GetDuration()));

	InOutTooltip.UpdateLayout();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FContextSwitchesTimingTrack::BuildContextMenu(FMenuBuilder& InOutMenuBuilder)
{
	if (SharedState.GetTimingView())
	{
		const TSharedPtr<const ITimingEvent> HoveredEvent = SharedState.GetTimingView()->GetHoveredEvent();
		if (HoveredEvent.IsValid() &&
			&HoveredEvent->GetTrack().Get() == this &&
			HoveredEvent->Is<FContextSwitchTimingEvent>())
		{
			const FContextSwitchTimingEvent& CpuCoreEvent = HoveredEvent->As<FContextSwitchTimingEvent>();
			const uint32 CoreNumber = CpuCoreEvent.GetCoreNumber();

			SharedState.SetTargetTimingEvent(HoveredEvent);

			FString SectionName = FString::Printf(TEXT("Core %d"), CoreNumber);
			InOutMenuBuilder.BeginSection("CpuCore", FText::FromString(SectionName));
			InOutMenuBuilder.AddMenuEntry(FContextSwitchesStateCommands::Get().Command_NavigateToCpuCoreEvent);
			InOutMenuBuilder.AddMenuEntry(FContextSwitchesStateCommands::Get().Command_DockCpuCoreTrackToTop);
			InOutMenuBuilder.EndSection();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FContextSwitchesTimingTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	TTimingEventSearch<TraceServices::FContextSwitch>::Search(

		InSearchParameters,

		// Search Predicate
		[this, &InSearchParameters]
		(TTimingEventSearch<TraceServices::FContextSwitch>::FContext& InContext)
		{
			TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
			if (Session.IsValid())
			{
				if (const TraceServices::IContextSwitchesProvider* ContextSwitchesProvider = TraceServices::ReadContextSwitchesProvider(*Session.Get()))
				{
					TraceServices::FProviderReadScopeLock ScopedLock(*ContextSwitchesProvider);
					ContextSwitchesProvider->EnumerateContextSwitches(ThreadId, InSearchParameters.StartTime, InSearchParameters.EndTime,
						[&InContext]
						(const TraceServices::FContextSwitch& ContextSwitchEvent)
						{
							InContext.Check(ContextSwitchEvent.Start, ContextSwitchEvent.End, 0, ContextSwitchEvent);
							return InContext.ShouldContinueSearching() ?
								TraceServices::EContextSwitchEnumerationResult::Continue :
								TraceServices::EContextSwitchEnumerationResult::Stop;
						});
				}
			}
		},

		// Filter Predicate
		TTimingEventSearch<TraceServices::FContextSwitch>::NoFilter,

		// Found Predicate
		[this, &FoundEvent]
		(double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const TraceServices::FContextSwitch& InEvent)
		{
			FoundEvent = MakeShared<FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth, InEvent.CoreNumber);
		},

		// Payload Matched Predicate
		TTimingEventSearch<TraceServices::FContextSwitch>::NoMatch

	); // Search

	return FoundEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ContextSwitches
