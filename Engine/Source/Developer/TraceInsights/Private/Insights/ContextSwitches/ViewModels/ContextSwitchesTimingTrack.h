// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/ViewModels/TimingEventsTrack.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

class FThreadTrackEvent;

namespace UE::Insights::ContextSwitches
{

class FContextSwitchesSharedState;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FContextSwitchesTimingTrack : public FTimingEventsTrack
{
	INSIGHTS_DECLARE_RTTI(FContextSwitchesTimingTrack, FTimingEventsTrack)

public:
	explicit FContextSwitchesTimingTrack(FContextSwitchesSharedState& InSharedState, const FString& InName, uint32 InTimelineIndex, uint32 InThreadId)
		: FTimingEventsTrack(InName)
		, SharedState(InSharedState)
		, TimelineIndex(InTimelineIndex)
		, ThreadId(InThreadId)
	{
	}

	virtual ~FContextSwitchesTimingTrack() = default;

	uint32 GetTimelineIndex() const { return TimelineIndex; }
	uint32 GetThreadId() const { return ThreadId; }

	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;

	virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	virtual void PostDraw(const ITimingTrackDrawContext& Context) const override;

	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;

	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;

	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;

protected:
	virtual const TSharedPtr<const ITimingEvent> GetEvent(double InTime, double SecondsPerPixel, int32 Depth) const override;

	void DrawLineEvents(const ITimingTrackDrawContext& Context, const float OffsetY = 1.0f) const;

private:
	FContextSwitchesSharedState& SharedState;

	uint32 TimelineIndex;
	uint32 ThreadId;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ContextSwitches
