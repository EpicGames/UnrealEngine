// Copyright Epic Games, Inc. All Rights Reserved.

#include "CpuCoreTimingTrack.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/ContextSwitches.h"
#include "TraceServices/Model/Threads.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Filter/ViewModels/FilterConfigurator.h"
#include "InsightsCore/Filter/ViewModels/Filters.h"

// TraceInsights
#include "Insights/ContextSwitches/ViewModels/ContextSwitchesSharedState.h"
#include "Insights/ContextSwitches/ViewModels/ContextSwitchTimingEvent.h"
#include "Insights/InsightsManager.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingViewDrawHelper.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Insights/Widgets/STimingView.h"

namespace UE::Insights::ContextSwitches
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FCpuCoreTimingTrackFilterContext
////////////////////////////////////////////////////////////////////////////////////////////////////

class FCpuCoreTimingTrackFilterContext : public FBaseFilterContext
{
public:
	FCpuCoreTimingTrackFilterContext(const FCpuCoreTimingTrack& InTrack)
		: Track(InTrack)
	{
	}
	virtual ~FCpuCoreTimingTrackFilterContext() = default;

	virtual bool HasData(int32 Key) const override
	{
		switch (Key)
		{
		case static_cast<int32>(EFilterField::StartTime):
		case static_cast<int32>(EFilterField::EndTime):
		case static_cast<int32>(EFilterField::Duration):
		case static_cast<int32>(EFilterField::TrackName):
		case static_cast<int32>(EFilterField::CoreEventName):
			return true;
		default:
			return false;
		}
	}

	virtual EFilterDataType GetDataType(int32 Key) const override
	{
		switch (Key)
		{
		case static_cast<int32>(EFilterField::StartTime):
		case static_cast<int32>(EFilterField::EndTime):
		case static_cast<int32>(EFilterField::Duration):
			return EFilterDataType::Double;
		case static_cast<int32>(EFilterField::TrackName):
			return EFilterDataType::String;
		case static_cast<int32>(EFilterField::CoreEventName): // exposed as SystemThreadId
			return EFilterDataType::Int64;
		default:
			return EFilterDataType::Void;
		}
	}

	virtual int64 GetDataAsInt64(int32 Key) const override
	{
		if (Key == static_cast<int32>(EFilterField::CoreEventName))
		{
			return int64(SystemThreadId);
		}
		return 0;
	}

	virtual double GetDataAsDouble(int32 Key) const override
	{
		if (Key == static_cast<int32>(EFilterField::StartTime))
		{
			return EventStartTime;
		}
		if (Key == static_cast<int32>(EFilterField::EndTime))
		{
			return EventEndTime;
		}
		if (Key == static_cast<int32>(EFilterField::Duration))
		{
			return EventEndTime - EventStartTime;
		}
		return 0.0;
	}

	virtual const FString& GetDataAsString(int32 Key) const override
	{
		if (Key == static_cast<int32>(EFilterField::TrackName))
		{
			return Track.GetName();
		}
		return EmptyString;
	}

	void SetEvent(const TraceServices::FCpuCoreEvent& Event)
	{
		EventStartTime = Event.Start;
		EventEndTime = Event.End;
		SystemThreadId = Event.SystemThreadId;
	}

private:
	const FCpuCoreTimingTrack& Track;
	double EventStartTime = 0.0;
	double EventEndTime = 0.0;
	uint32 SystemThreadId = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FCpuCoreTimingTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FCpuCoreTimingTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

FCpuCoreTimingTrack::FCpuCoreTimingTrack(FContextSwitchesSharedState& InSharedState, const FString& InName, uint32 InCoreNumber)
	: FTimingEventsTrack(InName)
	, SharedState(InSharedState)
	, CoreNumber(InCoreNumber)
	, NonTargetProcessEventsDrawState(MakeShared<FTimingEventsTrackDrawState>())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuCoreTimingTrack::Reset()
{
	NonTargetProcessEventsMaxDepth = -1;
	NonTargetProcessEventsDrawState->Reset();

	FTimingEventsTrack::Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuCoreTimingTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	if (const TraceServices::IContextSwitchesProvider* ContextSwitchesProvider = TraceServices::ReadContextSwitchesProvider(*Session.Get()))
	{
		TraceServices::FProviderReadScopeLock ScopedLock(*ContextSwitchesProvider);

		const FTimingTrackViewport& Viewport = Context.GetViewport();
		FTimingEventsTrackDrawStateBuilder NonTargetProcessEventsBuilder(*NonTargetProcessEventsDrawState, Viewport, Context.GetGeometry().Scale);
		bool bShowNonTargetProcessEvents = SharedState.AreNonTargetProcessEventsVisible();

		ContextSwitchesProvider->EnumerateCpuCoreEvents(CoreNumber, Viewport.GetStartTime(), Viewport.GetEndTime(),
			[this, &Builder, &NonTargetProcessEventsBuilder, ContextSwitchesProvider, bShowNonTargetProcessEvents]
			(const TraceServices::FCpuCoreEvent& CpuCoreEvent)
			{
				uint32 ThreadId = 0;
				if (ContextSwitchesProvider->GetThreadId(CpuCoreEvent.SystemThreadId, ThreadId))
				{
					this->AddCoreTimingEvent(Builder, CpuCoreEvent);
				}
				else if (bShowNonTargetProcessEvents)
				{
					this->AddCoreTimingEvent(NonTargetProcessEventsBuilder, CpuCoreEvent);
				}

				return TraceServices::EContextSwitchEnumerationResult::Continue;
			});

		NonTargetProcessEventsMaxDepth = NonTargetProcessEventsBuilder.GetMaxDepth();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuCoreTimingTrack::BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	const TSharedPtr<ITimingEventFilter> EventFilterPtr = Context.GetEventFilter();
	if (EventFilterPtr.IsValid() && EventFilterPtr->FilterTrack(*this))
	{
		bool bFilterOnlyByEventType = false; // this is the most often use case, so the below code tries to optimize it
		uint64 FilterEventType = 0;
		if (EventFilterPtr->Is<FTimingEventFilterByEventType>())
		{
			bFilterOnlyByEventType = true;
			const FTimingEventFilterByEventType& EventFilter = EventFilterPtr->As<FTimingEventFilterByEventType>();
			FilterEventType = EventFilter.GetEventType();
		}

		if (bFilterOnlyByEventType)
		{
			if (const TraceServices::IContextSwitchesProvider* ContextSwitchesProvider = TraceServices::ReadContextSwitchesProvider(*Session.Get()))
			{
				const FTimingTrackViewport& Viewport = Context.GetViewport();
				bool bShowNonTargetProcessEvents = SharedState.AreNonTargetProcessEventsVisible();

				TraceServices::FProviderReadScopeLock ScopedLock(*ContextSwitchesProvider);
				ContextSwitchesProvider->EnumerateCpuCoreEvents(CoreNumber, Viewport.GetStartTime(), Viewport.GetEndTime(),
					[this, &Builder, ContextSwitchesProvider, FilterEventType, bShowNonTargetProcessEvents]
					(const TraceServices::FCpuCoreEvent& CpuCoreEvent)
					{
						if (CpuCoreEvent.SystemThreadId != FilterEventType)
						{
							return TraceServices::EContextSwitchEnumerationResult::Continue;
						}

						uint32 ThreadId;
						if (!bShowNonTargetProcessEvents && !ContextSwitchesProvider->GetThreadId(CpuCoreEvent.SystemThreadId, ThreadId))
						{
							return TraceServices::EContextSwitchEnumerationResult::Continue;
						}

						this->AddCoreTimingEvent(Builder, CpuCoreEvent);

						return TraceServices::EContextSwitchEnumerationResult::Continue;
					});
			}
		}
		else // generic filter
		{
			//TODO: if (EventFilterPtr->FilterEvent(TimingEvent))
		}
	}

	TSharedPtr<TimingProfiler::STimingView> TimingView = SharedState.GetTimingView();
	if (!TimingView)
	{
		return;
	}

	TSharedPtr<FFilterConfigurator> FilterConfigurator = TimingView->GetFilterConfigurator();
	if (FilterConfigurator.IsValid() && !FilterConfigurator->IsEmpty()) // HasCustomFilter()
	{
		if (const TraceServices::IContextSwitchesProvider* ContextSwitchesProvider = TraceServices::ReadContextSwitchesProvider(*Session.Get()))
		{
			FCpuCoreTimingTrackFilterContext FilterContext(*this);
			const FTimingTrackViewport& Viewport = Context.GetViewport();
			bool bShowNonTargetProcessEvents = SharedState.AreNonTargetProcessEventsVisible();

			TraceServices::FProviderReadScopeLock ScopedLock(*ContextSwitchesProvider);
			ContextSwitchesProvider->EnumerateCpuCoreEvents(CoreNumber, Viewport.GetStartTime(), Viewport.GetEndTime(),
				[this, &Builder, ContextSwitchesProvider, &FilterContext, &FilterConfigurator, bShowNonTargetProcessEvents]
				(const TraceServices::FCpuCoreEvent& CpuCoreEvent)
				{
					FilterContext.SetEvent(CpuCoreEvent);
					if (!FilterConfigurator->ApplyFilters(FilterContext))
					{
						return TraceServices::EContextSwitchEnumerationResult::Continue;
					}

					uint32 ThreadId;
					if (!bShowNonTargetProcessEvents && !ContextSwitchesProvider->GetThreadId(CpuCoreEvent.SystemThreadId, ThreadId))
					{
						return TraceServices::EContextSwitchEnumerationResult::Continue;
					}

					this->AddCoreTimingEvent(Builder, CpuCoreEvent);

					return TraceServices::EContextSwitchEnumerationResult::Continue;
				});
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuCoreTimingTrack::AddCoreTimingEvent(ITimingEventsTrackDrawStateBuilder& Builder, const TraceServices::FCpuCoreEvent& CpuCoreEvent)
{
	constexpr uint32 EventDepth = 0;
	const uint32 EventColor = FTimingEvent::ComputeEventColor(CpuCoreEvent.SystemThreadId);
	const uint32 SystemThreadId = CpuCoreEvent.SystemThreadId;

	Builder.AddEvent(CpuCoreEvent.Start, CpuCoreEvent.End, EventDepth, EventColor,
		[SystemThreadId, &CpuCoreEvent, this](float Width) -> const FString
		{
			FString EventName = GetThreadName(SystemThreadId);

			const float MinWidth = static_cast<float>(EventName.Len()) * 4.0f + 32.0f;
			if (Width > MinWidth)
			{
				const double Duration = CpuCoreEvent.End - CpuCoreEvent.Start;
				FTimingEventsTrackDrawStateBuilder::AppendDurationToEventName(EventName, Duration);
			}

			return EventName;
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuCoreTimingTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	const FTimingViewDrawHelper& Helper = *static_cast<const FTimingViewDrawHelper*>(&Context.GetHelper());

	Helper.DrawFadedEvents(*NonTargetProcessEventsDrawState, *this, 1.0f, 0.1f);

	FTimingEventsTrack::Draw(Context);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuCoreTimingTrack::PostDraw(const ITimingTrackDrawContext& Context) const
{
	for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
	{
		Track->PostDraw(Context);
	}

	const TSharedPtr<const ITimingEvent> SelectedEventPtr = Context.GetSelectedEvent();
	if (SelectedEventPtr.IsValid() &&
		SelectedEventPtr->CheckTrack(this) &&
		SelectedEventPtr->Is<FTimingEvent>())
	{
		const FTimingEvent& SelectedEvent = SelectedEventPtr->As<FTimingEvent>();
		const ITimingViewDrawHelper& Helper = Context.GetHelper();

		FString Str = FString::Printf(TEXT("%s (Duration.: %s)"),
			*GetThreadName(static_cast<uint32>(SelectedEvent.GetType())),
			*FormatTimeAuto(SelectedEvent.GetDuration()));

		DrawSelectedEventInfo(Str, Context.GetViewport(), Context.GetDrawContext(), Helper.GetWhiteBrush(), Helper.GetEventFont());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FCpuCoreTimingTrack::GetEvent(double InTime, double SecondsPerPixel, int32 Depth) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return nullptr;
	}

	TraceServices::FCpuCoreEvent BestMatchEvent;
	double Delta = 2 * SecondsPerPixel;

	if (const TraceServices::IContextSwitchesProvider* ContextSwitchesProvider = TraceServices::ReadContextSwitchesProvider(*Session.Get()))
	{
		TraceServices::FProviderReadScopeLock ScopedLock(*ContextSwitchesProvider);

		bool bShowNonTargetProcessEvents = SharedState.AreNonTargetProcessEventsVisible();

		auto ShouldShowEvent =
			[bShowNonTargetProcessEvents, ContextSwitchesProvider]
			(const TraceServices::FCpuCoreEvent& CpuCoreEvent)
			{
				uint32 ThreadId;
				if (!bShowNonTargetProcessEvents)
				{
					return ContextSwitchesProvider->GetThreadId(CpuCoreEvent.SystemThreadId, ThreadId);
				}

				return true;
			};

		ContextSwitchesProvider->EnumerateCpuCoreEvents(CoreNumber, InTime - Delta, InTime + Delta,
			[InTime, &BestMatchEvent, &Delta, ShouldShowEvent]
			(const TraceServices::FCpuCoreEvent& CpuCoreEvent)
			{
				if (CpuCoreEvent.Start <= InTime && CpuCoreEvent.End >= InTime && ShouldShowEvent(CpuCoreEvent))
				{
					BestMatchEvent = CpuCoreEvent;
					Delta = 0.0f;
					return TraceServices::EContextSwitchEnumerationResult::Stop;
				}

				double DeltaLeft = InTime - CpuCoreEvent.End;
				if (DeltaLeft >= 0 && DeltaLeft < Delta && ShouldShowEvent(CpuCoreEvent))
				{
					Delta = DeltaLeft;
					BestMatchEvent = CpuCoreEvent;
				}

				double DeltaRight = CpuCoreEvent.Start - InTime;
				if (DeltaRight >= 0 && DeltaRight < Delta && ShouldShowEvent(CpuCoreEvent))
				{
					Delta = DeltaRight;
					BestMatchEvent = CpuCoreEvent;
				}

				return TraceServices::EContextSwitchEnumerationResult::Continue;
			});
	}
	else
	{
		return nullptr;
	}

	if (Delta < 2 * SecondsPerPixel)
	{
		TSharedPtr<FCpuCoreTimingEvent> TimingEvent = MakeShared<FCpuCoreTimingEvent>(SharedThis(this), BestMatchEvent.Start, BestMatchEvent.End, 0);
		TimingEvent->SetSystemThreadId(BestMatchEvent.SystemThreadId);
		return TimingEvent;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FCpuCoreTimingTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindCpuCoreTimingEvent(InSearchParameters,
		[this, &FoundEvent]
		(double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const TraceServices::FCpuCoreEvent& InEvent)
		{
			FoundEvent = MakeShared<FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth, InEvent.SystemThreadId);
		});

	return FoundEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FCpuCoreTimingTrack::FindCpuCoreTimingEvent(
	const FTimingEventSearchParameters& InParameters,
	TFunctionRef<void(double, double, uint32, const TraceServices::FCpuCoreEvent&)> InFoundPredicate) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return false;
	}

	FCpuCoreTimingTrackFilterContext FilterContext(*this);

	return TTimingEventSearch<TraceServices::FCpuCoreEvent>::Search(

		InParameters,

		// Search Predicate
		[this, &InParameters]
		(TTimingEventSearch<TraceServices::FCpuCoreEvent>::FContext& InContext)
		{
			TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
			if (Session.IsValid())
			{
				if (const TraceServices::IContextSwitchesProvider* ContextSwitchesProvider = TraceServices::ReadContextSwitchesProvider(*Session.Get()))
				{
					TraceServices::FProviderReadScopeLock ScopedLock(*ContextSwitchesProvider);

					auto Callback =
						[&InContext]
						(const TraceServices::FCpuCoreEvent& CpuCoreEvent)
						{
							InContext.Check(CpuCoreEvent.Start, CpuCoreEvent.End, 0, CpuCoreEvent);
							return InContext.ShouldContinueSearching() ?
								TraceServices::EContextSwitchEnumerationResult::Continue :
								TraceServices::EContextSwitchEnumerationResult::Stop;
						};

					if (InContext.GetParameters().SearchDirection == FTimingEventSearchParameters::ESearchDirection::Forward)
					{
						ContextSwitchesProvider->EnumerateCpuCoreEvents(CoreNumber, InParameters.StartTime, InParameters.EndTime, Callback);
					}
					else
					{
						ContextSwitchesProvider->EnumerateCpuCoreEventsBackwards(CoreNumber, InParameters.EndTime, InParameters.StartTime, Callback);
					}
				}
			}
		},

		// Filter Predicate
		[&FilterContext, &InParameters]
		(double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FCpuCoreEvent& Event)
		{
			if (!InParameters.FilterExecutor.IsValid())
			{
				return true;
			}

			FilterContext.SetEvent(Event);
			return InParameters.FilterExecutor->ApplyFilters(FilterContext);
		},

		// Found Predicate
		InFoundPredicate,

		// Payload Matched Predicate
		TTimingEventSearch<TraceServices::FCpuCoreEvent>::NoMatch

	); // Search
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuCoreTimingTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	InOutTooltip.ResetContent();

	if (!InTooltipEvent.CheckTrack(this) || !InTooltipEvent.Is<FCpuCoreTimingEvent>())
	{
		return;
	}

	const FCpuCoreTimingEvent& CpuCoreEvent = InTooltipEvent.As<FCpuCoreTimingEvent>();

	const uint32 SystemThreadId = CpuCoreEvent.GetSystemThreadId();
	uint32 ThreadId;
	const TCHAR* ThreadName;
	SharedState.GetThreadInfo(SystemThreadId, ThreadId, ThreadName);

	InOutTooltip.AddTitle(ThreadName);
	InOutTooltip.AddNameValueTextLine(TEXT("System Thread Id:"), FString::Printf(TEXT("%d"), SystemThreadId));
	if (ThreadId != ~0)
	{
		InOutTooltip.AddNameValueTextLine(TEXT("Thread Id:"), FString::Printf(TEXT("%d"), ThreadId));
	}

	using namespace UE::Insights;
	InOutTooltip.AddNameValueTextLine(TEXT("Start Time:"), FormatTimeAuto(InTooltipEvent.GetStartTime(), 6));
	InOutTooltip.AddNameValueTextLine(TEXT("End Time:"), FormatTimeAuto(InTooltipEvent.GetEndTime(), 6));
	InOutTooltip.AddNameValueTextLine(TEXT("Duration:"), FormatTimeAuto(InTooltipEvent.GetDuration()));

	InOutTooltip.UpdateLayout();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuCoreTimingTrack::BuildContextMenu(FMenuBuilder& InOutMenuBuilder)
{
	if (SharedState.GetTimingView())
	{
		const TSharedPtr<const ITimingEvent> HoveredEvent = SharedState.GetTimingView()->GetHoveredEvent();
		if (HoveredEvent.IsValid() &&
			&HoveredEvent->GetTrack().Get() == this &&
			HoveredEvent->Is<FCpuCoreTimingEvent>())
		{
			const FCpuCoreTimingEvent& CpuCoreEvent = HoveredEvent->As<FCpuCoreTimingEvent>();
			const uint32 SystemThreadId = CpuCoreEvent.GetSystemThreadId();
			uint32 ThreadId;
			const TCHAR* ThreadName;
			SharedState.GetThreadInfo(SystemThreadId, ThreadId, ThreadName);

			if (ThreadId != ~0)
			{
				SharedState.SetTargetTimingEvent(HoveredEvent);
			}
			else
			{
				SharedState.SetTargetTimingEvent(nullptr);
			}

			InOutMenuBuilder.BeginSection("CpuThread", FText::FromString(ThreadName));
			InOutMenuBuilder.AddMenuEntry(FContextSwitchesStateCommands::Get().Command_NavigateToCpuThreadEvent);
			InOutMenuBuilder.AddMenuEntry(FContextSwitchesStateCommands::Get().Command_DockCpuThreadTrackToBottom);
			InOutMenuBuilder.EndSection();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FCpuCoreTimingTrack::HasCustomFilter() const
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = SharedState.GetTimingView();
	if (!TimingView)
	{
		return false;
	}

	TSharedPtr<FFilterConfigurator> FilterConfigurator = TimingView->GetFilterConfigurator();
	return FilterConfigurator.IsValid() && !FilterConfigurator->IsEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FCpuCoreTimingTrack::GetThreadName(uint32 InSystemThreadId) const
{
	const TCHAR* ThreadName = nullptr;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		uint32 ThreadId = 0;
		bool bIsValidThreadId = false;
		if (const TraceServices::IContextSwitchesProvider* ContextSwitchesProvider = TraceServices::ReadContextSwitchesProvider(*Session.Get()))
		{
			TraceServices::FProviderReadScopeLock ScopedLock(*ContextSwitchesProvider);
			bIsValidThreadId = ContextSwitchesProvider->GetThreadId(InSystemThreadId, ThreadId);
		}
		if (bIsValidThreadId)
		{
			const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(*Session.Get());
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			ThreadName = ThreadProvider.GetThreadName(ThreadId);
		}
	}

	if (ThreadName)
	{
		return FString(ThreadName);
	}
	else
	{
		return FString::Printf(TEXT("Unknown %d"), InSystemThreadId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ContextSwitches
