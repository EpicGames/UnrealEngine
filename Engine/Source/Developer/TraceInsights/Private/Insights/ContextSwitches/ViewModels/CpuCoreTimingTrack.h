// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/ViewModels/TimingEventsTrack.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

class FThreadTrackEvent;
struct FTimingEventsTrackDrawState;

namespace TraceServices
{
	struct FCpuCoreEvent;
};

namespace UE::Insights::ContextSwitches
{

class FContextSwitchesSharedState;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FCpuCoreTimingTrack : public FTimingEventsTrack
{
	INSIGHTS_DECLARE_RTTI(FCpuCoreTimingTrack, FTimingEventsTrack)

public:
	explicit FCpuCoreTimingTrack(FContextSwitchesSharedState& InSharedState, const FString& InName, uint32 InCoreNumber);

	virtual ~FCpuCoreTimingTrack() = default;

	uint32 GetCoreNumber() const { return CoreNumber; }

	virtual void Reset() override;
	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	virtual void PostDraw(const ITimingTrackDrawContext& Context) const override;
	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;

protected:
	virtual const TSharedPtr<const ITimingEvent> GetEvent(double InTime, double SecondsPerPixel, int32 Depth) const override;
	virtual bool HasCustomFilter() const override;
	virtual int32 GetMaxDepth() const override { return NonTargetProcessEventsMaxDepth; }

private:
	bool FindCpuCoreTimingEvent(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const TraceServices::FCpuCoreEvent&)> InFoundPredicate) const;
	void AddCoreTimingEvent(ITimingEventsTrackDrawStateBuilder& Builder, const TraceServices::FCpuCoreEvent& CpuCoreEvent);
	FString GetThreadName(uint32 InSystemThreadId) const;

private:
	FContextSwitchesSharedState& SharedState;

	uint32 CoreNumber;

	TSharedRef<FTimingEventsTrackDrawState> NonTargetProcessEventsDrawState;

	int32 NonTargetProcessEventsMaxDepth = -1;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ContextSwitches
