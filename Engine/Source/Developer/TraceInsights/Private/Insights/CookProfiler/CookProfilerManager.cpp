// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookProfilerManager.h"

#include "Features/IModularFeatures.h"
#include "Framework/Docking/TabManager.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/CookProfilerProvider.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/CookProfiler/ViewModels/PackageTable.h"
#include "Insights/CookProfiler/Widgets/SPackageTableTreeView.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"

#define LOCTEXT_NAMESPACE "UE::Insights::CookProfiler"

namespace UE::Insights::CookProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

const FName FCookProfilerTabs::PackageTableTreeViewTabID(TEXT("PackageTableTreeView"));

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FCookProfilerManager> FCookProfilerManager::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FCookProfilerManager> FCookProfilerManager::Get()
{
	return FCookProfilerManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FCookProfilerManager> FCookProfilerManager::CreateInstance()
{
	ensure(!FCookProfilerManager::Instance.IsValid());
	if (FCookProfilerManager::Instance.IsValid())
	{
		FCookProfilerManager::Instance.Reset();
	}

	FCookProfilerManager::Instance = MakeShared<FCookProfilerManager>(FInsightsManager::Get()->GetCommandList());

	return FCookProfilerManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FCookProfilerManager::FCookProfilerManager(TSharedRef<FUICommandList> InCommandList)
	: bIsInitialized(false)
	, bIsAvailable(false)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCookProfilerManager::Initialize(IUnrealInsightsModule& InsightsModule)
{
	ensure(!bIsInitialized);
	if (bIsInitialized)
	{
		return;
	}
	bIsInitialized = true;

	// Register tick functions.
	OnTick = FTickerDelegate::CreateSP(this, &FCookProfilerManager::Tick);
	OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(OnTick, 0.0f);

	FOnRegisterMajorTabExtensions* TimingProfilerLayoutExtension = InsightsModule.FindMajorTabLayoutExtension(FInsightsManagerTabs::TimingProfilerTabId);
	if (TimingProfilerLayoutExtension)
	{
		TimingProfilerLayoutExtension->AddRaw(this, &FCookProfilerManager::RegisterTimingProfilerLayoutExtensions);
	}

	FInsightsManager::Get()->GetSessionChangedEvent().AddSP(this, &FCookProfilerManager::OnSessionChanged);
	OnSessionChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCookProfilerManager::Shutdown()
{
	if (!bIsInitialized)
	{
		return;
	}
	bIsInitialized = false;

	FInsightsManager::Get()->GetSessionChangedEvent().RemoveAll(this);

	// Unregister tick function.
	FTSTicker::RemoveTicker(OnTickHandle);

	FCookProfilerManager::Instance.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FCookProfilerManager::~FCookProfilerManager()
{
	ensure(!bIsInitialized);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCookProfilerManager::RegisterMajorTabs(IUnrealInsightsModule& InsightsModule)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCookProfilerManager::UnregisterMajorTabs()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FCookProfilerManager::Tick(float DeltaTime)
{
	// Check if session has Cook events (to spawn the Packages tab), but not too often.
	if (!bIsAvailable && AvailabilityCheck.Tick())
	{
		bool bShouldBeAvailable = false;

		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			const TraceServices::ICookProfilerProvider* CookProvider = TraceServices::ReadCookProfilerProvider(*Session.Get());
			if (CookProvider)
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*CookProvider);

				TSharedPtr<FTabManager> TabManagerShared = TimingTabManager.Pin();
				if (CookProvider && CookProvider->GetNumPackages() > 0 && TabManagerShared.IsValid())
				{
					bIsAvailable = true;
					TabManagerShared->TryInvokeTab(FCookProfilerTabs::PackageTableTreeViewTabID, true);
				}

				if (Session->IsAnalysisComplete())
				{
					// Never check again during this session.
					AvailabilityCheck.Disable();
				}
			}
		}
		else
		{
			// Do not check again until the next session changed event (see OnSessionChanged).
			AvailabilityCheck.Disable();
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCookProfilerManager::OnSessionChanged()
{
	bIsAvailable = false;
	if (FInsightsManager::Get()->GetSession().IsValid())
	{
		AvailabilityCheck.Enable(0.5);
	}
	else
	{
		AvailabilityCheck.Disable();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCookProfilerManager::RegisterTimingProfilerLayoutExtensions(FInsightsMajorTabExtender& InOutExtender)
{
	TimingTabManager = InOutExtender.GetTabManager();

	FMinorTabConfig& MinorTabConfig = InOutExtender.AddMinorTabConfig();
	MinorTabConfig.TabId = FCookProfilerTabs::PackageTableTreeViewTabID;
	MinorTabConfig.TabLabel = LOCTEXT("PackageTableTreeViewTabTitle", "Packages");
	MinorTabConfig.TabTooltip = LOCTEXT("PackageTableTreeViewTabTitleTooltip", "Opens the Packages Tree View tab, that allows cook profiling.");
	MinorTabConfig.TabIcon = FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.PackagesView");
	MinorTabConfig.OnSpawnTab = FOnSpawnTab::CreateRaw(this, &FCookProfilerManager::SpawnTab_PackageTableTreeView);
	MinorTabConfig.CanSpawnTab = FCanSpawnTab::CreateRaw(this, &FCookProfilerManager::CanSpawnTab_PackageTableTreeView);
	MinorTabConfig.WorkspaceGroup = InOutExtender.GetWorkspaceGroup();

	InOutExtender.GetLayoutExtender().ExtendLayout(FTimingProfilerTabs::StatsCountersID
		, ELayoutExtensionPosition::After
		, FTabManager::FTab(FCookProfilerTabs::PackageTableTreeViewTabID, ETabState::ClosedTab));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> FCookProfilerManager::SpawnTab_PackageTableTreeView(const FSpawnTabArgs& Args)
{
	TSharedRef<FPackageTable> PackageTable = MakeShared<FPackageTable>();
	PackageTable->Reset();
	PackageTable->SetDisplayName(LOCTEXT("PackageTableTreeViewTabTitle", "Packages"));

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(PackageTableTreeView, SPackageTableTreeView, PackageTable)
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FCookProfilerManager::OnPackageTableTreeViewTabClosed));

	return DockTab;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FCookProfilerManager::CanSpawnTab_PackageTableTreeView(const FSpawnTabArgs& Args)
{
	return bIsAvailable;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCookProfilerManager::OnPackageTableTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	if (PackageTableTreeView.IsValid())
	{
		PackageTableTreeView->OnClose();
	}

	PackageTableTreeView.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCookProfilerManager::OnWindowClosedEvent()
{
	if (PackageTableTreeView.IsValid())
	{
		PackageTableTreeView->OnClose();
	}

	TSharedPtr<FTabManager> TimingTabManagerSharedPtr = TimingTabManager.Pin();

	if (TimingTabManagerSharedPtr.IsValid())
	{
		TSharedPtr<SDockTab> Tab = TimingTabManagerSharedPtr->FindExistingLiveTab(FCookProfilerTabs::PackageTableTreeViewTabID);
		if (Tab.IsValid())
		{
			Tab->RequestCloseTab();
			Tab->RemoveTabFromParent();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::CookProfiler

#undef LOCTEXT_NAMESPACE
