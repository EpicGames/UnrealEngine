// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Ticker.h"
#include "Framework/Commands/UICommandList.h"
#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Common/AvailabilityCheck.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"

namespace UE::Insights::CookProfiler
{

class SPackageTableTreeView;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FCookProfilerTabs
{
	// Tab identifiers
	static const FName PackageTableTreeViewTabID;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * This class manages the Cooking Profiler state and settings.
 */
class FCookProfilerManager : public TSharedFromThis<FCookProfilerManager>, public IInsightsComponent
{
public:
	/** Creates the Cook Profiler manager, only one instance can exist. */
	FCookProfilerManager(TSharedRef<FUICommandList> InCommandList);

	/** Virtual destructor. */
	virtual ~FCookProfilerManager();

	/** Creates an instance of the CookProfiler manager. */
	static TSharedPtr<FCookProfilerManager> CreateInstance();

	/**
	 * @return the global instance of the Cook Profiler manager.
	 * This is an internal singleton and cannot be used outside TraceInsights.
	 * For external use:
	 *     IUnrealInsightsModule& Module = FModuleManager::Get().LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	 *     Module.GetCookProfilerManager();
	 */
	static TSharedPtr<FCookProfilerManager> Get();

	//////////////////////////////////////////////////
	// IInsightsComponent

	virtual void Initialize(IUnrealInsightsModule& InsightsModule) override;
	virtual void Shutdown() override;
	virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) override;
	virtual void UnregisterMajorTabs() override;
	virtual void OnWindowClosedEvent() override;

	//////////////////////////////////////////////////

	TSharedRef<SDockTab> SpawnTab_PackageTableTreeView(const FSpawnTabArgs& Args);
	bool CanSpawnTab_PackageTableTreeView(const FSpawnTabArgs& Args);
	void OnPackageTableTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	//////////////////////////////////////////////////

	bool IsAvailable() { return bIsAvailable; }

	void OnSessionChanged();

private:
	/** Updates this manager, done through FCoreTicker. */
	bool Tick(float DeltaTime);

	void RegisterTimingProfilerLayoutExtensions(FInsightsMajorTabExtender& InOutExtender);

private:
	bool bIsInitialized;
	bool bIsAvailable;
	FAvailabilityCheck AvailabilityCheck;

	/** The delegate to be invoked when this manager ticks. */
	FTickerDelegate OnTick;

	/** Handle to the registered OnTick. */
	FTSTicker::FDelegateHandle OnTickHandle;

	/** A shared pointer to the global instance of the Cook Profiler manager. */
	static TSharedPtr<FCookProfilerManager> Instance;

	TWeakPtr<FTabManager> TimingTabManager;

	TSharedPtr<SPackageTableTreeView> PackageTableTreeView;
};

} // namespace UE::Insights::CookProfiler
