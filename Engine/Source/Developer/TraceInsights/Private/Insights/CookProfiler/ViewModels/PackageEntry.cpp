// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageEntry.h"

// TraceServices
#include "TraceServices/Model/CookProfilerProvider.h"

namespace UE::Insights::CookProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FPackageEntry::FPackageEntry(const TraceServices::FPackageData& PackageData)
	: Id(PackageData.Id)
	, Name(PackageData.Name)
	, LoadTimeIncl(PackageData.LoadTimeIncl)
	, LoadTimeExcl(PackageData.LoadTimeExcl)
	, SaveTimeIncl(PackageData.SaveTimeIncl)
	, SaveTimeExcl(PackageData.SaveTimeExcl)
	, BeginCacheForCookedPlatformDataIncl(PackageData.BeginCacheForCookedPlatformDataIncl)
	, BeginCacheForCookedPlatformDataExcl(PackageData.BeginCacheForCookedPlatformDataExcl)
	, IsCachedCookedPlatformDataLoadedIncl(PackageData.IsCachedCookedPlatformDataLoadedIncl)
	, IsCachedCookedPlatformDataLoadedExcl(PackageData.IsCachedCookedPlatformDataLoadedExcl)
	, AssetClass(PackageData.AssetClass)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::CookProfiler
