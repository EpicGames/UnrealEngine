// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace TraceServices { struct FPackageData; }

namespace UE::Insights::CookProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FPackageEntry
{
	friend class SPackageTableTreeView;

public:
	FPackageEntry(const TraceServices::FPackageData& PackageData);
	~FPackageEntry() = default;

	uint64 GetId() const { return Id; }
	const TCHAR* GetName() const { return Name; }

	const double GetLoadTimeIncl() const { return LoadTimeIncl; }
	const double GetLoadTimeExcl() const { return LoadTimeExcl; }

	const double GetSaveTimeIncl() const { return SaveTimeIncl; }
	const double GetSaveTimeExcl() const { return SaveTimeExcl; }

	const double GetBeginCacheForCookedPlatformDataIncl() const { return BeginCacheForCookedPlatformDataIncl; }
	const double GetBeginCacheForCookedPlatformDataExcl() const { return BeginCacheForCookedPlatformDataExcl; }

	const double GetIsCachedCookedPlatformDataLoadedIncl() const { return IsCachedCookedPlatformDataLoadedIncl; }
	const double GetIsCachedCookedPlatformDataLoadedExcl() const { return IsCachedCookedPlatformDataLoadedExcl; }

	const TCHAR* GetAssetClass() const { return AssetClass; }

private:
	uint64 Id;
	const TCHAR* Name;
	double LoadTimeIncl;
	double LoadTimeExcl;

	double SaveTimeIncl;
	double SaveTimeExcl;

	double BeginCacheForCookedPlatformDataIncl;
	double BeginCacheForCookedPlatformDataExcl;

	double IsCachedCookedPlatformDataLoadedIncl;
	double IsCachedCookedPlatformDataLoadedExcl;

	const TCHAR* AssetClass;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::CookProfiler
