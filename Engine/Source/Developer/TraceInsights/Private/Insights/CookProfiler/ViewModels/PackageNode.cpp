// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageNode.h"

namespace UE::Insights::CookProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FPackageNode)

} // namespace UE::Insights::CookProfiler
