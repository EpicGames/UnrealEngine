// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"

// TraceInsights
#include "Insights/CookProfiler/ViewModels/PackageTable.h"
#include "Insights/CookProfiler/ViewModels/PackageEntry.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::CookProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FPackageNode;

/** Type definition for shared pointers to instances of FPackageNode. */
typedef TSharedPtr<class FPackageNode> FPackageNodePtr;

/** Type definition for shared references to instances of FPackageNode. */
typedef TSharedRef<class FPackageNode> FPackageNodeRef;

/** Type definition for shared references to const instances of FPackageNode. */
typedef TSharedRef<const class FPackageNode> FPackageNodeRefConst;

/** Type definition for weak references to instances of FPackageNode. */
typedef TWeakPtr<class FPackageNode> FPackageNodeWeak;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class used to store information about a package node (used in the SPackageTableTreeView).
 */
class FPackageNode : public FTableTreeNode
{
	INSIGHTS_DECLARE_RTTI(FPackageNode, FTableTreeNode)

public:
	/** Initialization constructor for the Package node.
	 * InPackageName needs to be a persistent string (ex: from the session's string store).
	 */
	explicit FPackageNode(TWeakPtr<FPackageTable> InParentTable, int32 InRowIndex, const TCHAR* InPackageName)
		: FTableTreeNode(InParentTable, InRowIndex)
		, PackageName(InPackageName)
	{
	}

	virtual const FText GetDisplayName() const override
	{
		return FText::FromString(PackageName);
	}

	FPackageTable& GetPackageTableChecked() const
	{
		const TSharedPtr<FTable>& TablePin = GetParentTable().Pin();
		check(TablePin.IsValid());
		return *StaticCastSharedPtr<FPackageTable>(TablePin);
	}

	bool IsValidPackage() const { return GetPackageTableChecked().IsValidRowIndex(GetRowIndex()); }
	const FPackageEntry* GetPackage() const { return GetPackageTableChecked().GetPackage(GetRowIndex()); }
	const FPackageEntry& GetPackageChecked() const { return GetPackageTableChecked().GetPackageChecked(GetRowIndex()); }

private:
	const TCHAR* PackageName = nullptr; // persistent string (from the session's string store)
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::CookProfiler
