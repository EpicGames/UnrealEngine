// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageTable.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/CookProfiler/ViewModels/PackageNode.h"
#include "Insights/CookProfiler/ViewModels/PackageTable.h"

#define LOCTEXT_NAMESPACE "UE::Insights::CookProfiler::FPackageTable"

namespace UE::Insights::CookProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Column identifiers

const FName FPackageTableColumns::IdColumnId(TEXT("Id"));
const FName FPackageTableColumns::NameColumnId(TEXT("Name"));

const FName FPackageTableColumns::LoadTimeInclColumnId(TEXT("LoadTimeIncl"));
const FName FPackageTableColumns::LoadTimeExclColumnId(TEXT("LoadTimeExcl"));

const FName FPackageTableColumns::SaveTimeInclColumnId(TEXT("SaveTimeIncl"));
const FName FPackageTableColumns::SaveTimeExclColumnId(TEXT("SaveTimeExcl"));

const FName FPackageTableColumns::BeginCacheForCookedPlatformDataTimeInclColumnId(TEXT("BeginCacheForCookedPlatformDataTimeIncl"));
const FName FPackageTableColumns::BeginCacheForCookedPlatformDataTimeExclColumnId(TEXT("BeginCacheForCookedPlatformDataTimeExcl"));

const FName FPackageTableColumns::GetIsCachedCookedPlatformDataLoadedInclColumnId(TEXT("GetIsCachedCookedPlatformDataLoadedIncl"));
const FName FPackageTableColumns::GetIsCachedCookedPlatformDataLoadedExclColumnId(TEXT("GetIsCachedCookedPlatformDataLoadedExcl"));

const FName FPackageTableColumns::PackageAssetClassColumnId(TEXT("AssetClass"));

////////////////////////////////////////////////////////////////////////////////////////////////////

typedef FTableCellValue (*PackageFieldGetter) (const FTableColumn&, const FPackageEntry&);

template<PackageFieldGetter Getter>
class FPackageColumnValueGetter : public FTableCellValueGetter
{
public:
	virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
	{
		if (Node.IsGroup())
		{
			const FTableTreeNode& NodePtr = static_cast<const FTableTreeNode&>(Node);
			if (NodePtr.HasAggregatedValue(Column.GetId()))
			{
				return NodePtr.GetAggregatedValue(Column.GetId());
			}
		}
		else //if (Node->Is<FackageNode>())
		{
			const FPackageNode& PackageNode = static_cast<const FPackageNode&>(Node);
			const FPackageEntry* Package = PackageNode.GetPackage();
			if (Package)
			{
				return Getter(Column, *Package);
			}
		}

		return TOptional<FTableCellValue>();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct DefaultPackageFieldGetterFuncts
{
	static FTableCellValue GetId(const FTableColumn& Column, const FPackageEntry& Package) { return FTableCellValue((int64)Package.GetId());	}
	static FTableCellValue GetName(const FTableColumn& Column, const FPackageEntry& Package) { return FTableCellValue((const TCHAR*)Package.GetName());	}

	static FTableCellValue GetLoadTimeIncl(const FTableColumn& Column, const FPackageEntry& Package) { return FTableCellValue((double)Package.GetLoadTimeIncl());	}
	static FTableCellValue GetLoadTimeExcl(const FTableColumn& Column, const FPackageEntry& Package) { return FTableCellValue((double)Package.GetLoadTimeExcl());	}

	static FTableCellValue GetSaveTimeIncl(const FTableColumn& Column, const FPackageEntry& Package) { return FTableCellValue((double)Package.GetSaveTimeIncl());	}
	static FTableCellValue GetSaveTimeExcl(const FTableColumn& Column, const FPackageEntry& Package) { return FTableCellValue((double)Package.GetSaveTimeExcl());	}

	static FTableCellValue GetBeginCacheForCookedPlatformDataIncl(const FTableColumn& Column, const FPackageEntry& Package) { return FTableCellValue((double)Package.GetBeginCacheForCookedPlatformDataIncl());	}
	static FTableCellValue GetBeginCacheForCookedPlatformDataExcl(const FTableColumn& Column, const FPackageEntry& Package) { return FTableCellValue((double)Package.GetBeginCacheForCookedPlatformDataExcl());	}

	static FTableCellValue GetIsCachedCookedPlatformDataLoadedIncl(const FTableColumn& Column, const FPackageEntry& Package) { return FTableCellValue((double)Package.GetIsCachedCookedPlatformDataLoadedIncl());	}
	static FTableCellValue GetIsCachedCookedPlatformDataLoadedExcl(const FTableColumn& Column, const FPackageEntry& Package) { return FTableCellValue((double)Package.GetIsCachedCookedPlatformDataLoadedExcl());	}

	static FTableCellValue GetAssetClass(const FTableColumn& Column, const FPackageEntry& Package) { return FTableCellValue((const TCHAR*)Package.GetAssetClass());	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FPackageTable
////////////////////////////////////////////////////////////////////////////////////////////////////

FPackageTable::FPackageTable()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FPackageTable::~FPackageTable()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FPackageTable::Reset()
{
	FTable::Reset();

	AddDefaultColumns();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FPackageTable::AddDefaultColumns()
{
	using namespace UE::Insights;

	//////////////////////////////////////////////////
	// Hierarchy Column
	{
		const int32 HierarchyColumnIndex = -1;
		const TCHAR* HierarchyColumnName = nullptr;
		AddHierarchyColumn(HierarchyColumnIndex, HierarchyColumnName);

		const TSharedRef<FTableColumn>& ColumnRef = GetColumns()[0];
		ColumnRef->SetInitialWidth(200.0f);
		ColumnRef->SetShortName(LOCTEXT("PackageColumnName", "Hierarchy"));
		ColumnRef->SetTitleName(LOCTEXT("PackageColumnTitle", "Package Hierarchy"));
		ColumnRef->SetDescription(LOCTEXT("PackageColumnDesc", "Hierarchy of the package's tree"));
	}

	int32 ColumnIndex = 0;

	//////////////////////////////////////////////////
	// Id Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FPackageTableColumns::IdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("CreatedTimestampColumnName", "Id"));
		Column.SetTitleName(LOCTEXT("CreatedTimestampColumnTitle", "Id"));
		Column.SetDescription(LOCTEXT("CreatedTimestampColumnDesc", "The id of the package."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(80.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPackageColumnValueGetter<DefaultPackageFieldGetterFuncts::GetId>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Inclusive Load Time Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FPackageTableColumns::LoadTimeInclColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("LoadTimeInclColumnName", "I. Load Time"));
		Column.SetTitleName(LOCTEXT("LoadTimeInclColumnTitle", "Inclusive Load Time"));
		Column.SetDescription(LOCTEXT("LoadTimeInclColumnDesc", "The inclusive time it took to load the package."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPackageColumnValueGetter<DefaultPackageFieldGetterFuncts::GetLoadTimeIncl>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Exclusive Load Time Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FPackageTableColumns::LoadTimeExclColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("LoadTimeExclColumnName", "E. Load Time"));
		Column.SetTitleName(LOCTEXT("LoadTimeExclColumnTitle", "Exclusive Load Time"));
		Column.SetDescription(LOCTEXT("LoadTimeExclColumnDesc", "The exclusive time it took to load the package."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPackageColumnValueGetter<DefaultPackageFieldGetterFuncts::GetLoadTimeExcl>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Inclusive Save Time Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FPackageTableColumns::SaveTimeInclColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SaveTimeInclColumnName", "I. Save Time"));
		Column.SetTitleName(LOCTEXT("SaveTimeInclColumnTitle", "Inclusive Save Time"));
		Column.SetDescription(LOCTEXT("SaveTimeInclColumnDesc", "The inclusive time it took to save the package."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPackageColumnValueGetter<DefaultPackageFieldGetterFuncts::GetSaveTimeIncl>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Exclusive Save Time Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FPackageTableColumns::SaveTimeExclColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SaveTimeExclColumnName", "E. Save Time"));
		Column.SetTitleName(LOCTEXT("SaveTimeExclColumnTitle", "Exclusive Save Time"));
		Column.SetDescription(LOCTEXT("SaveTimeEnclColumnDesc", "The exclusive time it took to save the package."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPackageColumnValueGetter<DefaultPackageFieldGetterFuncts::GetSaveTimeExcl>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Inclusive BeginCacheForCookedPlatformData Time Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FPackageTableColumns::BeginCacheForCookedPlatformDataTimeInclColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("BeginCacheForCookedPlatformDataInclColumnName", "I. Begin Cache Time"));
		Column.SetTitleName(LOCTEXT("BeginCacheForCookedPlatformDataInclColumnTitle", "Inclusive BeginCacheForCookedPlatformData"));
		Column.SetDescription(LOCTEXT("BeginCacheForCookedPlatformDataInclColumnDesc", "The total inclusive time spent in the BeginCacheForCookedPlatformData function for the package."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPackageColumnValueGetter<DefaultPackageFieldGetterFuncts::GetBeginCacheForCookedPlatformDataIncl>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Exclusive BeginCacheForCookedPlatformData Time Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FPackageTableColumns::BeginCacheForCookedPlatformDataTimeExclColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("BeginCacheForCookedPlatformDataExclColumnName", "E. Begin Cache Time"));
		Column.SetTitleName(LOCTEXT("BeginCacheForCookedPlatformDataExclColumnTitle", "Exclusive BeginCacheForCookedPlatformData"));
		Column.SetDescription(LOCTEXT("BeginCacheForCookedPlatformDataExclColumnDesc", "The total exclusive time spent in the BeginCacheForCookedPlatformData function for the package."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPackageColumnValueGetter<DefaultPackageFieldGetterFuncts::GetBeginCacheForCookedPlatformDataExcl>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Inclusive IsCachedCookedPlatformDataLoaded Time Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FPackageTableColumns::GetIsCachedCookedPlatformDataLoadedInclColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("GetIsCachedCookedPlatformDataLoadedInclColumnName", "I. IsCachedCooked"));
		Column.SetTitleName(LOCTEXT("GetIsCachedCookedPlatformDataLoadedInclColumnTitle", "Inclusive IsCachedCookedPlatformDataLoaded"));
		Column.SetDescription(LOCTEXT("GetIsCachedCookedPlatformDataLoadedInclColumnDesc", "The total inclusive time spent in the IsCachedCookedPlatformDataLoaded function for the package."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPackageColumnValueGetter<DefaultPackageFieldGetterFuncts::GetIsCachedCookedPlatformDataLoadedIncl>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Exclusive IsCachedCookedPlatformDataLoaded Time Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FPackageTableColumns::GetIsCachedCookedPlatformDataLoadedExclColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("GetIsCachedCookedPlatformDataLoadedExclColumnName", "E. IsCachedCooked"));
		Column.SetTitleName(LOCTEXT("GetIsCachedCookedPlatformDataLoadedExclColumnTitle", "Exclusive IsCachedCookedPlatformDataLoaded"));
		Column.SetDescription(LOCTEXT("GetIsCachedCookedPlatformDataLoadedExclColumnDesc", "The total exclusive time spent in the IsCachedCookedPlatformDataLoaded function for the package."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPackageColumnValueGetter<DefaultPackageFieldGetterFuncts::GetIsCachedCookedPlatformDataLoadedExcl>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Asset Class Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FPackageTableColumns::PackageAssetClassColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("AssetClassColumnName", "Asset Class"));
		Column.SetTitleName(LOCTEXT("AssetClassTitle", "Asset Class"));
		Column.SetDescription(LOCTEXT("AssetClassColumnDesc", "The class of the most significant asset in the package."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(200.0f);

		Column.SetDataType(ETableCellDataType::CString);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPackageColumnValueGetter<DefaultPackageFieldGetterFuncts::GetAssetClass>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Package Name Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FPackageTableColumns::NameColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("PackageNameColumnName", "Package Name"));
		Column.SetTitleName(LOCTEXT("PackageNameTitle", "Package Name"));
		Column.SetDescription(LOCTEXT("PackageNameColumnDesc", "The name of the package."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(400.0f);

		Column.SetDataType(ETableCellDataType::CString);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPackageColumnValueGetter<DefaultPackageFieldGetterFuncts::GetName>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::CookProfiler

#undef LOCTEXT_NAMESPACE
