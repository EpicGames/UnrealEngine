// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "UObject/NameTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/Table.h"

// TraceInsights
#include "Insights/CookProfiler/ViewModels/PackageEntry.h"

namespace UE::Insights { class FTableColumn; }

namespace UE::Insights::CookProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

// Column identifiers
struct FPackageTableColumns
{
	static const FName IdColumnId;
	static const FName NameColumnId;

	static const FName LoadTimeInclColumnId;
	static const FName LoadTimeExclColumnId;

	static const FName SaveTimeInclColumnId;
	static const FName SaveTimeExclColumnId;

	static const FName BeginCacheForCookedPlatformDataTimeInclColumnId;
	static const FName BeginCacheForCookedPlatformDataTimeExclColumnId;

	static const FName GetIsCachedCookedPlatformDataLoadedInclColumnId;
	static const FName GetIsCachedCookedPlatformDataLoadedExclColumnId;

	static const FName PackageAssetClassColumnId;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FPackageTable : public FTable
{
public:
	FPackageTable();
	virtual ~FPackageTable();

	virtual void Reset();

	TArray<FPackageEntry>& GetPackageEntries() { return PackageEntries; }
	const TArray<FPackageEntry>& GetPackageEntries() const { return PackageEntries; }

	bool IsValidRowIndex(int32 InIndex) const { return InIndex >= 0 && InIndex < PackageEntries.Num(); }
	const FPackageEntry* GetPackage(int32 InIndex) const { return IsValidRowIndex(InIndex) ? &PackageEntries[InIndex] : nullptr; }
	const FPackageEntry& GetPackageChecked(int32 InIndex) const { check(IsValidRowIndex(InIndex)); return PackageEntries[InIndex]; }

private:
	void AddDefaultColumns();

private:
	TArray<FPackageEntry> PackageEntries;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::CookProfiler
