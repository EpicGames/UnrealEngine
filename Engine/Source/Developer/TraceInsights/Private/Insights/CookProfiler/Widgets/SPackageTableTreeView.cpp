// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPackageTableTreeView.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/STextComboBox.h"

// TraceServices
#include "Common/ProviderLock.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/CookProfiler/CookProfilerManager.h"
#include "Insights/CookProfiler/ViewModels/PackageEntry.h"
#include "Insights/CookProfiler/ViewModels/PackageNode.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/Widgets/STimingView.h"

#include <limits>

#define LOCTEXT_NAMESPACE "UE::Insights::CookProfiler::SPackageTableTreeView"

namespace UE::Insights::CookProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FPackageTableTreeViewCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

class FPackageTableTreeViewCommands : public TCommands<FPackageTableTreeViewCommands>
{
public:
	FPackageTableTreeViewCommands()
	: TCommands<FPackageTableTreeViewCommands>(
		TEXT("PackageTableTreeViewCommands"),
		NSLOCTEXT("Contexts", "PackageTableTreeViewCommands", "Insights - Package Table Tree View"),
		NAME_None,
		FInsightsStyle::GetStyleSetName())
	{
	}

	virtual ~FPackageTableTreeViewCommands()
	{
	}

	virtual void RegisterCommands() override
	{
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// SPackageTableTreeView
////////////////////////////////////////////////////////////////////////////////////////////////////

SPackageTableTreeView::SPackageTableTreeView()
{
	bRunInAsyncMode = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SPackageTableTreeView::~SPackageTableTreeView()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPackageTableTreeView::Construct(const FArguments& InArgs, TSharedPtr<FPackageTable> InTablePtr)
{
	ConstructWidget(InTablePtr);

	AddCommmands();

	ViewPreset_OnSelectionChanged(AvailableViewPresets[0], ESelectInfo::Direct);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPackageTableTreeView::ExtendMenu(FMenuBuilder& MenuBuilder)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPackageTableTreeView::AddCommmands()
{
	FPackageTableTreeViewCommands::Register();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPackageTableTreeView::Reset()
{
	STableTreeView::Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPackageTableTreeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	STableTreeView::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (!bIsUpdateRunning)
	{
		RebuildTree(false);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPackageTableTreeView::RebuildTree(bool bResync)
{
	if (bDataLoaded == true)
	{
		return;
	}

	if (!Session.IsValid())
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

	if (!Session->IsAnalysisComplete())
	{
		return;
	}

	TSharedPtr<FPackageTable> PackageTable = GetPackageTable();
	TArray<FPackageEntry>& Packages = PackageTable->GetPackageEntries();
	Packages.Empty();
	TableRowNodes.Empty();

	const TraceServices::ICookProfilerProvider* CookProvider = TraceServices::ReadCookProfilerProvider(*Session.Get());

	if (CookProvider)
	{
		TraceServices::FProviderReadScopeLock ProviderReadScope(*CookProvider);

		TArray64<TraceServices::FPackageData> PackageAggregation;
		CookProvider->CreateAggregation(PackageAggregation);
		const uint32 NumPackages = CookProvider->GetNumPackages();
		Packages.Reserve(NumPackages);
		TableRowNodes.Reserve(NumPackages);

		TArray<FTableTreeNodePtr>* Nodes = &TableRowNodes;
		for (const TraceServices::FPackageData& Package : PackageAggregation)
		{
			Packages.Emplace(Package);
			uint32 Index = static_cast<uint32>(Packages.Num() - 1);
			FPackageNodePtr NodePtr = MakeShared<FPackageNode>(PackageTable, Index, Package.Name);
			Nodes->Add(NodePtr);
		}
	}

	bDataLoaded = true;

	UpdateTree();
	TreeView->RebuildList();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SPackageTableTreeView::IsRunning() const
{
	return STableTreeView::IsRunning();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double SPackageTableTreeView::GetAllOperationsDuration()
{
	return STableTreeView::GetAllOperationsDuration();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SPackageTableTreeView::GetCurrentOperationName() const
{
	return STableTreeView::GetCurrentOperationName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> SPackageTableTreeView::ConstructToolbar()
{
	TSharedPtr<SHorizontalBox> Box = SNew(SHorizontalBox);
	ConstructViewPreset(Box, 128.0f);
	return Box;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPackageTableTreeView::InternalCreateGroupings()
{
	STableTreeView::InternalCreateGroupings();

	AvailableGroupings.RemoveAll(
		[](TSharedPtr<FTreeNodeGrouping>& Grouping)
		{
			if (Grouping->Is<FTreeNodeGroupingByUniqueValue>())
			{
				const FName ColumnId = Grouping->As<FTreeNodeGroupingByUniqueValue>().GetColumnId();
				if (ColumnId == FPackageTableColumns::BeginCacheForCookedPlatformDataTimeInclColumnId ||
					ColumnId == FPackageTableColumns::BeginCacheForCookedPlatformDataTimeExclColumnId ||
					ColumnId == FPackageTableColumns::GetIsCachedCookedPlatformDataLoadedInclColumnId ||
					ColumnId == FPackageTableColumns::GetIsCachedCookedPlatformDataLoadedExclColumnId ||
					ColumnId == FPackageTableColumns::SaveTimeInclColumnId ||
					ColumnId == FPackageTableColumns::SaveTimeExclColumnId ||
					ColumnId == FPackageTableColumns::IdColumnId ||
					ColumnId == FPackageTableColumns::NameColumnId ||
					ColumnId == FPackageTableColumns::LoadTimeInclColumnId ||
					ColumnId == FPackageTableColumns::LoadTimeExclColumnId)
				{
					return true;
				}
			}
			else if (Grouping->Is<FTreeNodeGroupingByPathBreakdown>())
			{
				const FName ColumnId = Grouping->As<FTreeNodeGroupingByPathBreakdown>().GetColumnId();
				if (ColumnId == FPackageTableColumns::PackageAssetClassColumnId)
				{
					return true;
				}
			}
			return false;
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPackageTableTreeView::TreeView_OnMouseButtonDoubleClick(FTableTreeNodePtr TreeNode)
{
	STableTreeView::TreeView_OnMouseButtonDoubleClick(TreeNode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPackageTableTreeView::InitAvailableViewPresets()
{
	//////////////////////////////////////////////////
	// Default View

	class FDefaultViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("Default_PresetName", "Default");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("Default_PresetToolTip", "Default View\nConfigure the tree view to show default packages info.");
		}
		virtual FName GetSortColumn() const override
		{
			return FTable::GetHierarchyColumnId();
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Ascending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(), true, 500.0f });
			InOutConfigSet.Add({ FPackageTableColumns::IdColumnId, true, 80.0f });
			InOutConfigSet.Add({ FPackageTableColumns::LoadTimeExclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::SaveTimeExclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::BeginCacheForCookedPlatformDataTimeExclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::GetIsCachedCookedPlatformDataLoadedExclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::PackageAssetClassColumnId, true, 200.0f });
			InOutConfigSet.Add({ FPackageTableColumns::NameColumnId, false, 400.0f });
			InOutConfigSet.Add({ FPackageTableColumns::LoadTimeInclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::SaveTimeInclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::BeginCacheForCookedPlatformDataTimeInclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::GetIsCachedCookedPlatformDataLoadedInclColumnId, true, 100.0f });
		}
	};
	AvailableViewPresets.Add(MakeShared<FDefaultViewPreset>());

	//////////////////////////////////////////////////
	// Package Path Breakdown

	class FPackagePathViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("PackagePath_PresetName", "Package Path");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("PackagePath_PresetToolTip", "Configure the tree view to show the packages grouped by package path.");
		}
		virtual FName GetSortColumn() const override
		{
			return FTable::GetHierarchyColumnId();
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Ascending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);

			const TSharedPtr<FTreeNodeGrouping>* PackagePathGrouping = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FTreeNodeGroupingByPathBreakdown>() &&
						Grouping->As<FTreeNodeGroupingByPathBreakdown>().GetColumnId() == FPackageTableColumns::NameColumnId;
				});

			if (PackagePathGrouping)
			{
				InOutCurrentGroupings.Add(*PackagePathGrouping);
			}
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(), true, 500.0f });
			InOutConfigSet.Add({ FPackageTableColumns::IdColumnId, true, 80.0f });
			InOutConfigSet.Add({ FPackageTableColumns::LoadTimeExclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::SaveTimeExclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::BeginCacheForCookedPlatformDataTimeExclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::GetIsCachedCookedPlatformDataLoadedExclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::PackageAssetClassColumnId, true, 200.0f });
			InOutConfigSet.Add({ FPackageTableColumns::NameColumnId, false, 400.0f });
			InOutConfigSet.Add({ FPackageTableColumns::LoadTimeInclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::SaveTimeInclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::BeginCacheForCookedPlatformDataTimeInclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::GetIsCachedCookedPlatformDataLoadedInclColumnId, true, 100.0f });
		}
	};
	AvailableViewPresets.Add(MakeShared<FPackagePathViewPreset>());

	//////////////////////////////////////////////////
	// Asset Class Breakdown

	class FAssetClassViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("AssetClass_PresetName", "Asset Class");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("AssetClass_PresetToolTip", "Configure the tree view to show the packages grouped by their most important asset class.");
		}
		virtual FName GetSortColumn() const override
		{
			return FTable::GetHierarchyColumnId();
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Ascending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);

			const TSharedPtr<FTreeNodeGrouping>* PackagePathGrouping = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FTreeNodeGroupingByUniqueValue>() &&
						Grouping->As<FTreeNodeGroupingByUniqueValue>().GetColumnId() == FPackageTableColumns::PackageAssetClassColumnId;
				});

			if (PackagePathGrouping)
			{
				InOutCurrentGroupings.Add(*PackagePathGrouping);
			}
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(), true, 300.0f });
			InOutConfigSet.Add({ FPackageTableColumns::IdColumnId, true, 80.0f });
			InOutConfigSet.Add({ FPackageTableColumns::LoadTimeExclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::SaveTimeExclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::BeginCacheForCookedPlatformDataTimeExclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::GetIsCachedCookedPlatformDataLoadedExclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::NameColumnId, true, 400.0f });
			InOutConfigSet.Add({ FPackageTableColumns::PackageAssetClassColumnId, false, 200.0f });
			InOutConfigSet.Add({ FPackageTableColumns::LoadTimeInclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::SaveTimeInclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::BeginCacheForCookedPlatformDataTimeInclColumnId, true, 100.0f });
			InOutConfigSet.Add({ FPackageTableColumns::GetIsCachedCookedPlatformDataLoadedInclColumnId, true, 100.0f });

		}
	};
	AvailableViewPresets.Add(MakeShared<FAssetClassViewPreset>());

	SelectedViewPreset = AvailableViewPresets[0];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPackageTableTreeView::UpdateBannerText()
{
	if (!bDataLoaded)
	{
		TreeViewBannerText = LOCTEXT("DataWillLoad", "Package data will load when session analysis is complete.");
	}
	else
	{
		STableTreeView::UpdateBannerText();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::CookProfiler

#undef LOCTEXT_NAMESPACE
