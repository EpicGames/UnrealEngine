// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceServices
#include "TraceServices/Model/CookProfilerProvider.h"

// TraceInsights
#include "Insights/CookProfiler/ViewModels/PackageTable.h"
#include "Insights/Table/Widgets/SSessionTableTreeView.h"

class FMenuBuilder;

namespace UE::Insights::CookProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class SPackageTableTreeView : public SSessionTableTreeView
{
public:
	/** Default constructor. */
	SPackageTableTreeView();

	/** Virtual destructor. */
	virtual ~SPackageTableTreeView();

	SLATE_BEGIN_ARGS(SPackageTableTreeView) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<FPackageTable> InTablePtr);

	virtual TSharedPtr<SWidget> ConstructToolbar() override;

	TSharedPtr<FPackageTable> GetPackageTable() { return StaticCastSharedPtr<FPackageTable>(GetTable()); }
	const TSharedPtr<FPackageTable> GetPackageTable() const { return StaticCastSharedPtr<FPackageTable>(GetTable()); }

	virtual void Reset();

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/**
	 * Rebuilds the tree (if necessary).
	 * @param bResync - If true, it forces a resync even if the list did not changed since last sync.
	 */
	virtual void RebuildTree(bool bResync);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// IAsyncOperationStatusProvider implementation

	virtual bool IsRunning() const override;
	virtual double GetAllOperationsDuration() override;
	virtual FText GetCurrentOperationName() const override;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	virtual void TreeView_OnMouseButtonDoubleClick(FTableTreeNodePtr TreeNode) override;

protected:
	virtual void InternalCreateGroupings() override;

	virtual void ExtendMenu(FMenuBuilder& MenuBuilder) override;

	virtual void UpdateBannerText() override;

private:
	void AddCommmands();

	virtual void InitAvailableViewPresets() override;

private:
	bool bDataLoaded = false;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::CookProfiler
