// Copyright Epic Games, Inc. All Rights Reserved.

#include "TableImportTool.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableImporter.h"

// TraceInsights
#include "Insights/InsightsManager.h"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTableImportTool> FTableImportTool::Instance;

////////////////////////////////////////////////////////////////////////////////////////////////////

FTableImportTool::FTableImportTool()
	: TableImporter(MakeShared<FTableImporter>(FInsightsManager::Get()->GetLogListingName()))
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTableImportTool::~FTableImportTool()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTableImportTool> FTableImportTool::CreateInstance()
{
	ensure(Instance == nullptr);
	Instance = MakeShared<FTableImportTool>();

	return Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTableImportTool> FTableImportTool::Get()
{
	return Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableImportTool::StartImportProcess()
{
	TableImporter->StartImportProcess();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableImportTool::ImportFile(const FString& Filename)
{
	TableImporter->ImportFile(Filename);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableImportTool::StartDiffProcess()
{
	TableImporter->StartDiffProcess();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableImportTool::DiffFiles(const FString& FilenameA, const FString& FilenameB)
{
	TableImporter->DiffFiles(FilenameA, FilenameB);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTableImportTool::OnWindowClosedEvent()
{
	TableImporter->CloseAllOpenTabs();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
