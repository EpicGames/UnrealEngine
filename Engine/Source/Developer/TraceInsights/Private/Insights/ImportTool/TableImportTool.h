// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"

namespace UE::Insights
{

class FTableImporter;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTableImportTool : public TSharedFromThis<FTableImportTool>, public IInsightsComponent
{
public:
	FTableImportTool();
	virtual ~FTableImportTool();

	static TSharedPtr<FTableImportTool> CreateInstance();
	static TSharedPtr<FTableImportTool> Get();

	// IInsightsComponent
	virtual void Initialize(IUnrealInsightsModule& InsightsModule) override {}
	virtual void Shutdown() override {}
	virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) override {}
	virtual void UnregisterMajorTabs() override {}
	virtual void OnWindowClosedEvent() override;

	void StartImportProcess();
	void ImportFile(const FString& Filename);

	void StartDiffProcess();
	void DiffFiles(const FString& FilenameA, const FString& FilenameB);

private:
	static TSharedPtr<FTableImportTool> Instance;

	TSharedRef<FTableImporter> TableImporter;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
