// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/InsightsCommands.h"

#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManagerGeneric.h"
#include "Misc/Paths.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::FInsightsCommands"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FInsightsCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsCommands::FInsightsCommands()
: TCommands<FInsightsCommands>(
	TEXT("InsightsCommands"), // Context name for fast lookup
	NSLOCTEXT("Contexts", "InsightsCommands", "Insights"), // Localized context name for displaying
	NAME_None, // Parent
	FInsightsStyle::GetStyleSetName()) // Icon Style Set
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// UI_COMMAND takes long for the compile to optimize
UE_DISABLE_OPTIMIZATION_SHIP
void FInsightsCommands::RegisterCommands()
{
	UI_COMMAND(LoadTraceFile,
		"Load...",
		"Loads profiler data from a trace file.",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Control, EKeys::L));

	UI_COMMAND(ToggleDebugInfo,
		"Debug",
		"Toggles the display of debug info.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EModifierKey::Control, EKeys::D));

	UI_COMMAND(OpenSettings,
		"Settings",
		"Opens the Unreal Insights settings.",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Control, EKeys::O));
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////
// Toggle Commands
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FInsightsActionManager, This, ToggleDebugInfo, IsDebugInfoEnabled, SetDebugInfo)

////////////////////////////////////////////////////////////////////////////////////////////////////
// LoadTraceFile
////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsActionManager::Map_LoadTraceFile()
{
	FUIAction UIAction;
	UIAction.ExecuteAction = FExecuteAction::CreateRaw(this, &FInsightsActionManager::LoadTraceFile_Execute);
	UIAction.CanExecuteAction = FCanExecuteAction::CreateRaw(this, &FInsightsActionManager::LoadTraceFile_CanExecute);

	This->CommandList->MapAction(This->GetCommands().LoadTraceFile, UIAction);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsActionManager::LoadTraceFile_Execute()
{
	//const FString ProfilingDirectory(FPaths::ConvertRelativePathToFull(*FPaths::ProfilingDir()));
	const FString ProfilingDirectory(This->GetStoreDir());

	TArray<FString> OutFiles;

	bool bOpened = false;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform != nullptr)
	{
		FSlateApplication::Get().CloseToolTip();

		bOpened = DesktopPlatform->OpenFileDialog
		(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("LoadTrace_FileDesc", "Open trace file...").ToString(),
			ProfilingDirectory,
			TEXT(""),
			LOCTEXT("LoadTrace_FileFilter", "Trace files (*.utrace)|*.utrace|All files (*.*)|*.*").ToString(),
			EFileDialogFlags::None,
			OutFiles
		);
	}

	if (bOpened == true)
	{
		if (OutFiles.Num() == 1)
		{
			This->LoadTraceFile(OutFiles[0]);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsActionManager::LoadTraceFile_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// OpenSettings
////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsActionManager::Map_OpenSettings()
{
	FUIAction UIAction;
	UIAction.ExecuteAction = FExecuteAction::CreateRaw(this, &FInsightsActionManager::OpenSettings_Execute);
	UIAction.CanExecuteAction = FCanExecuteAction::CreateRaw(this, &FInsightsActionManager::OpenSettings_CanExecute);

	This->CommandList->MapAction(This->GetCommands().OpenSettings, UIAction);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsActionManager::OpenSettings_Execute()
{
	This->OpenSettings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsActionManager::OpenSettings_CanExecute() const
{
	return !This->Settings.IsEditing();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
