// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UIAction.h"

// TraceInsights
#include "Insights/Common/CommandUtils.h"

namespace UE::Insights
{

class FInsightsManager;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class that holds all profiler commands.
 */
class FInsightsCommands : public TCommands<FInsightsCommands>
{
public:
	/** Default constructor. */
	FInsightsCommands();

	/** Initialize commands. */
	virtual void RegisterCommands() override;

public:
	/** Toggles the debug info. */
	TSharedPtr<FUICommandInfo> ToggleDebugInfo;

	/** Load profiler data from a trace file. */
	TSharedPtr<FUICommandInfo> LoadTraceFile;

	/** Open settings for the profiler manager. */
	TSharedPtr<FUICommandInfo> OpenSettings;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class that provides helper functions for the commands to avoid cluttering manager with many small functions.
 * Can't contain any variables. Directly operates on the manager instance.
 */
class FInsightsActionManager
{
	friend class FInsightsManager;

private:
	/** Private constructor. */
	FInsightsActionManager(class FInsightsManager* Instance)
		: This(Instance)
	{}

	//////////////////////////////////////////////////
	// Toggle Commands

	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleDebugInfo)

	//////////////////////////////////////////////////
	// LoadTraceFile Command

public:
	void Map_LoadTraceFile(); /**< Maps UI command info LoadTraceFile with the specified UI command list. */
protected:
	void LoadTraceFile_Execute();
	bool LoadTraceFile_CanExecute() const;

	//////////////////////////////////////////////////
	// OpenSettings Command

public:
	void Map_OpenSettings(); /**< Maps UI command info OpenSettings with the specified UI command list. */
protected:
	void OpenSettings_Execute();
	bool OpenSettings_CanExecute() const;

	//////////////////////////////////////////////////

protected:
	/** Reference to the global instance of the Insights manager. */
	FInsightsManager* This;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
