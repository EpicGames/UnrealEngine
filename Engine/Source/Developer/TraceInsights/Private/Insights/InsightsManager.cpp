// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsManager.h"

#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformProcess.h"
#include "ISessionServicesModule.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"
#include "Logging/MessageLog.h"
#include "MessageLogModule.h"
#include "Misc/CString.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

// TraceTools
#include "TraceTools/Interfaces/ITraceToolsModule.h"

// TraceAnalysis
#include "Trace/DataStream.h"
#include "Trace/SecureDataStream.h"
#include "Trace/StoreClient.h"

// TraceServices
#include "TraceServices/AnalysisService.h"
#include "TraceServices/Model/Diagnostics.h"
#include "TraceServices/Model/Modules.h"

// TraceInsightsCore
#include "InsightsCore/Common/MiscUtils.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/Common/InsightsMenuBuilder.h"
#include "Insights/InsightsStyle.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/Log.h"
#include "Insights/Tests/InsightsTestRunner.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/Widgets/SSessionInfoWindow.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "InsightsManager"

const FName FInsightsManagerTabs::SessionInfoTabId(TEXT("SessionInfo"));
const FName FInsightsManagerTabs::TimingProfilerTabId(TEXT("TimingProfiler"));
const FName FInsightsManagerTabs::LoadingProfilerTabId(TEXT("LoadingProfiler"));
const FName FInsightsManagerTabs::MemoryProfilerTabId(TEXT("MemoryProfiler"));
const FName FInsightsManagerTabs::NetworkingProfilerTabId(TEXT("NetworkingProfiler"));

const FName FInsightsManagerTabs::AutomationWindowTabId(TEXT("AutomationWindow"));
const FName FInsightsManagerTabs::MessageLogTabId(TEXT("MessageLog"));
const FName FInsightsManagerTabs::TraceControlTabId(TEXT("TraceControl"));

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FInsightsManager
////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FInsightsManager::AutoQuitMsg = TEXT("Application is closing because it was started with the AutoQuit parameter and session analysis is complete.");
const TCHAR* FInsightsManager::AutoQuitMsgOnFail = TEXT("Application is closing because it was started with the AutoQuit parameter and session analysis failed to start.");

TSharedPtr<FInsightsManager> FInsightsManager::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FInsightsManager> FInsightsManager::Get()
{
	return FInsightsManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FInsightsManager> FInsightsManager::CreateInstance(TSharedRef<TraceServices::IAnalysisService> TraceAnalysisService,
															  TSharedRef<TraceServices::IModuleService> TraceModuleService)
{
	ensure(!FInsightsManager::Instance.IsValid());
	if (FInsightsManager::Instance.IsValid())
	{
		FInsightsManager::Instance.Reset();
	}

	FInsightsManager::Instance = MakeShared<FInsightsManager>(TraceAnalysisService, TraceModuleService);

	return FInsightsManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsManager::FInsightsManager(TSharedRef<TraceServices::IAnalysisService> InTraceAnalysisService,
								   TSharedRef<TraceServices::IModuleService> InTraceModuleService)
	: LogListingName(TEXT("UnrealInsights"))
	, AnalysisLogListingName(TEXT("TraceAnalysis"))
	, AnalysisService(InTraceAnalysisService)
	, ModuleService(InTraceModuleService)
	, CommandList(new FUICommandList())
	, ActionManager(this)
	, SourceFilePathHelper(MakeShared<FSourceFilePathHelper>())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::Initialize(IUnrealInsightsModule& InsightsModule)
{
	ensure(!bIsInitialized);
	if (bIsInitialized)
	{
		return;
	}
	bIsInitialized = true;

	InsightsMenuBuilder = MakeShared<FInsightsMenuBuilder>();

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.RegisterLogListing(GetLogListingName(), LOCTEXT("UnrealInsights", "Unreal Insights"));
	MessageLogModule.RegisterLogListing(AnalysisLogListingName, LOCTEXT("TraceAnalysis", "Trace Analysis"));
	MessageLogModule.EnableMessageLogDisplay(true);

	RegisterTraceControlTab();

	// Register tick functions.
	OnTick = FTickerDelegate::CreateSP(this, &FInsightsManager::Tick);
	OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(OnTick, 0.0f);

	FInsightsCommands::Register();
	BindCommands();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::Shutdown()
{
	if (!bIsInitialized)
	{
		return;
	}
	bIsInitialized = false;

	ResetSession(false);

	FInsightsCommands::Unregister();

	// Unregister tick function.
	FTSTicker::RemoveTicker(OnTickHandle);

	// If the MessageLog module was already unloaded as part of the global Shutdown process, do not load it again.
	if (FModuleManager::Get().IsModuleLoaded("MessageLog"))
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		if (MessageLogModule.IsRegisteredLogListing(GetLogListingName()))
		{
			MessageLogModule.UnregisterLogListing(GetLogListingName());
		}
	}

	FInsightsManager::Instance.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsManager::~FInsightsManager()
{
	ensure(!bIsInitialized);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::BindCommands()
{
	ActionManager.Map_LoadTraceFile();
	ActionManager.Map_ToggleDebugInfo();
	ActionManager.Map_OpenSettings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::RegisterMajorTabs(IUnrealInsightsModule& InsightsModule)
{
	const FInsightsMajorTabConfig& SessionInfoConfig = InsightsModule.FindMajorTabConfig(FInsightsManagerTabs::SessionInfoTabId);
	if (SessionInfoConfig.bIsAvailable)
	{
		// Register tab spawner for the Session Info.
		FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(FInsightsManagerTabs::SessionInfoTabId,
			FOnSpawnTab::CreateRaw(this, &FInsightsManager::SpawnSessionInfoTab))
			.SetDisplayName(SessionInfoConfig.TabLabel.IsSet() ? SessionInfoConfig.TabLabel.GetValue() : LOCTEXT("SessionInfoTabTitle", "Session"))
			.SetTooltipText(SessionInfoConfig.TabTooltip.IsSet() ? SessionInfoConfig.TabTooltip.GetValue() : LOCTEXT("SessionInfoTooltipText", "Open the Session tab."))
			.SetIcon(SessionInfoConfig.TabIcon.IsSet() ? SessionInfoConfig.TabIcon.GetValue() : FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.SessionInfo"));

		TSharedRef<FWorkspaceItem> Group = SessionInfoConfig.WorkspaceGroup.IsValid() ? SessionInfoConfig.WorkspaceGroup.ToSharedRef() : GetInsightsMenuBuilder()->GetInsightsToolsGroup();
		TabSpawnerEntry.SetGroup(Group);
	}

#if !WITH_EDITOR
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.RegisterMessageLogSpawner(GetInsightsMenuBuilder()->GetWindowsGroup());
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::UnregisterMajorTabs()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(FInsightsManagerTabs::SessionInfoTabId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> FInsightsManager::SpawnSessionInfoTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FInsightsManager::OnSessionInfoTabClosed));

	// Create the SSessionInfoWindow widget.
	TSharedRef<SSessionInfoWindow> Window = SNew(SSessionInfoWindow, DockTab, Args.GetOwnerWindow());
	DockTab->SetContent(Window);

	AssignSessionInfoWindow(Window);

	if (!bIsMainTabSet)
	{
		FGlobalTabmanager::Get()->SetMainTab(DockTab);
		bIsMainTabSet = true;
	}

	return DockTab;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::OnSessionInfoTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	RemoveSessionInfoWindow();

	// Disable TabClosed delegate.
	TabBeingClosed->SetOnTabClosed(SDockTab::FOnTabClosedCallback());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<const TraceServices::IAnalysisSession> FInsightsManager::GetSession() const
{
	return Session;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedRef<FUICommandList> FInsightsManager::GetCommandList() const
{
	return CommandList;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FInsightsCommands& FInsightsManager::GetCommands()
{
	return FInsightsCommands::Get();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsActionManager& FInsightsManager::GetActionManager()
{
	return FInsightsManager::Instance->ActionManager;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsSettings& FInsightsManager::GetSettings()
{
	return FInsightsManager::Instance->Settings;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsManager::Tick(float DeltaTime)
{
	if (RetryLoadLastLiveSessionTimer > 0.0f)
	{
		LoadLastLiveSession(0.0f);
		if (Session)
		{
			RetryLoadLastLiveSessionTimer = 0.0f;
		}
		else
		{
			RetryLoadLastLiveSessionTimer -= DeltaTime;
		}
	}

	AutoLoadLiveSession();

	UpdateSessionDuration();

	PollAnalysisInfo();

	if (!bIsSessionInfoSet && Session.IsValid())
	{
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			const TraceServices::IDiagnosticsProvider* DiagnosticsProvider = TraceServices::ReadDiagnosticsProvider(*Session.Get());
			if (DiagnosticsProvider && DiagnosticsProvider->IsSessionInfoAvailable())
			{
				bIsSessionInfoSet = true;
				SourceFilePathHelper->InitVFSMapping(DiagnosticsProvider->GetSessionInfo().VFSPaths);
				InstanceId = DiagnosticsProvider->GetSessionInfo().InstanceId;

#if !WITH_EDITOR
				TSharedPtr<SWidget> TraceControlPtr = TraceControl.Pin();
				if (TraceControlPtr.IsValid())
				{
					FModuleManager::LoadModuleChecked<UE::TraceTools::ITraceToolsModule>("TraceTools").SetTraceControlWidgetInstanceId(TraceControlPtr.ToSharedRef(), InstanceId);
				}
#endif // !WITH_EDITOR
			}
		}

#if !WITH_EDITOR
		if (bIsSessionInfoSet)
		{
			UpdateAppTitle();
		}
#endif // !WITH_EDITOR
	}

#if !WITH_EDITOR
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();
	SourceCodeAccessor.Tick(DeltaTime);

	CheckMemoryUsage();
#endif // !WITH_EDITOR

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::UpdateSessionDuration()
{
	using namespace UE::Insights;

	if (Session.IsValid())
	{
		if (!bIsAnalysisComplete)
		{
			AnalysisStopwatch.Update();
			AnalysisDuration = AnalysisStopwatch.GetAccumulatedTime();
			AnalysisSpeedFactor = SessionDuration / AnalysisDuration;
		}

		bool bLocalIsAnalysisComplete = false;
		double LocalSessionDuration = 0.0;
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			bLocalIsAnalysisComplete = Session->IsAnalysisComplete();
			LocalSessionDuration = Session->GetDurationSeconds();
		}

		if (LocalSessionDuration != SessionDuration)
		{
			SessionDuration = LocalSessionDuration;
			AnalysisSpeedFactor = SessionDuration / AnalysisDuration;
			if (bIsAnalysisComplete)
			{
				UE_LOGF(TraceInsights, Warning, "The session duration was updated (%ls) after the analysis has been completed.",
					*FormatTimeAuto(GetSessionDuration(), 2));
			}
		}

		if (bLocalIsAnalysisComplete && !bIsAnalysisComplete)
		{
			bIsAnalysisComplete = true;
			AnalysisStopwatch.Update();
			AnalysisDuration = AnalysisStopwatch.GetAccumulatedTime();
			AnalysisSpeedFactor = SessionDuration / AnalysisDuration;
			SessionAnalysisCompletedEvent.Broadcast();

			UE_LOGF(TraceInsights, Log, "Analysis has completed in %ls (%.1fX speed; session duration: %ls).",
				*FormatTimeAuto(AnalysisDuration, 2),
				AnalysisSpeedFactor,
				*FormatTimeAuto(SessionDuration, 2));

			OnSessionAnalysisCompleted();
		}

		if (bIsAnalysisComplete && !bIsSymbolResolvingComplete)
		{
			const TraceServices::IModuleProvider* ModuleProvider = ReadModuleProvider(*Session.Get());
			if (ModuleProvider)
			{
				if (ModuleProvider->HasFinishedResolving())
				{
					bIsSymbolResolvingComplete = true;
					OnSessionSymbolResolverCompleted();
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::CheckMemoryUsage()
{
	if (Session.IsValid()) // only check if we are in "viewer mode"
	{
		constexpr double MemUsageLimitPercent = 80.0;
		constexpr double MemUsageLimitHysteresisPercent = 50.0;

		const uint64 Time = FPlatformTime::Cycles64();
		const double DurationSeconds = static_cast<double>(Time - MemUsageLimitLastTimestamp) * FPlatformTime::GetSecondsPerCycle64();
		if (DurationSeconds > 1.0) // only check once per second
		{
			MemUsageLimitLastTimestamp = Time;

			FPlatformMemoryStats Stats = FPlatformMemory::GetStats();

			constexpr double GiB = 1024.0 * 1024.0 * 1024.0;
			const double UsedGiB = (double)(Stats.TotalPhysical - Stats.AvailablePhysical) / GiB;
			const double TotalGiB = (double)(Stats.TotalPhysical) / GiB;
			const double UsedPercent = (UsedGiB * 100.0) / TotalGiB;

			if (!bMemUsageLimitHysteresis)
			{
				if (UsedPercent >= MemUsageLimitPercent)
				{
					bMemUsageLimitHysteresis = true;

					const FText MessageBoxTextFmt = LOCTEXT("MemUsageWarning_TextFmt", "High System Memory Usage Detected: {0} / {1} GiB ({2}%)!\nUnreal Insights might need more memory!");
					const FText MessageBoxText = FText::Format(MessageBoxTextFmt,
						FText::AsNumber((uint32)(UsedGiB + 0.5)),
						FText::AsNumber((uint32)(TotalGiB + 0.5)),
						FText::AsNumber((uint32)(UsedPercent + 0.5)));

					FMessageLog ReportMessageLog(GetLogListingName());
					TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning, MessageBoxText);
					ReportMessageLog.AddMessage(Message);
					ReportMessageLog.Notify();
				}
			}
			else
			{
				if (UsedPercent <= MemUsageLimitHysteresisPercent)
				{
					bMemUsageLimitHysteresis = false;
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::ResetSession(bool bNotify)
{
	if (Session.IsValid())
	{
		Session->Stop(true);
		Session.Reset();

		CurrentTraceStreamType = ETraceStreamType::Unknown;
		CurrentTraceName.Reset();
		CurrentTraceId = 0;
		CurrentDirectTracePort = 0;

		if (bNotify)
		{
			OnSessionChanged();
		}
	}

	bIsSessionInfoSet = false;
	bIsAnalysisComplete = false;
	bIsSymbolResolvingComplete = false;
	SessionDuration = 0.0;
	AnalysisStopwatch.Restart();
	AnalysisDuration = 0.0;
	AnalysisSpeedFactor = 0.0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::PollAnalysisInfo()
{
	if (Session.IsValid())
	{
		if (Session->GetNumPendingMessages())
		{
			TSharedPtr<TraceServices::IAnalysisSession> EditableSession = ConstCastSharedPtr<TraceServices::IAnalysisSession>(Session);
			TraceServices::FAnalysisSessionEditScope SessionEditScope(*EditableSession.Get());
			const auto Messages = EditableSession->DrainPendingMessages();

			FMessageLog ReportMessageLog(AnalysisLogListingName);
			for (const auto& Message : Messages)
			{
				TSharedRef<FTokenizedMessage> LogMessage = FTokenizedMessage::Create(Message.Severity, FText::FromString(Message.Message));
				ReportMessageLog.AddMessage(LogMessage);
				if (Message.Severity == EMessageSeverity::Error)
				{
					ReportMessageLog.Notify();
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::OnSessionChanged()
{
	if (Session.IsValid())
	{
		FMessageLog(AnalysisLogListingName).NewPage(FText::FromString(Session->GetName()));
	}

	SessionChangedEvent.Broadcast();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::SpawnAndActivateTabs()
{
	// Open Session Info tab.
	if (FGlobalTabmanager::Get()->HasTabSpawner(FInsightsManagerTabs::SessionInfoTabId))
	{
		FGlobalTabmanager::Get()->TryInvokeTab(FInsightsManagerTabs::SessionInfoTabId);
	}

	// Open Timing Insights tab.
	if (FGlobalTabmanager::Get()->HasTabSpawner(FInsightsManagerTabs::TimingProfilerTabId))
	{
		FGlobalTabmanager::Get()->TryInvokeTab(FInsightsManagerTabs::TimingProfilerTabId);
	}

	// Open Asset Loading Insights tab.
	if (FGlobalTabmanager::Get()->HasTabSpawner(FInsightsManagerTabs::LoadingProfilerTabId))
	{
		FGlobalTabmanager::Get()->TryInvokeTab(FInsightsManagerTabs::LoadingProfilerTabId);
	}

	// Close the existing Networking Insights tabs.
	//for (int32 ReservedId = 0; ReservedId < 10; ++ReservedId)
	{
		FName TabId = FInsightsManagerTabs::NetworkingProfilerTabId;
		//TabId.SetNumber(ReservedId);
		if (FGlobalTabmanager::Get()->HasTabSpawner(TabId))
		{
			TSharedPtr<SDockTab> NetworkingProfilerTab;
			while ((NetworkingProfilerTab = FGlobalTabmanager::Get()->FindExistingLiveTab(TabId)).IsValid())
			{
				NetworkingProfilerTab->RequestCloseTab();
			}
		}
	}

	ActivateTimingInsightsTab();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::ActivateTimingInsightsTab()
{
	// Ensure Timing Insights / Timing View is the active tab / view.
	TimingProfiler::FTimingProfilerManager::Get()->ActivateMajorTab();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsManager::ShowOpenTraceFileDialog(FString& OutTraceFile) const
{
	static FString DefaultDirectory(FPaths::ConvertRelativePathToFull(GetStoreDir()));

	TArray<FString> OutFiles;
	bool bOpened = false;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform != nullptr)
	{
		FSlateApplication::Get().CloseToolTip();

		bOpened = DesktopPlatform->OpenFileDialog
		(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("LoadTrace_FileDesc", "Open trace file...").ToString(),
			DefaultDirectory,
			TEXT(""),
			LOCTEXT("LoadTrace_FileFilter", "Trace files (*.utrace)|*.utrace|All files (*.*)|*.*").ToString(),
			EFileDialogFlags::None,
			OutFiles
		);
	}

	if (bOpened == true && OutFiles.Num() == 1)
	{
		OutTraceFile = OutFiles[0];
		DefaultDirectory = FPaths::GetPath(OutTraceFile);
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::OpenTraceFile() const
{
	FString TraceFile;
	if (ShowOpenTraceFileDialog(TraceFile))
	{
		OpenTraceFile(TraceFile);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::OpenTraceFile(const FString& InTraceFile) const
{
	FString CmdLine = TEXT("-OpenTraceFile=\"") + InTraceFile + TEXT("\"");
	FMiscUtils::OpenUnrealInsights(*CmdLine);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::AutoLoadLiveSession()
{
	if (!bIsAutoLoadLiveSessionEnabled)
	{
		return;
	}

	UE::Trace::FStoreClient* StoreClient = TraceStoreConnection.GetStoreClient();
	if (!StoreClient)
	{
		return;
	}

	uint32 AutoLoadTraceId = 0;
	{
		FScopeLock _(&TraceStoreConnection.GetStoreClientCriticalSection());
		const uint32 SessionCount = StoreClient->GetSessionCount();
		for (uint32 SessionIndex = 0; SessionIndex < SessionCount; ++SessionIndex)
		{
			const UE::Trace::FStoreClient::FSessionInfo* SessionInfo = StoreClient->GetSessionInfo(SessionIndex);
			if (SessionInfo)
			{
				const uint32 TraceId = SessionInfo->GetTraceId();
				if (TraceId != CurrentTraceId && !AutoLoadedTraceIds.Contains(TraceId))
				{
					AutoLoadTraceId = TraceId;
					break;
				}
			}
		}
	}

	if (AutoLoadTraceId != 0)
	{
		AutoLoadedTraceIds.Add(AutoLoadTraceId);
		LoadTrace(AutoLoadTraceId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::LoadLastLiveSession(float InRetryTime)
{
	ResetSession();

	UE::Trace::FStoreClient* StoreClient = TraceStoreConnection.GetStoreClient();
	if (!StoreClient)
	{
		return;
	}

	uint32 LastLiveSessionTraceId = 0;
	{
		FScopeLock _(&TraceStoreConnection.GetStoreClientCriticalSection());
		const uint32 SessionCount = StoreClient->GetSessionCount();
		if (SessionCount != 0)
		{
			const UE::Trace::FStoreClient::FSessionInfo* SessionInfo = StoreClient->GetSessionInfo(SessionCount - 1);
			if (SessionInfo)
			{
				LastLiveSessionTraceId = SessionInfo->GetTraceId();
			}
		}
	}

	if (LastLiveSessionTraceId)
	{
		LoadTrace(LastLiveSessionTraceId);
	}

	if (Session == nullptr && InRetryTime > 0)
	{
		RetryLoadLastLiveSessionTimer = InRetryTime;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::LoadTrace(uint32 InTraceId)
{
	ResetSession();

	UE::Trace::FStoreClient* StoreClient = TraceStoreConnection.GetStoreClient();
	if (!StoreClient)
	{
		if (bAutoQuit)
		{
			RequestEngineExit(AutoQuitMsgOnFail);
		}
		return;
	}

	FScopeLock StoreClientLock(&TraceStoreConnection.GetStoreClientCriticalSection());

	UE::Trace::FStoreClient::FTraceData TraceData = StoreClient->ReadTrace(InTraceId);
	if (!TraceData)
	{
		if (bAutoQuit)
		{
			RequestEngineExit(AutoQuitMsgOnFail);
		}
		return;
	}

	FString TraceName;
	const UE::Trace::FStoreClient::FTraceInfo* TraceInfo = StoreClient->GetTraceInfoById(InTraceId);
	if (TraceInfo != nullptr)
	{
		FUtf8StringView Uri = TraceInfo->GetUri();
		if (Uri.Len() > 0)
		{
			TraceName = FString(Uri);
		}
		else
		{
			// Fallback for older versions of UTS which didn't write uri
			FString StoreDirectory(StoreClient->GetStatus()->GetStoreDir());
			FString Name(TraceInfo->GetName());
			TraceName = FPaths::SetExtension(FPaths::Combine(StoreDirectory, Name), TEXT(".utrace"));
			FPaths::MakePlatformFilename(TraceName);
		}
	}

	StoreClientLock.Unlock();

	Session = AnalysisService->StartAnalysis(InTraceId, *TraceName, MoveTemp(TraceData));

	if (Session)
	{
		CurrentTraceStreamType = ETraceStreamType::TraceStore;
		CurrentTraceName = TraceName;
		CurrentTraceId = InTraceId;
		CurrentDirectTracePort = 0;
		bIsSessionInfoSet = false;
		OnSessionChanged();
	}
	else if (bAutoQuit)
	{
		RequestEngineExit(AutoQuitMsgOnFail);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::LoadTraceFile()
{
	FString TraceFile;
	if (ShowOpenTraceFileDialog(TraceFile))
	{
		LoadTraceFile(TraceFile);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::LoadTraceFile(const FString& InTraceFilename)
{
	IPlatformFile& PlatformFile = IPlatformFile::GetPlatformPhysical();
	if (!PlatformFile.FileExists(*InTraceFilename))
	{
		const uint32 TraceId = uint32(FCString::Strtoui64(*InTraceFilename, nullptr, 10));
		return LoadTrace(TraceId);
	}

	ResetSession();

	Session = AnalysisService->StartAnalysis(*InTraceFilename);

	if (Session)
	{
		CurrentTraceStreamType = ETraceStreamType::TraceFile;
		CurrentTraceName = InTraceFilename;
		CurrentTraceId = 0;
		CurrentDirectTracePort = 0;
		bIsSessionInfoSet = false;
		OnSessionChanged();
	}
	else if (bAutoQuit)
	{
		RequestEngineExit(AutoQuitMsgOnFail);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint16 FInsightsManager::ListenForDirectTrace(const TCHAR* InStreamName, uint16 InPort)
{
	ResetSession();

	TUniquePtr<UE::Trace::FDirectSocketStream> TraceData = MakeUnique<UE::Trace::FDirectSocketStream>();
	uint16 Port = TraceData->StartListening(InPort);

	constexpr uint32 TraceId = 0;
	FString TraceName = InStreamName ? InStreamName : FString::Printf(TEXT("Direct Trace Stream on Port %u"), uint32(Port));
	Session = AnalysisService->StartAnalysis(TraceId, *TraceName, MoveTemp(TraceData));

	if (Session)
	{
		CurrentTraceStreamType = ETraceStreamType::DirectTrace;
		CurrentTraceName = TraceName;
		CurrentTraceId = 0;
		CurrentDirectTracePort = Port;
		bIsSessionInfoSet = false;
		OnSessionChanged();
	}
	else if (bAutoQuit)
	{
		RequestEngineExit(AutoQuitMsgOnFail);
	}

	return Port;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::ConnectToSecureTrace(FStringView Host, uint16 Port, TConstArrayView<uint8> AuthorizationToken, FStringView CertificateFile, TConstArrayView<uint8> CertificateAuthority, FAnsiStringView Identity)
{
#if UE_TRACE_ANALYSIS_SSS_ENABLED
	ResetSession();

	TUniquePtr<UE::Trace::FSecureSocketStream> TraceData = MakeUnique<UE::Trace::FSecureSocketStream>();
	bool bConnected = false;
	if (!CertificateFile.IsEmpty())
	{
		bConnected = TraceData->ConnectWithCertificate(Host, Port, AuthorizationToken, CertificateFile);
	}
	else if (!CertificateAuthority.IsEmpty())
	{
		bConnected = TraceData->ConnectWithSelfSigned(Host, Port, AuthorizationToken, CertificateAuthority);
	}
	else
	{
		bConnected = TraceData->ConnectWithPreSharedKey(Host, Port, AuthorizationToken, Identity);
	}

	bool bSuccess = false;
	if (bConnected)
	{

		constexpr uint32 TraceId = 0;
		TStringBuilder<64> TraceName;
		TraceName << TEXT("Secure Trace Stream from ") << Host;

		Session = AnalysisService->StartAnalysis(TraceId, TraceName.ToString(), MoveTemp(TraceData));

		if (Session)
		{
			CurrentTraceStreamType = ETraceStreamType::DirectTraceSecure;
			CurrentTraceName = TraceName;
			CurrentTraceId = 0;
			CurrentDirectTracePort = Port;
			bIsSessionInfoSet = false;
			OnSessionChanged();
			bSuccess = true;
		}
	}
	else
	{
		UE_LOGF(LogCore, Error, "Failed to connect to secure trace session.");
	}

	if (!bSuccess && bAutoQuit)
	{
		RequestEngineExit(AutoQuitMsgOnFail);
	}
#endif //UE_TRACE_ANALYSIS_SSS_ENABLED
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::StartAnalysisWithStream(TUniquePtr<UE::Trace::IInDataStream>&& InStream, const TCHAR* InStreamName)
{
	ResetSession();

	constexpr uint32 TraceId = 0;
	FString TraceName = InStreamName ? InStreamName : TEXT("CustomStream");
	Session = AnalysisService->StartAnalysis(TraceId, *TraceName, MoveTemp(InStream));

	if (Session)
	{
		CurrentTraceStreamType = ETraceStreamType::Custom;
		CurrentTraceName = TraceName;
		CurrentTraceId = 0;
		CurrentDirectTracePort = 0;
		bIsSessionInfoSet = false;
		OnSessionChanged();
	}
	else if (bAutoQuit)
	{
		RequestEngineExit(AutoQuitMsgOnFail);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsManager::OnDragOver(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FExternalDragOperation> DragDropOp = DragDropEvent.GetOperationAs<FExternalDragOperation>();
	if (DragDropOp.IsValid())
	{
		if (DragDropOp->HasFiles())
		{
			const TArray<FString>& Files = DragDropOp->GetFiles();
			if (Files.Num() == 1)
			{
				const FString DraggedFileExtension = FPaths::GetExtension(Files[0], true);
				if (DraggedFileExtension == TEXT(".utrace"))
				{
					return true;
				}
			}
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsManager::OnDrop(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FExternalDragOperation> DragDropOp = DragDropEvent.GetOperationAs<FExternalDragOperation>();
	if (DragDropOp.IsValid())
	{
		if (DragDropOp->HasFiles())
		{
			// For now, only allow a single file.
			const TArray<FString>& Files = DragDropOp->GetFiles();
			if (Files.Num() == 1)
			{
				const FString DraggedFileExtension = FPaths::GetExtension(Files[0], true);
				if (DraggedFileExtension == TEXT(".utrace"))
				{
					LoadTraceFile(Files[0]);
					UpdateAppTitle();
					return true;
				}
			}
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::UpdateAppTitle()
{
#if !WITH_EDITOR || defined(__INTELLISENSE__)
	TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
	if (RootWindow)
	{
		if (CurrentTraceName.IsEmpty())
		{
			const FText AppTitle = LOCTEXT("UnrealInsightsAppName", "Unreal Insights");
			RootWindow->SetTitle(AppTitle);
		}
		else
		{
			const FString SessionName =
				(CurrentTraceStreamType == ETraceStreamType::TraceFile ||
				 CurrentTraceStreamType == ETraceStreamType::TraceStore) ?
					FPaths::GetBaseFilename(CurrentTraceName) :
					CurrentTraceName;

			bool bWasAppTitleUpdated = false;
			if (Session.IsValid())
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
				const TraceServices::IDiagnosticsProvider* DiagnosticsProvider = TraceServices::ReadDiagnosticsProvider(*Session.Get());
				if (DiagnosticsProvider && DiagnosticsProvider->IsSessionInfoAvailable())
				{
					TraceServices::FSessionInfo SessionInfo = DiagnosticsProvider->GetSessionInfo();

					const FText AppTitle = FText::Format(LOCTEXT("UnrealInsightsAppNameFmt2", "{0}{1} - {2} - {3} - {4} - {5} Unreal Insights"),
						FText::FromString(SessionName),
						!SessionInfo.Branch.IsEmpty() ? FText::FromString(TEXT(" - ") + SessionInfo.Branch) : FText::GetEmpty(),
						FText::FromString(SessionInfo.Platform),
						FText::FromString(SessionInfo.AppName),
						FText::FromString(LexToString(SessionInfo.ConfigurationType)),
						FText::FromString(LexToString(SessionInfo.TargetType)));
					RootWindow->SetTitle(AppTitle);

					bWasAppTitleUpdated = true;
				}
			}

			if (!bWasAppTitleUpdated)
			{
				const FText AppTitle = FText::Format(LOCTEXT("UnrealInsightsAppNameFmt", "{0} - Unreal Insights"), FText::FromString(SessionName));
				RootWindow->SetTitle(AppTitle);
			}
		}
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::OpenSettings()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::ScheduleCommand(const FString& InCmd)
{
	SessionAnalysisCompletedCmd = InCmd;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::OnSessionAnalysisCompleted()
{
	if (!SessionAnalysisCompletedCmd.IsEmpty())
	{
		FOutputDevice& Ar = *GLog;
		Ar.Logf(TEXT("Executing commands on analysis completed..."));
		UE::Insights::FStopwatch Stopwatch;
		Stopwatch.Start();
		IUnrealInsightsModule& TraceInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		TraceInsightsModule.Exec(*SessionAnalysisCompletedCmd, Ar);
		Stopwatch.Stop();
		Ar.Logf(TEXT("Commands executed in %.3fs."), Stopwatch.GetAccumulatedTime());
	}

#if !UE_BUILD_SHIPPING && !WITH_EDITOR
	if (FInsightsTestRunner::Get().IsValid())
	{
		// Don't quit now. Let the test runner to execute.
		bAutoQuit = false;
	}
#endif

	if (bAutoQuit && !bWaitForSymbolResolver)
	{
		bAutoQuit = false;
		RequestEngineExit(AutoQuitMsg);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::OnSessionSymbolResolverCompleted()
{
	if (bAutoQuit && bWaitForSymbolResolver)
	{
		bAutoQuit = false;
		bWaitForSymbolResolver = false;
		RequestEngineExit(AutoQuitMsg);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsManager::Exec(const TCHAR* Cmd, FOutputDevice& Ar)
{
	// @=ResponseFile
	if (Cmd != nullptr &&
		Cmd[0] == TEXT('@') &&
		Cmd[1] == TEXT('='))
	{
		HandleResponseFileCmd(Cmd + 2, Ar);
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsManager::HandleResponseFileCmd(const TCHAR* ResponseFile, FOutputDevice& Ar)
{
	Ar.Logf(TEXT("Executing commands using response file (\"%s\")..."), ResponseFile);

	FString Contents;
	if (!FFileHelper::LoadFileToString(Contents, &IPlatformFile::GetPlatformPhysical(), ResponseFile))
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("Failed to open the response file (\"%s\")."), ResponseFile);
		return false;
	}

	if (Contents.IsEmpty())
	{
		return true;
	}

	IUnrealInsightsModule& TraceInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

	TCHAR* StartPos = &Contents[0];
	TCHAR* EndPos = StartPos + Contents.Len();
	TCHAR* CrtPos = StartPos;
	while (CrtPos != EndPos)
	{
		uint32 EndOfLine = 0;
		if (*CrtPos == TEXT('\r'))
		{
			if (*(CrtPos + 1) == TEXT('\n'))
			{
				EndOfLine = 2;
			}
			else
			{
				EndOfLine = 1;
			}
		}
		else if (*CrtPos == TEXT('\n'))
		{
			EndOfLine = 1;
		}

		if (EndOfLine > 0)
		{
			const uint32 LineLen = uint32(CrtPos - StartPos);
			if (LineLen > 0 && *StartPos != TEXT('#'))
			{
				StartPos[LineLen] = TEXT('\0');
				TraceInsightsModule.Exec(StartPos, Ar);
			}

			CrtPos += EndOfLine;
			StartPos = CrtPos;
		}
		else
		{
			++CrtPos;
		}
	}
	const uint32 LineLen = uint32(CrtPos - StartPos);
	if (LineLen > 0 && *StartPos != TEXT('#'))
	{
		StartPos[LineLen] = TEXT('\0');
		TraceInsightsModule.Exec(StartPos, Ar);
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::RegisterTraceControlTab()
{
#if !WITH_EDITOR
	FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(FInsightsManagerTabs::TraceControlTabId,
		FOnSpawnTab::CreateRaw(this, &FInsightsManager::SpawnTraceControlTab))
		.SetDisplayName(LOCTEXT("TraceControl", "Trace Control"))
		.SetTooltipText(LOCTEXT("TraceControlTooltip", "Open the Trace Control tab."))
		.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TraceControl"))
		.SetAutoGenerateMenuEntry(false);
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> FInsightsManager::SpawnTraceControlTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FInsightsManager::OnTraceControlTabClosed));

	ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");
	TSharedPtr<ITraceController> TraceController = SessionServicesModule.GetTraceController();

	TSharedRef<SWidget> TraceControlRef = FModuleManager::LoadModuleChecked<UE::TraceTools::ITraceToolsModule>("TraceTools").CreateTraceControlWidget(TraceController, InstanceId);

	DockTab->SetContent(TraceControlRef);

	TraceControl = TraceControlRef;

	return DockTab;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::OnTraceControlTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	TraceControl.Reset();

	// Disable TabClosed delegate.
	TabBeingClosed->SetOnTabClosed(SDockTab::FOnTabClosedCallback());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsManager::OpenTraceControlWindow()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FInsightsManagerTabs::TraceControlTabId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
