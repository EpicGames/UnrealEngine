// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "Framework/Commands/UICommandList.h"
#include "Input/DragAndDrop.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

// TraceAnalysis
#include "Trace/StoreConnection.h"

// TraceServices
#include "TraceServices/Model/AnalysisSession.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"

// TraceInsights
#include "Insights/Common/SourceFilePathHelper.h"
#include "Insights/IInsightsManager.h"
#include "Insights/InsightsCommands.h"
#include "Insights/InsightsSettings.h"

namespace UE::Trace
{
	class FStoreClient;
}

namespace TraceServices
{
	class IAnalysisService;
	class IModuleService;
}

class IUnrealInsightsModule;
class FInsightsTestRunner;

namespace UE::Insights
{

class SSessionInfoWindow;
class FInsightsMenuBuilder;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Struct that holds data about in progress async operations
 */
struct FAsyncTaskData
{
	FString Name;
	FGraphEventRef GraphEvent;

	FAsyncTaskData(FGraphEventRef InGraphEvent, const FString InName)
	{
		Name = InName;
		GraphEvent = InGraphEvent;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class ETraceStreamType
{
	Unknown,
	TraceFile,
	TraceStore,
	DirectTrace,
	DirectTraceSecure,
	Custom
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * This class manages following areas:
 *     Connecting/disconnecting to source trace
 *     Global Unreal Insights application state and settings
 */
class FInsightsManager : public TSharedFromThis<FInsightsManager>, public ::Insights::IInsightsManager
{
	friend class FInsightsActionManager;

public:
	/** Creates the main manager, only one instance can exist. */
	FInsightsManager(TSharedRef<TraceServices::IAnalysisService> TraceAnalysisService,
					 TSharedRef<TraceServices::IModuleService> TraceModuleService);

	/** Virtual destructor. */
	virtual ~FInsightsManager();

	/**
	 * Creates an instance of the main manager and initializes global instance with the previously created instance of the manager.
	 * @param TraceAnalysisService The trace analysis service
	 * @param TraceModuleService   The trace module service
	 */
	static TSharedPtr<FInsightsManager> CreateInstance(TSharedRef<TraceServices::IAnalysisService> TraceAnalysisService,
													   TSharedRef<TraceServices::IModuleService> TraceModuleService);

	/** @return the global instance of the main manager (FInsightsManager). */
	static TSharedPtr<FInsightsManager> Get();

	//////////////////////////////////////////////////
	// IInsightsComponent

	virtual void Initialize(IUnrealInsightsModule& InsightsModule) override;
	virtual void Shutdown() override;
	virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) override;
	virtual void UnregisterMajorTabs() override;
	virtual bool Exec(const TCHAR* Cmd, FOutputDevice& Ar) override;

	//////////////////////////////////////////////////

	TSharedRef<TraceServices::IAnalysisService> GetAnalysisService() const { return AnalysisService; }
	TSharedRef<TraceServices::IModuleService> GetModuleService() const { return ModuleService; }

	//////////////////////////////////////////////////
	// Trace Store Connection wrapper

	bool ConnectToStore(const TCHAR* Host, uint32 Port = 0) { return TraceStoreConnection.ConnectToStore(Host, Port); }
	bool ReconnectToStore() { return TraceStoreConnection.ReconnectToStore(); }

	UE::Trace::FStoreClient* GetStoreClient() const { return TraceStoreConnection.GetStoreClient(); }
	FCriticalSection& GetStoreClientCriticalSection() const { return TraceStoreConnection.GetStoreClientCriticalSection(); }

	bool GetStoreAddressAndPort(uint32& OutStoreAddress, uint32& OutStorePort) const { return TraceStoreConnection.GetStoreAddressAndPort(OutStoreAddress, OutStorePort); }
	FString GetStoreDir() const { return TraceStoreConnection.GetStoreDir(); }

	const FString& GetLastStoreHost() const { return TraceStoreConnection.GetLastStoreHost(); }
	uint32 GetLastStorePort() const { return TraceStoreConnection.GetLastStorePort(); }

	const bool IsLocalHost() const { return TraceStoreConnection.IsLocalHost(); }
	const bool CanChangeStoreSettings() const { return TraceStoreConnection.CanChangeStoreSettings(); }

	//////////////////////////////////////////////////
	// Trace Session

	/** @return an instance of the trace analysis session. */
	TSharedPtr<const TraceServices::IAnalysisSession> GetSession() const;

	/** @return the stream type of the trace being analyzed. */
	ETraceStreamType GetTraceStreamType() const { return CurrentTraceStreamType; }

	/** @return the name of the trace being analyzed. */
	const FString& GetTraceName() const { return CurrentTraceName; }

	/** @return the store id of the trace being analyzed. */
	uint32 GetTraceId() const { return CurrentTraceId; }

	/** @return the port of the direct trace connection. */
	uint16 GetDirectTracePort() const { return CurrentDirectTracePort; }

	//////////////////////////////////////////////////

	/** @return UI command list for the main manager. */
	const TSharedRef<FUICommandList> GetCommandList() const;

	/** @return an instance of the main commands. */
	static const FInsightsCommands& GetCommands();

	/** @return an instance of the main action manager. */
	static FInsightsActionManager& GetActionManager();

	/** @return an instance of the main settings. */
	static FInsightsSettings& GetSettings();

	//////////////////////////////////////////////////
	// Session Info

	void AssignSessionInfoWindow(const TSharedRef<SSessionInfoWindow>& InSessionInfoWindow)
	{
		SessionInfoWindow = InSessionInfoWindow;
	}

	void RemoveSessionInfoWindow()
	{
		SessionInfoWindow.Reset();
	}

	/**
	 * Converts profiler window weak pointer to a shared pointer and returns it.
	 * Make sure the returned pointer is valid before trying to dereference it.
	 */
	TSharedPtr<class SSessionInfoWindow> GetSessionInfoWindow() const
	{
		return SessionInfoWindow.Pin();
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Getters and setters used by Toggle Commands.

	/** @return true, if UI is allowed to display debug info. */
	bool IsDebugInfoEnabled() const { return bIsDebugInfoEnabled; }
	void SetDebugInfo(bool bDebugInfoEnabledState) { bIsDebugInfoEnabled = bDebugInfoEnabledState; }

	bool IsAutoQuitEnabled() const { return bAutoQuit; }
	bool IsWaitForSymbolResolverEnabled() const { return bAutoQuit && bWaitForSymbolResolver; }
	void SetAutoQuit(bool bInAutoQuit, bool bInWaitForSymbolResolver = false)
	{
		bAutoQuit = bInAutoQuit;
		bWaitForSymbolResolver = bInWaitForSymbolResolver;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	/**
	 * Shows the open file dialog for choosing a trace file.
	 * @param OutTraceFile - The chosen trace file, if successful
	 * @return True, if successful.
	 */
	bool ShowOpenTraceFileDialog(FString& OutTraceFile) const;

	/**
	 * Shows the open file dialog and starts analysis session for the chosen trace file, in a new Unreal Insights instance.
	 */
	void OpenTraceFile() const;

	/**
	 * Starts analysis session for the specified trace file, in a new Unreal Insights instance.
	 * @param TraceFilename - The trace file to analyze
	 */
	void OpenTraceFile(const FString& TraceFilename) const;

	void ToggleAutoLoadLiveSession() { bIsAutoLoadLiveSessionEnabled = !bIsAutoLoadLiveSessionEnabled; }
	bool IsAutoLoadLiveSessionEnabled() const { return bIsAutoLoadLiveSessionEnabled; }
	void AutoLoadLiveSession();

	/**
	 * Creates a new analysis session instance and loads the latest available trace that is live.
	 * Replaces the current analysis session. On failure, if InRetryTime is > 0, will retry connecting every frame for RetryTime seconds
	 * @param InRetryTime - On failure, how many seconds to retry connecting during FInsightsManager::Tick
	 */
	void LoadLastLiveSession(float InRetryTime = 1.0f);

	/**
	 * Creates a new analysis session instance using the specified trace id.
	 * Replaces the current analysis session.
	 * @param InTraceId - The id of the trace to analyze
	 */
	void LoadTrace(uint32 InTraceId);

	/**
	 * Shows the open file dialog and creates a new analysis session instance for the chosen trace file.
	 * Replaces the current analysis session.
	 */
	void LoadTraceFile();

	/**
	 * Creates a new analysis session instance and loads a trace file from the specified location.
	 * Replaces the current analysis session.
	 * @param InTraceFilename - The trace file to analyze
	 */
	void LoadTraceFile(const FString& InTraceFilename);

	/**
	 * Creates a new analysis session instance and opens a listening port to await connections.
	 * Replaces the current analysis session.
	 * @param InStreamName User facing name of stream
	 * @param InPort The specific port number to listen to or 0 to use any available port
	 * @return The actual port used for listening to incoming connections. 0 if failed to open port.
	 */
	uint16 ListenForDirectTrace(const TCHAR* InStreamName = nullptr, uint16 InPort = 0);

	/**
	 * Creates a new analysis session instance using a secure connection to the specified host.
	 * @param Host Hostname to connect to.
	 * @param Port Port to connect to.
	 * @param AuthorizationToken Optional authorization token to use when connecting.
	 * @param CertificateFile Optional certificate to use when connecting. Excludes usage of CertificateAuthority.
	 * @param CertificateAuthority Optional certificate to use as trusted. CertificateFile should be unset to use.
	 * @param Identity Identity string used to connect when using pre-shared keys.
	 * @note All view-type parameters are copied and does not have to outlive this function call.
	 */
	void ConnectToSecureTrace(FStringView Host, uint16 Port, TConstArrayView<uint8> AuthorizationToken, FStringView CertificateFile = FStringView(), TConstArrayView<uint8> CertificateAuthority = TConstArrayView<uint8>(), FAnsiStringView Identity = FAnsiStringView());

	/**
	 * Creates a new analysis session instance from a provided data stream.
	 * Replaces the current analysis session.
	 * @param InStream Stream to read from
	 * @param InStreamName User facing name of stream
	 */
	void StartAnalysisWithStream(TUniquePtr<UE::Trace::IInDataStream>&& InStream, const TCHAR* InStreamName = nullptr);

	bool OnDragOver(const FDragDropEvent& DragDropEvent);
	bool OnDrop(const FDragDropEvent& DragDropEvent);

	void UpdateAppTitle();

	/** Opens the Settings dialog. */
	void OpenSettings();

	void UpdateSessionDuration();
	void CheckMemoryUsage();

	bool IsAnalysisComplete() const { return bIsAnalysisComplete; }
	double GetSessionDuration() const { return SessionDuration; }
	double GetAnalysisDuration() const { return AnalysisDuration; }
	double GetAnalysisSpeedFactor() const { return AnalysisSpeedFactor; }

	TSharedPtr<FInsightsMenuBuilder> GetInsightsMenuBuilder() { return InsightsMenuBuilder; }

	const FName& GetLogListingName() const { return LogListingName; }

	void ScheduleCommand(const FString& InCmd);

	/** Resets (closes) current session instance. */
	void ResetSession(bool bNotify = true);

	void OpenTraceControlWindow();

	FSourceFilePathHelper& GetSourceFilePathHelper() { return SourceFilePathHelper.Get(); }
	const FSourceFilePathHelper& GetSourceFilePathHelper() const { return SourceFilePathHelper.Get(); }

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// SessionChangedEvent

public:
	/** The event to execute when the session has changed. */
	virtual ::Insights::IInsightsManager::FSessionChangedEvent& GetSessionChangedEvent() override { return SessionChangedEvent; }
private:
	/** The event to execute when the session has changed. */
	::Insights::IInsightsManager::FSessionChangedEvent SessionChangedEvent;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// SessionAnalysisCompletedEvent

public:
	/** The event to execute when session analysis is complete. */
	virtual ::Insights::IInsightsManager::FSessionAnalysisCompletedEvent& GetSessionAnalysisCompletedEvent() override { return SessionAnalysisCompletedEvent; }
private:
	/** The event to execute when session analysis is completed. */
	::Insights::IInsightsManager::FSessionAnalysisCompletedEvent SessionAnalysisCompletedEvent;

	////////////////////////////////////////////////////////////////////////////////////////////////////

private:
	/** Binds our UI commands to delegates. */
	void BindCommands();

	/** Called to spawn the Session Info major tab. */
	TSharedRef<SDockTab> SpawnSessionInfoTab(const FSpawnTabArgs& Args);

	/** Callback called when the Session Info major tab is closed. */
	void OnSessionInfoTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	/** Updates this manager, done through FCoreTicker. */
	bool Tick(float DeltaTime);

	/** Extract messages from the session */
	void PollAnalysisInfo();

	void OnSessionChanged();
	void OnSessionAnalysisCompleted();
	void OnSessionSymbolResolverCompleted();

	void SpawnAndActivateTabs();

	void ActivateTimingInsightsTab();

	bool HandleResponseFileCmd(const TCHAR* ResponseFile, FOutputDevice& Ar);

	void RegisterTraceControlTab();

	TSharedRef<SDockTab> SpawnTraceControlTab(const FSpawnTabArgs& Args);

	void OnTraceControlTabClosed(TSharedRef<SDockTab> TabBeingClosed);

private:
	bool bIsInitialized = false;

	/** If true, the "high system memory usage warning" will be disabled until the system memory usage first drops below a certain threshold. */
	bool bMemUsageLimitHysteresis = false;

	/** The timestamp when has occurred the last check for system memory usage. */
	uint64 MemUsageLimitLastTimestamp = 0;

	/** The name of the Unreal Insights log listing. */
	FName LogListingName;

	/** Name used for Analysis log in Message Log. */
	FName AnalysisLogListingName;

	/** The delegate to be invoked when this manager ticks. */
	FTickerDelegate OnTick;

	/** Handle to the registered OnTick. */
	FTSTicker::FDelegateHandle OnTickHandle;

	TSharedRef<TraceServices::IAnalysisService> AnalysisService;
	TSharedRef<TraceServices::IModuleService> ModuleService;

	/** The trace analysis session. */
	TSharedPtr<const TraceServices::IAnalysisSession> Session;

	/** The stream type of the trace being analyzed. */
	ETraceStreamType CurrentTraceStreamType = ETraceStreamType::Unknown;

	/** The name of the trace being analyzed. */
	FString CurrentTraceName;

	/** The id of the trace being analyzed. */
	uint32 CurrentTraceId = 0;

	/** The port of the direct trace connection. */
	uint16 CurrentDirectTracePort = 0;

	/** List of UI commands for this manager. This will be filled by this and corresponding classes. */
	TSharedRef<FUICommandList> CommandList;

	/** An instance of the main action manager. */
	FInsightsActionManager ActionManager;

	/** An instance of the main settings. */
	FInsightsSettings Settings;

	/** A weak pointer to the Session Info window. */
	TWeakPtr<class SSessionInfoWindow> SessionInfoWindow;

	/** If enabled, UI can display additional info for debugging purposes. */
	bool bIsDebugInfoEnabled = false;

	bool bIsMainTabSet = false;
	bool bIsSessionInfoSet = false;

	bool bIsAnalysisComplete = false;
	bool bAutoQuit = false;
	bool bIsSymbolResolvingComplete = false;
	bool bWaitForSymbolResolver = false;

	float RetryLoadLastLiveSessionTimer = 0.0f;
	bool bIsAutoLoadLiveSessionEnabled = false;
	TSet<uint32> AutoLoadedTraceIds; // list of trace ids for the auto loaded live sessions

	UE::Insights::FStopwatch AnalysisStopwatch;
	double SessionDuration = 0.0;
	double AnalysisDuration = 0.0;
	double AnalysisSpeedFactor = 0.0;

	TSharedPtr<FInsightsMenuBuilder> InsightsMenuBuilder;
	TSharedPtr<FInsightsTestRunner> TestRunner;

	FString SessionAnalysisCompletedCmd;

	static const TCHAR* AutoQuitMsg;
	static const TCHAR* AutoQuitMsgOnFail;

	TDoubleLinkedList<FAsyncTaskData> InProgressAsyncTasks;

	/** A shared pointer to the global instance of the main manager. */
	static TSharedPtr<FInsightsManager> Instance;

	/** The Trace Store connection */
	UE::Trace::FStoreConnection TraceStoreConnection;

	FGuid InstanceId;
	TWeakPtr<SWidget> TraceControl;

	TSharedRef<FSourceFilePathHelper> SourceFilePathHelper;
};

} // namespace UE::Insights
