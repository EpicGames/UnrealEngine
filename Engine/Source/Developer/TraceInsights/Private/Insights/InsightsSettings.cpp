// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsSettings.h"

#include "Containers/StringConv.h"
#include "HAL/PlatformMath.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/CString.h"
#include "ProfilingDebugging/MiscTrace.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsSettings FInsightsSettings::Defaults(true);

////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsSettings::FInsightsSettings(bool bInIsDefault)
	: bIsDefault(bInIsDefault)
{
	if (!bIsDefault)
	{
		LoadFromConfig();
	}
	else
	{
		TimersViewInstanceVisibleColumns.Add(TEXT("Count"));
		TimersViewInstanceVisibleColumns.Add(TEXT("TotalInclTime"));
		TimersViewInstanceVisibleColumns.Add(TEXT("TotalExclTime"));

		TimersViewGameFrameVisibleColumns.Add(TEXT("MaxInclTime"));
		TimersViewGameFrameVisibleColumns.Add(TEXT("AverageInclTime"));
		TimersViewGameFrameVisibleColumns.Add(TEXT("MedianInclTime"));
		TimersViewGameFrameVisibleColumns.Add(TEXT("MinInclTime"));

		TimersViewRenderingFrameVisibleColumns.Add(TEXT("MaxInclTime"));
		TimersViewRenderingFrameVisibleColumns.Add(TEXT("AverageInclTime"));
		TimersViewRenderingFrameVisibleColumns.Add(TEXT("MedianInclTime"));
		TimersViewRenderingFrameVisibleColumns.Add(TEXT("MinInclTime"));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsSettings::~FInsightsSettings()
{
	for (auto& KV : AllTimingViewSettings)
	{
		delete KV.Value;
	}
	AllTimingViewSettings.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsSettings::ResetToDefaults()
{
	DefaultZoomLevel = Defaults.DefaultZoomLevel;
	bAutoHideEmptyTracks = Defaults.bAutoHideEmptyTracks;
	bAllowPanningOnScreenEdges = Defaults.bAllowPanningOnScreenEdges;
	bAutoZoomOnFrameSelection = Defaults.bAutoZoomOnFrameSelection;
	AutoScrollFrameAlignment = Defaults.AutoScrollFrameAlignment;
	AutoScrollViewportOffsetPercent = Defaults.AutoScrollViewportOffsetPercent;
	AutoScrollMinDelay = Defaults.AutoScrollMinDelay;
	ThreadsToUseRatio = Defaults.ThreadsToUseRatio;
	bShowFormatInTimerNames = Defaults.bShowFormatInTimerNames;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsSettings::LoadFromConfig()
{
	if (!FConfigContext::ReadIntoGConfig().Load(TEXT("UnrealInsightsSettings"), SettingsIni))
	{
		return;
	}

	using namespace UE::Insights;

	//////////////////////////////////////////////////
	// [Insights.(TimingViewName).TimingView]
	{
		TArray<FString> PersistentTimingViewSettings;
		GConfig->GetArray(TEXT("Insights.PersistentTimingView"), TEXT("Name"), PersistentTimingViewSettings, SettingsIni);

		for (const FString& SettingsName : PersistentTimingViewSettings)
		{
			FTimingViewSettings& TimingViewSettings = GetOrCreateTimingViewSettings(*SettingsName);

			const FString Section = FString::Printf(TEXT("Insights.%s.TimingView"), *SettingsName);

			// Toggle Options
			{
				TimingViewSettings.EnumerateToggleOptions([this, &Section](FTimingViewSettings::FToggleOption& Option)
					{
						GConfig->GetBool(*Section, Option.Name, Option.bValue, SettingsIni);
					});
			}

			// CPU Thread Groups
			{
				TArray<FString> VisibleCpuThreadGroups;
				GConfig->GetArray(*Section, TEXT("VisibleCpuThreadGroups"), VisibleCpuThreadGroups, SettingsIni);
				for (const FString& CpuGroup : VisibleCpuThreadGroups)
				{
					TimingViewSettings.AddCpuThreadGroup(*CpuGroup, true);
				}

				TArray<FString> HiddenCpuThreadGroups;
				GConfig->GetArray(*Section, TEXT("HiddenCpuThreadGroups"), HiddenCpuThreadGroups, SettingsIni);
				for (const FString& CpuGroup : HiddenCpuThreadGroups)
				{
					TimingViewSettings.AddCpuThreadGroup(*CpuGroup, false);
				}
			}
		}
	}

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler]

	GConfig->GetDouble(TEXT("Insights.TimingProfiler"), TEXT("DefaultZoomLevel"), DefaultZoomLevel, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler"), TEXT("bAutoHideEmptyTracks"), bAutoHideEmptyTracks, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler"), TEXT("bAllowPanningOnScreenEdges"), bAllowPanningOnScreenEdges, SettingsIni);
	GConfig->GetDouble(TEXT("Insights.TimingProfiler"), TEXT("ThreadsToUseRatio"), ThreadsToUseRatio, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler"), TEXT("bShowFormatInTimerNames"), bShowFormatInTimerNames, SettingsIni);
	GConfig->GetInt(TEXT("Insights.TimingProfiler"), TEXT("TimingTrackDepthLimit.Min"), TimingTrackDepthLimitMin, SettingsIni);
	GConfig->GetInt(TEXT("Insights.TimingProfiler"), TEXT("TimingTrackDepthLimit.Max"), TimingTrackDepthLimitMax, SettingsIni);

	// Auto-scroll options
	GConfig->GetBool(TEXT("Insights.TimingProfiler"), TEXT("bAutoScroll"), bAutoScroll, SettingsIni);
	FString FrameAlignment;
	if (GConfig->GetString(TEXT("Insights.TimingProfiler"), TEXT("AutoScrollFrameAlignment"), FrameAlignment, SettingsIni))
	{
		FrameAlignment.TrimStartAndEndInline();
		if (FrameAlignment.Equals(TEXT("game"), ESearchCase::IgnoreCase))
		{
			static_assert((int32)TraceFrameType_Game == 0, "ETraceFrameType");
			AutoScrollFrameAlignment = (int32)TraceFrameType_Game;
		}
		else if (FrameAlignment.Equals(TEXT("rendering"), ESearchCase::IgnoreCase))
		{
			static_assert((int32)TraceFrameType_Rendering == 1, "ETraceFrameType");
			AutoScrollFrameAlignment = (int32)TraceFrameType_Rendering;
		}
		else
		{
			AutoScrollFrameAlignment = -1;
		}
	}
	GConfig->GetDouble(TEXT("Insights.TimingProfiler"), TEXT("AutoScrollViewportOffsetPercent"), AutoScrollViewportOffsetPercent, SettingsIni);
	GConfig->GetDouble(TEXT("Insights.TimingProfiler"), TEXT("AutoScrollMinDelay"), AutoScrollMinDelay, SettingsIni);

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler.FramesView]

	GConfig->GetBool(TEXT("Insights.TimingProfiler.FramesView"), TEXT("bShowUpperThresholdLine"), bShowUpperThresholdLine, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler.FramesView"), TEXT("bShowLowerThresholdLine"), bShowLowerThresholdLine, SettingsIni);

	static constexpr double MinThresholdTime = 0.001; // == 1ms == 1000 fps
	static constexpr double MaxThresholdTime = 1.0; // == 1s == 1 fps

	FString UpperThreshold;
	if (GConfig->GetString(TEXT("Insights.TimingProfiler.FramesView"), TEXT("UpperThreshold"), UpperThreshold, SettingsIni))
	{
		if (UpperThreshold.IsEmpty())
		{
			UpperThreshold = TEXT("30 fps");
		}
		if (UpperThreshold.EndsWith(TEXT("fps")))
		{
			double FPS = FCString::Atof(*UpperThreshold);
			UpperThresholdTime = 1.0 / FMath::Clamp(FPS, 1.0 / MaxThresholdTime, 1.0 / MinThresholdTime);
			bShowUpperThresholdAsFps = true;
		}
		else
		{
			double Time = FCString::Atof(*UpperThreshold);
			UpperThresholdTime = FMath::Clamp(Time, MinThresholdTime, MaxThresholdTime);
			bShowUpperThresholdAsFps = false;
		}
	}

	FString LowerThreshold;
	if (GConfig->GetString(TEXT("Insights.TimingProfiler.FramesView"), TEXT("LowerThreshold"), LowerThreshold, SettingsIni))
	{
		if (LowerThreshold.IsEmpty())
		{
			LowerThreshold = TEXT("60 fps");
		}
		if (LowerThreshold.EndsWith(TEXT("fps")))
		{
			double FPS = FCString::Atof(*LowerThreshold);
			LowerThresholdTime = 1.0 / FMath::Clamp(FPS, 1.0 / MaxThresholdTime, 1.0 / MinThresholdTime);
			bShowLowerThresholdAsFps = true;
		}
		else
		{
			double Time = FCString::Atof(*LowerThreshold);
			LowerThresholdTime = FMath::Clamp(Time, MinThresholdTime, MaxThresholdTime);
			bShowLowerThresholdAsFps = false;
		}
	}

	GConfig->GetBool(TEXT("Insights.TimingProfiler.FramesView"), TEXT("bAutoZoomOnFrameSelection"), bAutoZoomOnFrameSelection, SettingsIni);

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler.MainGraph]

	GConfig->GetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowPoints"), bTimingViewMainGraphShowPoints, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowPointsWithBorder"), bTimingViewMainGraphShowPointsWithBorder, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowConnectedLines"), bTimingViewMainGraphShowConnectedLines, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowPolygons"), bTimingViewMainGraphShowPolygons, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowEventDuration"), bTimingViewMainGraphShowEventDuration, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowBars"), bTimingViewMainGraphShowBars, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowGameFrames"), bTimingViewMainGraphShowGameFrames, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowRenderingFrame"), bTimingViewMainGraphShowRenderingFrames, SettingsIni);

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler.TimersView]

	GConfig->GetArray(TEXT("Insights.TimingProfiler.TimersView"), TEXT("InstanceColumns"), TimersViewInstanceVisibleColumns, SettingsIni);
	GConfig->GetArray(TEXT("Insights.TimingProfiler.TimersView"), TEXT("GameFrameColumns"), TimersViewGameFrameVisibleColumns, SettingsIni);
	GConfig->GetArray(TEXT("Insights.TimingProfiler.TimersView"), TEXT("RenderingFrameColumns"), TimersViewRenderingFrameVisibleColumns, SettingsIni);

	GConfig->GetInt(TEXT("Insights.TimingProfiler.TimersView"), TEXT("AggregationMode"), TimersViewAggregationMode, SettingsIni);

	GConfig->GetInt(TEXT("Insights.TimingProfiler.TimersView"), TEXT("GroupingMode"), TimersViewGroupingMode, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler.TimersView"), TEXT("ShowGpuTimers"), bTimersViewShowGpuTimers, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler.TimersView"), TEXT("ShowVerseSamplingTimers"), bTimersViewShowVerseSamplingTimers, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler.TimersView"), TEXT("ShowCpuTimers"), bTimersViewShowCpuTimers, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler.TimersView"), TEXT("ShowCpuSamplingTimers"), bTimersViewShowCpuSamplingTimers, SettingsIni);
	GConfig->GetBool(TEXT("Insights.TimingProfiler.TimersView"), TEXT("ShowZeroCountTimers"), bTimersViewShowZeroCountTimers, SettingsIni);

	//////////////////////////////////////////////////
	// [Insights.GoToFrame]

	{
		int32 FrameTypeInt = (int32)GoToFrameType;
		GConfig->GetInt(TEXT("Insights.GoToFrame"), TEXT("FrameType"), FrameTypeInt, SettingsIni);
		GoToFrameType = (ETraceFrameType)FrameTypeInt;
	}
	GConfig->GetInt64(TEXT("Insights.GoToFrame"), TEXT("FrameNumber"), GoToFrameFrameNumber, SettingsIni);
	GConfig->GetBool(TEXT("Insights.GoToFrame"), TEXT("bZoomTimingView"), bGoToFrameZoomTimingView, SettingsIni);
	GConfig->GetBool(TEXT("Insights.GoToFrame"), TEXT("bSelectTimeRange"), bGoToFrameSelectTimeRange, SettingsIni);

	//////////////////////////////////////////////////
	// [Insights.GoToTime]

	GConfig->GetDouble(TEXT("Insights.GoToTime"), TEXT("Time"), GoToTimeTime, SettingsIni);
	GConfig->GetDouble(TEXT("Insights.GoToTime"), TEXT("TimeRangeDuration"), GoToTimeTimeRangeDuration, SettingsIni);
	GConfig->GetBool(TEXT("Insights.GoToTime"), TEXT("bZoomTimingView"), bGoToTimeZoomTimingView, SettingsIni);
	GConfig->GetBool(TEXT("Insights.GoToTime"), TEXT("bSelectTimeRange"), bGoToTimeSelectTimeRange, SettingsIni);

	//////////////////////////////////////////////////
	// [Insights]
	GConfig->GetString(TEXT("Insights"), TEXT("SourceFilesSearchPath"), SourceFilesSearchPath, SettingsIni);
	GConfig->GetArray(TEXT("Insights"), TEXT("SymbolSearchPaths"), SymbolSearchPaths, SettingsIni);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsSettings::SaveToConfig()
{
	using namespace UE::Insights;

	//////////////////////////////////////////////////
	// [Insights.(TimingViewName).TimingView]
	{
		TArray<FString> PersistentTimingViewSettings;
		for (auto& KV : AllTimingViewSettings)
		{
			PersistentTimingViewSettings.Add(KV.Key.ToString());
		}
		GConfig->SetArray(TEXT("Insights.PersistentTimingView"), TEXT("Name"), PersistentTimingViewSettings, SettingsIni);

		for (auto& KV : AllTimingViewSettings)
		{
			FTimingViewSettings& TimingViewSettings = *KV.Value;
			FString SettingsName = TimingViewSettings.GetName().ToString();

			const FString Section = FString::Printf(TEXT("Insights.%s.TimingView"), *SettingsName);

			// Toggle Options
			{
				TimingViewSettings.EnumerateToggleOptions([this, &Section](const FTimingViewSettings::FToggleOption& Option)
					{
						GConfig->SetBool(*Section, Option.Name, Option.bValue, SettingsIni);
					});
			}

			// CPU Thread Groups
			{
				const bool bShowAllCpuTracks = TimingViewSettings.IsToggleOptionEnabled(FTimingViewSettings::Option_ShowAllCpuTracks);

				TArray<FString> VisibleCpuThreadGroups;
				TArray<FString> HiddenCpuThreadGroups;
				TimingViewSettings.EnumerateCpuThreadGroups([this, bShowAllCpuTracks, &VisibleCpuThreadGroups, &HiddenCpuThreadGroups](const FString& GroupName, bool bIsVisible)
					{
						if (bIsVisible != bShowAllCpuTracks)
						{
							if (bIsVisible)
							{
								VisibleCpuThreadGroups.Add(GroupName);
							}
							else
							{
								HiddenCpuThreadGroups.Add(GroupName);
							}
						}
					});
				GConfig->SetArray(*Section, TEXT("VisibleCpuThreadGroups"), VisibleCpuThreadGroups, SettingsIni);
				GConfig->SetArray(*Section, TEXT("HiddenCpuThreadGroups"), HiddenCpuThreadGroups, SettingsIni);
			}
		}
	}

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler]

	GConfig->SetDouble(TEXT("Insights.TimingProfiler"), TEXT("DefaultZoomLevel"), DefaultZoomLevel, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler"), TEXT("bAutoHideEmptyTracks"), bAutoHideEmptyTracks, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler"), TEXT("bAllowPanningOnScreenEdges"), bAllowPanningOnScreenEdges, SettingsIni);
	GConfig->SetDouble(TEXT("Insights.TimingProfiler"), TEXT("ThreadsToUseRatio"), ThreadsToUseRatio, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler"), TEXT("bShowFormatInTimerNames"), bShowFormatInTimerNames, SettingsIni);
	GConfig->SetInt(TEXT("Insights.TimingProfiler"), TEXT("TimingTrackDepthLimit.Min"), TimingTrackDepthLimitMin, SettingsIni);
	GConfig->SetInt(TEXT("Insights.TimingProfiler"), TEXT("TimingTrackDepthLimit.Max"), TimingTrackDepthLimitMax, SettingsIni);

	// Auto-scroll options
	GConfig->SetBool(TEXT("Insights.TimingProfiler"), TEXT("bAutoScroll"), bAutoScroll, SettingsIni);
	const TCHAR* FrameAlignment = (AutoScrollFrameAlignment == 0) ? TEXT("game") : (AutoScrollFrameAlignment == 1) ? TEXT("rendering") : TEXT("none");
	GConfig->SetString(TEXT("Insights.TimingProfiler"), TEXT("AutoScrollFrameAlignment"), FrameAlignment, SettingsIni);
	GConfig->SetDouble(TEXT("Insights.TimingProfiler"), TEXT("AutoScrollViewportOffsetPercent"), AutoScrollViewportOffsetPercent, SettingsIni);
	GConfig->SetDouble(TEXT("Insights.TimingProfiler"), TEXT("AutoScrollMinDelay"), AutoScrollMinDelay, SettingsIni);

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler.FramesView]

	GConfig->SetBool(TEXT("Insights.TimingProfiler.FramesView"), TEXT("bShowUpperThresholdLine"), bShowUpperThresholdLine, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler.FramesView"), TEXT("bShowLowerThresholdLine"), bShowLowerThresholdLine, SettingsIni);

	if (bShowUpperThresholdAsFps)
	{
		FString UpperThreshold = FString::Printf(TEXT("%g fps"), 1.0 / UpperThresholdTime);
		GConfig->SetString(TEXT("Insights.TimingProfiler.FramesView"), TEXT("UpperThreshold"), *UpperThreshold, SettingsIni);
	}
	else
	{
		FString UpperThreshold = FString::Printf(TEXT("%g"), UpperThresholdTime);
		GConfig->SetString(TEXT("Insights.TimingProfiler.FramesView"), TEXT("UpperThreshold"), *UpperThreshold, SettingsIni);
	}

	if (bShowLowerThresholdAsFps)
	{
		FString LowerThreshold = FString::Printf(TEXT("%g fps"), 1.0 / LowerThresholdTime);
		GConfig->SetString(TEXT("Insights.TimingProfiler.FramesView"), TEXT("LowerThreshold"), *LowerThreshold, SettingsIni);
	}
	else
	{
		FString LowerThreshold = FString::Printf(TEXT("%g"), LowerThresholdTime);
		GConfig->SetString(TEXT("Insights.TimingProfiler.FramesView"), TEXT("LowerThreshold"), *LowerThreshold, SettingsIni);
	}

	GConfig->SetBool(TEXT("Insights.TimingProfiler.FramesView"), TEXT("bAutoZoomOnFrameSelection"), bAutoZoomOnFrameSelection, SettingsIni);

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler.MainGraph]

	GConfig->SetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowPoints"), bTimingViewMainGraphShowPoints, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowPointsWithBorder"), bTimingViewMainGraphShowPointsWithBorder, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowConnectedLines"), bTimingViewMainGraphShowConnectedLines, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowPolygons"), bTimingViewMainGraphShowPolygons, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowEventDuration"), bTimingViewMainGraphShowEventDuration, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowBars"), bTimingViewMainGraphShowBars, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowGameFrames"), bTimingViewMainGraphShowGameFrames, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler.MainGraph"), TEXT("ShowRenderingFrame"), bTimingViewMainGraphShowRenderingFrames, SettingsIni);

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler.TimersView]

	GConfig->SetArray(TEXT("Insights.TimingProfiler.TimersView"), TEXT("InstanceColumns"), TimersViewInstanceVisibleColumns, SettingsIni);
	GConfig->SetArray(TEXT("Insights.TimingProfiler.TimersView"), TEXT("GameFrameColumns"), TimersViewGameFrameVisibleColumns, SettingsIni);
	GConfig->SetArray(TEXT("Insights.TimingProfiler.TimersView"), TEXT("RenderingFrameColumns"), TimersViewRenderingFrameVisibleColumns, SettingsIni);

	GConfig->SetInt(TEXT("Insights.TimingProfiler.TimersView"), TEXT("AggregationMode"), TimersViewAggregationMode, SettingsIni);

	GConfig->SetInt(TEXT("Insights.TimingProfiler.TimersView"), TEXT("GroupingMode"), TimersViewGroupingMode, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler.TimersView"), TEXT("ShowGpuTimers"), bTimersViewShowGpuTimers, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler.TimersView"), TEXT("ShowVerseSamplingTimers"), bTimersViewShowVerseSamplingTimers, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler.TimersView"), TEXT("ShowCpuTimers"), bTimersViewShowCpuTimers, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler.TimersView"), TEXT("ShowCpuSamplingTimers"), bTimersViewShowCpuSamplingTimers, SettingsIni);
	GConfig->SetBool(TEXT("Insights.TimingProfiler.TimersView"), TEXT("ShowZeroCountTimers"), bTimersViewShowZeroCountTimers, SettingsIni);

	//////////////////////////////////////////////////
	// [Insights.GoToFrame]

	GConfig->SetInt(TEXT("Insights.GoToFrame"), TEXT("FrameType"), (int32)GoToFrameType, SettingsIni);
	GConfig->SetString(TEXT("Insights.GoToFrame"), TEXT("FrameNumber"), *FString::Printf(TEXT("%lld"), GoToFrameFrameNumber), SettingsIni);
	GConfig->SetBool(TEXT("Insights.GoToFrame"), TEXT("bZoomTimingView"), bGoToFrameZoomTimingView, SettingsIni);
	GConfig->SetBool(TEXT("Insights.GoToFrame"), TEXT("bSelectTimeRange"), bGoToFrameSelectTimeRange, SettingsIni);

	//////////////////////////////////////////////////
	// [Insights.GoToTime]

	GConfig->SetDouble(TEXT("Insights.GoToTime"), TEXT("Time"), GoToTimeTime, SettingsIni);
	GConfig->SetDouble(TEXT("Insights.GoToTime"), TEXT("TimeRangeDuration"), GoToTimeTimeRangeDuration, SettingsIni);
	GConfig->SetBool(TEXT("Insights.GoToTime"), TEXT("bZoomTimingView"), bGoToTimeZoomTimingView, SettingsIni);
	GConfig->SetBool(TEXT("Insights.GoToTime"), TEXT("bSelectTimeRange"), bGoToTimeSelectTimeRange, SettingsIni);

	//////////////////////////////////////////////////
	// [Insights]
	GConfig->SetString(TEXT("Insights"), TEXT("SourceFilesSearchPath"), *SourceFilesSearchPath, SettingsIni);
	GConfig->SetArray(TEXT("Insights"), TEXT("SymbolSearchPaths"), SymbolSearchPaths, SettingsIni);

	//////////////////////////////////////////////////

	GConfig->Flush(false, SettingsIni);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////
