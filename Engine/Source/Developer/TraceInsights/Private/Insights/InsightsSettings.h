// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "UObject/NameTypes.h"

namespace UE::Insights
{

class FTimingViewSettings
{
public:
	struct FToggleOption
	{
		const TCHAR* Name = nullptr;
		bool bDefaultValue = false;
		mutable bool bValue = false;
	};

	typedef TMap<const TCHAR*, FToggleOption, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<const TCHAR*, FToggleOption>> FToggleOptionsMap;

	FTimingViewSettings()
	{
		RegisterDefaultToggleOptions();
	}

	FTimingViewSettings(const FName& InName)
		: TimingViewName(InName)
	{
		RegisterDefaultToggleOptions();
	}

	~FTimingViewSettings() = default;

	void Reset()
	{
		ResetToggleOptions();
		ResetCpuThreadGroups();
	}

	const FName& GetName() const
	{
		return TimingViewName;
	}

	//////////////////////////////////////////////////
	// Toggle Options

	const FToggleOptionsMap& GetToggleOptions() const
	{
		return ToggleOptions;
	}
	FToggleOptionsMap& GetToggleOptions()
	{
		return ToggleOptions;
	}

	void EnumerateToggleOptions(TFunctionRef<void(const FToggleOption& Option)> Callback) const
	{
		for (const auto& KV : ToggleOptions)
		{
			Callback(KV.Value);
		}
	}
	void EnumerateToggleOptions(TFunctionRef<void(FToggleOption& Option)> Callback)
	{
		for (auto& KV : ToggleOptions)
		{
			Callback(KV.Value);
		}
	}

	bool IsToggleOptionRegistered(const TCHAR* Key) const
	{
		return ToggleOptions.Contains(Key);
	}
	void RegisterToggleOptions(const TCHAR* Name, bool bDefaultValue = false)
	{
		ToggleOptions.Add(Name, FToggleOption{ Name, bDefaultValue, bDefaultValue });
	}

	bool GetDefaultToggleOptionValue(const TCHAR* Key) const
	{
		const FToggleOption* Found = ToggleOptions.Find(Key);
		return Found && Found->bDefaultValue;
	}
	bool IsToggleOptionEnabled(const TCHAR* Key) const
	{
		const FToggleOption* Found = ToggleOptions.Find(Key);
		return Found && Found->bValue;
	}
	void SetToggleOption(const TCHAR* Key, bool bValue)
	{
		const FToggleOption* Found = ToggleOptions.Find(Key);
		if (Found)
		{
			Found->bValue = bValue;
		}
	}
	void ResetToggleOptions()
	{
		for (auto& KV : ToggleOptions)
		{
			KV.Value.bValue = KV.Value.bDefaultValue;
		}
	}

	//////////////////////////////////////////////////
	// CPU Thread Groups

	void EnumerateCpuThreadGroups(TFunctionRef<void(const FString& Name, bool bIsVisible)> Callback) const
	{
		for (const auto& KV : CpuThreadGroups)
		{
			Callback(KV.Key, KV.Value);
		}
	}

	bool IsCpuThreadGroupVisible(const TCHAR* GroupName) const
	{
		const bool* Found = CpuThreadGroups.Find(FString(GroupName));
		return Found && *Found;
	}
	bool HasCpuThreadGroup(const TCHAR* GroupName) const
	{
		return CpuThreadGroups.Contains(FString(GroupName));
	}
	void AddCpuThreadGroup(const TCHAR* GroupName, bool bIsVisible)
	{
		CpuThreadGroups.Add(FString(GroupName), bIsVisible);
	}
	void RemoveCpuThreadGroup(const TCHAR* GroupName)
	{
		CpuThreadGroups.Remove(FString(GroupName));
	}
	void ResetCpuThreadGroups()
	{
		CpuThreadGroups.Reset();
	}

	//////////////////////////////////////////////////

private:
	void RegisterDefaultToggleOptions()
	{
		RegisterToggleOptions(Option_ShowAllGpuTracks, true);
		RegisterToggleOptions(Option_ShowGpuWorkTracks, true);
		RegisterToggleOptions(Option_ShowGpuWorkOverlays, true);
		RegisterToggleOptions(Option_ShowGpuWorkExtendedLines, true);
		RegisterToggleOptions(Option_ShowGpuFencesRelations, true);
		RegisterToggleOptions(Option_ShowGpuFencesTracks, false);
		RegisterToggleOptions(Option_ShowGpuFencesExtendedLines, true);

		RegisterToggleOptions(Option_ShowAllVerseTracks, true);

		RegisterToggleOptions(Option_ShowAllCpuTracks, true);

		RegisterToggleOptions(Option_ShowAllCpuSamplingTracks, true);

		RegisterToggleOptions(Option_ShowCpuCoreTracks, true);
		RegisterToggleOptions(Option_ShowContextSwitches, true);
		RegisterToggleOptions(Option_ShowContextSwitchOverlays, true);
		RegisterToggleOptions(Option_ShowContextSwitchExtendedLines, true);
		RegisterToggleOptions(Option_ShowNonTargetProcessEvents, true);
	}

public:
	/** Toggle all or no GPU-related Tracks in the Timing View. */
	static constexpr const TCHAR* Option_ShowAllGpuTracks = TEXT("ShowAllGpuTracks");
	/** Visibility of the GPU Work header tracks. */
	static constexpr const TCHAR* Option_ShowGpuWorkTracks = TEXT("ShowGpuWorkTracks");
	/** Extends the visualization of GPU work events over the GPU timing tracks. */
	static constexpr const TCHAR* Option_ShowGpuWorkOverlays = TEXT("ShowGpuWorkOverlays");
	/** Extended vertical lines at the edges of each GPU work event. */
	static constexpr const TCHAR* Option_ShowGpuWorkExtendedLines = TEXT("ShowGpuWorkExtendedLines");
	/** Relations between Signal and Wait fences will be displayed when selecting a Timing Event in a GPU Queue Track. */
	static constexpr const TCHAR* Option_ShowGpuFencesRelations = TEXT("ShowGpuFencesRelations");
	/** Visibility of the GPU Fences child tracks. */
	static constexpr const TCHAR* Option_ShowGpuFencesTracks = TEXT("ShowGpuFencesTracks");
	/** Extended vertical lines at the location of GPU fences. */
	static constexpr const TCHAR* Option_ShowGpuFencesExtendedLines = TEXT("ShowGpuFencesExtendedLines");

	/** Toggle all or no Verse Tracks in the Timing View. */
	static constexpr const TCHAR* Option_ShowAllVerseTracks = TEXT("ShowAllVerseTracks");

	/** Toggle all or no CPU Tracks in the Timing View. */
	static constexpr const TCHAR* Option_ShowAllCpuTracks = TEXT("ShowAllCpuTracks");

	/** Toggle all or no CPU Sampling Tracks in the Timing View. */
	static constexpr const TCHAR* Option_ShowAllCpuSamplingTracks = TEXT("ShowAllCpuSamplingTracks");

	/** Toggle all or no CPU Core Tracks in the Timing View. */
	static constexpr const TCHAR* Option_ShowCpuCoreTracks = TEXT("ShowCpuCoreTracks");
	/** Visibility of the Context Switch tracks. */
	static constexpr const TCHAR* Option_ShowContextSwitches = TEXT("ShowContextSwitches");
	/** Extends the visualization of context work events over the CPU timing tracks. */
	static constexpr const TCHAR* Option_ShowContextSwitchOverlays = TEXT("ShowContextSwitchOverlays");
	/** Extended vertical lines at the edges of each context work event. */
	static constexpr const TCHAR* Option_ShowContextSwitchExtendedLines = TEXT("ShowContextSwitchExtendedLines");
	/** Visibility of the Non-Target events. */
	static constexpr const TCHAR* Option_ShowNonTargetProcessEvents = TEXT("ShowNonTargetProcessEvents");

private:
	// The unique name of the Timing View settings.
	// When settings are saved it will use a section named [Insights.(TimingViewName).TimingView].
	FName TimingViewName;

	/** Enabled state for the toggle options. */
	/** Key - The Option_ for true/false toggle commands. */
	/** Value - The enabled state. */
	FToggleOptionsMap ToggleOptions;

	/** Visibility state for the CPU Thread Groups. */
	/** GroupName - The name of the CPU Thread Group. */
	/** bIsVisible - The visibility state. */
	TMap<FString, bool> CpuThreadGroups;
};

} // namespace UE::Insights

/** Contains all settings for the Unreal Insights, accessible through the main manager. */
class FInsightsSettings
{
	friend class SInsightsSettings;

public:
	FInsightsSettings(bool bInIsDefault = false);
	~FInsightsSettings();

	void LoadFromConfig();
	void SaveToConfig();

	const FInsightsSettings& GetDefaults() const
	{
		return Defaults;
	}

	void ResetToDefaults();

	void EnterEditMode()
	{
		bIsEditing = true;
	}

	void ExitEditMode()
	{
		bIsEditing = false;
	}

	const bool IsEditing() const
	{
		return bIsEditing;
	}

	#define SET_AND_SAVE(Option, Value) { if (Option != Value) { Option = Value; SaveToConfig(); } }

	//////////////////////////////////////////////////
	// [Insights.(TimingViewName).TimingView]

	UE::Insights::FTimingViewSettings& GetOrCreateTimingViewSettings(const FName& TimingViewName)
	{
		UE::Insights::FTimingViewSettings** Found = AllTimingViewSettings.Find(TimingViewName);
		if (Found)
		{
			return **Found;
		}
		UE::Insights::FTimingViewSettings* NewTimingViewSettings = new UE::Insights::FTimingViewSettings(TimingViewName);
		AllTimingViewSettings.Add(TimingViewName, NewTimingViewSettings);
		return *NewTimingViewSettings;
	}

	UE::Insights::FTimingViewSettings& GetTimingViewSettings(const FName& TimingViewName)
	{
		UE::Insights::FTimingViewSettings** Found = AllTimingViewSettings.Find(TimingViewName);
		if (Found)
		{
			return **Found;
		}
		return DefaultTimingViewSettings;
	}
	const UE::Insights::FTimingViewSettings& GetTimingViewSettings(const FName& TimingViewName) const
	{
		UE::Insights::FTimingViewSettings*const* Found = AllTimingViewSettings.Find(TimingViewName);
		if (Found)
		{
			return **Found;
		}
		return DefaultTimingViewSettings;
	}

	void ResetTimingViewSettings(const FName& TimingViewName)
	{
		GetTimingViewSettings(TimingViewName).Reset();
		SaveToConfig();
	}

	void SetAndSaveTimingViewToggleOption(const FName& TimingViewName, const TCHAR* Key, bool bNewValue)
	{
		UE::Insights::FTimingViewSettings& TimingViewSettings = GetTimingViewSettings(TimingViewName);
		bool bOldValue = TimingViewSettings.IsToggleOptionEnabled(Key);
		if (bNewValue != bOldValue)
		{
			TimingViewSettings.SetToggleOption(Key, bNewValue);
			SaveToConfig();
		}
	}

	void AddAndSaveTimingViewCpuThreadGroup(const FName& TimingViewName, const TCHAR* GroupName, bool bIsVisible)
	{
		UE::Insights::FTimingViewSettings& TimingViewSettings = GetTimingViewSettings(TimingViewName);
		TimingViewSettings.AddCpuThreadGroup(GroupName, bIsVisible);
		SaveToConfig();
	}
	void RemoveAndSaveTimingViewCpuThreadGroup(const FName& TimingViewName, const TCHAR* GroupName)
	{
		UE::Insights::FTimingViewSettings& TimingViewSettings = GetTimingViewSettings(TimingViewName);
		TimingViewSettings.RemoveCpuThreadGroup(GroupName);
		SaveToConfig();
	}
	void ResetAndSaveTimingViewCpuThreadGroups(const FName& TimingViewName)
	{
		UE::Insights::FTimingViewSettings& TimingViewSettings = GetTimingViewSettings(TimingViewName);
		TimingViewSettings.ResetCpuThreadGroups();
		SaveToConfig();
	}

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler]

	double GetDefaultZoomLevel() const { return DefaultZoomLevel; }
	void SetDefaultZoomLevel(double ZoomLevel) { DefaultZoomLevel = ZoomLevel; }
	void SetAndSaveDefaultZoomLevel(double ZoomLevel) { SET_AND_SAVE(DefaultZoomLevel, ZoomLevel); }

	bool IsAutoHideEmptyTracksEnabled() const { return bAutoHideEmptyTracks; }
	void SetAutoHideEmptyTracks(bool bOnOff) { bAutoHideEmptyTracks = bOnOff; }
	void SetAndSaveAutoHideEmptyTracks(bool bOnOff) { SET_AND_SAVE(bAutoHideEmptyTracks, bOnOff); }

	bool IsPanningOnScreenEdgesEnabled() const { return bAllowPanningOnScreenEdges; }
	void SetPanningOnScreenEdges(bool bOnOff) { bAllowPanningOnScreenEdges = bOnOff; }
	void SetAndSavePanningOnScreenEdges(bool bOnOff) { SET_AND_SAVE(bAllowPanningOnScreenEdges, bOnOff); }

	int32 GetTimingTrackDepthLimitMin() const { return TimingTrackDepthLimitMin; }
	void SetTimingTrackDepthLimitMin(int32 DepthLimit) { TimingTrackDepthLimitMin = DepthLimit; }
	void SetAndSaveTimingTrackDepthLimitMin(int32 DepthLimit) { SET_AND_SAVE(TimingTrackDepthLimitMin, DepthLimit); }

	int32 GetTimingTrackDepthLimitMax() const { return TimingTrackDepthLimitMax; }
	void SetTimingTrackDepthLimitMax(int32 DepthLimit) { TimingTrackDepthLimitMax = DepthLimit; }
	void SetAndSaveTimingTrackDepthLimitMax(int32 DepthLimit) { SET_AND_SAVE(TimingTrackDepthLimitMax, DepthLimit); }

	bool IsAutoScrollEnabled() const { return bAutoScroll; }
	void SetAutoScroll(bool bOnOff) { bAutoScroll = bOnOff; }
	void SetAndSaveAutoScroll(bool bOnOff) { SET_AND_SAVE(bAutoScroll, bOnOff); }

	int32 GetAutoScrollFrameAlignment() const { return AutoScrollFrameAlignment; }
	void SetAutoScrollFrameAlignment(int32 FrameType) { AutoScrollFrameAlignment = FrameType; }
	void SetAndSaveAutoScrollFrameAlignment(int32 FrameType) { SET_AND_SAVE(AutoScrollFrameAlignment, FrameType); }

	double GetAutoScrollViewportOffsetPercent() const { return AutoScrollViewportOffsetPercent; }
	void SetAutoScrollViewportOffsetPercent(double OffsetPercent) { AutoScrollViewportOffsetPercent = OffsetPercent; }
	void SetAndSaveAutoScrollViewportOffsetPercent(double OffsetPercent) { SET_AND_SAVE(AutoScrollViewportOffsetPercent, OffsetPercent); }

	double GetAutoScrollMinDelay() const { return AutoScrollMinDelay; }
	void SetAutoScrollMinDelay(double Delay) { AutoScrollMinDelay = Delay; }
	void SetAndSaveAutoScrollMinDelay(double Delay) { SET_AND_SAVE(AutoScrollMinDelay, Delay); }

	double GetThreadRatio() const { return ThreadsToUseRatio; }
	void SetThreadRatio(double Ratio) { ThreadsToUseRatio = Ratio; }
	void SetAndSaveThreadRatio(double Ratio) { SET_AND_SAVE(ThreadsToUseRatio, Ratio); }

	bool GetShowFormatInTimerNames() const { return bShowFormatInTimerNames; }
	void SetShowFormatInTimerNames(bool Value) { bShowFormatInTimerNames = Value; }
	void SetAndSaveShowFormatInTimerNames(bool Value) { SET_AND_SAVE(bShowFormatInTimerNames, Value); }

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler.FramesView]

	bool IsShowUpperThresholdLineEnabled() const { return bShowUpperThresholdLine; }
	void SetShowUpperThresholdLineEnabled(bool bOnOff) { bShowUpperThresholdLine = bOnOff; }
	void SetAndSaveShowUpperThresholdLineEnabled(bool bOnOff) { SET_AND_SAVE(bShowUpperThresholdLine, bOnOff); }

	bool IsShowLowerThresholdLineEnabled() const { return bShowLowerThresholdLine; }
	void SetShowLowerThresholdLineEnabled(bool bOnOff) { bShowLowerThresholdLine = bOnOff; }
	void SetAndSaveShowLowerThresholdLineEnabled(bool bOnOff) { SET_AND_SAVE(bShowLowerThresholdLine, bOnOff); }

	double GetUpperThresholdTime() const { return UpperThresholdTime; }
	void SetUpperThresholdTime(double InUpperThresholdTime) { UpperThresholdTime = InUpperThresholdTime; }
	void SetAndSaveUpperThresholdTime(double InUpperThresholdTime) { SET_AND_SAVE(UpperThresholdTime, InUpperThresholdTime); }

	double GetLowerThresholdTime() const { return LowerThresholdTime; }
	void SetLowerThresholdTime(double InLowerThresholdTime) { LowerThresholdTime = InLowerThresholdTime; }
	void SetAndSaveLowerThresholdTime(double InLowerThresholdTime) { SET_AND_SAVE(LowerThresholdTime, InLowerThresholdTime); }

	bool IsShowUpperThresholdAsFpsEnabled() const { return bShowUpperThresholdAsFps; }
	void SetShowUpperThresholdAsFpsEnabled(bool bOnOff) { bShowUpperThresholdAsFps = bOnOff; }
	void SetAndSaveShowUpperThresholdAsFpsEnabled(bool bOnOff) { SET_AND_SAVE(bShowUpperThresholdAsFps, bOnOff); }

	bool IsShowLowerThresholdAsFpsEnabled() const { return bShowLowerThresholdAsFps; }
	void SetShowLowerThresholdAsFpsEnabled(bool bOnOff) { bShowLowerThresholdAsFps = bOnOff; }
	void SetAndSaveShowLowerThresholdAsFpsEnabled(bool bOnOff) { SET_AND_SAVE(bShowLowerThresholdAsFps, bOnOff); }

	void SetAndSaveThresholds(double InUpperThresholdTime, double InLowerThresholdTime, bool bInShowUpperThresholdAsFps, bool bInShowLowerThresholdAsFps)
	{
		bool bChanged = false;
		if (UpperThresholdTime != InUpperThresholdTime)
		{
			UpperThresholdTime = InUpperThresholdTime;
			bChanged = true;
		}
		if (LowerThresholdTime != InLowerThresholdTime)
		{
			LowerThresholdTime = InLowerThresholdTime;
			bChanged = true;
		}
		if (bShowUpperThresholdAsFps != bInShowUpperThresholdAsFps)
		{
			bShowUpperThresholdAsFps = bInShowUpperThresholdAsFps;
			bChanged = true;
		}
		if (bShowLowerThresholdAsFps != bInShowLowerThresholdAsFps)
		{
			bShowLowerThresholdAsFps = bInShowLowerThresholdAsFps;
			bChanged = true;
		}
		if (bChanged)
		{
			SaveToConfig();
		}
	}

	bool IsAutoZoomOnFrameSelectionEnabled() const { return bAutoZoomOnFrameSelection; }
	void SetAutoZoomOnFrameSelection(bool bOnOff) { bAutoZoomOnFrameSelection = bOnOff; }
	void SetAndSaveAutoZoomOnFrameSelection(bool bOnOff) { SET_AND_SAVE(bAutoZoomOnFrameSelection, bOnOff); }

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler.MainGraph]

	bool GetTimingViewMainGraphShowPoints() const { return bTimingViewMainGraphShowPoints; }
	void SetTimingViewMainGraphShowPoints(bool InValue) { bTimingViewMainGraphShowPoints = InValue; }
	void SetAndSaveTimingViewMainGraphShowPoints(bool InValue) { SET_AND_SAVE(bTimingViewMainGraphShowPoints, InValue); }

	bool GetTimingViewMainGraphShowPointsWithBorder() const { return bTimingViewMainGraphShowPointsWithBorder; }
	void SetTimingViewMainGraphShowPointsWithBorder(bool InValue) { bTimingViewMainGraphShowPointsWithBorder = InValue; }
	void SetAndSaveTimingViewMainGraphShowPointsWithBorder(bool InValue) { SET_AND_SAVE(bTimingViewMainGraphShowPointsWithBorder, InValue); }

	bool GetTimingViewMainGraphShowConnectedLines() const { return bTimingViewMainGraphShowConnectedLines; }
	void SetTimingViewMainGraphShowConnectedLines(bool InValue) { bTimingViewMainGraphShowConnectedLines = InValue; }
	void SetAndSaveTimingViewMainGraphShowConnectedLines(bool InValue) { SET_AND_SAVE(bTimingViewMainGraphShowConnectedLines, InValue); }

	bool GetTimingViewMainGraphShowPolygons() const { return bTimingViewMainGraphShowPolygons; }
	void SetTimingViewMainGraphShowPolygons(bool InValue) { bTimingViewMainGraphShowPolygons = InValue; }
	void SetAndTimingViewMainGraphShowPolygons(bool InValue) { SET_AND_SAVE(bTimingViewMainGraphShowPolygons, InValue); }

	bool GetTimingViewMainGraphShowEventDuration() const { return bTimingViewMainGraphShowEventDuration; }
	void SetTimingViewMainGraphShowEventDuration(bool InValue) { bTimingViewMainGraphShowEventDuration = InValue; }
	void SetAndSaveTimingViewMainGraphShowEventDuration(bool InValue) { SET_AND_SAVE(bTimingViewMainGraphShowEventDuration, InValue); }

	bool GetTimingViewMainGraphShowBars() const { return bTimingViewMainGraphShowBars; }
	void SetTimingViewMainGraphShowBars(bool InValue) { bTimingViewMainGraphShowBars = InValue; }
	void SetAndSaveTimingViewMainGraphShowBars(bool InValue) { SET_AND_SAVE(bTimingViewMainGraphShowBars, InValue); }

	bool GetTimingViewMainGraphShowGameFrames() const { return bTimingViewMainGraphShowGameFrames; }
	void SetTimingViewMainGraphShowGameFrames(bool InValue) { bTimingViewMainGraphShowGameFrames = InValue; }
	void SetAndSaveTimingViewMainGraphShowGameFrames(bool InValue) { SET_AND_SAVE(bTimingViewMainGraphShowGameFrames, InValue); }

	bool GetTimingViewMainGraphShowRenderingFrames() const { return bTimingViewMainGraphShowRenderingFrames; }
	void SetTimingViewMainGraphShowRenderingFrames(bool InValue) { bTimingViewMainGraphShowRenderingFrames = InValue; }
	void SetAndSaveTimingViewMainGraphShowRenderingFrames(bool InValue) { SET_AND_SAVE(bTimingViewMainGraphShowRenderingFrames, InValue); }

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler.TimersView]

	const TArray<FString>& GetTimersViewInstanceVisibleColumns() const { return TimersViewInstanceVisibleColumns; }
	void SetTimersViewInstanceVisibleColumns(const TArray<FString>& Columns) { TimersViewInstanceVisibleColumns = Columns; }
	void SetAndSaveTimersViewInstanceVisibleColumns(const TArray<FString>& Columns) { SET_AND_SAVE(TimersViewInstanceVisibleColumns, Columns); }

	const TArray<FString>& GetTimersViewGameFrameVisibleColumns() const { return TimersViewGameFrameVisibleColumns; }
	void SetTimersViewGameFrameVisibleColumns(const TArray<FString>& Columns) { TimersViewGameFrameVisibleColumns = Columns; }
	void SetAndSaveTimersViewGameFrameVisibleColumns(const TArray<FString>& Columns) { SET_AND_SAVE(TimersViewGameFrameVisibleColumns, Columns); }

	const TArray<FString>& GetTimersViewRenderingFrameVisibleColumns() const { return TimersViewRenderingFrameVisibleColumns; }
	void SetTimersViewRenderingFrameVisibleColumns(const TArray<FString>& Columns) { TimersViewRenderingFrameVisibleColumns = Columns; }
	void SetAndSaveTimersViewRenderingFrameVisibleColumns(const TArray<FString>& Columns) { SET_AND_SAVE(TimersViewRenderingFrameVisibleColumns, Columns); }

	int32 GetTimersViewAggregationMode() const { return TimersViewAggregationMode; }
	void SetTimersViewAggregationMode(int32 InMode) { TimersViewAggregationMode = InMode; }
	void SetAndSaveTimersViewAggregationMode(int32 InMode) { SET_AND_SAVE(TimersViewAggregationMode, InMode); }

	int32 GetTimersViewGroupingMode() const { return TimersViewGroupingMode; }
	void SetTimersViewGroupingMode(int32 InValue) { TimersViewGroupingMode = InValue; }
	void SetAndSaveTimersViewGroupingMode(int32 InValue) { SET_AND_SAVE(TimersViewGroupingMode, InValue); }

	bool GetTimersViewShowGpuTimers() const { return bTimersViewShowGpuTimers; }
	void SetTimersViewShowGpuTimers(bool InValue) { bTimersViewShowGpuTimers = InValue; }
	void SetAndSaveTimersViewShowGpuTimers(bool InValue) { SET_AND_SAVE(bTimersViewShowGpuTimers, InValue); }

	bool GetTimersViewShowVerseSamplingTimers() const { return bTimersViewShowVerseSamplingTimers; }
	void SetTimersViewShowVerseSamplingTimers(bool InValue) { bTimersViewShowVerseSamplingTimers = InValue; }
	void SetAndSaveTimersViewShowVerseSamplingTimers(bool InValue) { SET_AND_SAVE(bTimersViewShowVerseSamplingTimers, InValue); }

	bool GetTimersViewShowCpuTimers() const { return bTimersViewShowCpuTimers; }
	void SetTimersViewShowCpuTimers(bool InValue) { bTimersViewShowCpuTimers = InValue; }
	void SetAndSaveTimersViewShowCpuTimers(bool InValue) { SET_AND_SAVE(bTimersViewShowCpuTimers, InValue); }

	bool GetTimersViewShowCpuSamplingTimers() const { return bTimersViewShowCpuSamplingTimers; }
	void SetTimersViewShowCpuSamplingTimers(bool InValue) { bTimersViewShowCpuSamplingTimers = InValue; }
	void SetAndSaveTimersViewShowCpuSamplingTimers(bool InValue) { SET_AND_SAVE(bTimersViewShowCpuSamplingTimers, InValue); }

	bool GetTimersViewShowZeroCountTimers() const { return bTimersViewShowZeroCountTimers; }
	void SetTimersViewShowZeroCountTimers(bool InValue) { bTimersViewShowZeroCountTimers = InValue; }
	void SetAndSaveTimersViewShowZeroCountTimers(bool InValue) { SET_AND_SAVE(bTimersViewShowZeroCountTimers, InValue); }

	//////////////////////////////////////////////////
	// [Insights.GoToFrame]

	ETraceFrameType GetGoToFrameType() const { return GoToFrameType; }
	void SetGoToFrameType(ETraceFrameType InType) { GoToFrameType = InType; }
	void SetAndSaveGoToFrameType(ETraceFrameType InType) { SET_AND_SAVE(GoToFrameType, InType); }

	int64 GetGoToFrameFrameNumber() const { return GoToFrameFrameNumber; }
	void SetGoToFrameFrameNumber(int64 InFrameNumber) { GoToFrameFrameNumber = InFrameNumber; }
	void SetAndSaveGoToFrameFrameNumber(int64 InFrameNumber) { SET_AND_SAVE(GoToFrameFrameNumber, InFrameNumber); }

	bool IsGoToFrameZoomTimingViewEnabled() const { return bGoToFrameZoomTimingView; }
	void SetGoToFrameZoomTimingView(bool bOnOff) { bGoToFrameZoomTimingView = bOnOff; }
	void SetAndSaveGoToFrameZoom(bool bOnOff) { SET_AND_SAVE(bGoToFrameZoomTimingView, bOnOff); }

	bool IsGoToFrameSelectTimeRangeEnabled() const { return bGoToFrameSelectTimeRange; }
	void SetGoToFrameSelectTimeRange(bool bOnOff) { bGoToFrameSelectTimeRange = bOnOff; }
	void SetAndSaveGoToFrameSelectTimeRange(bool bOnOff) { SET_AND_SAVE(bGoToFrameSelectTimeRange, bOnOff); }

	//////////////////////////////////////////////////
	// [Insights.GoToTime]

	double GetGoToTimeTime() const { return GoToTimeTime; }
	void SetGoToTimeTime(double InTime) { GoToTimeTime = InTime; }
	void SetAndSaveGoToTimeTime(double InTime) { SET_AND_SAVE(GoToTimeTime, InTime); }

	double GetGoToTimeTimeRangeDuration() const { return GoToTimeTimeRangeDuration; }
	void SetGoToTimeTimeRangeDuration(double InDuration) { GoToTimeTimeRangeDuration = InDuration; }
	void SetAndSaveGoToTimeTimeRangeDuration(double InDuration) { SET_AND_SAVE(GoToTimeTimeRangeDuration, InDuration); }

	bool IsGoToTimeZoomTimingViewEnabled() const { return bGoToTimeZoomTimingView; }
	void SetGoToTimeZoomTimingView(bool bOnOff) { bGoToTimeZoomTimingView = bOnOff; }
	void SetAndSaveGoToTimeZoom(bool bOnOff) { SET_AND_SAVE(bGoToTimeZoomTimingView, bOnOff); }

	bool IsGoToTimeSelectTimeRangeEnabled() const { return bGoToTimeSelectTimeRange; }
	void SetGoToTimeSelectTimeRange(bool bOnOff) { bGoToTimeSelectTimeRange = bOnOff; }
	void SetAndSaveGoToTimeSelectTimeRange(bool bOnOff) { SET_AND_SAVE(bGoToTimeSelectTimeRange, bOnOff); }

	//////////////////////////////////////////////////
	// [Insights]

	const FString& GetSourceFilesSearchPath() const { return SourceFilesSearchPath; }
	void SetSourceFilesSearchPath(const FString& SearchPath) { SourceFilesSearchPath = SearchPath; }
	void SetAndSaveSourceFilesSearchPath(const FString& SearchPath) { SET_AND_SAVE(SourceFilesSearchPath, SearchPath); }

	const TArray<FString>& GetSymbolSearchPaths() const { return SymbolSearchPaths; }
	void SetSymbolSearchPaths(const TArray<FString>& SearchPaths) { SymbolSearchPaths = SearchPaths; }
	void SetAndSaveSymbolSearchPaths(const TArray<FString>& SearchPaths) { SET_AND_SAVE(SymbolSearchPaths, SearchPaths); }

	//////////////////////////////////////////////////

	#undef SET_AND_SAVE

private:
	/** Contains default settings. */
	static FInsightsSettings Defaults;

	/** Whether this instance contains defaults. */
	bool bIsDefault = false;

	/** Whether profiler settings is in edit mode. */
	bool bIsEditing = false;

	/** Settings filename ini. */
	FString SettingsIni;

	//////////////////////////////////////////////////
	// [Insights.(TimingViewName).TimingView]

	/** Settings for each registered Timing View instance. */
	TMap<FName, UE::Insights::FTimingViewSettings*> AllTimingViewSettings;

	/** The default settings for a Timing View. */
	inline static UE::Insights::FTimingViewSettings DefaultTimingViewSettings = { };

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler]

	/** The default (initial) zoom level of the Timing view. */
	double DefaultZoomLevel = 5.0; // 5 seconds between major tick marks

	/** Auto hide empty tracks (ex.: ones without timing events in the current viewport). */
	bool bAutoHideEmptyTracks = true;

	/** If enabled, the panning is allowed to continue when mouse cursor reaches the edges of the screen. */
	bool bAllowPanningOnScreenEdges = false;

	/** The minimum number of lanes in a timing events track. */
	int32 TimingTrackDepthLimitMin;

	/** The maximum number of lanes in a timing events track. */
	int32 TimingTrackDepthLimitMax;

	/** If enabled, the Timing View will start with auto-scroll enabled. */
	bool bAutoScroll = false;

	/** -1 to disable frame alignment or the type of frame to align with (0 = Game or 1 = Rendering). */
	int32 AutoScrollFrameAlignment = 0; // -1 = none, 0 = game, 1 = rendering

	/**
	 * Viewport offset while auto-scrolling, as percent of viewport width.
	 * If positive, it offsets the viewport forward, allowing an empty space at the right side of the viewport (i.e. after end of session).
	 * If negative, it offsets the viewport backward (i.e. end of session will be outside viewport).
	 */
	double AutoScrollViewportOffsetPercent = 0.1; // scrolls forward 10% of viewport's width

	/** Minimum time between two auto-scroll updates, in [seconds]. */
	double AutoScrollMinDelay = 0.3; // [seconds]

	/** The ratio of available threads to use when performing per-frame aggregations. */
	double ThreadsToUseRatio = 0.75;

	/** When enabled, the format will be appended to the timers static name for events with metadata specs. */
	bool bShowFormatInTimerNames = true;

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler.FramesView]

	/** If enabled, the upper threshold line is visible.The frame coloring by threshold is enabled regardless of this setting. */
	bool bShowUpperThresholdLine = false;

	/** If enabled, the lower threshold line is visible.The frame coloring by threshold is enabled regardless of this setting. */
	bool bShowLowerThresholdLine = false;

	/**
	 * The upper threshold for frames.
	 * Can be specified as a frame duration([0.001 .. 1.0] seconds; ex.: "0.010" for 10 ms) or as a framerate([1 fps .. 1000 fps]; ex: "100 fps").
	 */
	double UpperThresholdTime = 1.0 / 30.0;
	/**
	 * The lower threshold for frames.
	 * Can be specified as a frame duration([0.001 .. 1.0] seconds; ex.: "0.010" for 10 ms) or as a framerate([1 fps .. 1000 fps]; ex: "100 fps").
	 */
	double LowerThresholdTime = 1.0 / 60.0;

	bool bShowUpperThresholdAsFps = true;
	bool bShowLowerThresholdAsFps = true;

	/** If enabled, the Timing View will also be zoomed when a new frame is selected in the Frames track. */
	bool bAutoZoomOnFrameSelection = false;

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler.MainGraph]

	/** If enabled, values will be displayed as points in the Main Graph Track in Timing Insights. */
	bool bTimingViewMainGraphShowPoints = false;

	/** If enabled, values will be displayed as points with border in the Main Graph Track in Timing Insights. */
	bool bTimingViewMainGraphShowPointsWithBorder = true;

	/** If enabled, values will be displayed as connected lines in the Main Graph Track in Timing Insights. */
	bool bTimingViewMainGraphShowConnectedLines = true;

	/** If enabled, values will be displayed as polygons in the Main Graph Track in Timing Insights. */
	bool bTimingViewMainGraphShowPolygons = true;

	/** If enabled, uses duration of timing events for connected lines and polygons in the Main Graph Track in Timing Insights. */
	bool bTimingViewMainGraphShowEventDuration = true;

	/** If enabled, shows bars corresponding to the duration of the timing events in the Main Graph Track in Timing Insights. */
	bool bTimingViewMainGraphShowBars = false;

	/** If enabled, shows game frames in the Main Graph Track in Timing Insights. */
	bool bTimingViewMainGraphShowGameFrames = true;

	/** If enabled, shows rendering frames in the Main Graph Track in Timing Insights. */
	bool bTimingViewMainGraphShowRenderingFrames = true;

	//////////////////////////////////////////////////
	// [Insights.TimingProfiler.TimersView]

	/** The list of visible columns in the Timers view in the Instance mode. */
	TArray<FString> TimersViewInstanceVisibleColumns;

	/** The list of visible columns in the Timers view in the Game Frame mode. */
	TArray<FString> TimersViewGameFrameVisibleColumns;

	/** The list of visible columns in the Timers view in the Rendering Frame mode. */
	TArray<FString> TimersViewRenderingFrameVisibleColumns;

	/**
	 * The mode for the timers panel.
	 * See ETimerAggregationMode in TimingProfiler\ViewModels\TimerAggregator.h.
	 */
	int32 TimersViewAggregationMode = 0; // 0=Instance, 1=GameFrame, 2=RenderingFrame

	/** The grouping mode for the timers panel. */
	int32 TimersViewGroupingMode = 3; // ByType

	/** If enabled, GPU Scope timers will be displayed in the Timing View. */
	bool bTimersViewShowGpuTimers = true;

	/** If enabled, Verse Sampling timers will be displayed in the Timing View. */
	bool bTimersViewShowVerseSamplingTimers = true;

	/** If enabled, CPU Scope timers will be displayed in the Timing View. */
	bool bTimersViewShowCpuTimers = true;

	/** If enabled, CPU Sampling timers will be displayed in the Timing View. */
	bool bTimersViewShowCpuSamplingTimers = true;

	/** If enabled, timers with no instances in the selected interval will still be displayed in the Timers View. */
	bool bTimersViewShowZeroCountTimers = true;

	//////////////////////////////////////////////////
	// [Insights.GoToFrame]

	/** Last selected frame type in the Go to Frame dialog. */
	ETraceFrameType GoToFrameType = TraceFrameType_Game;

	/** Last frame number entered in the Go to Frame dialog. */
	int64 GoToFrameFrameNumber = 0;

	/** If true, zooms in on the target frame; if false, centers the view on it. */
	bool bGoToFrameZoomTimingView = true;

	/** If true, selects the frame's time range when navigating. */
	bool bGoToFrameSelectTimeRange = true;

	//////////////////////////////////////////////////
	// [Insights.GoToTime]

	/** Last time (in seconds) entered in the Go to Time dialog. Negative if none saved. */
	double GoToTimeTime = -1.0;

	/** Time range duration in seconds used for selection and/or zoom when navigating to a time. */
	double GoToTimeTimeRangeDuration = 1.0;

	/** If true, zooms in on the target time; if false, centers the view on it. */
	bool bGoToTimeZoomTimingView = true;

	/** If true, selects the time range starting at the target time. */
	bool bGoToTimeSelectTimeRange = true;

	//////////////////////////////////////////////////
	// [Insights]

	/** Search path for the open source files feature. */
	FString SourceFilesSearchPath;

	/** List of search paths to look for symbol files */
	TArray<FString> SymbolSearchPaths;

	//////////////////////////////////////////////////
};
