// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsStyle.h"

#include "Brushes/SlateNoResource.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/LowLevelMemTracker.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"

////////////////////////////////////////////////////////////////////////////////////////////////////
// FInsightsStyle
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FInsightsStyle::FStyle> FInsightsStyle::StyleInstance = nullptr;

////////////////////////////////////////////////////////////////////////////////////////////////////

const ISlateStyle& FInsightsStyle::Get()
{
	return *StyleInstance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsStyle::Initialize()
{
	LLM_SCOPE_BYNAME(TEXT("Insights/Style"));

	// The UE Core style must be initialized before the Insights style
	FSlateApplication::InitializeCoreStyle();

	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FInsightsStyle::FStyle> FInsightsStyle::Create()
{
	TSharedRef<class FInsightsStyle::FStyle> NewStyle = MakeShareable(new FInsightsStyle::FStyle(FInsightsStyle::GetStyleSetName()));
	NewStyle->Initialize();
	return NewStyle;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FName FInsightsStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("InsightsStyle"));
	return StyleSetName;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FInsightsStyle::FStyle
////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsStyle::FStyle::FStyle(const FName& InStyleSetName)
	: FSlateStyleSet(InStyleSetName)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsStyle::FStyle::SyncParentStyles()
{
	const ISlateStyle* ParentStyle = GetParentStyle();

	NormalText = ParentStyle->GetWidgetStyle<FTextBlockStyle>("NormalText");
	Button = ParentStyle->GetWidgetStyle<FButtonStyle>("Button");

	SelectorColor = ParentStyle->GetSlateColor("SelectorColor");
	SelectionColor = ParentStyle->GetSlateColor("SelectionColor");
	SelectionColor_Inactive = ParentStyle->GetSlateColor("SelectionColor_Inactive");
	SelectionColor_Pressed = ParentStyle->GetSlateColor("SelectionColor_Pressed");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#define EDITOR_IMAGE_BRUSH(RelativePath, ...)      IMAGE_BRUSH      ("../../../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_IMAGE_BRUSH_SVG(RelativePath, ...)  IMAGE_BRUSH_SVG  ("../../../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_BOX_BRUSH(RelativePath, ...)        BOX_BRUSH        ("../../../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_BOX_BRUSH_SVG(RelativePath, ...)    BOX_BRUSH_SVG    ("../../../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_BORDER_BRUSH(RelativePath, ...)     BORDER_BRUSH     ("../../../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_BORDER_BRUSH_SVG(RelativePath, ...) BORDER_BRUSH_SVG ("../../../Editor/Slate/" RelativePath, __VA_ARGS__)

#define TODO_IMAGE_BRUSH(...) EDITOR_IMAGE_BRUSH_SVG("Starship/Common/StaticMesh", __VA_ARGS__)

void FInsightsStyle::FStyle::Initialize()
{
	SetParentStyleName("InsightsCoreStyle");

	// Sync styles from the parent style that will be used as templates for styles defined here
	SyncParentStyles();

	SetContentRoot(FPaths::EngineContentDir() / TEXT("Slate/Starship/Insights"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FVector2D Icon12x12(12.0f, 12.0f); // for TreeItem icons
	const FVector2D Icon16x16(16.0f, 16.0f); // for regular icons
	const FVector2D Icon20x20(20.0f, 20.0f); // for ToolBar icons

	Set("AppIcon", new IMAGE_BRUSH_SVG("UnrealInsights", FVector2D(45.0f, 45.0f)));
	Set("AppIconPadding", FMargin(5.0f, 5.0f, 5.0f, 5.0f));

	Set("AppIcon.Small", new IMAGE_BRUSH_SVG("UnrealInsights", FVector2D(24.0f, 24.0f)));
	Set("AppIconPadding.Small", FMargin(4.0f, 4.0f, 0.0f, 0.0f));

	//////////////////////////////////////////////////
	// Session Info

	Set("Icons.SessionInfo", new IMAGE_BRUSH_SVG("Session", Icon16x16));

	//////////////////////////////////////////////////
	// Timing Insights

	Set("Icons.TimingProfiler", new IMAGE_BRUSH_SVG("Timing", Icon16x16));

	Set("Icons.FramesTrack", new IMAGE_BRUSH_SVG("Frames", Icon16x16));
	Set("Icons.FramesTrack.ToolBar", new IMAGE_BRUSH_SVG("Frames_20", Icon20x20));

	Set("Icons.TimingView", new IMAGE_BRUSH_SVG("Timing", Icon16x16));
	Set("Icons.TimingView.ToolBar", new IMAGE_BRUSH_SVG("Timing_20", Icon20x20));

	Set("Icons.TimersView", new IMAGE_BRUSH_SVG("Timer", Icon16x16));
	Set("Icons.TimersView.ToolBar", new IMAGE_BRUSH_SVG("Timer_20", Icon20x20));

	Set("Icons.CountersView", new IMAGE_BRUSH_SVG("Counter", Icon16x16));
	Set("Icons.CountersView.ToolBar", new IMAGE_BRUSH_SVG("Counter_20", Icon20x20));

	Set("Icons.CallersView", new IMAGE_BRUSH_SVG("Callers", Icon16x16));
	Set("Icons.CallersView.ToolBar", new IMAGE_BRUSH_SVG("Callers_20", Icon20x20));

	Set("Icons.CalleesView", new IMAGE_BRUSH_SVG("Callees", Icon16x16));
	Set("Icons.CalleesView.ToolBar", new IMAGE_BRUSH_SVG("Callees_20", Icon20x20));

	Set("Icons.LogView", new IMAGE_BRUSH_SVG("Log", Icon16x16));
	Set("Icons.LogView.ToolBar", new IMAGE_BRUSH_SVG("Log_20", Icon20x20));

	Set("Icons.TableTreeView", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Spreadsheet", Icon16x16));
	Set("Icons.TableTreeView.ToolBar", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Spreadsheet", Icon20x20));

	Set("Icons.TasksView", new IMAGE_BRUSH_SVG("Tasks", Icon16x16));
	Set("Icons.PackagesView", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Spreadsheet", Icon16x16));

	Set("Icons.ObjectsView", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Spreadsheet", Icon16x16));

	Set("Icons.PanelMenu.Toolbar", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Menu", Icon20x20));

	//////////////////////////////////////////////////
	// Timing View

	Set("Icons.AllTracksMenu.ToolBar", new IMAGE_BRUSH_SVG("AllTracks_20", Icon20x20));
	Set("Icons.CpuGpuTracksMenu.ToolBar", new IMAGE_BRUSH_SVG("CpuGpuTracks_20", Icon20x20));
	Set("Icons.OtherTracksMenu.ToolBar", new IMAGE_BRUSH_SVG("SpecialTracks_20", Icon20x20));
	Set("Icons.PluginTracksMenu.ToolBar", new IMAGE_BRUSH_SVG("PluginTracks_20", Icon20x20));
	Set("Icons.ViewModeMenu.ToolBar", new IMAGE_BRUSH_SVG("ViewMode_20", Icon20x20));

	Set("Icons.HighlightEvents.ToolBar", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Visualizer", Icon20x20));
	Set("Icons.ResetHighlight.ToolBar", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Reject", Icon20x20));

	Set("Icons.TimeMarker", new IMAGE_BRUSH_SVG("TimeMarker", Icon16x16));

	Set("Icons.Annotation", new IMAGE_BRUSH_SVG("Annotation", Icon16x16));
	Set("Icons.Annotation.ToolBar", new IMAGE_BRUSH_SVG("Annotation_20", Icon20x20));

	//////////////////////////////////////////////////

	Set("Icons.FindFirst", new IMAGE_BRUSH_SVG("ControlsFirst", Icon16x16));
	Set("Icons.FindFirst.ToolBar", new IMAGE_BRUSH_SVG("ControlsFirst", Icon20x20));
	Set("Icons.FindPrevious", new IMAGE_BRUSH_SVG("ControlsPrevious", Icon16x16));
	Set("Icons.FindPrevious.ToolBar", new IMAGE_BRUSH_SVG("ControlsPrevious", Icon20x20));
	Set("Icons.FindNext", new IMAGE_BRUSH_SVG("ControlsNext", Icon16x16));
	Set("Icons.FindNext.ToolBar", new IMAGE_BRUSH_SVG("ControlsNext", Icon20x20));
	Set("Icons.FindLast", new IMAGE_BRUSH_SVG("ControlsLast", Icon16x16));
	Set("Icons.FindLast.ToolBar", new IMAGE_BRUSH_SVG("ControlsLast", Icon20x20));

	//////////////////////////////////////////////////

	Set("Icons.SizeSmall", new IMAGE_BRUSH_SVG("SizeSmall", Icon16x16));
	Set("Icons.SizeSmall.ToolBar", new IMAGE_BRUSH_SVG("SizeSmall_20", Icon20x20));
	Set("Icons.SizeMedium", new IMAGE_BRUSH_SVG("SizeMedium", Icon16x16));
	Set("Icons.SizeMedium.ToolBar", new IMAGE_BRUSH_SVG("SizeMedium_20", Icon20x20));
	Set("Icons.SizeLarge", new IMAGE_BRUSH_SVG("SizeLarge", Icon16x16));
	Set("Icons.SizeLarge.ToolBar", new IMAGE_BRUSH_SVG("SizeLarge_20", Icon20x20));

	//////////////////////////////////////////////////
	// Asset Loading Insights

	Set("Icons.LoadingProfiler", new CORE_IMAGE_BRUSH_SVG("Starship/Common/file", Icon16x16));

	//////////////////////////////////////////////////
	// Networking Insights

	Set("Icons.NetworkingProfiler", new IMAGE_BRUSH_SVG("Networking", Icon16x16));

	Set("Icons.PacketView", new IMAGE_BRUSH_SVG("Packets", Icon16x16));
	Set("Icons.PacketView.ToolBar", new IMAGE_BRUSH_SVG("Packets_20", Icon20x20));

	Set("Icons.PacketContentView", new IMAGE_BRUSH_SVG("PacketContent", Icon16x16));
	Set("Icons.PacketContentView.ToolBar", new IMAGE_BRUSH_SVG("PacketContent_20", Icon20x20));

	Set("Icons.NetStatsView", new IMAGE_BRUSH_SVG("NetStats", Icon16x16));
	Set("Icons.NetStatsView.ToolBar", new IMAGE_BRUSH_SVG("NetStats_20", Icon20x20));

	//////////////////////////////////////////////////
	// Memory Insights

	Set("Icons.MemoryProfiler", new IMAGE_BRUSH_SVG("Memory", Icon16x16));

	Set("Icons.MemTagTreeView", new IMAGE_BRUSH_SVG("MemTags", Icon16x16));
	Set("Icons.MemTagTreeView.ToolBar", new IMAGE_BRUSH_SVG("MemTags_20", Icon20x20));

	Set("Icons.MemInvestigationView", new IMAGE_BRUSH_SVG("MemInvestigation", Icon16x16));
	Set("Icons.MemInvestigationView.ToolBar", new IMAGE_BRUSH_SVG("MemInvestigation_20", Icon20x20));

	Set("Icons.MemAllocTableTreeView", new IMAGE_BRUSH_SVG("MemAllocTable", Icon16x16));

	Set("Icons.ModulesView", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Spreadsheet", Icon16x16));
	Set("Icons.ModulesView.ToolBar", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Spreadsheet", Icon20x20));

	Set("Icons.AddMemTagGraphs", new CORE_IMAGE_BRUSH_SVG("Starship/Common/plus", Icon16x16));
	Set("Icons.RemoveMemTagGraphs", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Delete", Icon16x16));

	Set("Icons.TagSet.Systems", new IMAGE_BRUSH_SVG("MemTagSet_Systems", Icon16x16));
	Set("Icons.TagSet.Assets", new IMAGE_BRUSH_SVG("MemTagSet_Assets", Icon16x16));
	Set("Icons.TagSet.AssetClasses", new IMAGE_BRUSH_SVG("MemTagSet_AssetClasses", Icon16x16));
	Set("Icons.TagSet.UObjectClasses", new IMAGE_BRUSH_SVG("MemTagSet_AssetClasses", Icon16x16));
	Set("Icons.TagSet.CodeOrContent", new IMAGE_BRUSH_SVG("MemTagSet_AssetClasses", Icon16x16));

	Set("Icons.BudgetSettings", new IMAGE_BRUSH_SVG("BudgetSettings", Icon16x16));
	Set("Icons.TimeMarkerSettings", new IMAGE_BRUSH_SVG("TimeMarkerSettings", Icon16x16));

	//////////////////////////////////////////////////
	// Object Insights

	Set("Icons.ObjectProfiler", new IMAGE_BRUSH_SVG("MemoryAnalysis_16", Icon16x16));

	Set("Icons.ObjectTableTreeView", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Spreadsheet", Icon16x16));
	Set("Icons.ObjectTableTreeView.ToolBar", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Spreadsheet", Icon20x20));

	Set("Icons.ObjectDetailsView", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Spreadsheet", Icon16x16));
	Set("Icons.ObjectDetailsView.ToolBar", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Spreadsheet", Icon20x20));

	Set("ObjectDetailsView.ListView", FTableViewStyle()
		.SetBackgroundBrush(FSlateNoResource()));

	//////////////////////////////////////////////////
	// Tasks

	Set("Icons.GoToTask", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_ViewColumn_32x", Icon16x16));
	Set("Icons.ShowTaskCriticalPath", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_HotPath_32x", Icon16x16));
	Set("Icons.ShowTaskTransitions", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_Calls_32x", Icon16x16));
	Set("Icons.ShowTaskConnections", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_Calls_32x", Icon16x16));
	Set("Icons.ShowTaskPrerequisites", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_Calls_32x", Icon16x16));
	Set("Icons.ShowTaskSubsequents", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_Calls_32x", Icon16x16));
	Set("Icons.ShowParentTasks", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_Calls_32x", Icon16x16));
	Set("Icons.ShowNestedTasks", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_Calls_32x", Icon16x16));
	Set("Icons.ShowTaskTrack", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_Calls_32x", Icon16x16));
	Set("Icons.ShowDetailedTaskTrackInfo", new CORE_IMAGE_BRUSH("Icons/Profiler/profiler_Calls_32x", Icon16x16));

	//////////////////////////////////////////////////

	Set("MainFrame.OpenVisualStudio", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/VisualStudio", Icon16x16));
	Set("MainFrame.OpenSourceCodeEditor", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/SourceCodeEditor", Icon16x16));

	//////////////////////////////////////////////////

	Set("Icons.AddGraphSeries", new CORE_IMAGE_BRUSH_SVG("Starship/Common/plus", Icon16x16));
	Set("Icons.RemoveGraphSeries", new CORE_IMAGE_BRUSH_SVG("Starship/Common/close", Icon16x16));

	Set("Icons.AutoScroll", new IMAGE_BRUSH_SVG("AutoScrollRight_20", Icon16x16));

	Set("Icons.ZeroCountFilter", new IMAGE_BRUSH_SVG("ZeroCountFilter", Icon16x16));

	Set("Icons.Function", new IMAGE_BRUSH_SVG("Function", Icon16x16));

	Set("Icons.Pinned", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Pinned", Icon16x16));
	Set("Icons.Unpinned", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Unpinned", Icon16x16));

	Set("Icons.SelectEventRange", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/SelectInViewport", Icon16x16));

	Set("Icons.FindInstance", new CORE_IMAGE_BRUSH_SVG("Starship/Common/search", Icon16x16));
	Set("Icons.FindMinInstance", new IMAGE_BRUSH_SVG("SizeSmall", Icon16x16));
	Set("Icons.FindMinInstance.ToolBar", new IMAGE_BRUSH_SVG("SizeSmall_20", Icon20x20));
	Set("Icons.FindMaxInstance", new IMAGE_BRUSH_SVG("SizeLarge", Icon16x16));
	Set("Icons.FindMaxInstance.ToolBar", new IMAGE_BRUSH_SVG("SizeLarge_20", Icon20x20));

	//////////////////////////////////////////////////
	// Icons for tree/table items

	Set("Icons.HotPath.TreeItem", new IMAGE_BRUSH_SVG("HotPath_12", Icon12x12));
	Set("Icons.GpuTimer.TreeItem", new TODO_IMAGE_BRUSH(Icon12x12));
	Set("Icons.CpuTimer.TreeItem", new TODO_IMAGE_BRUSH(Icon12x12));
	Set("Icons.Counter.TreeItem", new TODO_IMAGE_BRUSH(Icon12x12));
	Set("Icons.StatCounter.TreeItem", new TODO_IMAGE_BRUSH(Icon12x12));
	Set("Icons.DataTypeDouble.TreeItem", new TODO_IMAGE_BRUSH(Icon12x12));
	Set("Icons.DataTypeInt64.TreeItem", new TODO_IMAGE_BRUSH(Icon12x12));
	Set("Icons.NetEvent.TreeItem", new TODO_IMAGE_BRUSH(Icon12x12));

	Set("Icons.MemTag.TreeItem", new IMAGE_BRUSH_SVG("MemTags", Icon12x12));
	Set("Icons.SystemMemTag.TreeItem", new IMAGE_BRUSH_SVG("MemTag_System_12", Icon12x12));
	Set("Icons.AssetMemTag.TreeItem", new IMAGE_BRUSH_SVG("MemTag_Asset_12", Icon12x12));
	Set("Icons.ClassMemTag.TreeItem", new IMAGE_BRUSH_SVG("MemTag_Class_12", Icon12x12));

	// Core UObject Classes
	Set("Icons.UObject.TreeItem",       new IMAGE_BRUSH_SVG("Object", Icon12x12));
	Set("Icons.UObject.Group.TreeItem", new IMAGE_BRUSH_SVG("Object", Icon12x12));
	Set("Icons.UField.TreeItem",        new CORE_IMAGE_BRUSH_SVG("Starship/Common/box-perspective", Icon12x12)); // UField -> UObject
	Set("Icons.UEnum.TreeItem",         new CORE_IMAGE_BRUSH_SVG("Starship/Common/file-tree", Icon12x12)); // UEnum -> UField
	Set("Icons.UProperty.TreeItem",     new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/tool-properties", Icon12x12)); //  UProperty -> UField
	Set("Icons.UStruct.TreeItem",       new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Struct", Icon12x12)); // UStruct -> UField
	Set("Icons.UClass.TreeItem",        new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Class", Icon12x12)); // UClass -> UStruct
	Set("Icons.UFunction.TreeItem",     new EDITOR_IMAGE_BRUSH_SVG("Starship/GraphEditors/FunctionInputParameter", Icon12x12)); // UFunction -> UStruct
	Set("Icons.UPackage.TreeItem",      new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/ProjectPackage", Icon12x12)); // UPackage -> UObject

	// Engine UObject Classes
	Set("Icons.UTexture.TreeItem",      new CORE_IMAGE_BRUSH_SVG("Starship/Common/box-perspective", Icon12x12)); // UTexture -> UStreamableRenderAsset -> UObject
	Set("Icons.UMaterial.TreeItem",     new CORE_IMAGE_BRUSH_SVG("Starship/Common/box-perspective", Icon12x12)); // UMaterial -> UMaterialInterface -> UObject
	Set("Icons.UStaticMesh.TreeItem",   new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/StaticMesh", Icon12x12)); // UStaticMesh -> UStreamableRenderAsset -> UObject
	Set("Icons.USkeletalMesh.TreeItem", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/SkeletalMesh", Icon12x12)); // USkeletalMesh -> USkinnedAsset -> UStreamableRenderAsset -> UObject
	Set("Icons.ULevel.TreeItem",        new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Levels", Icon12x12)); // ULevel -> UObject
	Set("Icons.UWorld.TreeItem",        new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/World", Icon12x12)); // UWorld -> UObject
	Set("Icons.AActor.TreeItem",        new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/LitCube", Icon12x12)); // AActor -> UObject

	Set("Icons.HasGraph.TreeItem", new IMAGE_BRUSH_SVG("RoundedBullet", Icon12x12));

	//////////////////////////////////////////////////
	// Trace Control

	Set("Icons.TraceControl", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/TraceDataFiltering", Icon16x16));

	//////////////////////////////////////////////////
}

#undef TODO_IMAGE_BRUSH
#undef EDITOR_BOX_BRUSH
#undef EDITOR_IMAGE_BRUSH_SVG
#undef EDITOR_IMAGE_BRUSH

////////////////////////////////////////////////////////////////////////////////////////////////////
