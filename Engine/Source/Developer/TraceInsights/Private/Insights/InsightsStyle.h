// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Styling/SlateColor.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

struct FSlateBrush;

/** Style data for Insights tools */
class FInsightsStyle
{
public:
	static void Initialize();
	static void Shutdown();

	static const ISlateStyle& Get();
	static FName GetStyleSetName();

	static const FLinearColor& GetColor(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return StyleInstance->GetColor(PropertyName, Specifier);
	}

	static const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return StyleInstance->GetBrush(PropertyName, Specifier);
	}

	static const FSlateBrush* GetOutlineBrush(EHorizontalAlignment HAlign)
	{
		if (HAlign == HAlign_Left)
		{
			return StyleInstance->GetBrush("Border.L");
		}
		else if (HAlign == HAlign_Right)
		{
			return StyleInstance->GetBrush("Border.R");
		}
		else
		{
			return StyleInstance->GetBrush("Border.TB");
		}
	}

	class FStyle : public FSlateStyleSet
	{
	public:
		FStyle(const FName& InStyleSetName);

		void Initialize();

		void SyncParentStyles();

		// Styles inherited from the parent style
		FTextBlockStyle NormalText;
		FButtonStyle Button;
		FSlateColor SelectorColor;
		FSlateColor SelectionColor;
		FSlateColor SelectionColor_Inactive;
		FSlateColor SelectionColor_Pressed;
	};

private:
	static TSharedRef<FInsightsStyle::FStyle> Create();

	static TSharedPtr<FInsightsStyle::FStyle> StyleInstance;
};
