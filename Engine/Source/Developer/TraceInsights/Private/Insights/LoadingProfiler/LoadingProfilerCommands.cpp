// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoadingProfilerCommands.h"

#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/LoadingProfiler/LoadingProfilerManager.h"

#define LOCTEXT_NAMESPACE "UE::Insights::LoadingProfiler"

namespace UE::Insights::LoadingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FLoadingProfilerCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

FLoadingProfilerCommands::FLoadingProfilerCommands()
: TCommands<FLoadingProfilerCommands>(
	TEXT("LoadingProfilerCommands"),
	NSLOCTEXT("Contexts", "LoadingProfilerCommands", "Insights - Asset Loading Insights"),
	NAME_None,
	FInsightsStyle::GetStyleSetName())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// UI_COMMAND takes long for the compiler to optimize
UE_DISABLE_OPTIMIZATION_SHIP
void FLoadingProfilerCommands::RegisterCommands()
{
	UI_COMMAND(ToggleTimingViewVisibility,
		"Timing",
		"Toggles the visibility of the main Timing view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleEventAggregationTreeViewVisibility,
		"Event Aggregation",
		"Toggles the visibility of the Event Aggregation table/tree view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleObjectTypeAggregationTreeViewVisibility,
		"Object Type Aggregation",
		"Toggles the visibility of the Object Type Aggregation table/tree view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(TogglePackageDetailsTreeViewVisibility,
		"Package Details",
		"Toggles the visibility of the Package Details table/tree view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleExportDetailsTreeViewVisibility,
		"Export Details",
		"Toggles the visibility of the Export Details table/tree view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleRequestsTreeViewVisibility,
		"Requests",
		"Toggles the visibility of the Requests table/tree view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////
// Toggle Commands
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FLoadingProfilerActionManager, This, ToggleTimingViewVisibility, IsTimingViewVisible, ShowHideTimingView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FLoadingProfilerActionManager, This, ToggleEventAggregationTreeViewVisibility, IsEventAggregationTreeViewVisible, ShowHideEventAggregationTreeView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FLoadingProfilerActionManager, This, ToggleObjectTypeAggregationTreeViewVisibility, IsObjectTypeAggregationTreeViewVisible, ShowHideObjectTypeAggregationTreeView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FLoadingProfilerActionManager, This, TogglePackageDetailsTreeViewVisibility, IsPackageDetailsTreeViewVisible, ShowHidePackageDetailsTreeView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FLoadingProfilerActionManager, This, ToggleExportDetailsTreeViewVisibility, IsExportDetailsTreeViewVisible, ShowHideExportDetailsTreeView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FLoadingProfilerActionManager, This, ToggleRequestsTreeViewVisibility, IsRequestsTreeViewVisible, ShowHideRequestsTreeView)

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::LoadingProfiler

#undef LOCTEXT_NAMESPACE
