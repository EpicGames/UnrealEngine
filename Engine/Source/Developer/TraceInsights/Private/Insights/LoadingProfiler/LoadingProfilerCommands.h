// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UIAction.h"
#include "Styling/SlateTypes.h" // for ECheckBoxState
#include "Templates/SharedPointer.h"

// TraceInsights
#include "Insights/Common/CommandUtils.h"

namespace UE::Insights::LoadingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class that holds all profiler commands.
 */
class FLoadingProfilerCommands : public TCommands<FLoadingProfilerCommands>
{
public:
	/** Default constructor. */
	FLoadingProfilerCommands();

	/** Initialize commands. */
	virtual void RegisterCommands() override;

public:
	/** Toggles visibility for the Timing view. */
	TSharedPtr<FUICommandInfo> ToggleTimingViewVisibility;

	/** Toggles visibility for the Event Aggregation tree view. */
	TSharedPtr<FUICommandInfo> ToggleEventAggregationTreeViewVisibility;

	/** Toggles visibility for the Object Type Aggregation tree view. */
	TSharedPtr<FUICommandInfo> ToggleObjectTypeAggregationTreeViewVisibility;

	/** Toggles visibility for the Package Details tree view. */
	TSharedPtr<FUICommandInfo> TogglePackageDetailsTreeViewVisibility;

	/** Toggles visibility for the Export Details tree view. */
	TSharedPtr<FUICommandInfo> ToggleExportDetailsTreeViewVisibility;

	/** Toggles visibility for the Requests tree view. */
	TSharedPtr<FUICommandInfo> ToggleRequestsTreeViewVisibility;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class that provides helper functions for the commands to avoid cluttering profiler manager with many small functions.
 * Can't contain any variables. Directly operates on the profiler manager instance.
 */
class FLoadingProfilerActionManager
{
	friend class FLoadingProfilerManager;

private:
	/** Private constructor. */
	FLoadingProfilerActionManager(class FLoadingProfilerManager* Instance)
		: This(Instance)
	{}

	//////////////////////////////////////////////////
	// Toggle Commands

	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleTimingViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleEventAggregationTreeViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleObjectTypeAggregationTreeViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(TogglePackageDetailsTreeViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleExportDetailsTreeViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleRequestsTreeViewVisibility)

	//////////////////////////////////////////////////

protected:
	/** Reference to the global instance of the profiler manager. */
	class FLoadingProfilerManager* This;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::LoadingProfiler
