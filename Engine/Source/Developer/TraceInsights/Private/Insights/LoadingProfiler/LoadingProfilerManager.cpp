// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoadingProfilerManager.h"

#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

// TraceServices
#include "TraceServices/Model/LoadTimeProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Table/Widgets/STableTreeView.h"

// TraceInsights
#include "Insights/Common/InsightsMenuBuilder.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/LoadingProfiler/Widgets/SLoadingProfilerWindow.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::LoadingProfiler"

namespace UE::Insights::LoadingProfiler
{

DEFINE_LOG_CATEGORY(LogLoadingProfiler);

TSharedPtr<FLoadingProfilerManager> FLoadingProfilerManager::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FLoadingProfilerManager> FLoadingProfilerManager::Get()
{
	return FLoadingProfilerManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FLoadingProfilerManager> FLoadingProfilerManager::CreateInstance()
{
	ensure(!FLoadingProfilerManager::Instance.IsValid());
	if (FLoadingProfilerManager::Instance.IsValid())
	{
		FLoadingProfilerManager::Instance.Reset();
	}

	FLoadingProfilerManager::Instance = MakeShared<FLoadingProfilerManager>(FInsightsManager::Get()->GetCommandList());

	return FLoadingProfilerManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLoadingProfilerManager::FLoadingProfilerManager(TSharedRef<FUICommandList> InCommandList)
	: CommandList(InCommandList)
	, ActionManager(this)
	, MajorTabId(FInsightsManagerTabs::LoadingProfilerTabId)
	, LogListingName("LoadingInsights")
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingProfilerManager::Initialize(IUnrealInsightsModule& InsightsModule)
{
	ensure(!bIsInitialized);
	if (bIsInitialized)
	{
		return;
	}
	bIsInitialized = true;

	UE_LOGF(LogLoadingProfiler, Log, "Initialize");

	// Register tick functions.
	OnTick = FTickerDelegate::CreateSP(this, &FLoadingProfilerManager::Tick);
	OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(OnTick, 0.0f);

	FLoadingProfilerCommands::Register();
	BindCommands();

	FInsightsManager::Get()->GetSessionChangedEvent().AddSP(this, &FLoadingProfilerManager::OnSessionChanged);
	OnSessionChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingProfilerManager::Shutdown()
{
	if (!bIsInitialized)
	{
		return;
	}
	bIsInitialized = false;

	// If the MessageLog module was already unloaded as part of the global Shutdown process, do not load it again.
	if (FModuleManager::Get().IsModuleLoaded("MessageLog"))
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		if (MessageLogModule.IsRegisteredLogListing(GetLogListingName()))
		{
			MessageLogModule.UnregisterLogListing(GetLogListingName());
		}
	}

	FInsightsManager::Get()->GetSessionChangedEvent().RemoveAll(this);

	FLoadingProfilerCommands::Unregister();

	// Unregister tick function.
	FTSTicker::RemoveTicker(OnTickHandle);

	FLoadingProfilerManager::Instance.Reset();

	UE_LOGF(LogLoadingProfiler, Log, "Shutdown");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLoadingProfilerManager::~FLoadingProfilerManager()
{
	ensure(!bIsInitialized);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingProfilerManager::BindCommands()
{
	ActionManager.Map_ToggleTimingViewVisibility();
	ActionManager.Map_ToggleEventAggregationTreeViewVisibility();
	ActionManager.Map_ToggleObjectTypeAggregationTreeViewVisibility();
	ActionManager.Map_TogglePackageDetailsTreeViewVisibility();
	ActionManager.Map_ToggleExportDetailsTreeViewVisibility();
	ActionManager.Map_ToggleRequestsTreeViewVisibility();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingProfilerManager::RegisterMajorTabs(IUnrealInsightsModule& InsightsModule)
{
	if (!bIsTabRegistered)
	{
		bIsTabRegistered = true;
		const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(MajorTabId);
		if (Config.bIsAvailable)
		{
			// Register tab spawner for the Asset Loading Insights.
			FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(MajorTabId,
					FOnSpawnTab::CreateRaw(this, &FLoadingProfilerManager::SpawnTab),
					FCanSpawnTab::CreateRaw(this, &FLoadingProfilerManager::CanSpawnTab))
				.SetDisplayName(Config.TabLabel.IsSet() ? Config.TabLabel.GetValue() : LOCTEXT("LoadingProfilerTabTitle", "Asset Loading Insights"))
				.SetTooltipText(Config.TabTooltip.IsSet() ? Config.TabTooltip.GetValue() : LOCTEXT("LoadingProfilerTooltipText", "Open the Asset Loading Insights tab."))
				.SetIcon(Config.TabIcon.IsSet() ? Config.TabIcon.GetValue() : FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.LoadingProfiler"));

			TSharedRef<FWorkspaceItem> Group = Config.WorkspaceGroup.IsValid() ? Config.WorkspaceGroup.ToSharedRef() : FInsightsManager::Get()->GetInsightsMenuBuilder()->GetInsightsToolsGroup();
			TabSpawnerEntry.SetGroup(Group);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingProfilerManager::UnregisterMajorTabs()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(MajorTabId);
	bIsTabRegistered = false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> FLoadingProfilerManager::SpawnTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	// Register OnTabClosed to handle Loading profiler manager shutdown.
	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FLoadingProfilerManager::OnTabClosed));

	// Create the SLoadingProfilerWindow widget.
	TSharedRef<SLoadingProfilerWindow> Window = SNew(SLoadingProfilerWindow, DockTab, Args.GetOwnerWindow());
	DockTab->SetContent(Window);

	AssignProfilerWindow(Window);

	return DockTab;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FLoadingProfilerManager::CanSpawnTab(const FSpawnTabArgs& Args) const
{
	return bIsAvailable;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingProfilerManager::OnTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	OnWindowClosedEvent();

	RemoveProfilerWindow();

	// Disable TabClosed delegate.
	TabBeingClosed->SetOnTabClosed(SDockTab::FOnTabClosedCallback());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedRef<FUICommandList> FLoadingProfilerManager::GetCommandList() const
{
	return CommandList;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FLoadingProfilerCommands& FLoadingProfilerManager::GetCommands()
{
	return FLoadingProfilerCommands::Get();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLoadingProfilerActionManager& FLoadingProfilerManager::GetActionManager()
{
	return FLoadingProfilerManager::Instance->ActionManager;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FLoadingProfilerManager::Tick(float DeltaTime)
{
	// Check if session has Load Time events (to spawn the tab), but not too often.
	if (!bIsAvailable && AvailabilityCheck.Tick())
	{
		bool bIsProviderAvailable = false;

		TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();
		check(InsightsManager.IsValid());

		TSharedPtr<const TraceServices::IAnalysisSession> Session = InsightsManager->GetSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			if (Session->IsAnalysisComplete())
			{
				// Never check again during this session.
				AvailabilityCheck.Disable();
			}

			const TraceServices::ILoadTimeProfilerProvider* LoadTimeProfilerProvider = TraceServices::ReadLoadTimeProfilerProvider(*Session.Get());
			if (LoadTimeProfilerProvider)
			{
				bIsProviderAvailable = (LoadTimeProfilerProvider->GetTimelineCount() > 0);
			}
		}
		else
		{
			// Do not check again until the next session changed event (see OnSessionChanged).
			AvailabilityCheck.Disable();
		}

		if (bIsProviderAvailable)
		{
			bIsAvailable = true;

			FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
			MessageLogModule.RegisterLogListing(GetLogListingName(), LOCTEXT("LoadingInsights", "Loading Insights"));
			MessageLogModule.EnableMessageLogDisplay(true);

			if (FGlobalTabmanager::Get()->HasTabSpawner(MajorTabId))
			{
				UE_LOGF(LogLoadingProfiler, Log, "Opening the \"Asset Loading Insights\" tab...");
				FGlobalTabmanager::Get()->TryInvokeTab(MajorTabId);
			}
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingProfilerManager::OnSessionChanged()
{
	UE_LOGF(LogLoadingProfiler, Log, "OnSessionChanged");

	bIsAvailable = false;
	if (FInsightsManager::Get()->GetSession().IsValid())
	{
		AvailabilityCheck.Enable(0.3);
	}
	else
	{
		AvailabilityCheck.Disable();
	}

	TSharedPtr<SLoadingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->Reset();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingProfilerManager::ShowHideTimingView(const bool bIsVisible)
{
	bIsTimingViewVisible = bIsVisible;

	TSharedPtr<SLoadingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FLoadingProfilerTabs::TimingViewID, bIsVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingProfilerManager::ShowHideEventAggregationTreeView(const bool bIsVisible)
{
	bIsEventAggregationTreeViewVisible = bIsVisible;

	TSharedPtr<SLoadingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FLoadingProfilerTabs::EventAggregationTreeViewID, bIsVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingProfilerManager::ShowHideObjectTypeAggregationTreeView(const bool bIsVisible)
{
	bIsObjectTypeAggregationTreeViewVisible = bIsVisible;

	TSharedPtr<SLoadingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FLoadingProfilerTabs::ObjectTypeAggregationTreeViewID, bIsVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingProfilerManager::ShowHidePackageDetailsTreeView(const bool bIsVisible)
{
	bIsPackageDetailsTreeViewVisible = bIsVisible;

	TSharedPtr<SLoadingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FLoadingProfilerTabs::PackageDetailsTreeViewID, bIsVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingProfilerManager::ShowHideExportDetailsTreeView(const bool bIsVisible)
{
	bIsExportDetailsTreeViewVisible = bIsVisible;

	TSharedPtr<SLoadingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FLoadingProfilerTabs::ExportDetailsTreeViewID, bIsVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingProfilerManager::ShowHideRequestsTreeView(const bool bIsVisible)
{
	bIsRequestsTreeViewVisible = bIsVisible;

	TSharedPtr<SLoadingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FLoadingProfilerTabs::RequestsTreeViewID, bIsVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingProfilerManager::OnWindowClosedEvent()
{
	TSharedPtr<SLoadingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		TSharedPtr<TimingProfiler::STimingView> TimingView = Wnd->GetTimingView();
		if (TimingView)
		{
			TimingView->CloseQuickFindTab();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::LoadingProfiler

#undef LOCTEXT_NAMESPACE
