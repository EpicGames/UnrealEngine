// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Ticker.h"
#include "Framework/Commands/UICommandList.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Common/AvailabilityCheck.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/LoadingProfiler/LoadingProfilerCommands.h"

namespace UE::Insights::LoadingProfiler
{

class SLoadingProfilerWindow;

DECLARE_LOG_CATEGORY_EXTERN(LogLoadingProfiler, Log, All);

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * This class manages the Loading Profiler (Asset Loading Insights) state and settings.
 */
class FLoadingProfilerManager : public TSharedFromThis<FLoadingProfilerManager>, public IInsightsComponent
{
	friend class FLoadingProfilerActionManager;

public:
	/** Creates the Loading Profiler manager, only one instance can exist. */
	FLoadingProfilerManager(TSharedRef<FUICommandList> InCommandList);

	/** Virtual destructor. */
	virtual ~FLoadingProfilerManager();

	/** Creates an instance of the Loading Profiler manager. */
	static TSharedPtr<FLoadingProfilerManager> CreateInstance();

	/**
	 * @return the global instance of the Loading Profiler manager.
	 * This is an internal singleton and cannot be used outside TraceInsights.
	 * For external use:
	 *     IUnrealInsightsModule& Module = FModuleManager::Get().LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	 *     Module.GetLoadingProfilerManager();
	 */
	static TSharedPtr<FLoadingProfilerManager> Get();

	bool IsAvailable() const { return bIsAvailable; }

	//////////////////////////////////////////////////
	// IInsightsComponent

	virtual void Initialize(IUnrealInsightsModule& InsightsModule) override;
	virtual void Shutdown() override;
	virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) override;
	virtual void UnregisterMajorTabs() override;
	virtual void OnWindowClosedEvent() override;

	//////////////////////////////////////////////////

	/** @return UI command list for the Loading Profiler manager. */
	const TSharedRef<FUICommandList> GetCommandList() const;

	/** @return an instance of the Loading Profiler commands. */
	static const FLoadingProfilerCommands& GetCommands();

	/** @return an instance of the Loading Profiler action manager. */
	static FLoadingProfilerActionManager& GetActionManager();

	/**
	 * Converts profiler window weak pointer to a shared pointer and returns it.
	 * Make sure the returned pointer is valid before trying to dereference it.
	 */
	TSharedPtr<SLoadingProfilerWindow> GetProfilerWindow() const
	{
		return ProfilerWindowWeakPtr.Pin();
	}

	const FName& GetMajorTabId() const { return MajorTabId; }
	const FName& GetLogListingName() const { return LogListingName; }

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Getters and setters used by Toggle Commands.

	/** @return true, if the Timing view is visible */
	const bool IsTimingViewVisible() const { return bIsTimingViewVisible; }
	void SetTimingViewVisible(const bool bIsVisible) { bIsTimingViewVisible = bIsVisible; }
	void ShowHideTimingView(const bool bIsVisible);

	/** @return true, if the Event Aggregation tree view is visible */
	const bool IsEventAggregationTreeViewVisible() const { return bIsEventAggregationTreeViewVisible; }
	void SetEventAggregationTreeViewVisible(const bool bIsVisible) { bIsEventAggregationTreeViewVisible = bIsVisible; }
	void ShowHideEventAggregationTreeView(const bool bIsVisible);

	/** @return true, if the Object Type Aggregation tree view is visible */
	const bool IsObjectTypeAggregationTreeViewVisible() const { return bIsObjectTypeAggregationTreeViewVisible; }
	void SetObjectTypeAggregationTreeViewVisible(const bool bIsVisible) { bIsObjectTypeAggregationTreeViewVisible = bIsVisible; }
	void ShowHideObjectTypeAggregationTreeView(const bool bIsVisible);

	/** @return true, if the Package Details tree view is visible */
	const bool IsPackageDetailsTreeViewVisible() const { return bIsPackageDetailsTreeViewVisible; }
	void SetPackageDetailsTreeViewVisible(const bool bIsVisible) { bIsPackageDetailsTreeViewVisible = bIsVisible; }
	void ShowHidePackageDetailsTreeView(const bool bIsVisible);

	/** @return true, if the Export Details tree view is visible */
	const bool IsExportDetailsTreeViewVisible() const { return bIsExportDetailsTreeViewVisible; }
	void SetExportDetailsTreeViewVisible(const bool bIsVisible) { bIsExportDetailsTreeViewVisible = bIsVisible; }
	void ShowHideExportDetailsTreeView(const bool bIsVisible);

	/** @return true, if the Requests tree view is visible */
	const bool IsRequestsTreeViewVisible() const { return bIsRequestsTreeViewVisible; }
	void SetRequestsTreeViewVisible(const bool bIsVisible) { bIsRequestsTreeViewVisible = bIsVisible; }
	void ShowHideRequestsTreeView(const bool bIsVisible);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void OnSessionChanged();

private:
	/** Binds our UI commands to delegates. */
	void BindCommands();

	/** Called to spawn the Loading Profiler major tab. */
	TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args);

	bool CanSpawnTab(const FSpawnTabArgs& Args) const;

	/** Callback called when the Loading Profiler major tab is closed. */
	void OnTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	/** Updates this manager, done through FCoreTicker. */
	bool Tick(float DeltaTime);

	void AssignProfilerWindow(const TSharedRef<SLoadingProfilerWindow>& InProfilerWindow)
	{
		ProfilerWindowWeakPtr = InProfilerWindow;
	}

	void RemoveProfilerWindow()
	{
		ProfilerWindowWeakPtr.Reset();
	}

private:
	bool bIsInitialized = false;
	bool bIsAvailable = false;

	/** If the tab is registered in the tab manager. */
	bool bIsTabRegistered = false;

	FAvailabilityCheck AvailabilityCheck;

	/** The delegate to be invoked when this manager ticks. */
	FTickerDelegate OnTick;

	/** Handle to the registered OnTick. */
	FTSTicker::FDelegateHandle OnTickHandle;

	/** List of UI commands for this manager. This will be filled by this and corresponding classes. */
	TSharedRef<FUICommandList> CommandList;

	/** An instance of the Loading Profiler action manager. */
	FLoadingProfilerActionManager ActionManager;

	/** A weak pointer to the Asset Loading Insights window. */
	TWeakPtr<SLoadingProfilerWindow> ProfilerWindowWeakPtr;

	/** If the Timing view is visible or hidden. */
	bool bIsTimingViewVisible = false;

	/** If the Event Aggregation tree view is visible or hidden. */
	bool bIsEventAggregationTreeViewVisible = false;

	/** If the Object Type Aggregation tree view is visible or hidden. */
	bool bIsObjectTypeAggregationTreeViewVisible = false;

	/** If the Package Details tree view is visible or hidden. */
	bool bIsPackageDetailsTreeViewVisible = false;

	/** If the Export Details tree view is visible or hidden. */
	bool bIsExportDetailsTreeViewVisible = false;

	/** If the Requests tree view is visible or hidden. */
	bool bIsRequestsTreeViewVisible = false;

	/** The id of the Asset Loading Insights major tab. */
	FName MajorTabId;

	/** The name of the Loading Profiler log listing. */
	FName LogListingName;

	/** A shared pointer to the global instance of the Loading Profiler manager. */
	static TSharedPtr<FLoadingProfilerManager> Instance;
};

} // namespace UE::Insights::LoadingProfiler
