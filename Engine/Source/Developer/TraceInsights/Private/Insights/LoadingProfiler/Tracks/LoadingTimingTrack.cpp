// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoadingTimingTrack.h"

// TraceServices
#include "TraceServices/Containers/Timelines.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/LoadTimeProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Filter/ViewModels/FilterConfigurator.h"
#include "InsightsCore/Filter/ViewModels/Filters.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/LoadingProfiler/ViewModels/LoadingSharedState.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"

#define LOCTEXT_NAMESPACE "UE::Insights::LoadingProfiler"

namespace UE::Insights::LoadingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FLoadingTimingTrackFilterContext
////////////////////////////////////////////////////////////////////////////////////////////////////

class FLoadingTimingTrackFilterContext : public FBaseFilterContext
{
public:
	FLoadingTimingTrackFilterContext(const FLoadingTimingTrack& InTrack)
		: Track(InTrack)
	{
	}
	virtual ~FLoadingTimingTrackFilterContext() = default;

	virtual bool HasData(int32 Key) const override
	{
		switch (Key)
		{
		case static_cast<int32>(EFilterField::StartTime):
		case static_cast<int32>(EFilterField::EndTime):
		case static_cast<int32>(EFilterField::Duration):
		case static_cast<int32>(EFilterField::TrackName):
			return true;
		default:
			return false;
		}
	}

	virtual EFilterDataType GetDataType(int32 Key) const override
	{
		switch (Key)
		{
		case static_cast<int32>(EFilterField::StartTime):
		case static_cast<int32>(EFilterField::EndTime):
		case static_cast<int32>(EFilterField::Duration):
			return EFilterDataType::Double;
		case static_cast<int32>(EFilterField::TrackName):
			return EFilterDataType::String;
		default:
			return EFilterDataType::Void;
		}
	}

	virtual double GetDataAsDouble(int32 Key) const override
	{
		if (Key == static_cast<int32>(EFilterField::StartTime))
		{
			return EventStartTime;
		}
		if (Key == static_cast<int32>(EFilterField::EndTime))
		{
			return EventEndTime;
		}
		if (Key == static_cast<int32>(EFilterField::Duration))
		{
			return EventEndTime - EventStartTime;
		}
		return 0.0;
	}

	virtual const FString& GetDataAsString(int32 Key) const override
	{
		if (Key == static_cast<int32>(EFilterField::TrackName))
		{
			return Track.GetName();
		}
		return EmptyString;
	}

	void SetEvent(double StartTime, double EndTime, uint32 Depth, const TraceServices::FLoadTimeProfilerCpuEvent& InEvent)
	{
		EventStartTime = StartTime;
		EventEndTime = EndTime;
		EventDepth = Depth;
		Event = &InEvent;
	}

private:
	const FLoadingTimingTrack& Track;
	double EventStartTime = 0.0;
	double EventEndTime = 0.0;
	uint32 EventDepth = 0;
	const TraceServices::FLoadTimeProfilerCpuEvent* Event;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FLoadingTimingTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FLoadingTimingTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingTimingTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	if (const TraceServices::ILoadTimeProfilerProvider* LoadTimeProfilerProvider = TraceServices::ReadLoadTimeProfilerProvider(*Session.Get()))
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const FTimingTrackViewport& Viewport = Context.GetViewport();

		LoadTimeProfilerProvider->ReadTimeline(TimelineIndex,
			[this, &Builder, &Viewport]
			(const TraceServices::ILoadTimeProfilerProvider::CpuTimeline& Timeline)
			{
				if (FTimingEventsTrack::bUseDownSampling)
				{
					const double SecondsPerPixel = 1.0 / Viewport.GetScaleX();
					Timeline.EnumerateEventsDownSampled(Viewport.GetStartTime(), Viewport.GetEndTime(), SecondsPerPixel,
						[this, &Builder]
						(double StartTime, double EndTime, uint32 Depth, const TraceServices::FLoadTimeProfilerCpuEvent& Event)
						{
							if (Event.Package)
							{
								const TCHAR* Name = SharedState.GetEventName(Depth, Event);
								const uint64 Type = static_cast<uint64>(Event.EventType);
								const uint32 Color = 0;
								Builder.AddEvent(StartTime, EndTime, Depth, Name, Type, Color);
							}
							return TraceServices::EEventEnumerate::Continue;
						});
				}
				else
				{
					Timeline.EnumerateEvents(Viewport.GetStartTime(), Viewport.GetEndTime(),
						[this, &Builder]
						(double StartTime, double EndTime, uint32 Depth, const TraceServices::FLoadTimeProfilerCpuEvent& Event)
						{
							if (Event.Package)
							{
								const TCHAR* Name = SharedState.GetEventName(Depth, Event);
								const uint64 Type = static_cast<uint64>(Event.EventType);
								const uint32 Color = 0;
								Builder.AddEvent(StartTime, EndTime, Depth, Name, Type, Color);
							}
							return TraceServices::EEventEnumerate::Continue;
						});
				}
			}); // ReadTimeline
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingTimingTrack::BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	if (HasCustomFilter())
	{
		if (const TraceServices::ILoadTimeProfilerProvider* LoadTimeProfilerProvider = TraceServices::ReadLoadTimeProfilerProvider(*Session.Get()))
		{
			FLoadingTimingTrackFilterContext FilterContext(*this);
			const FTimingTrackViewport& Viewport = Context.GetViewport();

			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			LoadTimeProfilerProvider->ReadTimeline(TimelineIndex,
				[this, &Builder, &Viewport, &FilterContext]
				(const TraceServices::ILoadTimeProfilerProvider::CpuTimeline& Timeline)
				{
					Timeline.EnumerateEvents(Viewport.GetStartTime(), Viewport.GetEndTime(),
						[this, &Builder, &FilterContext]
						(double StartTime, double EndTime, uint32 Depth, const TraceServices::FLoadTimeProfilerCpuEvent& Event)
						{
							if (Event.Package)
							{
								FilterContext.SetEvent(StartTime, EndTime, Depth, Event);
								if (FilterConfigurator->ApplyFilters(FilterContext))
								{
									const TCHAR* Name = SharedState.GetEventName(Depth, Event);
									const uint64 Type = static_cast<uint64>(Event.EventType);
									const uint32 Color = 0;
									Builder.AddEvent(StartTime, EndTime, Depth, Name, Type, Color);
								}
							}
							return TraceServices::EEventEnumerate::Continue;
						}); // EnumerateEvents
				}); // ReadTimeline
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingTimingTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	if (InTooltipEvent.CheckTrack(this) && InTooltipEvent.Is<FTimingEvent>())
	{
		const FTimingEvent& TooltipEvent = InTooltipEvent.As<FTimingEvent>();

		auto MatchEvent = [&TooltipEvent](double InStartTime, double InEndTime, uint32 InDepth)
		{
			return InDepth == TooltipEvent.GetDepth()
				&& InStartTime == TooltipEvent.GetStartTime()
				&& InEndTime == TooltipEvent.GetEndTime();
		};

		FTimingEventSearchParameters SearchParameters(TooltipEvent.GetStartTime(), TooltipEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch, MatchEvent);
		FindLoadTimeProfilerCpuEvent(SearchParameters, [this, &InOutTooltip, &TooltipEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const TraceServices::FLoadTimeProfilerCpuEvent& InFoundEvent)
		{
			InOutTooltip.ResetContent();

			InOutTooltip.AddTitle(SharedState.GetEventName(TooltipEvent.GetDepth(), InFoundEvent));

			const TraceServices::FPackageExportInfo* Export = InFoundEvent.Export;
			const TraceServices::FPackageInfo* Package = InFoundEvent.Export ? InFoundEvent.Export->Package : InFoundEvent.Package;

			InOutTooltip.AddNameValueTextLine(TEXT("Duration:"), UE::Insights::FormatTimeAuto(TooltipEvent.GetDuration()));
			InOutTooltip.AddNameValueTextLine(TEXT("Depth:"), FString::Printf(TEXT("%d"), TooltipEvent.GetDepth()));

			if (Package)
			{
				InOutTooltip.AddNameValueTextLine(TEXT("Package Name:"), Package->Name);
				InOutTooltip.AddNameValueTextLine(TEXT("Header Size:"), FString::Printf(TEXT("%s bytes"), *FText::AsNumber(Package->Summary.TotalHeaderSize).ToString()));
				InOutTooltip.AddNameValueTextLine(TEXT("Package Summary:"), FString::Printf(TEXT("%d imports, %d exports"), Package->Summary.ImportCount, Package->Summary.ExportCount));
				InOutTooltip.AddNameValueTextLine(TEXT("Request Priority:"), FString::Printf(TEXT("%d"), Package->Summary.Priority));
				if (!Export)
				{
					InOutTooltip.AddNameValueTextLine(TEXT("Event:"), TEXT("ProcessPackageSummary"));
				}
			}

			if (Export)
			{
				InOutTooltip.AddNameValueTextLine(TEXT("Event:"), FString::Printf(TEXT("%s"), TraceServices::GetLoadTimeProfilerObjectEventTypeString(InFoundEvent.EventType)));
				InOutTooltip.AddNameValueTextLine(TEXT("Export Class:"), Export->Class ? Export->Class->Name : TEXT("N/A"));
				InOutTooltip.AddNameValueTextLine(TEXT("Serial Size:"), FString::Printf(TEXT("%s bytes"), *FText::AsNumber(Export->SerialSize).ToString()));
			}

			InOutTooltip.UpdateLayout();
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FLoadingTimingTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindLoadTimeProfilerCpuEvent(InSearchParameters,
		[this, &FoundEvent]
		(double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const TraceServices::FLoadTimeProfilerCpuEvent& InFoundEvent)
		{
			FoundEvent = MakeShared<FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth);
		});

	return FoundEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FLoadingTimingTrack::FindLoadTimeProfilerCpuEvent(
	const FTimingEventSearchParameters& InParameters,
	TFunctionRef<void(double, double, uint32, const TraceServices::FLoadTimeProfilerCpuEvent&)> InFoundPredicate) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return false;
	}

	FLoadingTimingTrackFilterContext FilterContext(*this);

	return TTimingEventSearch<TraceServices::FLoadTimeProfilerCpuEvent>::Search(

		InParameters,

		// Search Predicate
		[this]
		(TTimingEventSearch<TraceServices::FLoadTimeProfilerCpuEvent>::FContext& InContext)
		{
			TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
			if (Session.IsValid())
			{
				if (const TraceServices::ILoadTimeProfilerProvider* LoadTimeProfilerProvider = TraceServices::ReadLoadTimeProfilerProvider(*Session.Get()))
				{
					TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

					LoadTimeProfilerProvider->ReadTimeline(TimelineIndex,
						[&InContext]
						(const TraceServices::ILoadTimeProfilerProvider::CpuTimeline& Timeline)
						{
							auto Callback =
								[&InContext]
								(double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FLoadTimeProfilerCpuEvent& Event)
								{
									if (!Event.Package)
									{
										return TraceServices::EEventEnumerate::Continue;
									}
									InContext.Check(EventStartTime, EventEndTime, EventDepth, Event);
									return InContext.ShouldContinueSearching() ?
										TraceServices::EEventEnumerate::Continue :
										TraceServices::EEventEnumerate::Stop;
								};

							if (InContext.GetParameters().SearchDirection == FTimingEventSearchParameters::ESearchDirection::Forward)
							{
								Timeline.EnumerateEvents(InContext.GetParameters().StartTime, InContext.GetParameters().EndTime, Callback);
							}
							else
							{
								Timeline.EnumerateEventsBackwards(InContext.GetParameters().EndTime, InContext.GetParameters().StartTime, Callback);
							}
						}); // ReadTimeline
				}
			}
		},

		// Filter Predicate
		[&FilterContext, &InParameters]
		(double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FLoadTimeProfilerCpuEvent& Event)
		{
			if (!InParameters.FilterExecutor.IsValid())
			{
				return true;
			}

			FilterContext.SetEvent(EventStartTime, EventEndTime, EventDepth, Event);
			return InParameters.FilterExecutor->ApplyFilters(FilterContext);
		},

		// Found Predicate
		InFoundPredicate,

		// Payload Matched Predicate
		TTimingEventSearch<TraceServices::FLoadTimeProfilerCpuEvent>::NoMatch

	); // Search
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FLoadingTimingTrack::HasCustomFilter() const
{
	return FilterConfigurator.IsValid() && !FilterConfigurator->IsEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingTimingTrack::SetFilterConfigurator(TSharedPtr<FFilterConfigurator> InFilterConfigurator)
{
	if (FilterConfigurator != InFilterConfigurator)
	{
		FilterConfigurator = InFilterConfigurator;
		SetDirtyFlag();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::LoadingProfiler

#undef LOCTEXT_NAMESPACE
