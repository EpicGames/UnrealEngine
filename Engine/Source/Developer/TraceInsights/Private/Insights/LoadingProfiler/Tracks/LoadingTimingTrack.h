// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Framework/Commands/Commands.h"

// TraceInsights
#include "Insights/ViewModels/TimingEventsTrack.h"

namespace TraceServices { struct FLoadTimeProfilerCpuEvent; }

namespace UE::Insights { class FFilterConfigurator; }

class FTimingEventSearchParameters;

namespace UE::Insights::LoadingProfiler
{

class FLoadingSharedState;
class STimingView;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FLoadingTimingTrack : public FTimingEventsTrack
{
	INSIGHTS_DECLARE_RTTI(FLoadingTimingTrack, FTimingEventsTrack)

public:
	explicit FLoadingTimingTrack(FLoadingSharedState& InSharedState, uint32 InTimelineIndex, const FString& InName)
		: FTimingEventsTrack(InName)
		, SharedState(InSharedState)
		, TimelineIndex(InTimelineIndex)
	{
	}
	virtual ~FLoadingTimingTrack() = default;

	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;

	virtual void BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;

	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;

	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;

	virtual void SetFilterConfigurator(TSharedPtr<UE::Insights::FFilterConfigurator> InFilterConfigurator) override;

protected:
	// Helper function to find an event given search parameters
	bool FindLoadTimeProfilerCpuEvent(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const TraceServices::FLoadTimeProfilerCpuEvent&)> InFoundPredicate) const;

	virtual bool HasCustomFilter() const override;

protected:
	FLoadingSharedState& SharedState;
	uint32 TimelineIndex;

	TSharedPtr<UE::Insights::FFilterConfigurator> FilterConfigurator;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::LoadingProfiler
