// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoadingSharedState.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

// TraceServices
#include "TraceServices/Containers/Timelines.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/LoadTimeProfiler.h"
#include "TraceServices/Model/Threads.h"

// TraceInsights
#include "Insights/LoadingProfiler/Tracks/LoadingTimingTrack.h"
#include "Insights/LoadingProfiler/ViewModels/LoadingTimingViewCommands.h"
#include "Insights/Widgets/STimingView.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"

#define LOCTEXT_NAMESPACE "UE::Insights::LoadingProfiler"

namespace UE::Insights::LoadingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FLoadingSharedState::FLoadingSharedState(UE::Insights::TimingProfiler::STimingView* InTimingView)
	: TimingView(InTimingView)
	, bShowHideAllLoadingTracks(false)
	//, LoadingTracks
	, LoadTimeProfilerTimelineCount(0)
	//, GetEventNameDelegate
{
	check(TimingView != nullptr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingSharedState::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	if (TimingView->GetName() == FInsightsManagerTabs::LoadingProfilerTabId)
	{
		bShowHideAllLoadingTracks = true;
	}
	else
	{
		bShowHideAllLoadingTracks = false;
	}

	LoadingTracks.Reset();

	LoadTimeProfilerTimelineCount = 0;

	SetColorSchema(1);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingSharedState::OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	bShowHideAllLoadingTracks = false;

	LoadingTracks.Reset();

	LoadTimeProfilerTimelineCount = 0;

	GetEventNameDelegate = nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingSharedState::Tick(UE::Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	const TraceServices::ILoadTimeProfilerProvider* LoadTimeProfilerProvider = TraceServices::ReadLoadTimeProfilerProvider(InAnalysisSession);
	if (LoadTimeProfilerProvider)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(InAnalysisSession);

		const uint64 CurrentLoadTimeProfilerTimelineCount = LoadTimeProfilerProvider->GetTimelineCount();
		if (CurrentLoadTimeProfilerTimelineCount != LoadTimeProfilerTimelineCount)
		{
			LoadTimeProfilerTimelineCount = CurrentLoadTimeProfilerTimelineCount;

			// Iterate through threads.
			const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(InAnalysisSession);
			ThreadProvider.EnumerateThreads([this, &InSession, LoadTimeProfilerProvider](const TraceServices::FThreadInfo& ThreadInfo)
				{
					// Check available Asset Loading tracks.
					uint32 LoadingTimelineIndex;
					if (LoadTimeProfilerProvider->GetCpuThreadTimelineIndex(ThreadInfo.Id, LoadingTimelineIndex))
					{
						if (!LoadingTracks.Contains(LoadingTimelineIndex))
						{
							//const TCHAR* const GroupName = ThreadInfo.GroupName ? ThreadInfo.GroupName : ThreadInfo.Name;
							const FString TrackName(ThreadInfo.Name && *ThreadInfo.Name ? FString::Printf(TEXT("Loading - %s"), ThreadInfo.Name) : FString::Printf(TEXT("Loading - Thread %u"), ThreadInfo.Id));
							TSharedRef<FLoadingTimingTrack> LoadingThreadTrack = MakeShared<FLoadingTimingTrack>(*this, LoadingTimelineIndex, TrackName);
							static_assert(FTimingTrackOrder::GroupRange > 1000, "Order group range too small");
							LoadingThreadTrack->SetOrder(FTimingTrackOrder::Cpu - 1000 + LoadingTracks.Num() * 10);
							LoadingThreadTrack->SetVisibilityFlag(bShowHideAllLoadingTracks);
							InSession.AddScrollableTrack(LoadingThreadTrack);
							LoadingTracks.Add(LoadingTimelineIndex, LoadingThreadTrack);
						}
					}
				});
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingSharedState::ExtendOtherTracksFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InOutMenuBuilder)
{
	if (&InSession != TimingView)
	{
		return;
	}

	InOutMenuBuilder.BeginSection("Asset Loading", LOCTEXT("ContextMenu_Section_AssetLoading", "Asset Loading"));
	{
		InOutMenuBuilder.AddMenuEntry(FLoadingTimingViewCommands::Get().ShowHideAllLoadingTracks);
	}
	InOutMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingSharedState::BindCommands()
{
	FLoadingTimingViewCommands::Register();

	TSharedPtr<FUICommandList> CommandList = TimingView->GetCommandList();
	ensure(CommandList.IsValid());

	CommandList->MapAction(
		FLoadingTimingViewCommands::Get().ShowHideAllLoadingTracks,
		FExecuteAction::CreateSP(this, &FLoadingSharedState::ShowHideAllLoadingTracks),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FLoadingSharedState::IsAllLoadingTracksToggleOn));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingSharedState::SetAllLoadingTracksToggle(bool bOnOff)
{
	bShowHideAllLoadingTracks = bOnOff;

	for (const auto& KV : LoadingTracks)
	{
		FLoadingTimingTrack& Track = *KV.Value;
		Track.SetVisibilityFlag(bShowHideAllLoadingTracks);
	}

	TimingView->HandleTrackVisibilityChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FLoadingSharedState::GetEventNameByEventType(uint32 Depth, const TraceServices::FLoadTimeProfilerCpuEvent& Event) const
{
	if (Event.Export)
	{
		return TraceServices::GetLoadTimeProfilerObjectEventTypeString(Event.EventType);
	}
	else
	{
		return TEXT("ProcessPackageHeader");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FLoadingSharedState::GetEventNameByPackageName(uint32 Depth, const TraceServices::FLoadTimeProfilerCpuEvent& Event) const
{
	return Event.Package ? Event.Package->Name : TEXT("");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FLoadingSharedState::GetEventNameByExportClassName(uint32 Depth, const TraceServices::FLoadTimeProfilerCpuEvent& Event) const
{
	return Event.Export && Event.Export->Class ? Event.Export->Class->Name : TEXT("");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FLoadingSharedState::GetEventNameByPackageAndExportClassName(uint32 Depth, const TraceServices::FLoadTimeProfilerCpuEvent& Event) const
{
	if (Depth == 0)
	{
		if (Event.Package)
		{
			return Event.Package->Name;
		}
	}

	if (Event.Export && Event.Export->Class)
	{
		return Event.Export->Class->Name;
	}

	return TEXT("");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FLoadingSharedState::GetEventName(uint32 Depth, const TraceServices::FLoadTimeProfilerCpuEvent& Event) const
{
	if (GetEventNameDelegate.IsBound())
	{
		return GetEventNameDelegate.Execute(Depth, Event);
	}
	return TEXT("");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLoadingSharedState::SetColorSchema(int32 Schema)
{
	switch (Schema)
	{
	case 0: GetEventNameDelegate = FLoadingTrackGetEventNameDelegate::CreateRaw(this, &FLoadingSharedState::GetEventNameByEventType); break;
	case 1: GetEventNameDelegate = FLoadingTrackGetEventNameDelegate::CreateRaw(this, &FLoadingSharedState::GetEventNameByPackageName); break;
	case 2: GetEventNameDelegate = FLoadingTrackGetEventNameDelegate::CreateRaw(this, &FLoadingSharedState::GetEventNameByExportClassName); break;
	case 3: GetEventNameDelegate = FLoadingTrackGetEventNameDelegate::CreateRaw(this, &FLoadingSharedState::GetEventNameByPackageAndExportClassName); break;
	default: GetEventNameDelegate.Unbind();
	};

	for (const auto& KV : LoadingTracks)
	{
		FLoadingTimingTrack& Track = *KV.Value;
		Track.SetDirtyFlag();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::LoadingProfiler

#undef LOCTEXT_NAMESPACE
