// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Map.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"

// TraceInsights
#include "Insights/ITimingViewExtender.h"

namespace TraceServices
{
	struct FLoadTimeProfilerCpuEvent;
}

namespace UE::Insights::TimingProfiler
{
	class STimingView;
}

namespace UE::Insights::LoadingProfiler
{

class FLoadingTimingTrack;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Defines FLoadingTrackGetEventNameDelegate delegate interface. Returns the name for a timing event in a Loading track. */
DECLARE_DELEGATE_RetVal_TwoParams(const TCHAR*, FLoadingTrackGetEventNameDelegate, uint32 /*Depth*/, const TraceServices::FLoadTimeProfilerCpuEvent& /*Event*/);

////////////////////////////////////////////////////////////////////////////////////////////////////

class FLoadingSharedState : public UE::Insights::Timing::ITimingViewExtender, public TSharedFromThis<FLoadingSharedState>
{
public:
	explicit FLoadingSharedState(UE::Insights::TimingProfiler::STimingView* InTimingView);
	virtual ~FLoadingSharedState() = default;

	//////////////////////////////////////////////////
	// ITimingViewExtender interface

	virtual void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void Tick(UE::Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;
	virtual void ExtendOtherTracksFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;

	//////////////////////////////////////////////////

	void BindCommands();

	const TCHAR* GetEventName(uint32 Depth, const TraceServices::FLoadTimeProfilerCpuEvent& Event) const;
	void SetColorSchema(int32 Schema);

	TSharedPtr<FLoadingTimingTrack> GetLoadingTrack(uint32 InThreadId)
	{
		TSharedPtr<FLoadingTimingTrack>* TrackPtrPtr = LoadingTracks.Find(InThreadId);
		return TrackPtrPtr ? *TrackPtrPtr : nullptr;
	}

	bool IsAllLoadingTracksToggleOn() const { return bShowHideAllLoadingTracks; }
	void SetAllLoadingTracksToggle(bool bOnOff);
	void ShowAllLoadingTracks() { SetAllLoadingTracksToggle(true); }
	void HideAllLoadingTracks() { SetAllLoadingTracksToggle(false); }
	void ShowHideAllLoadingTracks() { SetAllLoadingTracksToggle(!IsAllLoadingTracksToggleOn()); }

private:
	const TCHAR* GetEventNameByEventType(uint32 Depth, const TraceServices::FLoadTimeProfilerCpuEvent& Event) const;
	const TCHAR* GetEventNameByPackageName(uint32 Depth, const TraceServices::FLoadTimeProfilerCpuEvent& Event) const;
	const TCHAR* GetEventNameByExportClassName(uint32 Depth, const TraceServices::FLoadTimeProfilerCpuEvent& Event) const;
	const TCHAR* GetEventNameByPackageAndExportClassName(uint32 Depth, const TraceServices::FLoadTimeProfilerCpuEvent& Event) const;

private:
	UE::Insights::TimingProfiler::STimingView* TimingView;

	bool bShowHideAllLoadingTracks;

	/** Maps thread id to track pointer. */
	TMap<uint32, TSharedPtr<FLoadingTimingTrack>> LoadingTracks;

	uint64 LoadTimeProfilerTimelineCount;

	FLoadingTrackGetEventNameDelegate GetEventNameDelegate;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::LoadingProfiler
