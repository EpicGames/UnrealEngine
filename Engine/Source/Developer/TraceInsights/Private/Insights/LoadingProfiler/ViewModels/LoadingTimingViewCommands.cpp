// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoadingTimingViewCommands.h"

// TraceInsights
#include "Insights/InsightsStyle.h"

#define LOCTEXT_NAMESPACE "UE::Insights::LoadingProfiler"

namespace UE::Insights::LoadingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FLoadingTimingViewCommands::FLoadingTimingViewCommands()
	: TCommands<FLoadingTimingViewCommands>(
		TEXT("LoadingTimingViewCommands"),
		NSLOCTEXT("Contexts", "LoadingTimingViewCommands", "Insights - Timing View - Asset Loading"),
		NAME_None,
		FInsightsStyle::GetStyleSetName())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLoadingTimingViewCommands::~FLoadingTimingViewCommands()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// UI_COMMAND takes long for the compiler to optimize
UE_DISABLE_OPTIMIZATION_SHIP
void FLoadingTimingViewCommands::RegisterCommands()
{
	UI_COMMAND(ShowHideAllLoadingTracks,
		"Asset Loading Tracks",
		"Shows/hides the Asset Loading tracks.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::L));
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////

} // UE::Insights::LoadingProfiler

#undef LOCTEXT_NAMESPACE
