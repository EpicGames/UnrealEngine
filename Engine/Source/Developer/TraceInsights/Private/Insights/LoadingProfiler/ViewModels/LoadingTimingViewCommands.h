// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Framework/Commands/Commands.h"
#include "Templates/SharedPointer.h"

namespace UE::Insights::LoadingProfiler
{
	
class FLoadingTimingViewCommands : public TCommands<FLoadingTimingViewCommands>
{
public:
	FLoadingTimingViewCommands();
	virtual ~FLoadingTimingViewCommands();
	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> ShowHideAllLoadingTracks;
};

} // namespace UE::Insights::LoadingProfiler
