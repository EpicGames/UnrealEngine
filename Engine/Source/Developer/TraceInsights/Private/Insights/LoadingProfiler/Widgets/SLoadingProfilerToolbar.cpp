// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLoadingProfilerToolbar.h"

#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SBoxPanel.h"

// TraceInsights
#include "Insights/InsightsCommands.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/LoadingProfiler/LoadingProfilerCommands.h"

namespace UE::Insights::LoadingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

SLoadingProfilerToolbar::SLoadingProfilerToolbar()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SLoadingProfilerToolbar::~SLoadingProfilerToolbar()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerToolbar::Construct(const FArguments& InArgs, const FInsightsMajorTabConfig& Config)
{
	struct Local
	{
		static void FillViewToolbar(FToolBarBuilder& ToolbarBuilder, const FArguments &InArgs, const FInsightsMajorTabConfig& Config)
		{
			ToolbarBuilder.BeginSection("View");
			{
				if (Config.ShouldRegisterMinorTab(FLoadingProfilerTabs::TimingViewID))
				{
					ToolbarBuilder.AddToolBarButton(FLoadingProfilerCommands::Get().ToggleTimingViewVisibility,
						NAME_None, TAttribute<FText>(), TAttribute<FText>(),
						FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TimingView.ToolBar"));
				}
				if (Config.ShouldRegisterMinorTab(FLoadingProfilerTabs::EventAggregationTreeViewID))
				{
					ToolbarBuilder.AddToolBarButton(FLoadingProfilerCommands::Get().ToggleEventAggregationTreeViewVisibility,
						NAME_None, TAttribute<FText>(), TAttribute<FText>(),
						FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TableTreeView.ToolBar"));
				}
				if (Config.ShouldRegisterMinorTab(FLoadingProfilerTabs::ObjectTypeAggregationTreeViewID))
				{
					ToolbarBuilder.AddToolBarButton(FLoadingProfilerCommands::Get().ToggleObjectTypeAggregationTreeViewVisibility,
						NAME_None, TAttribute<FText>(), TAttribute<FText>(),
						FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TableTreeView.ToolBar"));
				}
				if (Config.ShouldRegisterMinorTab(FLoadingProfilerTabs::PackageDetailsTreeViewID))
				{
					ToolbarBuilder.AddToolBarButton(FLoadingProfilerCommands::Get().TogglePackageDetailsTreeViewVisibility,
						NAME_None, TAttribute<FText>(), TAttribute<FText>(),
						FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TableTreeView.ToolBar"));
				}
				if (Config.ShouldRegisterMinorTab(FLoadingProfilerTabs::ExportDetailsTreeViewID))
				{
					ToolbarBuilder.AddToolBarButton(FLoadingProfilerCommands::Get().ToggleExportDetailsTreeViewVisibility,
						NAME_None, TAttribute<FText>(), TAttribute<FText>(),
						FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TableTreeView.ToolBar"));
				}
				if (Config.ShouldRegisterMinorTab(FLoadingProfilerTabs::RequestsTreeViewID))
				{
					ToolbarBuilder.AddToolBarButton(FLoadingProfilerCommands::Get().ToggleRequestsTreeViewVisibility,
						NAME_None, TAttribute<FText>(), TAttribute<FText>(),
						FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TableTreeView.ToolBar"));
				}
			}
			ToolbarBuilder.EndSection();

			if (InArgs._ToolbarExtender.IsValid())
			{
				InArgs._ToolbarExtender->Apply("MainToolbar", EExtensionHook::First, ToolbarBuilder);
			}
		}

		static void FillRightSideToolbar(FToolBarBuilder& ToolbarBuilder, const FArguments &InArgs)
		{
			ToolbarBuilder.BeginSection("Debug");
			{
				ToolbarBuilder.AddToolBarButton(FInsightsCommands::Get().ToggleDebugInfo,
					NAME_None, FText(), TAttribute<FText>(),
					FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.Debug.ToolBar"));
			}
			ToolbarBuilder.EndSection();

			if (InArgs._ToolbarExtender.IsValid())
			{
				InArgs._ToolbarExtender->Apply("RightSideToolbar", EExtensionHook::First, ToolbarBuilder);
			}
		}
	};

	TSharedPtr<FUICommandList> CommandList = FInsightsManager::Get()->GetCommandList();

	FSlimHorizontalToolBarBuilder ToolbarBuilder(CommandList.ToSharedRef(), FMultiBoxCustomization::None);
	ToolbarBuilder.SetStyle(&FInsightsStyle::Get(), "PrimaryToolbar");
	Local::FillViewToolbar(ToolbarBuilder, InArgs, Config);

	FSlimHorizontalToolBarBuilder RightSideToolbarBuilder(CommandList.ToSharedRef(), FMultiBoxCustomization::None);
	RightSideToolbarBuilder.SetStyle(&FInsightsStyle::Get(), "PrimaryToolbar");
	Local::FillRightSideToolbar(RightSideToolbarBuilder, InArgs);

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.FillWidth(1.0)
		.Padding(0.0f)
		[
			ToolbarBuilder.MakeWidget()
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0.0f)
		[
			RightSideToolbarBuilder.MakeWidget()
		]
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::LoadingProfiler
