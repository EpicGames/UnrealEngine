// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FExtender;
struct FInsightsMajorTabConfig;

namespace UE::Insights::LoadingProfiler
{

/** Ribbon based toolbar used as a main menu in the Profiler window. */
class SLoadingProfilerToolbar : public SCompoundWidget
{
public:
	/** Default constructor. */
	SLoadingProfilerToolbar();

	/** Virtual destructor. */
	virtual ~SLoadingProfilerToolbar();

	SLATE_BEGIN_ARGS(SLoadingProfilerToolbar) {}
	SLATE_ARGUMENT(TSharedPtr<FExtender>, ToolbarExtender);
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs, const FInsightsMajorTabConfig& Config);
};

} // namespace UE::Insights::LoadingProfiler
