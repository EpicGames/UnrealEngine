// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLoadingProfilerWindow.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Docking/SDockTab.h"

// TraceServices
#include "TraceServices/Model/LoadTimeProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/UntypedTable.h"
#include "InsightsCore/Table/Widgets/SUntypedTableTreeView.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/LoadingProfiler/LoadingProfilerManager.h"
#include "Insights/LoadingProfiler/Widgets/SLoadingProfilerToolbar.h"
#include "Insights/TimingProfiler/Widgets/SUserAnnotationsPanel.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::LoadingProfiler"

const FName FLoadingProfilerTabs::TimingViewID(TEXT("TimingView"));
const FName FLoadingProfilerTabs::EventAggregationTreeViewID(TEXT("EventAggregation"));
const FName FLoadingProfilerTabs::ObjectTypeAggregationTreeViewID(TEXT("ObjectTypeAggregation"));
const FName FLoadingProfilerTabs::PackageDetailsTreeViewID(TEXT("PackageDetails"));
const FName FLoadingProfilerTabs::ExportDetailsTreeViewID(TEXT("ExportDetails"));
const FName FLoadingProfilerTabs::RequestsTreeViewID(TEXT("Requests"));
const FName FLoadingProfilerTabs::UserAnnotationsID(TEXT("UserAnnotations"));

namespace UE::Insights::LoadingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

SLoadingProfilerWindow::SLoadingProfilerWindow()
	: SMajorTabWindow(FInsightsManagerTabs::LoadingProfilerTabId)
	, SelectionStartTime(0.0f)
	, SelectionEndTime(0.0f)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SLoadingProfilerWindow::~SLoadingProfilerWindow()
{
	CloseAllOpenTabs();

	check(RequestsTreeView == nullptr);
	check(ExportDetailsTreeView == nullptr);
	check(PackageDetailsTreeView == nullptr);
	check(ObjectTypeAggregationTreeView == nullptr);
	check(EventAggregationTreeView == nullptr);
	check(TimingView == nullptr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* SLoadingProfilerWindow::GetAnalyticsEventName() const
{
	return TEXT("Insights.Usage.LoadingProfiler");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::Reset()
{
	if (TimingView)
	{
		TimingView->Reset();
	}

	if (EventAggregationTreeView)
	{
		EventAggregationTreeView->Reset();
	}

	if (ObjectTypeAggregationTreeView)
	{
		ObjectTypeAggregationTreeView->Reset();
	}

	if (PackageDetailsTreeView)
	{
		PackageDetailsTreeView->Reset();
	}

	if (ExportDetailsTreeView)
	{
		ExportDetailsTreeView->Reset();
	}

	if (RequestsTreeView)
	{
		RequestsTreeView->Reset();
	}

	UpdateTableTreeViews();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::UpdateTableTreeViews()
{
	UpdateEventAggregationTreeView();
	UpdateObjectTypeAggregationTreeView();
	UpdatePackageDetailsTreeView();
	UpdateExportDetailsTreeView();
	UpdateRequestsTreeView();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::UpdateEventAggregationTreeView()
{
	using namespace TraceServices;

	if (EventAggregationTreeView)
	{
		TSharedPtr<IUntypedTable> SourceTable;

		TSharedPtr<const IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			const ILoadTimeProfilerProvider* LoadTimeProfilerProvider = ReadLoadTimeProfilerProvider(*Session.Get());
			if (LoadTimeProfilerProvider)
			{
				FAnalysisSessionReadScope SessionReadScope(*Session.Get());
				ITable<FLoadTimeProfilerAggregatedStats>* EventAggregationTable =
					LoadTimeProfilerProvider->CreateEventAggregation(SelectionStartTime, SelectionEndTime);
				SourceTable = MakeShareable(EventAggregationTable);
			}
		}

		EventAggregationTreeView->UpdateSourceTable(SourceTable);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::UpdateObjectTypeAggregationTreeView()
{
	using namespace TraceServices;

	if (ObjectTypeAggregationTreeView)
	{
		TSharedPtr<IUntypedTable> SourceTable;

		TSharedPtr<const IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			const ILoadTimeProfilerProvider* LoadTimeProfilerProvider = ReadLoadTimeProfilerProvider(*Session.Get());
			if (LoadTimeProfilerProvider)
			{
				FAnalysisSessionReadScope SessionReadScope(*Session.Get());
				ITable<FLoadTimeProfilerAggregatedStats>* ObjectTypeAggregationTable =
					LoadTimeProfilerProvider->CreateObjectTypeAggregation(SelectionStartTime, SelectionEndTime);
				SourceTable = MakeShareable(ObjectTypeAggregationTable);
			}
		}

		ObjectTypeAggregationTreeView->UpdateSourceTable(SourceTable);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::UpdatePackageDetailsTreeView()
{
	using namespace TraceServices;

	if (PackageDetailsTreeView)
	{
		TSharedPtr<IUntypedTable> SourceTable;

		TSharedPtr<const IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			const ILoadTimeProfilerProvider* LoadTimeProfilerProvider = ReadLoadTimeProfilerProvider(*Session.Get());
			if (LoadTimeProfilerProvider)
			{
				FAnalysisSessionReadScope SessionReadScope(*Session.Get());
				ITable<FPackagesTableRow>* PackageDetailsTable =
					LoadTimeProfilerProvider->CreatePackageDetailsTable(SelectionStartTime, SelectionEndTime);
				SourceTable = MakeShareable(PackageDetailsTable);
			}
		}

		PackageDetailsTreeView->UpdateSourceTable(SourceTable);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::UpdateExportDetailsTreeView()
{
	using namespace TraceServices;

	if (ExportDetailsTreeView)
	{
		TSharedPtr<IUntypedTable> SourceTable;

		TSharedPtr<const IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			const ILoadTimeProfilerProvider* LoadTimeProfilerProvider = ReadLoadTimeProfilerProvider(*Session.Get());
			if (LoadTimeProfilerProvider)
			{
				FAnalysisSessionReadScope SessionReadScope(*Session.Get());
				ITable<FExportsTableRow>* ExportDetailsTable =
					LoadTimeProfilerProvider->CreateExportDetailsTable(SelectionStartTime, SelectionEndTime);
				SourceTable = MakeShareable(ExportDetailsTable);
			}
		}

		ExportDetailsTreeView->UpdateSourceTable(SourceTable);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::UpdateRequestsTreeView()
{
	using namespace TraceServices;

	if (RequestsTreeView)
	{
		TSharedPtr<IUntypedTable> SourceTable;

		TSharedPtr<const IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			const ILoadTimeProfilerProvider* LoadTimeProfilerProvider = ReadLoadTimeProfilerProvider(*Session.Get());
			if (LoadTimeProfilerProvider)
			{
				FAnalysisSessionReadScope SessionReadScope(*Session.Get());
				ITable<FRequestsTableRow>* RequestsTable =
					LoadTimeProfilerProvider->CreateRequestsTable(SelectionStartTime, SelectionEndTime);
				SourceTable = MakeShareable(RequestsTable);
			}
		}

		RequestsTreeView->UpdateSourceTable(SourceTable);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SLoadingProfilerWindow::SpawnTab_TimingView(const FSpawnTabArgs& Args)
{
	FLoadingProfilerManager::Get()->SetTimingViewVisible(true);

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(TimingView, TimingProfiler::STimingView, FInsightsManagerTabs::LoadingProfilerTabId)
		];

	TimingView->Reset(true);
	TimingView->OnSelectionChanged().AddSP(this, &SLoadingProfilerWindow::OnTimeSelectionChanged);
	TimingView->SelectTimeInterval(SelectionStartTime, SelectionEndTime - SelectionStartTime);

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SLoadingProfilerWindow::OnTimingViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::OnTimingViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FLoadingProfilerManager::Get()->SetTimingViewVisible(false);
	if (TimingView)
	{
		TimingView->OnSelectionChanged().RemoveAll(this);
		TimingView = nullptr;
	}

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SLoadingProfilerWindow::SpawnTab_EventAggregationTreeView(const FSpawnTabArgs& Args)
{
	FLoadingProfilerManager::Get()->SetEventAggregationTreeViewVisible(true);

	TSharedRef<FUntypedTable> Table = MakeShared<FUntypedTable>();
	Table->SetDisplayName(LOCTEXT("EventAggregation_TableName", "Event Aggregation"));

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(EventAggregationTreeView, SUntypedTableTreeView, Table)
		];

	EventAggregationTreeView->SetLogListingName(FLoadingProfilerManager::Get()->GetLogListingName());
	UpdateEventAggregationTreeView();

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SLoadingProfilerWindow::OnEventAggregationTreeViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::OnEventAggregationTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FLoadingProfilerManager::Get()->SetEventAggregationTreeViewVisible(false);
	EventAggregationTreeView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SLoadingProfilerWindow::SpawnTab_ObjectTypeAggregationTreeView(const FSpawnTabArgs& Args)
{
	FLoadingProfilerManager::Get()->SetObjectTypeAggregationTreeViewVisible(true);

	TSharedRef<FUntypedTable> Table = MakeShared<FUntypedTable>();
	Table->SetDisplayName(LOCTEXT("ObjectTypeAggregation_TableName", "Object Type Aggregation"));

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(ObjectTypeAggregationTreeView, SUntypedTableTreeView, Table)
		];

	ObjectTypeAggregationTreeView->SetLogListingName(FLoadingProfilerManager::Get()->GetLogListingName());
	UpdateObjectTypeAggregationTreeView();

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SLoadingProfilerWindow::OnObjectTypeAggregationTreeViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::OnObjectTypeAggregationTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FLoadingProfilerManager::Get()->SetObjectTypeAggregationTreeViewVisible(false);
	ObjectTypeAggregationTreeView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SLoadingProfilerWindow::SpawnTab_PackageDetailsTreeView(const FSpawnTabArgs& Args)
{
	FLoadingProfilerManager::Get()->SetPackageDetailsTreeViewVisible(true);

	TSharedRef<FUntypedTable> Table = MakeShared<FUntypedTable>();
	Table->SetDisplayName(LOCTEXT("PackageDetails_TableName", "Package Details"));

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(PackageDetailsTreeView, SUntypedTableTreeView, Table)
		];

	PackageDetailsTreeView->SetLogListingName(FLoadingProfilerManager::Get()->GetLogListingName());
	UpdatePackageDetailsTreeView();

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SLoadingProfilerWindow::OnPackageDetailsTreeViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::OnPackageDetailsTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FLoadingProfilerManager::Get()->SetPackageDetailsTreeViewVisible(false);
	PackageDetailsTreeView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SLoadingProfilerWindow::SpawnTab_ExportDetailsTreeView(const FSpawnTabArgs& Args)
{
	FLoadingProfilerManager::Get()->SetExportDetailsTreeViewVisible(true);

	TSharedRef<FUntypedTable> Table = MakeShared<FUntypedTable>();
	Table->SetDisplayName(LOCTEXT("ExportDetails_TableName", "Export Details"));

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(ExportDetailsTreeView, SUntypedTableTreeView, Table)
		];

	ExportDetailsTreeView->SetLogListingName(FLoadingProfilerManager::Get()->GetLogListingName());
	UpdateExportDetailsTreeView();

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SLoadingProfilerWindow::OnExportDetailsTreeViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::OnExportDetailsTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FLoadingProfilerManager::Get()->SetExportDetailsTreeViewVisible(false);
	ExportDetailsTreeView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SLoadingProfilerWindow::SpawnTab_RequestsTreeView(const FSpawnTabArgs& Args)
{
	FLoadingProfilerManager::Get()->SetRequestsTreeViewVisible(true);

	TSharedRef<FUntypedTable> Table = MakeShared<FUntypedTable>();
	Table->SetDisplayName(LOCTEXT("Requests_TableName", "Requests"));

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(RequestsTreeView, SUntypedTableTreeView, Table)
		];

	RequestsTreeView->SetLogListingName(FLoadingProfilerManager::Get()->GetLogListingName());
	UpdateRequestsTreeView();

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SLoadingProfilerWindow::OnRequestsTreeViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::OnRequestsTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FLoadingProfilerManager::Get()->SetRequestsTreeViewVisible(false);
	RequestsTreeView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SLoadingProfilerWindow::SpawnTab_UserAnnotations(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(UserAnnotationsPanel, TimingProfiler::SUserAnnotationsPanel)
			.HostTimingView(TimingView)
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SLoadingProfilerWindow::OnUserAnnotationsTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::OnUserAnnotationsTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	UserAnnotationsPanel = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	TSharedPtr<FLoadingProfilerManager> LoadingProfilerManager = FLoadingProfilerManager::Get();
	ensure(LoadingProfilerManager.IsValid());

	SetCommandList(LoadingProfilerManager->GetCommandList());

	SMajorTabWindow::FArguments Args;
	SMajorTabWindow::Construct(Args, ConstructUnderMajorTab, ConstructUnderWindow);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FWorkspaceItem> SLoadingProfilerWindow::CreateWorkspaceMenuGroup()
{
	return GetTabManager()->AddLocalWorkspaceMenuCategory(LOCTEXT("LoadingProfilerMenuGroupName", "Asset Loading Insights"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::RegisterTabSpawners()
{
	check(GetTabManager().IsValid());
	FTabManager* TabManagerPtr = GetTabManager().Get();
	check(GetWorkspaceMenuGroup().IsValid());
	const TSharedRef<FWorkspaceItem> Group = GetWorkspaceMenuGroup().ToSharedRef();

	IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(FInsightsManagerTabs::LoadingProfilerTabId);

	if (Config.ShouldRegisterMinorTab(FLoadingProfilerTabs::TimingViewID))
	{

		TabManagerPtr->RegisterTabSpawner(FLoadingProfilerTabs::TimingViewID, FOnSpawnTab::CreateRaw(this, &SLoadingProfilerWindow::SpawnTab_TimingView))
			.SetDisplayName(LOCTEXT("TimingViewTabTitle", "Timing View"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TimingView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FLoadingProfilerTabs::EventAggregationTreeViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FLoadingProfilerTabs::EventAggregationTreeViewID, FOnSpawnTab::CreateRaw(this, &SLoadingProfilerWindow::SpawnTab_EventAggregationTreeView))
			.SetDisplayName(LOCTEXT("EventAggregationTreeViewTabTitle", "Event Aggregation"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TableTreeView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FLoadingProfilerTabs::ObjectTypeAggregationTreeViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FLoadingProfilerTabs::ObjectTypeAggregationTreeViewID, FOnSpawnTab::CreateRaw(this, &SLoadingProfilerWindow::SpawnTab_ObjectTypeAggregationTreeView))
			.SetDisplayName(LOCTEXT("ObjectTypeAggregationTreeViewTabTitle", "Object Type Aggregation"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TableTreeView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FLoadingProfilerTabs::PackageDetailsTreeViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FLoadingProfilerTabs::PackageDetailsTreeViewID, FOnSpawnTab::CreateRaw(this, &SLoadingProfilerWindow::SpawnTab_PackageDetailsTreeView))
			.SetDisplayName(LOCTEXT("PackageDetailsTreeViewTabTitle", "Package Details"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TableTreeView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FLoadingProfilerTabs::ExportDetailsTreeViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FLoadingProfilerTabs::ExportDetailsTreeViewID, FOnSpawnTab::CreateRaw(this, &SLoadingProfilerWindow::SpawnTab_ExportDetailsTreeView))
			.SetDisplayName(LOCTEXT("ExportDetailsTreeViewTabTitle", "Export Details"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TableTreeView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FLoadingProfilerTabs::RequestsTreeViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FLoadingProfilerTabs::RequestsTreeViewID, FOnSpawnTab::CreateRaw(this, &SLoadingProfilerWindow::SpawnTab_RequestsTreeView))
			.SetDisplayName(LOCTEXT("RequestsTreeViewTabTitle", "Requests"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TableTreeView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FLoadingProfilerTabs::UserAnnotationsID))
	{
		TabManagerPtr->RegisterTabSpawner(FLoadingProfilerTabs::UserAnnotationsID, FOnSpawnTab::CreateRaw(this, &SLoadingProfilerWindow::SpawnTab_UserAnnotations))
			.SetDisplayName(LOCTEXT("UserAnnotationsTabTitle", "Annotations"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.Annotation"))
			.SetGroup(Group);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<FTabManager::FLayout> SLoadingProfilerWindow::CreateDefaultTabLayout() const
{
	return FTabManager::NewLayout("InsightsLoadingProfilerLayout_v1.3")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.5f)
				->SetHideTabWell(true)
				->AddTab(FLoadingProfilerTabs::TimingViewID, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.35f)
				->AddTab(FLoadingProfilerTabs::EventAggregationTreeViewID, ETabState::OpenedTab)
				->AddTab(FLoadingProfilerTabs::ObjectTypeAggregationTreeViewID, ETabState::OpenedTab)
				->AddTab(FLoadingProfilerTabs::PackageDetailsTreeViewID, ETabState::OpenedTab)
				->AddTab(FLoadingProfilerTabs::ExportDetailsTreeViewID, ETabState::OpenedTab)
				->AddTab(FLoadingProfilerTabs::RequestsTreeViewID, ETabState::OpenedTab)
				->AddTab(FLoadingProfilerTabs::UserAnnotationsID, ETabState::OpenedTab)
				->SetForegroundTab(FLoadingProfilerTabs::PackageDetailsTreeViewID)
			)
		);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SLoadingProfilerWindow::CreateToolbar(TSharedPtr<FExtender> Extender)
{
	IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(FInsightsManagerTabs::LoadingProfilerTabId);

	return SNew(SLoadingProfilerToolbar, Config).ToolbarExtender(Extender);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLoadingProfilerWindow::OnTimeSelectionChanged(Timing::ETimeChangedFlags InFlags, double InStartTime, double InEndTime)
{
	if (InFlags != Timing::ETimeChangedFlags::Interactive)
	{
		if (InStartTime < InEndTime)
		{
			SelectionStartTime = InStartTime;
			SelectionEndTime = InEndTime;
			UpdateTableTreeViews();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::LoadingProfiler

#undef LOCTEXT_NAMESPACE
