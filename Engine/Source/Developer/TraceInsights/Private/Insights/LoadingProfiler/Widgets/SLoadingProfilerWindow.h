// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// TraceInsights
#include "Insights/ITimingViewSession.h" // for ETimeChangedFlags
#include "Insights/Widgets/SMajorTabWindow.h"

namespace UE::Insights { class SUntypedTableTreeView; }
namespace UE::Insights::TimingProfiler { class STimingView; class SUserAnnotationsPanel; }

namespace UE::Insights::LoadingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Implements the Asset Loading Insights window. */
class SLoadingProfilerWindow : public ::Insights::SMajorTabWindow
{
public:
	/** Default constructor. */
	SLoadingProfilerWindow();

	/** Virtual destructor. */
	virtual ~SLoadingProfilerWindow();

	SLATE_BEGIN_ARGS(SLoadingProfilerWindow) {}
	SLATE_END_ARGS()

	virtual void Reset() override;

	void UpdateTableTreeViews();
	void UpdateEventAggregationTreeView();
	void UpdateObjectTypeAggregationTreeView();
	void UpdatePackageDetailsTreeView();
	void UpdateExportDetailsTreeView();
	void UpdateRequestsTreeView();

	/** Constructs this widget. */
	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);

	TSharedPtr<TimingProfiler::STimingView> GetTimingView() const { return TimingView; }
	TSharedPtr<SUntypedTableTreeView> GetEventAggregationTreeView() const { return EventAggregationTreeView; }
	TSharedPtr<SUntypedTableTreeView> GetObjectTypeAggregationTreeView() const { return ObjectTypeAggregationTreeView; }
	TSharedPtr<SUntypedTableTreeView> GetPackageDetailsTreeView() const { return PackageDetailsTreeView; }
	TSharedPtr<SUntypedTableTreeView> GetExportDetailsTreeView() const { return ExportDetailsTreeView; }
	TSharedPtr<SUntypedTableTreeView> GetRequestsTreeView() const { return RequestsTreeView; }

protected:
	virtual const TCHAR* GetAnalyticsEventName() const override;
	virtual TSharedRef<FWorkspaceItem> CreateWorkspaceMenuGroup() override;
	virtual void RegisterTabSpawners() override;
	virtual TSharedRef<FTabManager::FLayout> CreateDefaultTabLayout() const override;
	virtual TSharedRef<SWidget> CreateToolbar(TSharedPtr<FExtender> Extender);

private:
	TSharedRef<SDockTab> SpawnTab_TimingView(const FSpawnTabArgs& Args);
	void OnTimingViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_EventAggregationTreeView(const FSpawnTabArgs& Args);
	void OnEventAggregationTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_ObjectTypeAggregationTreeView(const FSpawnTabArgs& Args);
	void OnObjectTypeAggregationTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_PackageDetailsTreeView(const FSpawnTabArgs& Args);
	void OnPackageDetailsTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_ExportDetailsTreeView(const FSpawnTabArgs& Args);
	void OnExportDetailsTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_RequestsTreeView(const FSpawnTabArgs& Args);
	void OnRequestsTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_UserAnnotations(const FSpawnTabArgs& Args);
	void OnUserAnnotationsTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	void OnTimeSelectionChanged(Timing::ETimeChangedFlags InFlags, double InStartTime, double InEndTime);

private:
	/** The Timing view (multi-track) widget */
	TSharedPtr<TimingProfiler::STimingView> TimingView;

	/** The Event Aggregation tree view widget */
	TSharedPtr<SUntypedTableTreeView> EventAggregationTreeView;

	/** The Object Type Aggregation tree view widget */
	TSharedPtr<SUntypedTableTreeView> ObjectTypeAggregationTreeView;

	/** The Package Details tree view widget */
	TSharedPtr<SUntypedTableTreeView> PackageDetailsTreeView;

	/** The Export Details tree view widget */
	TSharedPtr<SUntypedTableTreeView> ExportDetailsTreeView;

	/** The Requests tree view widget */
	TSharedPtr<SUntypedTableTreeView> RequestsTreeView;

	/** The User Annotations panel widget. */
	TSharedPtr<TimingProfiler::SUserAnnotationsPanel> UserAnnotationsPanel;

	double SelectionStartTime;
	double SelectionEndTime;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::LoadingProfiler
