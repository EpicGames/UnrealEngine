// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"

// TraceServices
#include "TraceServices/Model/Modules.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FSymbolSearchPathsHelper"

namespace UE::Insights
{

class FSymbolSearchPathsHelper
{
public:
	/**
	 * Get localized text describing symbol search paths, or if none is set, instruction on how to setup those.
	 * Safe to call from any thread.
	 * @param ModuleProvider Provider to ask.
	 * @return Text listing symbol search paths.
	 */
	static FText GetLocalizedSymbolSearchPathsText(const TraceServices::IModuleProvider* ModuleProvider)
	{
		TStringBuilder<2048> PathsListStr;
		ModuleProvider->EnumerateSymbolSearchPaths([&](FStringView Path) { PathsListStr << TEXT("\n \u2022 ") << Path;});
		if (PathsListStr.Len())
		{
			return FText::Format(LOCTEXT("SymbolResolutionPaths", "Symbol search paths (in order of precedence): {0}"),
					 FText::FromStringView(PathsListStr));
		}
		return LOCTEXT("SymbolResolutionNoPaths", "No symbol paths has been setup.\nUse configuration file, UE_INSIGHTS_SYMBOL_PATH environment variable or manually select a file or directory from the Modules view.");
	}
};

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
