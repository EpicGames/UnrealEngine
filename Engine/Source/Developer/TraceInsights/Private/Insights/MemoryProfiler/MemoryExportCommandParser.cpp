// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryExportCommandParser.h"

#include "Misc/Parse.h"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY(LogMemoryCommandParser);

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryExportCommandParser::FMemoryExportCommandParser() :
	TimeA(-1.0), TimeB(-1.0), TimeC(-1.0), TimeD(-1.0), MaxResults(0)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static bool ParseTimeParameter(double& OutTime, const FString& Value, const TCHAR* TimeName)
{
	if (!Value.IsNumeric())
	{
		UE_LOGF(LogMemoryCommandParser, Error, "Time%ls value (%ls) is not a valid number.", TimeName, *Value);
		return false;
	}
	OutTime = FCString::Atod(*Value);
	if (OutTime < 0.0)
	{
		UE_LOGF(LogMemoryCommandParser, Error, "Cannot pass a negative value (%.2f) for Time%ls.", OutTime, TimeName);
		return false;
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemoryExportCommandParser::ParseParameters(const TCHAR* Cmd, FOutputDevice& Ar)
{
	UE_LOGF(LogMemoryCommandParser, Log, "Validating parameters necessary to execute chosen rule...");
	const bool UseEscape = true;
	while (Cmd && Cmd[0] != TEXT('\0'))
	{
		FString Token;
		if (FParse::Token(Cmd, Token, UseEscape))
		{
			UE_LOGF(LogMemoryCommandParser, Log, "Reading Parameter %ls", *Token);

			static constexpr TCHAR RuleToken[] = TEXT("-Rule=");
			static constexpr TCHAR OutputToken[] = TEXT("-Output=");
			static constexpr TCHAR BookmarkAToken[] = TEXT("-BookmarkA=");
			static constexpr TCHAR BookmarkBToken[] = TEXT("-BookmarkB=");
			static constexpr TCHAR BookmarkCToken[] = TEXT("-BookmarkC=");
			static constexpr TCHAR BookmarkDToken[] = TEXT("-BookmarkD=");
			static constexpr TCHAR TimeAToken[] = TEXT("-TimeA=");
			static constexpr TCHAR TimeBToken[] = TEXT("-TimeB=");
			static constexpr TCHAR TimeCToken[] = TEXT("-TimeC=");
			static constexpr TCHAR TimeDToken[] = TEXT("-TimeD=");
			static constexpr TCHAR MaxResultsToken[] = TEXT("-MaxResults=");
			static constexpr TCHAR ColumnsToken[] = TEXT("-Columns=");

			if (Token.StartsWith(RuleToken))
			{
				Token.RightChopInline(UE_ARRAY_COUNT(RuleToken) - 1);
				Token.TrimQuotesInline();
				if (Token.IsEmpty())
				{
					UE_LOGF(LogMemoryCommandParser, Error, "Exporter stopped because no Rule was provided! Please, specify one. Ex: -Rule=AaBCf");
					return false;
				}
				TOptional<EQueryRule> OptRule = GetRule(Token);
				if (!OptRule.IsSet())
				{
					UE_LOGF(LogMemoryCommandParser, Error, "Exporter stopped because Rule (%ls) is not valid!", *Token);
					UE_LOGF(LogMemoryCommandParser, Error, "Available rules: %ls", *GetAllRulesString());
					return false;
				}
				RuleName = Token;
				RuleEnum = OptRule.GetValue();
			}
			else if (Token.StartsWith(OutputToken))
			{
				Token.RightChopInline(UE_ARRAY_COUNT(OutputToken) - 1);
				Token.TrimQuotesInline();
				OutputPath = Token;
			}
			else if (Token.StartsWith(BookmarkAToken))
			{
				Token.RightChopInline(UE_ARRAY_COUNT(BookmarkAToken) - 1);
				Token.TrimQuotesInline();
				BookmarkA = Token;
			}
			else if (Token.StartsWith(BookmarkBToken))
			{
				Token.RightChopInline(UE_ARRAY_COUNT(BookmarkBToken) - 1);
				Token.TrimQuotesInline();
				BookmarkB = Token;
			}
			else if (Token.StartsWith(BookmarkCToken))
			{
				Token.RightChopInline(UE_ARRAY_COUNT(BookmarkCToken) - 1);
				Token.TrimQuotesInline();
				BookmarkC = Token;
			}
			else if (Token.StartsWith(BookmarkDToken))
			{
				Token.RightChopInline(UE_ARRAY_COUNT(BookmarkDToken) - 1);
				Token.TrimQuotesInline();
				BookmarkD = Token;
			}
			else if (Token.StartsWith(TimeAToken))
			{
				Token.RightChopInline(UE_ARRAY_COUNT(TimeAToken) - 1);
				if (Token.IsEmpty())
				{
					continue;
				}
				if (!ParseTimeParameter(TimeA, Token, TEXT("A")))
				{
					return false;
				}
			}
			else if (Token.StartsWith(TimeBToken))
			{
				Token.RightChopInline(UE_ARRAY_COUNT(TimeBToken) - 1);
				if (Token.IsEmpty())
				{
					continue;
				}
				if (!ParseTimeParameter(TimeB, Token, TEXT("B")))
				{
					return false;
				}
			}
			else if (Token.StartsWith(TimeCToken))
			{
				Token.RightChopInline(UE_ARRAY_COUNT(TimeCToken) - 1);
				if (Token.IsEmpty())
				{
					continue;
				}
				if (!ParseTimeParameter(TimeC, Token, TEXT("C")))
				{
					return false;
				}
			}
			else if (Token.StartsWith(TimeDToken))
			{
				Token.RightChopInline(UE_ARRAY_COUNT(TimeDToken) - 1);
				if (Token.IsEmpty())
				{
					continue;
				}
				if (!ParseTimeParameter(TimeD, Token, TEXT("D")))
				{
					return false;
				}
			}
			else if (Token.StartsWith(MaxResultsToken))
			{
				Token.RightChopInline(UE_ARRAY_COUNT(MaxResultsToken) - 1);

				// If user pass the parameter without values or with garbage, Atoi handles it
				// and MaxResults will default to 0
				MaxResults = FCString::Atoi(*Token);
				if (MaxResults < 0)
				{
					UE_LOGF(LogMemoryCommandParser, Error, "Cannot pass a negative value (%d) for MaxResults!", MaxResults);
					return false;
				}
			}
			else if (Token.StartsWith(ColumnsToken))
			{
				Token.RightChopInline(UE_ARRAY_COUNT(ColumnsToken) - 1);
				Columns = Token;
			}
			else
			{
				UE_LOGF(LogMemoryCommandParser, Warning, "Skipped unknown parameter: %ls", *Token);
			}
		}
		else
		{
			UE_LOGF(LogMemoryCommandParser, Error, "Failed parsing command parameters at: %ls", Cmd);
			return false;
		}
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemoryExportCommandParser::ValidateParameters(const FMemoryExporter& Exporter)
{
	if (RuleName.IsEmpty())
	{
		UE_LOGF(LogMemoryCommandParser, Error, "Rule must be specified. Ex: -Rule=AaBCf");
		return false;
	}

	// All rules need OutputPath
	if (OutputPath.IsEmpty())
	{
		UE_LOGF(LogMemoryCommandParser, Error, "Missing required parameter: -Output=\"path/to/file.csv\"");
		return false;
	}

	// Validate BookmarkX and TimeX parameters
	if (!ValidateTimeParameters(RuleEnum, 1, BookmarkA, TimeA, TEXT("A"), Exporter))
	{
		return false;
	}
	if (!ValidateTimeParameters(RuleEnum, 2, BookmarkB, TimeB, TEXT("B"), Exporter))
	{
		return false;
	}
	if (!ValidateTimeParameters(RuleEnum, 3, BookmarkC, TimeC, TEXT("C"), Exporter))
	{
		return false;
	}
	if (!ValidateTimeParameters(RuleEnum, 4, BookmarkD, TimeD, TEXT("D"), Exporter))
	{
		return false;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FMemoryExportCommandParser::GetNumTimeMarkers(EQueryRule Rule)
{
	switch (Rule)
	{
	case EQueryRule::aAf:
	case EQueryRule::afA:
	case EQueryRule::Aaf:
		return 1;

	case EQueryRule::aAfB:
	case EQueryRule::AaBf:
	case EQueryRule::aAfaBf:
	case EQueryRule::AfB:
	case EQueryRule::AaB:
	case EQueryRule::AafB:
	case EQueryRule::aABf:
	case EQueryRule::AoB:
	case EQueryRule::AiB:
		return 2;

	case EQueryRule::AaBCf:
	case EQueryRule::AaBfC:
	case EQueryRule::aABfC:
		return 3;

	case EQueryRule::AaBCfD:
		return 4;

	default:
		return 0;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemoryExportCommandParser::ValidateTimeParameters(
	EQueryRule Rule,
	uint32 TimeMarkerId,
	const FString& BookmarkX,
	double& TimeX,
	const TCHAR* TimeName,
	const FMemoryExporter& Exporter)
{
	const uint32 NumTimeMarkers = GetNumTimeMarkers(Rule);

	if (TimeMarkerId <= NumTimeMarkers)
	{
		if (BookmarkX.IsEmpty() && TimeX < 0.0)
		{
			UE_LOGF(LogMemoryCommandParser, Error,
				"Time%ls must be specified using either -Bookmark%ls=\"<a bookmark name>\" or -Time%ls=<a real positive number in seconds> parameters!",
				TimeName, TimeName, TimeName);
			return false;
		}
		if (!BookmarkX.IsEmpty() && TimeX >= 0.0)
		{
			UE_LOGF(LogMemoryCommandParser, Error, "Cannot specify both -Bookmark%ls= and -Time%ls= parameters! Choose one.",
				TimeName, TimeName);
			return false;
		}
		// Only when a bookmark has been provided, then lookup the Timestamp
		if (!BookmarkX.IsEmpty() && !Exporter.FindBookmarkByName(BookmarkX, TimeX))
		{
			UE_LOGF(LogMemoryCommandParser, Error, "Bookmark '%ls' not found! Please re-run trace with valid bookmark name.", *BookmarkX);
			return false;
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FMemoryExportCommandParser::FStringToQueryRuleMap& FMemoryExportCommandParser::QueryRuleMap()
{
	static const FStringToQueryRuleMap Map =
	{
		{ TEXT("aAf"),     EQueryRule::aAf },
		{ TEXT("afA"),     EQueryRule::afA },
		{ TEXT("Aaf"),     EQueryRule::Aaf },
		{ TEXT("aAfB"),    EQueryRule::aAfB },
		{ TEXT("AaBf"),    EQueryRule::AaBf },
		{ TEXT("aAfaBf"),  EQueryRule::aAfaBf },
		{ TEXT("AfB"),     EQueryRule::AfB },
		{ TEXT("AaB"),     EQueryRule::AaB },
		{ TEXT("AafB"),    EQueryRule::AafB },
		{ TEXT("aABf"),    EQueryRule::aABf },
		{ TEXT("AaBCf"),   EQueryRule::AaBCf },
		{ TEXT("AaBfC"),   EQueryRule::AaBfC },
		{ TEXT("aABfC"),   EQueryRule::aABfC },
		{ TEXT("AaBCfD"),  EQueryRule::AaBCfD },
		{ TEXT("AoB"),     EQueryRule::AoB },
		{ TEXT("AiB"),     EQueryRule::AiB }
	};
	return Map;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TOptional<FMemoryExportCommandParser::EQueryRule> FMemoryExportCommandParser::GetRule(const FString& InRuleName)
{
	const EQueryRule* FoundRule = QueryRuleMap().Find(InRuleName);
	return (FoundRule) ? *FoundRule : TOptional<EQueryRule>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FMemoryExportCommandParser::GetAllRulesString()
{
	TStringBuilder<1024> AllRules;
	bool bShouldAddComma = false;
	for (const auto& KV : QueryRuleMap())
	{
		if (bShouldAddComma)
		{
			AllRules.Append(TEXT(", "));
		}
		else
		{
			bShouldAddComma = true;
		}
		AllRules.Append(KV.Key);
	}
	return FString(AllRules.ToString());
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
