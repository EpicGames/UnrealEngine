// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Hash/Fnv.h"

// TraceServices
#include "TraceServices/Model/AllocationsProvider.h"
#include "TraceServices/Model/Memory.h"

// TraceInsights
#include "ViewModels/MemoryExporter.h"

namespace UE::Insights::MemoryProfiler
{

DECLARE_LOG_CATEGORY_EXTERN(LogMemoryCommandParser, Log, All);

class FMemoryExportCommandParser
{
public:
	typedef TraceServices::IAllocationsProvider::EQueryRule EQueryRule;

public:
	FMemoryExportCommandParser();
	~FMemoryExportCommandParser() {}

	bool ParseParameters(const TCHAR* Cmd, FOutputDevice& Ar);
	bool ValidateParameters(const FMemoryExporter& Exporter);

	// Getters
	const FString& GetRuleName() const { return RuleName; }
	uint32 GetRuleNumTimeMarkers() const { return GetNumTimeMarkers(RuleEnum); }
	const EQueryRule GetRuleEnum() const { return RuleEnum; }
	const FString& GetOutputPathName() const { return OutputPath; }
	const FString& GetBookmarkA() const { return BookmarkA; }
	const FString& GetBookmarkB() const { return BookmarkB; }
	const FString& GetBookmarkC() const { return BookmarkC; }
	const FString& GetBookmarkD() const { return BookmarkD; }
	double GetTimeA() const { return TimeA; }
	double GetTimeB() const { return TimeB; }
	double GetTimeC() const { return TimeC; }
	double GetTimeD() const { return TimeD; }
	int32 GetMaxResults() const { return MaxResults; }
	const FString& GetColumns() const { return Columns; }

private:
	bool ValidateTimeParameters(
		EQueryRule Rule,
		uint32 TimeMarkerId,
		const FString& Bookmark,
		double& Time,
		const TCHAR* TimeName,
		const FMemoryExporter& Exporter);

	static uint32 GetNumTimeMarkers(EQueryRule Rule);

	template <typename ValueType>
	struct FCaseSensitiveStringMapFuncs : BaseKeyFuncs<ValueType, FString, /*bInAllowDuplicateKeys*/ false>
	{
		static FORCEINLINE const FString& GetSetKey(const TPair<FString, ValueType>& Element)
		{
			return Element.Key;
		}
		static FORCEINLINE bool Matches(const FString& A, const FString& B)
		{
			return A.Equals(B, ESearchCase::CaseSensitive);
		}
		static FORCEINLINE uint32 GetKeyHash(const FString& Key)
		{
			return UE::HashStringFNV1a32(MakeStringView(Key));
		}
	};

	using FStringToQueryRuleMap = TMap<FString, EQueryRule, FDefaultSetAllocator, FCaseSensitiveStringMapFuncs<EQueryRule>>;

	static const FStringToQueryRuleMap& QueryRuleMap();
	static TOptional<EQueryRule> GetRule(const FString& InRuleName);
	static FString GetAllRulesString();

private:
	FString RuleName;
	EQueryRule RuleEnum;
	FString OutputPath;
	FString BookmarkA, BookmarkB, BookmarkC, BookmarkD;
	double TimeA, TimeB, TimeC, TimeD;
	int32 MaxResults;
	FString Columns;
};

} // namespace UE::Insights::MemoryProfiler