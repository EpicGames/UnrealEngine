// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryProfilerCommands.h"

#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/MemoryProfiler/MemoryProfilerManager.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemoryProfilerCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryProfilerCommands::FMemoryProfilerCommands()
: TCommands<FMemoryProfilerCommands>(
	TEXT("MemoryProfilerCommands"),
	NSLOCTEXT("Contexts", "MemoryProfilerCommands", "Insights - Memory Insights"),
	NAME_None,
	FInsightsStyle::GetStyleSetName())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryProfilerCommands::~FMemoryProfilerCommands()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// UI_COMMAND takes long for the compiler to optimize
UE_DISABLE_OPTIMIZATION_SHIP
void FMemoryProfilerCommands::RegisterCommands()
{
	UI_COMMAND(ToggleTimingViewVisibility,
		"Timing",
		"Toggles the visibility of the main Timing view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleMemInvestigationViewVisibility,
		"Investigation",
		"Toggles the visibility of the Memory Investigation (Alloc Queries) view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleMemTagTreeViewVisibility,
		"Memory Tags",
		"Toggles the visibility of the Memory Tags (LLM) tree view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleModulesViewVisibility,
		"Modules",
		"Toggles the visibility of the Modules view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////
// Toggle Commands
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FMemoryProfilerActionManager, This, ToggleTimingViewVisibility, IsTimingViewVisible, ShowHideTimingView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FMemoryProfilerActionManager, This, ToggleMemInvestigationViewVisibility, IsMemInvestigationViewVisible, ShowHideMemInvestigationView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FMemoryProfilerActionManager, This, ToggleMemTagTreeViewVisibility, IsMemTagTreeViewVisible, ShowHideMemTagTreeView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FMemoryProfilerActionManager, This, ToggleModulesViewVisibility, IsModulesViewVisible, ShowHideModulesView)

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
