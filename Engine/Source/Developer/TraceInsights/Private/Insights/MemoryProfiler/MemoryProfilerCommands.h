// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/Commands.h"

// TraceInsights
#include "Insights/Common/CommandUtils.h"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class that holds all profiler commands.
 */
class FMemoryProfilerCommands : public TCommands<FMemoryProfilerCommands>
{
public:
	FMemoryProfilerCommands();
	virtual ~FMemoryProfilerCommands();
	virtual void RegisterCommands() override;

public:
	/** Toggles visibility for the Timing view. */
	TSharedPtr<FUICommandInfo> ToggleTimingViewVisibility;

	/** Toggles visibility for the Memory Investigation view. */
	TSharedPtr<FUICommandInfo> ToggleMemInvestigationViewVisibility;

	/** Toggles visibility for the Memory Tags tree view. */
	TSharedPtr<FUICommandInfo> ToggleMemTagTreeViewVisibility;

	/** Toggles visibility for the Modules view. */
	TSharedPtr<FUICommandInfo> ToggleModulesViewVisibility;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class that provides helper functions for the commands to avoid cluttering profiler manager with many small functions.
 * Can't contain any variables. Directly operates on the profiler manager instance.
 */
class FMemoryProfilerActionManager
{
	friend class FMemoryProfilerManager;

private:
	/** Private constructor. */
	FMemoryProfilerActionManager(class FMemoryProfilerManager* Instance)
		: This(Instance)
	{}

	//////////////////////////////////////////////////
	// Toggle Commands

	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleTimingViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleMemInvestigationViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleMemTagTreeViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleModulesViewVisibility)

	//////////////////////////////////////////////////

protected:
	/** Reference to the global instance of the profiler manager. */
	class FMemoryProfilerManager* This;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
