// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryProfilerManager.h"

#include "HAL/IConsoleManager.h"
#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AllocationsProvider.h"
#include "TraceServices/Model/Memory.h"

// TraceInsightsCore
#include "InsightsCore/Table/Widgets/STableTreeView.h"

// TraceInsights
#include "Insights/Common/InsightsMenuBuilder.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/MemoryProfiler/MemoryExportCommandParser.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryExporter.h"
#include "Insights/MemoryProfiler/ViewModels/MemorySharedState.h"
#include "Insights/MemoryProfiler/Widgets/SMemoryProfilerWindow.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler"

// For Memory Insights in editor preview.
static bool GEnableMemoryInsightsInEditor = false;
static FAutoConsoleVariableRef CVAREnableMemoryInsights(
	TEXT("r.EnableMemoryInsights"),
	GEnableMemoryInsightsInEditor,
	TEXT("Enables the Memory Insights feature in the Editor."),
	ECVF_Default
);

namespace UE::Insights::MemoryProfiler
{

DEFINE_LOG_CATEGORY(LogMemoryProfiler);

TSharedPtr<FMemoryProfilerManager> FMemoryProfilerManager::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FMemoryProfilerManager> FMemoryProfilerManager::Get()
{
	return FMemoryProfilerManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FMemoryProfilerManager> FMemoryProfilerManager::CreateInstance()
{
	ensure(!FMemoryProfilerManager::Instance.IsValid());
	if (FMemoryProfilerManager::Instance.IsValid())
	{
		FMemoryProfilerManager::Instance.Reset();
	}

	FMemoryProfilerManager::Instance = MakeShared<FMemoryProfilerManager>(FInsightsManager::Get()->GetCommandList());

	return FMemoryProfilerManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryProfilerManager::FMemoryProfilerManager(TSharedRef<FUICommandList> InCommandList)
	: CommandList(InCommandList)
	, ActionManager(this)
	, MajorTabId(FInsightsManagerTabs::MemoryProfilerTabId)
	, LogListingName("MemoryInsights")
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryProfilerManager::Initialize(IUnrealInsightsModule& InsightsModule)
{
	ensure(!bIsInitialized);
	if (bIsInitialized)
	{
		return;
	}
	bIsInitialized = true;

	UE_LOGF(LogMemoryProfiler, Log, "Initialize");

	// Register tick functions.
	OnTick = FTickerDelegate::CreateSP(this, &FMemoryProfilerManager::Tick);
	OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(OnTick, 0.0f);

	FMemoryProfilerCommands::Register();
	BindCommands();

	FInsightsManager::Get()->GetSessionChangedEvent().AddSP(this, &FMemoryProfilerManager::OnSessionChanged);
	OnSessionChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryProfilerManager::Shutdown()
{
	if (!bIsInitialized)
	{
		return;
	}
	bIsInitialized = false;

	// If the MessageLog module was already unloaded as part of the global Shutdown process, do not load it again.
	if (FModuleManager::Get().IsModuleLoaded("MessageLog"))
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		if (MessageLogModule.IsRegisteredLogListing(GetLogListingName()))
		{
			MessageLogModule.UnregisterLogListing(GetLogListingName());
		}
	}

	FInsightsManager::Get()->GetSessionChangedEvent().RemoveAll(this);

	FMemoryProfilerCommands::Unregister();

	// Unregister tick function.
	FTSTicker::RemoveTicker(OnTickHandle);

	FMemoryProfilerManager::Instance.Reset();

	UE_LOGF(LogMemoryProfiler, Log, "Shutdown");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryProfilerManager::~FMemoryProfilerManager()
{
	ensure(!bIsInitialized);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryProfilerManager::BindCommands()
{
	ActionManager.Map_ToggleTimingViewVisibility();
	ActionManager.Map_ToggleMemInvestigationViewVisibility();
	ActionManager.Map_ToggleMemTagTreeViewVisibility();
	ActionManager.Map_ToggleModulesViewVisibility();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryProfilerManager::RegisterMajorTabs(IUnrealInsightsModule& InsightsModule)
{
	bool bShouldRegister = true;

#if WITH_EDITOR
	#if UE_BUILD_SHIPPING
		bShouldRegister = false;
	#else
		bShouldRegister = GEnableMemoryInsightsInEditor;
	#endif
#endif

	if (!bIsTabRegistered && bShouldRegister)
	{
		bIsTabRegistered = true;
		const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(MajorTabId);
		if (Config.bIsAvailable)
		{
			// Register tab spawner for the Memory Insights.
			FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(MajorTabId,
					FOnSpawnTab::CreateRaw(this, &FMemoryProfilerManager::SpawnTab),
					FCanSpawnTab::CreateRaw(this, &FMemoryProfilerManager::CanSpawnTab))
				.SetDisplayName(Config.TabLabel.IsSet() ? Config.TabLabel.GetValue() : LOCTEXT("MemoryProfilerTabTitle", "Memory Insights"))
				.SetTooltipText(Config.TabTooltip.IsSet() ? Config.TabTooltip.GetValue() : LOCTEXT("MemoryProfilerTooltipText", "Open the Memory Insights tab."))
				.SetIcon(Config.TabIcon.IsSet() ? Config.TabIcon.GetValue() : FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.MemoryProfiler"));

			TSharedRef<FWorkspaceItem> Group = Config.WorkspaceGroup.IsValid() ? Config.WorkspaceGroup.ToSharedRef() : FInsightsManager::Get()->GetInsightsMenuBuilder()->GetInsightsToolsGroup();
			TabSpawnerEntry.SetGroup(Group);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryProfilerManager::UnregisterMajorTabs()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(MajorTabId);
	bIsTabRegistered = false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> FMemoryProfilerManager::SpawnTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	// Register OnTabClosed to handle Memory profiler manager shutdown.
	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FMemoryProfilerManager::OnTabClosed));

	// Create the SMemoryProfilerWindow widget.
	TSharedRef<SMemoryProfilerWindow> Window = SNew(SMemoryProfilerWindow, DockTab, Args.GetOwnerWindow());
	DockTab->SetContent(Window);

	AssignProfilerWindow(Window);

	return DockTab;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemoryProfilerManager::CanSpawnTab(const FSpawnTabArgs& Args) const
{
#if WITH_EDITOR
	return true;
#else
	return bIsAvailable;
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryProfilerManager::OnTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	OnWindowClosedEvent();

	RemoveProfilerWindow();

	// Disable TabClosed delegate.
	TabBeingClosed->SetOnTabClosed(SDockTab::FOnTabClosedCallback());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedRef<FUICommandList> FMemoryProfilerManager::GetCommandList() const
{
	return CommandList;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FMemoryProfilerCommands& FMemoryProfilerManager::GetCommands()
{
	return FMemoryProfilerCommands::Get();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryProfilerActionManager& FMemoryProfilerManager::GetActionManager()
{
	return FMemoryProfilerManager::Instance->ActionManager;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemorySharedState* FMemoryProfilerManager::GetSharedState()
{
	TSharedPtr<SMemoryProfilerWindow> Wnd = GetProfilerWindow();
	return Wnd.IsValid() ? &Wnd->GetSharedState() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemoryProfilerManager::Tick(float DeltaTime)
{
#if WITH_EDITOR
	if (!bIsTabRegistered && GEnableMemoryInsightsInEditor)
	{
		IUnrealInsightsModule& TraceInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		RegisterMajorTabs(TraceInsightsModule);
	}
#endif

	// Check if the session has Memory events (to spawn the tab), but not too often.
	if (!bIsAvailable && AvailabilityCheck.Tick())
	{
		bool bShouldBeAvailable = false;

		TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();
		check(InsightsManager.IsValid());

		TSharedPtr<const TraceServices::IAnalysisSession> Session = InsightsManager->GetSession();
		if (Session.IsValid())
		{
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
				if (Session->IsAnalysisComplete())
				{
					// Never check again during this session.
					AvailabilityCheck.Disable();
				}
			}

			const TraceServices::IMemoryProvider* MemoryProvider = TraceServices::ReadMemoryProvider(*Session.Get());
			if (MemoryProvider)
			{
				TraceServices::FProviderReadScopeLock _(*MemoryProvider);
				uint32 TagCount = MemoryProvider->GetTagCount();
				if (TagCount > 0)
				{
					bShouldBeAvailable = true;
				}
			}

			const TraceServices::IAllocationsProvider* AllocationsProvider = TraceServices::ReadAllocationsProvider(*Session.Get());
			if (AllocationsProvider)
			{
				TraceServices::FProviderReadScopeLock _(*AllocationsProvider);
				if (AllocationsProvider->IsInitialized())
				{
					bShouldBeAvailable = true;
				}
			}
		}
		else
		{
			// Do not check again until the next session changed event (see OnSessionChanged).
			AvailabilityCheck.Disable();
		}

		if (bShouldBeAvailable)
		{
			bIsAvailable = true;

			FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
			MessageLogModule.RegisterLogListing(GetLogListingName(), LOCTEXT("MemoryInsights", "Memory Insights"));
			MessageLogModule.EnableMessageLogDisplay(true);

#if !WITH_EDITOR
			if (FGlobalTabmanager::Get()->HasTabSpawner(MajorTabId))
			{
				UE_LOGF(LogMemoryProfiler, Log, "Opening the \"Memory Insights\" tab...");
				FGlobalTabmanager::Get()->TryInvokeTab(MajorTabId);
			}
#endif
		}
	}

	FMemorySharedState* SharedState = GetSharedState();
	if (SharedState)
	{
		SharedState->UpdateMemoryTags();
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryProfilerManager::OnSessionChanged()
{
	UE_LOGF(LogMemoryProfiler, Log, "OnSessionChanged");

	bIsAvailable = false;
	if (FInsightsManager::Get()->GetSession().IsValid())
	{
		AvailabilityCheck.Enable(0.5);
	}
	else
	{
		AvailabilityCheck.Disable();
	}

	TSharedPtr<SMemoryProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->Reset();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryProfilerManager::ShowHideTimingView(const bool bIsVisible)
{
	bIsTimingViewVisible = bIsVisible;

	TSharedPtr<SMemoryProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FMemoryProfilerTabs::TimingViewID, bIsVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryProfilerManager::ShowHideMemInvestigationView(const bool bIsVisible)
{
	bIsMemInvestigationViewVisible = bIsVisible;

	TSharedPtr<SMemoryProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FMemoryProfilerTabs::MemInvestigationViewID, bIsVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryProfilerManager::ShowHideMemTagTreeView(const bool bIsVisible)
{
	bIsMemTagTreeViewVisible = bIsVisible;

	TSharedPtr<SMemoryProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FMemoryProfilerTabs::MemTagTreeViewID, bIsVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryProfilerManager::ShowHideModulesView(const bool bIsVisible)
{
	bIsModulesViewVisible = bIsVisible;

	TSharedPtr<SMemoryProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FMemoryProfilerTabs::ModulesViewID, bIsVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryProfilerManager::OnWindowClosedEvent()
{
	TSharedPtr<SMemoryProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		// Need to close MemAlloc window to prevent it being saved in the layout and spawning as an "Unregister Tab" on the next application start.
		Wnd->CloseMemAllocTableTreeTabs();

		TSharedPtr<TimingProfiler::STimingView> TimingView = Wnd->GetTimingView();
		if (TimingView)
		{
			TimingView->CloseQuickFindTab();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemoryProfilerManager::Exec(const TCHAR* Cmd, FOutputDevice& Ar)
{
	// Export allocations based on memory investigation rules
	if (FParse::Command(&Cmd, TEXT("MemoryInsights.ExportAllocs")))
	{
		FStopwatch Stopwatch;
		Stopwatch.Start();

		TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();
		if (!InsightsManager.IsValid())
		{
			UE_LOGF(LogMemoryProfiler, Error, "InsightsManager is invalid");
			return false;
		}

		if (!InsightsManager->GetSession().IsValid())
		{
			UE_LOGF(LogMemoryProfiler, Error, "Session is invalid");
			return false;
		}

		FMemoryExporter Exporter(*InsightsManager->GetSession().Get());

		FMemoryExportCommandParser ExportCommandParser;

		if (!ExportCommandParser.ParseParameters(Cmd, Ar))
		{
			UE_LOGF(LogMemoryProfiler, Error, "Exporter stopped while parsing cmd parameters");
			return false;
		}

		if (!ExportCommandParser.ValidateParameters(Exporter))
		{
			UE_LOGF(LogMemoryProfiler, Error, "Exporter stopped while validating parameters");
			return false;
		}

		// All QueryRule are supported
		UE_LOGF(LogMemoryProfiler, Log, " === MemoryInsights.ExportAllocs -Rule=\"%ls\" === ", *ExportCommandParser.GetRuleName());

		FMemoryExporter::FExportMemoryAllocsParams Params;
		Params.Rule = ExportCommandParser.GetRuleEnum();
		Params.OutputPathName = ExportCommandParser.GetOutputPathName();
		Params.TimeA = ExportCommandParser.GetTimeA();
		Params.TimeB = ExportCommandParser.GetTimeB();
		Params.TimeC = ExportCommandParser.GetTimeC();
		Params.TimeD = ExportCommandParser.GetTimeD();
		Params.MaxResults = ExportCommandParser.GetMaxResults();
		Params.Columns = ExportCommandParser.GetColumns();

		UE_LOGF(LogMemoryProfiler, Log, "Parameters provided by user:");
		UE_LOGF(LogMemoryProfiler, Log, "-> Rule:       %ls", *ExportCommandParser.GetRuleName());
		UE_LOGF(LogMemoryProfiler, Log, "-> Output:     %ls", *Params.OutputPathName);

		const uint32 RuleNumTimeMarkers = ExportCommandParser.GetRuleNumTimeMarkers();
		if (RuleNumTimeMarkers >= 1)
		{
			FString BookmarkA = ExportCommandParser.GetBookmarkA();
			UE_LOGF(LogMemoryProfiler, Log, "-> TimeA:      %.2f%ls",
				Params.TimeA, BookmarkA.IsEmpty() ? TEXT("") : *FString::Printf(TEXT(" (from bookmark '%s')"), *BookmarkA));
		}
		if (RuleNumTimeMarkers >= 2)
		{
			FString BookmarkB = ExportCommandParser.GetBookmarkB();
			UE_LOGF(LogMemoryProfiler, Log, "-> TimeB:      %.2f%ls",
				Params.TimeB, BookmarkB.IsEmpty() ? TEXT("") : *FString::Printf(TEXT(" (from bookmark '%s')"), *BookmarkB));
		}
		if (RuleNumTimeMarkers >= 3)
		{
			FString BookmarkC = ExportCommandParser.GetBookmarkC();
			UE_LOGF(LogMemoryProfiler, Log, "-> TimeC:      %.2f%ls",
				Params.TimeC, BookmarkC.IsEmpty() ? TEXT("") : *FString::Printf(TEXT(" (from bookmark '%s')"), *BookmarkC));
		}
		if (RuleNumTimeMarkers >= 4)
		{
			FString BookmarkD = ExportCommandParser.GetBookmarkD();
			UE_LOGF(LogMemoryProfiler, Log, "-> TimeD:      %.2f%ls",
				Params.TimeD, BookmarkD.IsEmpty() ? TEXT("") : *FString::Printf(TEXT(" (from bookmark '%s')"), *BookmarkD));
		}

		UE_LOGF(LogMemoryProfiler, Log, "-> MaxResults: %d (0 = unlimited)", Params.MaxResults);

		UE_LOGF(LogMemoryProfiler, Log, "Starting export...");
		int ResultCount = Exporter.ExportMemoryAllocsAsText(Params);

		if (ResultCount >= 0)
		{
			UE_LOGF(LogMemoryProfiler, Log, "SUCCESS! Exported %d allocations", ResultCount);
			UE_LOGF(LogMemoryProfiler, Log, "Output file: %ls", *Params.OutputPathName);
			return true;
		}
		else
		{
			UE_LOGF(LogMemoryProfiler, Error, "Export failed with code %d", ResultCount);
			return false;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
