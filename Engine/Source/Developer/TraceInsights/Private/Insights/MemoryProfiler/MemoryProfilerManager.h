// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Ticker.h"
#include "Framework/Commands/UICommandList.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Common/AvailabilityCheck.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/MemoryProfiler/MemoryProfilerCommands.h"

namespace UE::Insights::MemoryProfiler
{

class FMemorySharedState;
class SMemoryProfilerWindow;

DECLARE_LOG_CATEGORY_EXTERN(LogMemoryProfiler, Log, All);

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * This class manages the Memory Profiler (Memory Insights) state and settings.
 */
class FMemoryProfilerManager : public TSharedFromThis<FMemoryProfilerManager>, public IInsightsComponent
{
	friend class FMemoryProfilerActionManager;

public:
	/** Creates the Memory Profiler manager, only one instance can exist. */
	FMemoryProfilerManager(TSharedRef<FUICommandList> InCommandList);

	/** Virtual destructor. */
	virtual ~FMemoryProfilerManager();

	/** Creates an instance of the Memory Profiler manager. */
	static TSharedPtr<FMemoryProfilerManager> CreateInstance();

	/**
	 * @return the global instance of the Memory Profiler manager.
	 * This is an internal singleton and cannot be used outside TraceInsights.
	 * For external use:
	 *     IUnrealInsightsModule& Module = FModuleManager::Get().LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	 *     Module.GetMemoryProfilerManager();
	 */
	static TSharedPtr<FMemoryProfilerManager> Get();

	bool IsAvailable() const { return bIsAvailable; }

	//////////////////////////////////////////////////
	// IInsightsComponent

	virtual void Initialize(IUnrealInsightsModule& InsightsModule) override;
	virtual void Shutdown() override;
	virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) override;
	virtual void UnregisterMajorTabs() override;
	virtual void OnWindowClosedEvent() override;
	virtual bool Exec(const TCHAR* Cmd, FOutputDevice& Ar) override;

	//////////////////////////////////////////////////

	/** @return UI command list for the Memory Profiler manager. */
	const TSharedRef<FUICommandList> GetCommandList() const;

	/** @return an instance of the Memory Profiler commands. */
	static const FMemoryProfilerCommands& GetCommands();

	/** @return an instance of the Memory Profiler action manager. */
	static FMemoryProfilerActionManager& GetActionManager();

	/**
	 * Converts profiler window weak pointer to a shared pointer and returns it.
	 * Make sure the returned pointer is valid before trying to dereference it.
	 */
	TSharedPtr<SMemoryProfilerWindow> GetProfilerWindow() const
	{
		return ProfilerWindowWeakPtr.Pin();
	}

	const FName& GetMajorTabId() const { return MajorTabId; }
	const FName& GetLogListingName() const { return LogListingName; }

	FMemorySharedState* GetSharedState();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Getters and setters used by Toggle Commands.

	const bool IsTimingViewVisible() const { return bIsTimingViewVisible; }
	void SetTimingViewVisible(const bool bIsVisible) { bIsTimingViewVisible = bIsVisible; }
	void ShowHideTimingView(const bool bIsVisible);

	const bool IsMemInvestigationViewVisible() const { return bIsMemInvestigationViewVisible; }
	void SetMemInvestigationViewVisible(const bool bIsVisible) { bIsMemInvestigationViewVisible = bIsVisible; }
	void ShowHideMemInvestigationView(const bool bIsVisible);

	const bool IsMemTagTreeViewVisible() const { return bIsMemTagTreeViewVisible; }
	void SetMemTagTreeViewVisible(const bool bIsVisible) { bIsMemTagTreeViewVisible = bIsVisible; }
	void ShowHideMemTagTreeView(const bool bIsVisible);

	const bool IsModulesViewVisible() const { return bIsModulesViewVisible; }
	void SetModulesViewVisible(const bool bIsVisible) { bIsModulesViewVisible = bIsVisible; }
	void ShowHideModulesView(const bool bIsVisible);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void OnSessionChanged();

private:
	/** Binds our UI commands to delegates. */
	void BindCommands();

	/** Called to spawn the Memory Profiler major tab. */
	TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args);

	bool CanSpawnTab(const FSpawnTabArgs& Args) const;

	/** Callback called when the Memory Profiler major tab is closed. */
	void OnTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	/** Updates this manager, done through FCoreTicker. */
	bool Tick(float DeltaTime);

	void AssignProfilerWindow(const TSharedRef<SMemoryProfilerWindow>& InProfilerWindow)
	{
		ProfilerWindowWeakPtr = InProfilerWindow;
	}

	void RemoveProfilerWindow()
	{
		ProfilerWindowWeakPtr.Reset();
	}

private:
	bool bIsInitialized = false;
	bool bIsAvailable = false;

	/** If the tab is registered in the tab manager. */
	bool bIsTabRegistered = false;

	FAvailabilityCheck AvailabilityCheck;

	/** The delegate to be invoked when this manager ticks. */
	FTickerDelegate OnTick;

	/** Handle to the registered OnTick. */
	FTSTicker::FDelegateHandle OnTickHandle;

	/** List of UI commands for this manager. This will be filled by this and corresponding classes. */
	TSharedRef<FUICommandList> CommandList;

	/** An instance of the Memory Profiler action manager. */
	FMemoryProfilerActionManager ActionManager;

	/** A weak pointer to the Memory Insights window. */
	TWeakPtr<SMemoryProfilerWindow> ProfilerWindowWeakPtr;

	/** If the Timing view is visible or hidden. */
	bool bIsTimingViewVisible = false;

	/** If the Memory Investigation (Alloc Queries) view is visible or hidden. */
	bool bIsMemInvestigationViewVisible = false;

	/** If the Memory Tags tree view is visible or hidden. */
	bool bIsMemTagTreeViewVisible = false;

	/** If the Modules view is visible or hidden. */
	bool bIsModulesViewVisible = false;

	/** The id of the Memory Insights major tab. */
	FName MajorTabId;

	/** The name of the Memory Profiler log listing. */
	FName LogListingName;

	/** A shared pointer to the global instance of the Memory Profiler manager. */
	static TSharedPtr<FMemoryProfilerManager> Instance;
};

} // namespace UE::Insights::MemoryProfiler
