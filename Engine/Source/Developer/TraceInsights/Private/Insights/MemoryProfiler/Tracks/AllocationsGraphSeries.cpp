// Copyright Epic Games, Inc. All Rights Reserved.

#include "AllocationsGraphSeries.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AllocationsProvider.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/MemoryProfiler/Tracks/MemoryGraphTrack.h"
#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "Insights/ViewModels/TimingTrackViewport.h"

#include <limits>

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemoryGraphTrack"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FAllocationsGraphSeries
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FAllocationsGraphSeries)

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationsGraphSeries::FAllocationsGraphSeries(ETimeline InTimeline)
	: Timeline(InTimeline)
{
	Initialize();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAllocationsGraphSeries::~FAllocationsGraphSeries()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsGraphSeries::Initialize()
{
	switch (Timeline)
	{
		case ETimeline::MinTotalMem:
		{
			ValueType = EValueType::IntegerBytes;
			SetName(LOCTEXT("TotalAllocatedMemoryMin_TrackName", "Total Allocated Memory (Min)"));
			SetDescription(LOCTEXT("TotalAllocatedMemoryMin_TrackDesc", "Minimum value per sample for the Total Allocated Memory"));
			SetColor(FLinearColor(0.0f, 0.5f, 1.0f, 1.0f));
			break;
		}

		case ETimeline::MaxTotalMem:
		{
			ValueType = EValueType::IntegerBytes;
			SetName(LOCTEXT("TotalAllocatedMemoryMax_TrackName", "Total Allocated Memory (Max)"));
			SetDescription(LOCTEXT("TotalAllocatedMemoryMax_TrackDesc", "Maximum value per sample for the Total Allocated Memory"));
			SetColor(FLinearColor(1.0f, 0.25f, 1.0f, 1.0f));
			break;
		}

		case ETimeline::MinLiveAllocs:
		{
			ValueType = EValueType::IntegerCounter;
			SetName(LOCTEXT("LiveAllocationCountMin_TrackName", "Live Allocation Count (Min)"));
			SetDescription(LOCTEXT("LiveAllocationCountMin_TrackDesc", "Minimum value per sample for the Live Allocation Count"));
			SetColor(FLinearColor(1.0f, 1.0f, 0.25f, 1.0f));
			break;
		}

		case ETimeline::MaxLiveAllocs:
		{
			ValueType = EValueType::IntegerCounter;
			SetName(LOCTEXT("LiveAllocationCountMax_TrackName", "Live Allocation Count (Max)"));
			SetDescription(LOCTEXT("LiveAllocationCountMax_TrackDesc", "Maximum value per sample for the Live Allocation Count"));
			SetColor(FLinearColor(1.0f, 0.25f, 1.0f, 1.0f));
			break;
		}

		case ETimeline::MinSwapMem:
		{
			ValueType = EValueType::IntegerBytes;
			SetName(LOCTEXT("TotalSwapMemoryMin_TrackName", "Total Swap Memory (Min)"));
			SetDescription(LOCTEXT("TotalSwapMemoryMin_TrackDesc", "Minimum value per sample for the Total Swap Memory"));
			SetColor(FLinearColor(0.0f, 0.5f, 1.0f, 1.0f));
			break;
		}

		case ETimeline::MaxSwapMem:
		{
			ValueType = EValueType::IntegerBytes;
			SetName(LOCTEXT("TotalSwapMemoryMax_TrackName", "Total Swap Memory (Max)"));
			SetDescription(LOCTEXT("TotalSwapMemoryMax_TrackDesc", "Maximum value per sample for the Total Swap Memory"));
			SetColor(FLinearColor(1.0f, 0.25f, 1.0f, 1.0f));
			break;
		}

		case ETimeline::MinCompressedSwapMem:
		{
			ValueType = EValueType::IntegerBytes;
			SetName(LOCTEXT("TotalCompressedSwapMemoryMin_TrackName", "Total Compressed Swap Memory (Min)"));
			SetDescription(LOCTEXT("TotalCompressedSwapMemoryMin_TrackDesc", "Minimum value per sample for the Total Compressed Swap Memory"));
			SetColor(FLinearColor(1.0f, 1.0f, 0.25f, 1.0f));
			break;
		}

		case ETimeline::MaxCompressedSwapMem:
		{
			ValueType = EValueType::IntegerBytes;
			SetName(LOCTEXT("TotalCompressedSwapMemoryMax_TrackName", "Total Compressed Swap Memory (Max)"));
			SetDescription(LOCTEXT("TotalCompressedSwapMemoryMax_TrackDesc", "Maximum value per sample for the Total Compressed Swap Memory"));
			SetColor(FLinearColor(1.0f, 0.25f, 1.0f, 1.0f));
			break;
		}

		case ETimeline::AllocEvents:
		{
			ValueType = EValueType::IntegerCounter;
			SetName(LOCTEXT("AllocEventCount_TrackName", "Alloc Event Count"));
			SetDescription(LOCTEXT("AllocEventCount_TrackDesc", "Number of alloc events per sample"));
			SetColor(FLinearColor(0.0f, 1.0f, 0.5f, 1.0f));
			break;
		}

		case ETimeline::FreeEvents:
		{
			ValueType = EValueType::IntegerCounter;
			SetName(LOCTEXT("FreeEventCount_TrackName", "Free Event Count"));
			SetDescription(LOCTEXT("FreeEventCount_TrackDesc", "Number of free events per sample"));
			SetColor(FLinearColor(1.0f, 0.5f, 0.25f, 1.0f));
			break;
		}

		case ETimeline::PageInEvents:
		{
			ValueType = EValueType::IntegerCounter;
			SetName(LOCTEXT("PageInEventCount_TrackName", "Page In Event Count"));
			SetDescription(LOCTEXT("PageInEventCount_TrackDesc", "Number of page in events per sample"));
			SetColor(FLinearColor(0.0f, 1.0f, 0.5f, 1.0f));
			break;
		}

		case ETimeline::PageOutEvents:
		{
			ValueType = EValueType::IntegerCounter;
			SetName(LOCTEXT("PageOutEventCount_TrackName", "Page Out Event Count"));
			SetDescription(LOCTEXT("PageOutEventCount_TrackDesc", "Number of page out events per sample"));
			SetColor(FLinearColor(1.0f, 0.5f, 0.25f, 1.0f));
			break;
		}

		case ETimeline::SwapFreeEvents:
		{
			ValueType = EValueType::IntegerCounter;
			SetName(LOCTEXT("SwapFreeEventCount_TrackName", "Swap Free Event Count"));
			SetDescription(LOCTEXT("SwapFreeEventCount_TrackDesc", "Number of swap free events per sample"));
			SetColor(FLinearColor(0.25f, 0.5f, 1.0f, 1.0f));
			break;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FAllocationsGraphSeries::FormatValue(double Value) const
{
	if (ValueType == EValueType::IntegerBytes)
	{
		const int64 ValueInt64 = static_cast<int64>(Value);
		if (ValueInt64 == 0)
		{
			return TEXT("0");
		}

		double UnitValue;
		const TCHAR* UnitText;
		FMemoryGraphTrack::GetUnit(EGraphTrackLabelUnit::Auto, FMath::Abs(Value), UnitValue, UnitText);

		constexpr int32 DefaultDecimalDigitCount = 2;

		if (ValueInt64 < 0)
		{
			FString Auto = FMemoryGraphTrack::FormatValue(-Value, UnitValue, UnitText, DefaultDecimalDigitCount);
			return FString::Printf(TEXT("-%s (%s bytes)"), *Auto, *FText::AsNumber(ValueInt64).ToString());
		}
		else
		{
			FString Auto = FMemoryGraphTrack::FormatValue(Value, UnitValue, UnitText, DefaultDecimalDigitCount);
			return FString::Printf(TEXT("%s (%s bytes)"), *Auto, *FText::AsNumber(ValueInt64).ToString());
		}
	}
	else if (ValueType == EValueType::IntegerCounter)
	{
		const int64 ValueInt64 = static_cast<int64>(Value);
		if (ValueInt64 == 0)
		{
			return TEXT("0");
		}

		return FText::AsNumber(ValueInt64).ToString();
	}
	else
	{
		return FText::AsNumber(Value).ToString();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsGraphSeries::PreUpdate(FGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::IAllocationsProvider* AllocationsProvider = TraceServices::ReadAllocationsProvider(*Session.Get());
		if (AllocationsProvider)
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*AllocationsProvider);

			int32 StartIndex = -1;
			int32 EndIndex = -1;

			AllocationsProvider->GetTimelineIndexRange(Viewport.GetStartTime(), Viewport.GetEndTime(), StartIndex, EndIndex);

			if (EndIndex >= 0)
			{
				--StartIndex; // include one more point on the left side
				++EndIndex; // include one more point on the right side
			}

			uint64 LocalMinValue = std::numeric_limits<uint64>::max();
			uint64 LocalMaxValue = 0;

			// Compute Min/Max values.
			auto Callback64 = [&LocalMinValue, &LocalMaxValue](double Time, double Duration, uint64 Value)
			{
				if (Value < LocalMinValue)
				{
					LocalMinValue = Value;
				}
				if (Value > LocalMaxValue)
				{
					LocalMaxValue = Value;
				}
			};
			auto Callback32 = [&LocalMinValue, &LocalMaxValue](double Time, double Duration, uint32 Value)
			{
				if (Value < LocalMinValue)
				{
					LocalMinValue = Value;
				}
				if (Value > LocalMaxValue)
				{
					LocalMaxValue = Value;
				}
			};

			using ETimelineU32 = TraceServices::IAllocationsProvider::ETimelineU32;
			using ETimelineU64 = TraceServices::IAllocationsProvider::ETimelineU64;
			switch (Timeline)
			{
			case ETimeline::MinTotalMem:
				AllocationsProvider->EnumerateTimeline(ETimelineU64::MinTotalAllocatedMemory, StartIndex, EndIndex, Callback64);
				break;
			case ETimeline::MaxTotalMem:
				AllocationsProvider->EnumerateTimeline(ETimelineU64::MaxTotalAllocatedMemory, StartIndex, EndIndex, Callback64);
				break;
			case ETimeline::MinLiveAllocs:
				AllocationsProvider->EnumerateTimeline(ETimelineU32::MinLiveAllocations, StartIndex, EndIndex, Callback32);
				break;
			case ETimeline::MaxLiveAllocs:
				AllocationsProvider->EnumerateTimeline(ETimelineU32::MaxLiveAllocations, StartIndex, EndIndex, Callback32);
				break;
			case ETimeline::MinSwapMem:
				AllocationsProvider->EnumerateTimeline(ETimelineU64::MinTotalSwapMemory, StartIndex, EndIndex, Callback64);
				break;
			case ETimeline::MaxSwapMem:
				AllocationsProvider->EnumerateTimeline(ETimelineU64::MaxTotalSwapMemory, StartIndex, EndIndex, Callback64);
				break;
			case ETimeline::MinCompressedSwapMem:
				AllocationsProvider->EnumerateTimeline(ETimelineU64::MinTotalCompressedSwapMemory, StartIndex, EndIndex, Callback64);
				break;
			case ETimeline::MaxCompressedSwapMem:
				AllocationsProvider->EnumerateTimeline(ETimelineU64::MaxTotalCompressedSwapMemory, StartIndex, EndIndex, Callback64);
				break;
			case ETimeline::AllocEvents:
				AllocationsProvider->EnumerateTimeline(ETimelineU32::AllocEvents, StartIndex, EndIndex, Callback32);
				break;
			case ETimeline::FreeEvents:
				AllocationsProvider->EnumerateTimeline(ETimelineU32::FreeEvents, StartIndex, EndIndex, Callback32);
				break;
			case ETimeline::PageInEvents:
				AllocationsProvider->EnumerateTimeline(ETimelineU32::PageInEvents, StartIndex, EndIndex, Callback32);
				break;
			case ETimeline::PageOutEvents:
				AllocationsProvider->EnumerateTimeline(ETimelineU32::PageOutEvents, StartIndex, EndIndex, Callback32);
				break;
			case ETimeline::SwapFreeEvents:
				AllocationsProvider->EnumerateTimeline(ETimelineU32::SwapFreeEvents, StartIndex, EndIndex, Callback32);
				break;
			}

			if (Timeline == ETimeline::FreeEvents)
			{
				// Shows FreeEvents as negative values in order to be displayed on same graph as AllocEvents.
				SetValueRange(-static_cast<double>(LocalMaxValue), -static_cast<double>(LocalMinValue));
			}
			else
			{
				SetValueRange(static_cast<double>(LocalMinValue), static_cast<double>(LocalMaxValue));
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAllocationsGraphSeries::Update(FGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport)
{
	FGraphTrackBuilder Builder(GraphTrack, *this, Viewport);

	//////////////////////////////////////////////////
	// Update auto-zoom (vertical scaling)...

	const float TopY = 4.0f;
	const float BottomY = GraphTrack.GetHeight() - 4.0f;

	if (IsAutoZoomEnabled() && TopY < BottomY)
	{
		UpdateAutoZoom(TopY, BottomY, GetMinValue(), GetMaxValue());
	}

	//////////////////////////////////////////////////

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::IAllocationsProvider* AllocationsProvider = TraceServices::ReadAllocationsProvider(*Session.Get());
		if (AllocationsProvider)
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*AllocationsProvider);

			int32 StartIndex = -1;
			int32 EndIndex = -1;

			AllocationsProvider->GetTimelineIndexRange(Viewport.GetStartTime(), Viewport.GetEndTime(), StartIndex, EndIndex);

			if (EndIndex >= 0)
			{
				--StartIndex; // include one more point on the left side
				++EndIndex; // include one more point on the right side
			}

			auto Callback64 = [&Builder](double Time, double Duration, uint64 Value)
			{
				Builder.AddEvent(Time, Duration, static_cast<double>(Value));
			};
			auto Callback32 = [&Builder](double Time, double Duration, uint32 Value)
			{
				Builder.AddEvent(Time, Duration, static_cast<double>(Value));
			};
			auto Callback32Negative = [&Builder](double Time, double Duration, uint32 Value)
			{
				// Shows FreeEvents as negative values in order to be displayed on same graph as AllocEvents.
				Builder.AddEvent(Time, Duration, -static_cast<double>(Value));
			};

			using ETimelineU32 = TraceServices::IAllocationsProvider::ETimelineU32;
			using ETimelineU64 = TraceServices::IAllocationsProvider::ETimelineU64;
			switch (Timeline)
			{
			case ETimeline::MinTotalMem:
				AllocationsProvider->EnumerateTimeline(ETimelineU64::MinTotalAllocatedMemory, StartIndex, EndIndex, Callback64);
				break;
			case ETimeline::MaxTotalMem:
				AllocationsProvider->EnumerateTimeline(ETimelineU64::MaxTotalAllocatedMemory, StartIndex, EndIndex, Callback64);
				break;
			case ETimeline::MinLiveAllocs:
				AllocationsProvider->EnumerateTimeline(ETimelineU32::MinLiveAllocations, StartIndex, EndIndex, Callback32);
				break;
			case ETimeline::MaxLiveAllocs:
				AllocationsProvider->EnumerateTimeline(ETimelineU32::MaxLiveAllocations, StartIndex, EndIndex, Callback32);
				break;
			case ETimeline::MinSwapMem:
				AllocationsProvider->EnumerateTimeline(ETimelineU64::MinTotalSwapMemory, StartIndex, EndIndex, Callback64);
				break;
			case ETimeline::MaxSwapMem:
				AllocationsProvider->EnumerateTimeline(ETimelineU64::MaxTotalSwapMemory, StartIndex, EndIndex, Callback64);
				break;
			case ETimeline::MinCompressedSwapMem:
				AllocationsProvider->EnumerateTimeline(ETimelineU64::MinTotalCompressedSwapMemory, StartIndex, EndIndex, Callback64);
				break;
			case ETimeline::MaxCompressedSwapMem:
				AllocationsProvider->EnumerateTimeline(ETimelineU64::MaxTotalCompressedSwapMemory, StartIndex, EndIndex, Callback64);
				break;
			case ETimeline::AllocEvents:
				AllocationsProvider->EnumerateTimeline(ETimelineU32::AllocEvents, StartIndex, EndIndex, Callback32);
				break;
			case ETimeline::FreeEvents:
				AllocationsProvider->EnumerateTimeline(ETimelineU32::FreeEvents, StartIndex, EndIndex, Callback32Negative);
				break;
			case ETimeline::PageInEvents:
				AllocationsProvider->EnumerateTimeline(ETimelineU32::PageInEvents, StartIndex, EndIndex, Callback32);
				break;
			case ETimeline::PageOutEvents:
				AllocationsProvider->EnumerateTimeline(ETimelineU32::PageOutEvents, StartIndex, EndIndex, Callback32Negative);
				break;
			case ETimeline::SwapFreeEvents:
				AllocationsProvider->EnumerateTimeline(ETimelineU32::SwapFreeEvents, StartIndex, EndIndex, Callback32);
				break;
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
