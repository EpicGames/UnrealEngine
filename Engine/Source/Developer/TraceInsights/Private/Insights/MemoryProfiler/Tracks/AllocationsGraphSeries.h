// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/MemoryProfiler/Tracks/MemoryGraphSeries.h"

namespace UE::Insights::MemoryProfiler
{

class FAllocationsGraphSeries : public FMemoryGraphSeries
{
	INSIGHTS_DECLARE_RTTI(FAllocationsGraphSeries, FMemoryGraphSeries)

public:
	enum class ETimeline
	{
		Unknown = -1,

		MinTotalMem = 0,
		MaxTotalMem,
		MinLiveAllocs,
		MaxLiveAllocs,
		MinSwapMem,
		MaxSwapMem,
		MinCompressedSwapMem,
		MaxCompressedSwapMem,
		AllocEvents,
		FreeEvents,
		PageInEvents,
		PageOutEvents,
		SwapFreeEvents,

		/** Not an actual parameter. Number of known timelines. */
		Count,
	};

	enum class EValueType
	{
		Unknown,
		IntegerBytes,
		IntegerCounter,
	};

public:
	FAllocationsGraphSeries(ETimeline InTimeline);
	virtual ~FAllocationsGraphSeries();

	virtual FString FormatValue(double Value) const override;
	virtual void PreUpdate(FGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport) override;
	virtual void Update(FGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport) override;

	ETimeline GetTimeline() const { return Timeline; }

private:
	void Initialize();

private:
	ETimeline Timeline = ETimeline::Unknown;
	EValueType ValueType = EValueType::Unknown;
};

} // namespace UE::Insights::MemoryProfiler
