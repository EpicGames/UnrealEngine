// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemTagGraphSeries.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/Memory.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/MemoryProfiler/Tracks/MemoryGraphTrack.h"
#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "Insights/ViewModels/TimingTrackViewport.h"

#include <cmath>
#include <limits>

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemoryGraphTrack"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemTagGraphSeries
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FMemTagGraphSeries)

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemTagGraphSeries::FMemTagGraphSeries(FMemoryTrackerId InTrackerId, FMemoryTagSetId InTagSetId, FMemoryTagId InTagId)
	: TrackerId(InTrackerId)
	, TagSetId(InTagSetId)
	, TagId(InTagId)
{
	SetName(TEXT("LLM Tag"));
	SetDescription(TEXT("Low Level Memory Tag"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemTagGraphSeries::~FMemTagGraphSeries()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemTagGraphSeries::HasHighThresholdValue() const
{
	return std::isfinite(HighThresholdValue);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double FMemTagGraphSeries::GetHighThresholdValue() const
{
	return HighThresholdValue;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagGraphSeries::SetHighThresholdValue(double InValue)
{
	HighThresholdValue = InValue;
	SetAutoZoomDirty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagGraphSeries::ResetHighThresholdValue()
{
	HighThresholdValue = +std::numeric_limits<double>::infinity();
	SetAutoZoomDirty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemTagGraphSeries::HasLowThresholdValue() const
{
	return std::isfinite(LowThresholdValue);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double FMemTagGraphSeries::GetLowThresholdValue() const
{
	return LowThresholdValue;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagGraphSeries::SetLowThresholdValue(double InValue)
{
	LowThresholdValue = InValue;
	SetAutoZoomDirty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagGraphSeries::ResetLowThresholdValue()
{
	LowThresholdValue = -std::numeric_limits<double>::infinity();
	SetAutoZoomDirty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FMemTagGraphSeries::FormatValue(double Value) const
{
	const int64 ValueInt64 = static_cast<int64>(Value);
	if (ValueInt64 == 0)
	{
		return TEXT("0");
	}

	double UnitValue;
	const TCHAR* UnitText;
	FMemoryGraphTrack::GetUnit(EGraphTrackLabelUnit::Auto, FMath::Abs(Value), UnitValue, UnitText);

	constexpr int32 DefaultDecimalDigitCount = 2;

	if (ValueInt64 < 0)
	{
		FString Auto = FMemoryGraphTrack::FormatValue(-Value, UnitValue, UnitText, DefaultDecimalDigitCount);
		return FString::Printf(TEXT("-%s (%s bytes)"), *Auto, *FText::AsNumber(ValueInt64).ToString());
	}
	else
	{
		FString Auto = FMemoryGraphTrack::FormatValue(Value, UnitValue, UnitText, DefaultDecimalDigitCount);
		return FString::Printf(TEXT("%s (%s bytes)"), *Auto, *FText::AsNumber(ValueInt64).ToString());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagGraphSeries::PreUpdate(FGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport)
{
	double LocalMinValue = +std::numeric_limits<double>::infinity();
	double LocalMaxValue = -std::numeric_limits<double>::infinity();

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::IMemoryProvider* MemoryProvider = TraceServices::ReadMemoryProvider(*Session.Get());
		if (MemoryProvider)
		{
			TraceServices::FProviderReadScopeLock _(*MemoryProvider);

			const uint64 TotalSampleCount = MemoryProvider->GetTagSampleCount(TrackerId, TagId);
			if (TotalSampleCount > 0)
			{
				// Compute Min/Max values.
				MemoryProvider->EnumerateTagSamples(TrackerId, TagId, Viewport.GetStartTime(), Viewport.GetEndTime(), true,
					[&LocalMinValue, &LocalMaxValue](double Time, double Duration, const TraceServices::FMemoryTagSample& Sample)
					{
						const double Value = static_cast<double>(Sample.Value);
						ExpandRange(LocalMinValue, LocalMaxValue, Value);
					});
			}
		}
	}

	SetValueRange(LocalMinValue, LocalMaxValue);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagGraphSeries::Update(FGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport)
{
	FGraphTrackBuilder Builder(GraphTrack, *this, Viewport);

	//////////////////////////////////////////////////
	// Update auto-zoom (vertical scaling)...

	const float TopY = 4.0f;
	const float BottomY = GraphTrack.GetHeight() - 4.0f;

	if (IsAutoZoomEnabled() && TopY < BottomY)
	{
		double ZoomMinValue = GetMinValue();
		double ZoomMaxValue = GetMaxValue();

		bool bAutoZoomIncludesBaseline = GraphTrack.IsAnyOptionEnabled(EGraphOptions::AutoZoomIncludesBaseline);
		if (bAutoZoomIncludesBaseline)
		{
			ExpandRange(ZoomMinValue, ZoomMaxValue, 0.0);
		}

		bool bAutoZoomIncludesThresholds = GraphTrack.IsAnyOptionEnabled(EGraphOptions::AutoZoomIncludesThresholds);
		if (bAutoZoomIncludesThresholds)
		{
			if (HasHighThresholdValue())
			{
				ExpandRange(ZoomMinValue, ZoomMaxValue, GetHighThresholdValue());
			}
			if (HasLowThresholdValue())
			{
				ExpandRange(ZoomMinValue, ZoomMaxValue, GetLowThresholdValue());
			}
		}

		UpdateAutoZoom(TopY, BottomY, ZoomMinValue, ZoomMaxValue);
	}

	//////////////////////////////////////////////////

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::IMemoryProvider* MemoryProvider = TraceServices::ReadMemoryProvider(*Session.Get());
		if (MemoryProvider)
		{
			TraceServices::FProviderReadScopeLock _(*MemoryProvider);

			const uint64 TotalSampleCount = MemoryProvider->GetTagSampleCount(TrackerId, TagId);
			if (TotalSampleCount > 0)
			{
				MemoryProvider->EnumerateTagSamples(TrackerId, TagId, Viewport.GetStartTime(), Viewport.GetEndTime(), true,
					[this, &Builder](double Time, double Duration, const TraceServices::FMemoryTagSample& Sample)
					{
						Builder.AddEvent(Time, Duration, static_cast<double>(Sample.Value));
					});
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
