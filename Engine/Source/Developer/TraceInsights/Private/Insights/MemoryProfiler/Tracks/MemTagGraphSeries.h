// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/MemoryProfiler/Tracks/MemoryGraphSeries.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryTag.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryTracker.h"

#include <limits>

namespace UE::Insights::MemoryProfiler
{

class FMemTagGraphSeries : public FMemoryGraphSeries
{
	INSIGHTS_DECLARE_RTTI(FMemTagGraphSeries, FMemoryGraphSeries)

public:
	FMemTagGraphSeries(FMemoryTrackerId InTrackerId, FMemoryTagSetId InTagSetId, FMemoryTagId InTagId);
	virtual ~FMemTagGraphSeries();

	virtual bool HasHighThresholdValue() const override;
	virtual double GetHighThresholdValue() const override;
	virtual void SetHighThresholdValue(double InValue) override;
	virtual void ResetHighThresholdValue() override;

	virtual bool HasLowThresholdValue() const override;
	virtual double GetLowThresholdValue() const override;
	virtual void SetLowThresholdValue(double InValue) override;
	virtual void ResetLowThresholdValue() override;

	virtual FString FormatValue(double Value) const override;
	virtual void PreUpdate(FGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport) override;
	virtual void Update(FGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport) override;

	FMemoryTrackerId GetTrackerId() const { return TrackerId; }
	FMemoryTagSetId GetTagSetId() const { return TagSetId; }
	FMemoryTagId GetTagId() const { return TagId; }

private:
	FMemoryTrackerId TrackerId = FMemoryTracker::InvalidTrackerId; // LLM tracker id
	FMemoryTagSetId TagSetId = FMemoryTagSet::InvalidTagSetId; // LLM tag set id
	FMemoryTagId TagId = FMemoryTag::InvalidTagId; // LLM tag id

	double HighThresholdValue = +std::numeric_limits<double>::infinity();
	double LowThresholdValue  = -std::numeric_limits<double>::infinity();
};

} // namespace UE::Insights::MemoryProfiler
