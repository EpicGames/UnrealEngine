// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryGraphSeries.h"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemoryGraphSeries
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FMemoryGraphSeries)

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FMemoryGraphSeries::FormatValue(double Value) const
{
	const int64 ValueInt64 = static_cast<int64>(Value);
	if (ValueInt64 == 0)
	{
		return TEXT("0");
	}

	return FText::AsNumber(ValueInt64).ToString();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryGraphSeries::ExpandRange(double& InOutMinValue, double& InOutMaxValue, double InValue)
{
	if (InValue < InOutMinValue)
	{
		InOutMinValue = InValue;
	}
	if (InValue > InOutMaxValue)
	{
		InOutMaxValue = InValue;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
