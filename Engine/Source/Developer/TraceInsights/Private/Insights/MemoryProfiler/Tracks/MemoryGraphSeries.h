// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/ViewModels/GraphSeries.h"
#include "Insights/ViewModels/GraphTrack.h"

namespace UE::Insights::MemoryProfiler
{

class FMemoryGraphSeries : public FGraphSeries
{
	INSIGHTS_DECLARE_RTTI(FMemoryGraphSeries, FGraphSeries)

public:
	FMemoryGraphSeries()
	{
	}
	virtual ~FMemoryGraphSeries() = default;

	virtual FString FormatValue(double Value) const override;
	virtual void PreUpdate(FGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport) = 0;
	virtual void Update(FGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport) = 0;

	double GetMinValue() const { return MinValue; }
	double GetMaxValue() const { return MaxValue; }
	void SetValueRange(double Min, double Max) { MinValue = Min; MaxValue = Max; }

protected:
	static void ExpandRange(double& InOutMinValue, double& InOutMaxValue, double InValue);

private:
	double MinValue = 0.0;
	double MaxValue = 0.0;
};

} // namespace UE::Insights::MemoryProfiler
