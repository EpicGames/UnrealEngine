// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryGraphTrack.h"

#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/SlateBrush.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/MemoryProfiler/Tracks/AllocationsGraphSeries.h"
#include "Insights/MemoryProfiler/Tracks/MemTagGraphSeries.h"
#include "Insights/MemoryProfiler/ViewModels/MemorySharedState.h"
#include "Insights/ViewModels/AxisViewportDouble.h"
#include "Insights/ViewModels/ITimingViewDrawHelper.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"

#include <limits>
#include <cmath>

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemoryGraphTrack"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemoryGraphTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FMemoryGraphTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryGraphTrack::FMemoryGraphTrack(FMemorySharedState& InSharedState)
	: FGraphTrack()
	, SharedState(InSharedState)
{
	EnabledOptions = //EGraphOptions::ShowDebugInfo |
					 //EGraphOptions::ShowPoints |
					 EGraphOptions::ShowPointsWithBorder |
					 EGraphOptions::ShowLines |
					 EGraphOptions::ShowPolygon |
					 EGraphOptions::UseEventDuration |
					 //EGraphOptions::ShowBars |
					 //EGraphOptions::ShowBaseline |
					 //EGraphOptions::ShowThresholds |
					 EGraphOptions::ShowVerticalAxisGrid |
					 EGraphOptions::ShowHeader |
					 EGraphOptions::None;
	//VisibleOptions |= EGraphOptions::AutoZoomIncludesBaseline | EGraphOptions::AutoZoomIncludesThresholds;
	//EditableOptions |= EGraphOptions::AutoZoomIncludesBaseline | EGraphOptions::AutoZoomIncludesThresholds;

	for (uint32 Mode = 0; Mode < static_cast<uint32>(EMemoryTrackHeightMode::Count); ++Mode)
	{
		SetAvailableTrackHeight(static_cast<EMemoryTrackHeightMode>(Mode), 100.0f * static_cast<float>(Mode + 1));
	}
	SetHeight(200.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryGraphTrack::ResetDefaultValueRange()
{
	DefaultMinValue = +std::numeric_limits<double>::infinity();
	DefaultMaxValue = -std::numeric_limits<double>::infinity();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryGraphTrack::~FMemoryGraphTrack()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static void ExpandRange(double& InOutMinValue, double& InOutMaxValue, double InValue)
{
	if (InValue < InOutMinValue)
	{
		InOutMinValue = InValue;
	}
	if (InValue > InOutMaxValue)
	{
		InOutMaxValue = InValue;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryGraphTrack::Update(const ITimingTrackUpdateContext& Context)
{
	FGraphTrack::Update(Context);

	const bool bIsEntireGraphTrackDirty = IsDirty() || Context.GetViewport().IsHorizontalViewportDirty();
	bool bNeedsUpdate = bIsEntireGraphTrackDirty;

	if (!bNeedsUpdate)
	{
		for (TSharedPtr<FGraphSeries>& Series : AllSeries)
		{
			if (Series->IsVisible() && Series->IsDirty())
			{
				// At least one series is dirty.
				bNeedsUpdate = true;
				break;
			}
		}
	}

	if (bNeedsUpdate)
	{
		ClearDirtyFlag();

		NumAddedEvents = 0;

		const FTimingTrackViewport& Viewport = Context.GetViewport();

		AllSeriesMinValue = DefaultMinValue;
		AllSeriesMaxValue = DefaultMaxValue;

		if (IsAnyOptionEnabled(EGraphOptions::AutoZoomIncludesBaseline))
		{
			ExpandRange(AllSeriesMinValue, AllSeriesMaxValue, 0.0);
		}

		for (TSharedPtr<FGraphSeries>& Series : AllSeries)
		{
			if (Series->IsVisible() &&
				Series->Is<FMemoryGraphSeries>())
			{
				FMemoryGraphSeries& MemorySeries = Series->As<FMemoryGraphSeries>();

				if (bIsEntireGraphTrackDirty || MemorySeries.IsDirty())
				{
					MemorySeries.PreUpdate(*this, Viewport);
				}

				const double SeriesMinValue = MemorySeries.GetMinValue();
				const double SeriesMaxValue = MemorySeries.GetMaxValue();
				if (SeriesMinValue <= SeriesMaxValue && std::isfinite(SeriesMinValue))
				{
					ExpandRange(AllSeriesMinValue, AllSeriesMaxValue, SeriesMinValue);
					ExpandRange(AllSeriesMinValue, AllSeriesMaxValue, SeriesMaxValue);
				}

				if (IsAnyOptionEnabled(EGraphOptions::AutoZoomIncludesThresholds))
				{
					if (Series->HasHighThresholdValue())
					{
						ExpandRange(AllSeriesMinValue, AllSeriesMaxValue, Series->GetHighThresholdValue());
					}
					if (Series->HasLowThresholdValue())
					{
						ExpandRange(AllSeriesMinValue, AllSeriesMaxValue, Series->GetLowThresholdValue());
					}
				}
			}
		}

		if (bAutoZoom)
		{
			const float TopY = 4.0f;
			const float BottomY = GetHeight() - 4.0f;
			if (TopY < BottomY)
			{
				for (TSharedPtr<FGraphSeries>& Series : AllSeries)
				{
					if (Series->IsVisible())
					{
						if (Series->UpdateAutoZoomEx(TopY, BottomY, AllSeriesMinValue, AllSeriesMaxValue, true))
						{
							Series->SetDirtyFlag();
						}
					}
				}
			}
		}

		for (TSharedPtr<FGraphSeries>& Series : AllSeries)
		{
			if (Series->IsVisible() && (bIsEntireGraphTrackDirty || Series->IsDirty()))
			{
				// Clear the flag before updating, because the update itself may further need to set the series as dirty.
				Series->ClearDirtyFlag();

				if (Series->Is<FMemoryGraphSeries>())
				{
					FMemoryGraphSeries& MemorySeries = Series->As<FMemoryGraphSeries>();
					MemorySeries.Update(*this, Viewport);
				}

				if (Series->IsAutoZoomDirty())
				{
					Series->SetDirtyFlag();
				}
			}
		}

		UpdateStats();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemoryGraphTrack - LLM Tag Series
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FMemTagGraphSeries> FMemoryGraphTrack::GetMemTagSeries(FMemoryTrackerId InMemTrackerId, FMemoryTagSetId InMemTagSetId, FMemoryTagId InMemTagId)
{
	TSharedPtr<FGraphSeries>* FoundSeries = AllSeries.FindByPredicate(
		[InMemTrackerId, InMemTagSetId, InMemTagId](const TSharedPtr<FGraphSeries>& GraphSeries)
		{
			if (!GraphSeries->Is<FMemTagGraphSeries>())
			{
				return false;
			}
			const FMemTagGraphSeries& MemTagSeries = GraphSeries->As<FMemTagGraphSeries>();
			return MemTagSeries.GetTrackerId() == InMemTrackerId &&
				   MemTagSeries.GetTagSetId() == InMemTagSetId &&
				   MemTagSeries.GetTagId() == InMemTagId;
		});
	return FoundSeries ? StaticCastSharedPtr<FMemTagGraphSeries>(*FoundSeries) : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FMemTagGraphSeries> FMemoryGraphTrack::AddMemTagSeries(FMemoryTrackerId InMemTrackerId, FMemoryTagSetId InMemTagSetId, FMemoryTagId InMemTagId)
{
	TSharedPtr<FMemTagGraphSeries> Series = GetMemTagSeries(InMemTrackerId, InMemTagSetId, InMemTagId);

	if (!Series.IsValid())
	{
		Series = MakeShared<FMemTagGraphSeries>(InMemTrackerId, InMemTagSetId, InMemTagId);

		Series->SetValueRange(0.0f, 0.0f);

		Series->SetBaselineY(GetHeight() - 1.0f);
		Series->SetScaleY(1.0);

		AllSeries.Add(Series);
		SetDirtyFlag();
	}

	return Series;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FMemoryGraphTrack::RemoveMemTagSeries(FMemoryTrackerId InMemTrackerId, FMemoryTagSetId InMemTagSetId, FMemoryTagId InMemTagId)
{
	SetDirtyFlag();
	return AllSeries.RemoveAll(
		[InMemTrackerId, InMemTagSetId, InMemTagId](const TSharedPtr<FGraphSeries>& GraphSeries)
		{
			if (!GraphSeries->Is<FMemTagGraphSeries>())
			{
				return false;
			}
			const FMemTagGraphSeries& MemTagSeries = GraphSeries->As<FMemTagGraphSeries>();
			return MemTagSeries.GetTrackerId() == InMemTrackerId &&
				   MemTagSeries.GetTagSetId() == InMemTagSetId &&
				   MemTagSeries.GetTagId() == InMemTagId;
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FMemoryGraphTrack::RemoveAllMemTagSeries()
{
	SetDirtyFlag();
	return AllSeries.RemoveAll([](const TSharedPtr<FGraphSeries>& GraphSeries) { return GraphSeries->Is<FMemTagGraphSeries>(); });
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemoryGraphTrack - (Allocations) Timeline Series
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FAllocationsGraphSeries> FMemoryGraphTrack::GetTimelineSeries(FAllocationsGraphSeries::ETimeline InTimeline)
{
	TSharedPtr<FGraphSeries>* FoundSeries = AllSeries.FindByPredicate(
		[InTimeline](const TSharedPtr<FGraphSeries>& GraphSeries)
		{
			if (!GraphSeries->Is<FAllocationsGraphSeries>())
			{
				return false;
			}
			const FAllocationsGraphSeries& AllocationsSeries = GraphSeries->As<FAllocationsGraphSeries>();
			return AllocationsSeries.GetTimeline() == InTimeline;
		});
	return FoundSeries ? StaticCastSharedPtr<FAllocationsGraphSeries>(*FoundSeries) : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FAllocationsGraphSeries> FMemoryGraphTrack::AddTimelineSeries(FAllocationsGraphSeries::ETimeline InTimeline)
{
	TSharedPtr<FAllocationsGraphSeries> Series = GetTimelineSeries(InTimeline);

	if (!Series.IsValid())
	{
		Series = MakeShared<FAllocationsGraphSeries>(InTimeline);

		Series->SetValueRange(0.0f, 0.0f);

		Series->SetBaselineY(GetHeight() - 1.0f);
		Series->SetScaleY(1.0);

		AllSeries.Add(Series);
		SetDirtyFlag();
	}

	return Series;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemoryGraphTrack - misc
////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryGraphTrack::DrawVerticalAxisGrid(const ITimingTrackDrawContext& Context) const
{
	TSharedPtr<FMemoryGraphSeries> Series = MainSeries;

	if (!Series.IsValid())
	{
		// Use the first visible series.
		for (const TSharedPtr<FGraphSeries>& GraphSeries : AllSeries)
		{
			if (GraphSeries->IsVisible() && GraphSeries->Is<FMemoryGraphSeries>())
			{
				Series = StaticCastSharedPtr<FMemoryGraphSeries>(GraphSeries);
				break;
			}
		}
	}

	if (!Series.IsValid())
	{
		return;
	}

	FAxisViewportDouble ViewportY;
	ViewportY.SetSize(GetHeight());
	ViewportY.SetScaleLimits(std::numeric_limits<double>::min(), std::numeric_limits<double>::max());
	ViewportY.SetScale(Series->GetScaleY());
	ViewportY.ScrollAtPos(static_cast<float>(Series->GetBaselineY()) - GetHeight());

	const float ViewWidth = Context.GetViewport().GetWidth();
	const float RoundedViewHeight = FMath::RoundToFloat(GetHeight());

	const float X0 = ViewWidth - 12.0f; // let some space for the vertical scrollbar
	const float Y0 = GetPosY();

	constexpr float MinDY = 32.0f; // min vertical distance between horizontal grid lines
	constexpr float TextH = 14.0f; // label height

	const float MinLabelY = Y0 + 1.0f;
	const float MaxLabelY = Y0 + RoundedViewHeight - TextH;

	float MinValueY = Y0 - MinDY; // a value below the track
	float MaxValueY = Y0 + RoundedViewHeight + MinDY; // a value above the track
	float ActualMinValueY = MinValueY;
	float ActualMaxValueY = MaxValueY;

	const bool bHasMinMax = (AllSeriesMinValue <= AllSeriesMaxValue);
	if (bHasMinMax)
	{
		const float MinValueOffset = ViewportY.GetOffsetForValue(AllSeriesMinValue);
		const float MinValueRoundedOffset = FMath::RoundToFloat(MinValueOffset);
		ActualMinValueY = Y0 + RoundedViewHeight - MinValueRoundedOffset;
		MinValueY = FMath::Min(MaxLabelY, FMath::Max(MinLabelY, ActualMinValueY - TextH / 2));

		const float MaxValueOffset = ViewportY.GetOffsetForValue(AllSeriesMaxValue);
		const float MaxValueRoundedOffset = FMath::RoundToFloat(MaxValueOffset);
		ActualMaxValueY = Y0 + RoundedViewHeight - MaxValueRoundedOffset;
		MaxValueY = FMath::Min(MaxLabelY, FMath::Max(MinLabelY, ActualMaxValueY - TextH / 2));
	}

	// Label for the High Threshold value.
	float HighThresholdLabelY = 0.0;
	bool bShowHighThresholdLabel = IsAnyOptionEnabled(EGraphOptions::ShowThresholds) && Series->HasHighThresholdValue();
	if (bShowHighThresholdLabel)
	{
		HighThresholdLabelY = Y0 + RoundedViewHeight - FMath::RoundToFloat(ViewportY.GetOffsetForValue(Series->GetHighThresholdValue())) - TextH / 2;
		if (HighThresholdLabelY < MinLabelY ||
			HighThresholdLabelY > MaxLabelY ||
			(bHasMinMax && FMath::Abs(MinValueY - HighThresholdLabelY) < TextH) ||
			(bHasMinMax && FMath::Abs(MaxValueY - HighThresholdLabelY) < TextH))
		{
			bShowHighThresholdLabel = false;
		}
	}

	// Label for the Low Threshold value.
	float LowThresholdLabelY = 0.0;
	bool bShowLowThresholdLabel = IsAnyOptionEnabled(EGraphOptions::ShowThresholds) && Series->HasLowThresholdValue();
	if (bShowLowThresholdLabel)
	{
		LowThresholdLabelY = Y0 + RoundedViewHeight - FMath::RoundToFloat(ViewportY.GetOffsetForValue(Series->GetLowThresholdValue())) - TextH / 2;
		if (LowThresholdLabelY < MinLabelY ||
			LowThresholdLabelY > MaxLabelY ||
			(bHasMinMax && FMath::Abs(MinValueY - LowThresholdLabelY) < TextH) ||
			(bHasMinMax && FMath::Abs(MaxValueY - LowThresholdLabelY) < TextH))
		{
			bShowLowThresholdLabel = false;
		}
	}

	FDrawContext& DrawContext = Context.GetDrawContext();
	const FSlateBrush* Brush = Context.GetHelper().GetWhiteBrush();
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	if (AllSeries.Num() == 1 && Series->Is<FMemTagGraphSeries>())
	{
		// Show name of track (for LLM tag series) in the lower right side of the track.
		// (to avoid switching attention between left and right sides of the track when looking at grid values)
		TDrawLabelParams Params(DrawContext, Brush, FontMeasureService);
		Params.TextBgColor = (Series->GetColor() * 0.05f).CopyWithNewOpacity(0.2f);
		Params.TextColor = Series->GetBorderColor().CopyWithNewOpacity(0.2f);
		Params.X = ViewWidth - 150.0f;
		Params.Y = Y0 + RoundedViewHeight - 20.0f;
		Params.Precision = std::numeric_limits<double>::quiet_NaN();
		Params.Prefix = GetName();
		DrawLabel(Params);
	}

	const double TopValue = ViewportY.GetValueAtOffset(RoundedViewHeight);
	const double GridValue = ViewportY.GetValueAtOffset(MinDY);
	const double BottomValue = ViewportY.GetValueAtOffset(0.0f);
	const double Delta = GridValue - BottomValue;

	double Precision = FMath::Abs(TopValue - BottomValue);
	for (int32 Digit = FMath::Abs(LabelDecimalDigitCount); Digit > 0; --Digit)
	{
		Precision *= 10.0;
	}
	Precision = FMath::Min(Precision, TopValue);

	if (Delta > 0.0)
	{
		double Grid;

		if (Series->Is<FMemTagGraphSeries>() ||
			(Series->Is<FAllocationsGraphSeries>() &&
			 Series->As<FAllocationsGraphSeries>().GetTimeline() <= FAllocationsGraphSeries::ETimeline::MaxTotalMem))
		{
			const uint64 DeltaBytes = FMath::Max(1ULL, static_cast<uint64>(Delta));
			Grid = static_cast<double>(FMath::RoundUpToPowerOfTwo64(DeltaBytes));
		}
		else
		{
			const uint64 DeltaCount = FMath::Max(1ULL, static_cast<uint64>(Delta));

			// Compute rounding based on magnitude of visible range of values (Delta).
			uint64 Delta10 = DeltaCount;
			uint64 Power10 = 1;
			while (Delta10 > 0)
			{
				Delta10 /= 10;
				Power10 *= 10;
			}
			if (Power10 >= 100)
			{
				Power10 /= 100;
			}
			else
			{
				Power10 = 1;
			}

			// Compute Grid as the next value divisible with a multiple of 10.
			Grid = static_cast<double>(((DeltaCount + Power10 - 1) / Power10) * Power10);
		}

		const double StartValue = FMath::GridSnap(BottomValue, Grid);

		TDrawLabelParams Params(DrawContext, Brush, FontMeasureService);
		Params.TextBgColor = FLinearColor(0.05f, 0.05f, 0.05f, 1.0f);
		Params.TextColor = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
		Params.X = X0;
		Params.Precision = Precision;

		const FLinearColor GridLineColor(0.0f, 0.0f, 0.0f, 0.1f);

		for (double Value = StartValue; Value < TopValue; Value += Grid)
		{
			const float Y = Y0 + RoundedViewHeight - FMath::RoundToFloat(ViewportY.GetOffsetForValue(Value));

			const float LabelY = FMath::Min(MaxLabelY, FMath::Max(MinLabelY, Y - TextH / 2));

			// Do not overlap with the min/max values.
			if (bHasMinMax && (FMath::Abs(LabelY - MinValueY) < TextH || FMath::Abs(LabelY - MaxValueY) < TextH))
			{
				continue;
			}
			// Do not overlap with the High Threshold value.
			if (bShowHighThresholdLabel && FMath::Abs(LabelY - HighThresholdLabelY) < TextH)
			{
				continue;
			}
			// Do not overlap with the Low Threshold value.
			if (bShowLowThresholdLabel && FMath::Abs(LabelY - LowThresholdLabelY) < TextH)
			{
				continue;
			}

			// Draw horizontal grid line.
			DrawContext.DrawBox(0, Y, ViewWidth, 1, Brush, GridLineColor);

			// Draw label.
			Params.Y = LabelY;
			Params.Value = Value;
			DrawLabel(Params);
		}
	}

	const bool bIsMinHeight = (GetHeight() >= TextH);

	// Draw label for the High Threshold value.
	if (bShowHighThresholdLabel && bIsMinHeight)
	{
		TDrawLabelParams Params(DrawContext, Brush, FontMeasureService);
		Params.TextBgColor = FLinearColor(0.1f, 0.05f, 0.05f, 1.0f);
		Params.TextColor = FLinearColor(1.0f, 0.3f, 0.3f, 1.0f);
		Params.X = X0;
		Params.Y = HighThresholdLabelY;
		Params.Value = Series->GetHighThresholdValue();
		Params.Precision = -Precision;
		DrawLabel(Params);
	}

	// Draw label for the Low Threshold value.
	if (bShowLowThresholdLabel && bIsMinHeight)
	{
		TDrawLabelParams Params(DrawContext, Brush, FontMeasureService);
		Params.TextBgColor = FLinearColor(0.1f, 0.1f, 0.05f, 1.0f);
		Params.TextColor = FLinearColor(1.0f, 1.0f, 0.3f, 1.0f);
		Params.X = X0;
		Params.Y = LowThresholdLabelY;
		Params.Value = Series->GetLowThresholdValue();
		Params.Precision = -Precision;
		DrawLabel(Params);
	}

	if (bHasMinMax && bIsMinHeight)
	{
		TDrawLabelParams Params(DrawContext, Brush, FontMeasureService);

		if (MainSeries.IsValid() || AllSeries.Num() == 1)
		{
			Params.TextBgColor = (Series->GetColor() * 0.05f).CopyWithNewOpacity(1.0f);
			Params.TextColor= Series->GetBorderColor().CopyWithNewOpacity(1.0f);
		}
		else
		{
			Params.TextBgColor = FLinearColor(0.02f, 0.02f, 0.02f, 1.0f);
			Params.TextColor = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
		}

		Params.X = X0;
		Params.Precision = -Precision; // format with detailed text

		int32 MinMaxAxis = 0;

		// Draw label for the max value.
		if (MaxValueY >= Y0 && MaxValueY <= Y0 + RoundedViewHeight)
		{
			Params.Y = MaxValueY;
			Params.Value = AllSeriesMaxValue;
			DrawLabel(Params);
			++MinMaxAxis;
		}

		// Draw label for the min value.
		if (MinValueY >= Y0 && MinValueY <= Y0 + RoundedViewHeight && FMath::Abs(MaxValueY - MinValueY) > TextH)
		{
			Params.Y = MinValueY;
			Params.Value = AllSeriesMinValue;
			DrawLabel(Params);
			++MinMaxAxis;
		}

		// If mouse hovers close to labels area, show also the Max-Min delta value.
		if (MinMaxAxis == 2)
		{
			const float MX = static_cast<float>(Context.GetMousePosition().X);
			const float MY = static_cast<float>(Context.GetMousePosition().Y);

			constexpr float MX2 = 120.0f; // width of the hover area

			if (MX > ViewWidth - MX2 && MY >= MaxValueY && MY < MinValueY + TextH)
			{
				const float LineX = MX - 16.0f;
				DrawContext.DrawBox(DrawContext.LayerId + 1, LineX, ActualMaxValueY, X0 - LineX, 1.0f, Params.Brush, Params.TextBgColor);
				DrawContext.DrawBox(DrawContext.LayerId + 1, LineX, ActualMaxValueY, 1.0f, ActualMinValueY - ActualMaxValueY, Params.Brush, Params.TextBgColor);
				DrawContext.DrawBox(DrawContext.LayerId + 1, LineX, ActualMinValueY, X0 - LineX, 1.0f, Params.Brush, Params.TextBgColor);

				DrawContext.LayerId += 3; // ensure to draw on top of other labels

				Params.X = MX;
				Params.Y = MY - TextH / 2;
				Params.Value = AllSeriesMaxValue - AllSeriesMinValue;
				Params.Precision = -Precision; // format with detailed text
				Params.Prefix = TEXT("\u0394=");
				DrawLabel(Params);
			}
		}
	}

	DrawContext.LayerId += 3;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryGraphTrack::DrawLabel(const TDrawLabelParams& Params) const
{
	FString LabelText = Params.Prefix;

	if (std::isnan(Params.Precision))
	{
		// Draw only the Prefix text.
	}
	else if (FMath::IsNearlyZero(Params.Value, 0.5))
	{
		LabelText += TEXT("0");
	}
	else
	{
		double UnitValue;
		const TCHAR* UnitText;
		GetUnit(LabelUnit, FMath::Abs(Params.Precision), UnitValue, UnitText);

		LabelText += FormatValue(Params.Value, UnitValue, UnitText, LabelDecimalDigitCount);

		if (Params.Precision < 0 && LabelUnit == EGraphTrackLabelUnit::Auto)
		{
			double ValueUnitValue;
			const TCHAR* ValueUnitText;
			GetUnit(LabelUnit, Params.Value, ValueUnitValue, ValueUnitText);
			if (ValueUnitValue > UnitValue)
			{
				FString LabelTextDetail = FormatValue(Params.Value, ValueUnitValue, ValueUnitText, LabelDecimalDigitCount);
				LabelText += TEXT(" (");
				LabelText += LabelTextDetail;
				LabelText += TEXT(")");
			}
		}
	}

	const float FontScale = Params.DrawContext.Geometry.Scale;
	const FVector2D TextSize = Params.FontMeasureService->Measure(LabelText, Font, FontScale) / FontScale;
	const float TextW = static_cast<float>(TextSize.X);
	constexpr float TextH = 14.0f;

	// Draw background for value text.
	Params.DrawContext.DrawBox(Params.DrawContext.LayerId + 1, Params.X - TextW - 4.0f, Params.Y, TextW + 5.0f, TextH, Params.Brush, Params.TextBgColor);

	// Draw value text.
	Params.DrawContext.DrawText(Params.DrawContext.LayerId + 2, Params.X - TextW - 2.0f, Params.Y + 1.0f, LabelText, Font, Params.TextColor);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryGraphTrack::GetUnit(const EGraphTrackLabelUnit InLabelUnit, const double InPrecision, double& OutUnitValue, const TCHAR*& OutUnitText)
{
	constexpr double KiB = (double)(1LL << 10); // 2^10 bytes
	constexpr double MiB = (double)(1LL << 20); // 2^20 bytes
	constexpr double GiB = (double)(1LL << 30); // 2^30 bytes
	constexpr double TiB = (double)(1LL << 40); // 2^40 bytes
	constexpr double PiB = (double)(1LL << 50); // 2^50 bytes
	constexpr double EiB = (double)(1LL << 60); // 2^60 bytes

	constexpr double K10 = 1000.0;    // 10^3
	constexpr double M10 = K10 * K10; // 10^6
	constexpr double G10 = M10 * K10; // 10^9
	constexpr double T10 = G10 * K10; // 10^12
	constexpr double P10 = T10 * K10; // 10^15
	constexpr double E10 = P10 * K10; // 10^18

	switch (InLabelUnit)
	{
		case EGraphTrackLabelUnit::Auto:
		{
			if (InPrecision >= EiB)
			{
				OutUnitValue = EiB;
				OutUnitText = TEXT("EiB");
			}
			else if (InPrecision >= PiB)
			{
				OutUnitValue = PiB;
				OutUnitText = TEXT("PiB");
			}
			else if (InPrecision >= TiB)
			{
				OutUnitValue = TiB;
				OutUnitText = TEXT("TiB");
			}
			else if (InPrecision >= GiB)
			{
				OutUnitValue = GiB;
				OutUnitText = TEXT("GiB");
			}
			else if (InPrecision >= MiB)
			{
				OutUnitValue = MiB;
				OutUnitText = TEXT("MiB");
			}
			else if (InPrecision >= KiB)
			{
				OutUnitValue = KiB;
				OutUnitText = TEXT("KiB");
			}
			else
			{
				OutUnitValue = 1.0;
				OutUnitText = TEXT("B");
			}
		}
		break;

		case EGraphTrackLabelUnit::KiB:
			OutUnitValue = KiB;
			OutUnitText = TEXT("KiB");
			break;

		case EGraphTrackLabelUnit::MiB:
			OutUnitValue = MiB;
			OutUnitText = TEXT("MiB");
			break;

		case EGraphTrackLabelUnit::GiB:
			OutUnitValue = GiB;
			OutUnitText = TEXT("GiB");
			break;

		case EGraphTrackLabelUnit::TiB:
			OutUnitValue = TiB;
			OutUnitText = TEXT("TiB");
			break;

		case EGraphTrackLabelUnit::PiB:
			OutUnitValue = PiB;
			OutUnitText = TEXT("PiB");
			break;

		case EGraphTrackLabelUnit::EiB:
			OutUnitValue = EiB;
			OutUnitText = TEXT("EiB");
			break;

		case EGraphTrackLabelUnit::Byte:
			OutUnitValue = 1.0;
			OutUnitText = TEXT("B");
			break;

		case EGraphTrackLabelUnit::AutoCount:
		{
			if (InPrecision >= E10)
			{
				OutUnitValue = E10;
				OutUnitText = TEXT("E");
			}
			else if (InPrecision >= P10)
			{
				OutUnitValue = P10;
				OutUnitText = TEXT("P");
			}
			else if (InPrecision >= T10)
			{
				OutUnitValue = T10;
				OutUnitText = TEXT("T");
			}
			else if (InPrecision >= G10)
			{
				OutUnitValue = G10;
				OutUnitText = TEXT("G");
			}
			else if (InPrecision >= M10)
			{
				OutUnitValue = M10;
				OutUnitText = TEXT("M");
			}
			else if (InPrecision >= K10)
			{
				OutUnitValue = K10;
				OutUnitText = TEXT("K");
			}
			else
			{
				OutUnitValue = 1.0;
				OutUnitText = TEXT("");
			}
		}
		break;

		case EGraphTrackLabelUnit::Count:
		default:
			OutUnitValue = 1.0;
			OutUnitText = TEXT("");
			break;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FMemoryGraphTrack::FormatValue(const double InValue, const double InUnitValue, const TCHAR* InUnitText, const int32 InDecimalDigitCount)
{
	if (InUnitText[0] == TEXT('\0') && InDecimalDigitCount == 0)
	{
		return FText::AsNumber(static_cast<int64>(InValue)).ToString();
	}

	FString OutText = FString::Printf(TEXT("%.*f"), FMath::Abs(InDecimalDigitCount), InValue / InUnitValue);

	if (InDecimalDigitCount < 0)
	{
		// Remove ending 0s.
		while (OutText.Len() > 0 && OutText[OutText.Len() - 1] == TEXT('0'))
		{
			OutText.RemoveAt(OutText.Len() - 1, 1);
		}
		// Remove ending dot.
		if (OutText.Len() > 0 && OutText[OutText.Len() - 1] == TEXT('.'))
		{
			OutText.RemoveAt(OutText.Len() - 1, 1);
		}
	}

	if (InUnitText[0] != TEXT('\0'))
	{
		OutText += TEXT(' ');
		OutText += InUnitText;
	}

	return OutText;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryGraphTrack::SetAvailableTrackHeight(EMemoryTrackHeightMode InMode, float InTrackHeight)
{
	check(static_cast<uint32>(InMode) < static_cast<uint32>(EMemoryTrackHeightMode::Count));
	AvailableTrackHeights[static_cast<uint32>(InMode)] = InTrackHeight;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryGraphTrack::SetCurrentTrackHeight(EMemoryTrackHeightMode InMode)
{
	check(static_cast<uint32>(InMode) < static_cast<uint32>(EMemoryTrackHeightMode::Count));
	SetHeight(AvailableTrackHeights[static_cast<uint32>(InMode)]);

	const float TrackHeight = GetHeight();
	for (TSharedPtr<FGraphSeries> Series : AllSeries)
	{
		Series->SetBaselineY(TrackHeight - 1.0f);
		Series->SetDirtyFlag();
	}

	SetDirtyFlag();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryGraphTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	if (InTooltipEvent.CheckTrack(this) && InTooltipEvent.Is<FGraphTrackEvent>())
	{
		const FGraphTrackEvent& TooltipEvent = InTooltipEvent.As<FGraphTrackEvent>();
		const TSharedRef<const FGraphSeries> GraphSeries = TooltipEvent.GetSeries();
		if (GraphSeries->Is<FMemoryGraphSeries>())
		{
			const FMemoryGraphSeries& Series = GraphSeries->As<FMemoryGraphSeries>();

			InOutTooltip.ResetContent();

			InOutTooltip.AddTitle(Series.GetName().ToString(), Series.GetColor());

			if (Series.Is<FMemTagGraphSeries>())
			{
				const FMemTagGraphSeries& MemTagSeries = Series.As<FMemTagGraphSeries>();
				FString SubTitle = FString::Printf(TEXT("(tag id 0x%llX, tag set id %i, tracker id %i)"), (uint64)MemTagSeries.GetTagId(), (int32)MemTagSeries.GetTagSetId(), (int32)MemTagSeries.GetTrackerId());
				InOutTooltip.AddTitle(SubTitle, Series.GetColor());
			}

			const double Precision = FMath::Max(1.0 / TimeScaleX, FTimeValue::Nanosecond);
			InOutTooltip.AddNameValueTextLine(TEXT("Time:"), FormatTime(TooltipEvent.GetStartTime(), Precision));
			if (Series.HasEventDuration())
			{
				InOutTooltip.AddNameValueTextLine(TEXT("Duration:"), FormatTimeAuto(TooltipEvent.GetDuration()));
			}
			InOutTooltip.AddNameValueTextLine(TEXT("Value:"), Series.FormatValue(TooltipEvent.GetValue()));

			InOutTooltip.UpdateLayout();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
