// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Math/Color.h"

// TraceInsights
#include "Insights/MemoryProfiler/Tracks/AllocationsGraphSeries.h"
#include "Insights/MemoryProfiler/Tracks/MemTagGraphSeries.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryTag.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryTracker.h"
#include "Insights/ViewModels/GraphTrack.h"

#include <limits>

class FSlateFontMeasure;
struct FSlateBrush;

namespace UE::Insights::MemoryProfiler
{

class FMemorySharedState;

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class EGraphTrackLabelUnit
{
	Auto,
	Byte,
	KiB, // 2^10 bytes (kibibyte)
	MiB, // 2^20 bytes (mebibyte)
	GiB, // 2^30 bytes (gibibyte)
	TiB, // 2^40 bytes (tebibyte)
	PiB, // 2^50 bytes (pebibyte)
	EiB, // 2^60 bytes (exbibyte)
	AutoCount,
	Count,
};

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class EMemoryTrackHeightMode
{
	Small = 0,
	Medium,
	Large,

	Count
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemoryGraphTrack : public FGraphTrack
{
	INSIGHTS_DECLARE_RTTI(FMemoryGraphTrack, FGraphTrack)

public:
	FMemoryGraphTrack(FMemorySharedState& InSharedState);
	virtual ~FMemoryGraphTrack();

	void SetLabelUnit(EGraphTrackLabelUnit InLabelUnit, int32 InLabelDecimalDigitCount) { LabelUnit = InLabelUnit; LabelDecimalDigitCount = InLabelDecimalDigitCount; }

	bool IsAutoZoomEnabled() const { return bAutoZoom; }
	void EnableAutoZoom() { bAutoZoom = true; }
	void DisableAutoZoom() { bAutoZoom = false; }
	void SetAutoZoom(bool bOnOff) { bAutoZoom = bOnOff; }

	void SetDefaultValueRange(double InDefaultMinValue, double InDefaultMaxValue) { DefaultMinValue = InDefaultMinValue; DefaultMaxValue = InDefaultMaxValue; }
	void ResetDefaultValueRange();

	bool IsStacked() const { return bIsStacked; }
	void SetStacked(bool bOnOff) { bIsStacked = bOnOff; }

	TSharedPtr<FMemoryGraphSeries> GetMainSeries() const { return MainSeries; }
	void SetMainSeries(TSharedPtr<FMemoryGraphSeries> InMainSeries) { MainSeries = InMainSeries; }

	virtual void Update(const ITimingTrackUpdateContext& Context) override;
	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;

	TSharedPtr<FMemTagGraphSeries> GetMemTagSeries(FMemoryTrackerId InMemTrackerId, FMemoryTagSetId InMemTagSetId, FMemoryTagId InMemTagId);
	TSharedPtr<FMemTagGraphSeries> AddMemTagSeries(FMemoryTrackerId InMemTrackerId, FMemoryTagSetId InMemTagSetId, FMemoryTagId InMemTagId);
	int32 RemoveMemTagSeries(FMemoryTrackerId InMemTrackerId, FMemoryTagSetId InMemTagSetId, FMemoryTagId InMemTagId);
	int32 RemoveAllMemTagSeries();

	TSharedPtr<FAllocationsGraphSeries> GetTimelineSeries(FAllocationsGraphSeries::ETimeline InTimeline);
	TSharedPtr<FAllocationsGraphSeries> AddTimelineSeries(FAllocationsGraphSeries::ETimeline InTimeline);

	void SetAvailableTrackHeight(EMemoryTrackHeightMode InMode, float InTrackHeight);
	void SetCurrentTrackHeight(EMemoryTrackHeightMode InMode);

	static void GetUnit(const EGraphTrackLabelUnit InLabelUnit, const double InPrecision, double& OutUnitValue, const TCHAR*& OutUnitText);
	static FString FormatValue(const double InValue, const double InUnitValue, const TCHAR* InUnitText, const int32 InDecimalDigitCount);

protected:
	virtual void DrawVerticalAxisGrid(const ITimingTrackDrawContext& Context) const override;

	struct TDrawLabelParams
	{
		TDrawLabelParams(FDrawContext& InDrawContext, const FSlateBrush* InBrush, const TSharedRef<FSlateFontMeasure>& InFontMeasureService)
		: DrawContext(InDrawContext), Brush(InBrush), FontMeasureService(InFontMeasureService)
		{
		}

		FDrawContext& DrawContext;
		const FSlateBrush* Brush;
		const TSharedRef<FSlateFontMeasure>& FontMeasureService;
		FLinearColor TextBgColor;
		FLinearColor TextColor;
		float X = 0.0f;
		float Y = 0.0f;
		double Value = 0.0;
		double Precision = 0.0; // if Precision < 0, formats the value with detailed text
		FString Prefix;
	};
	void DrawLabel(const TDrawLabelParams& Params) const;

protected:
	FMemorySharedState& SharedState;

	EGraphTrackLabelUnit LabelUnit = EGraphTrackLabelUnit::Auto;

	/**
	 * Number of decimal digits for labels.
	 * Specifies the number of decimal digits to use when formatting labels of the vertical axis grid.
	 * If negative, the formatting will use maximum the number of decimal digits specified (trims trailing 0s),
	 * otherwise, it will use exactly the number of decimal digits specified.
	 */
	int32 LabelDecimalDigitCount = 2;

	double DefaultMinValue = +std::numeric_limits<double>::infinity();
	double DefaultMaxValue = -std::numeric_limits<double>::infinity();
	double AllSeriesMinValue = 0.0;
	double AllSeriesMaxValue = 0.0;

	bool bAutoZoom = false; // all series will share same scale
	bool bIsStacked = false;

	float AvailableTrackHeights[static_cast<uint32>(EMemoryTrackHeightMode::Count)];

	TSharedPtr<FMemoryGraphSeries> MainSeries;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
