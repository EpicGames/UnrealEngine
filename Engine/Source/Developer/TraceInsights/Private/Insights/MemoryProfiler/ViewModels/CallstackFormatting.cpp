// Copyright Epic Games, Inc. All Rights Reserved.

#include "CallstackFormatting.h"

// TraceServices
#include "TraceServices/Model/Callstack.h"
#include "TraceServices/Model/Modules.h"

// TraceInsights
#include "Insights/InsightsManager.h"

#define LOCTEXT_NAMESPACE "UE::Insights::CallstackFormatting"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FText GetCallstackNotAvailableString()
{
	return LOCTEXT("UnknownCallstack", "Unknown Callstack");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText GetNoCallstackString()
{
	return LOCTEXT("NoCallstackRecorded", "No Callstack Recorded");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText GetEmptyCallstackString()
{
	return LOCTEXT("EmptyCallstack", "Empty Callstack");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FormatStackFrame(const TraceServices::FStackFrame& Frame, FStringBuilderBase& OutString, EStackFrameFormatFlags FormatFlags)
{
	using namespace TraceServices;
	const ESymbolQueryResult Result = Frame.Symbol->GetResult();
	switch (Result)
	{
		case ESymbolQueryResult::OK:
			if (EnumHasAnyFlags(FormatFlags, EStackFrameFormatFlags::Module))
			{
				OutString.Append(Frame.Symbol->Module);
				if (FormatFlags != EStackFrameFormatFlags::Module &&
					FormatFlags != (EStackFrameFormatFlags::Module | EStackFrameFormatFlags::Multiline))
				{
					OutString.AppendChar(EnumHasAnyFlags(FormatFlags, EStackFrameFormatFlags::Multiline) ? TEXT('\n') : TEXT('!'));
				}
			}
			if (EnumHasAnyFlags(FormatFlags, EStackFrameFormatFlags::Symbol))
			{
				OutString.Append(Frame.Symbol->Name);
			}
			if (EnumHasAnyFlags(FormatFlags, EStackFrameFormatFlags::Symbol) &&
				EnumHasAnyFlags(FormatFlags, EStackFrameFormatFlags::File | EStackFrameFormatFlags::Line))
			{
				OutString.AppendChar(EnumHasAnyFlags(FormatFlags, EStackFrameFormatFlags::Multiline) ? TEXT('\n') : TEXT(' '));
			}
			if (EnumHasAnyFlags(FormatFlags, EStackFrameFormatFlags::File))
			{
				FString UsablePath;
				FInsightsManager::Get()->GetSourceFilePathHelper().GetUsableFilePath(Frame.Symbol->File, UsablePath);
				OutString.Append(UsablePath);
			}
			if (EnumHasAnyFlags(FormatFlags, EStackFrameFormatFlags::Line))
			{
				OutString.Appendf(TEXT("(%d)"), Frame.Symbol->Line);
			}
			break;

		case ESymbolQueryResult::Mismatch:
		case ESymbolQueryResult::NotFound:
		case ESymbolQueryResult::NotLoaded:
			if (EnumHasAnyFlags(FormatFlags, EStackFrameFormatFlags::Module))
			{
				OutString.Append(Frame.Symbol->Module);
				if (FormatFlags != EStackFrameFormatFlags::Module &&
					FormatFlags != (EStackFrameFormatFlags::Module | EStackFrameFormatFlags::Multiline))
				{
					OutString.AppendChar(EnumHasAnyFlags(FormatFlags, EStackFrameFormatFlags::Multiline) ? TEXT('\n') : TEXT('!'));
				}
			}
			if (EnumHasAnyFlags(FormatFlags, EStackFrameFormatFlags::Symbol))
			{
				if (Frame.Addr == 0)
				{
					OutString.Append(TEXT("0x00000000"));
				}
				else
				{
					OutString.Appendf(TEXT("0x%llX"), Frame.Addr);
				}
			}
			if (EnumHasAnyFlags(FormatFlags, EStackFrameFormatFlags::ModuleAndSymbol))
			{
				OutString.AppendChar(EnumHasAnyFlags(FormatFlags, EStackFrameFormatFlags::Multiline) ? TEXT('\n') : TEXT(' '));
			}
			OutString.Appendf(TEXT("(%s)"), QueryResultToString((Result)));
			break;

		case ESymbolQueryResult::Pending:
		default:
			OutString.Append(QueryResultToString(Result));
			break;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
