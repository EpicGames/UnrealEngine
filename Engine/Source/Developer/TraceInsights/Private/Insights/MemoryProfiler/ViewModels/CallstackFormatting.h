// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Internationalization/Text.h"
#include "Misc/EnumClassFlags.h"

namespace TraceServices { struct FStackFrame; }

namespace UE::Insights
{

enum class EStackFrameFormatFlags : uint8
{
	Module    = 1 << 0, // include module name
	Symbol    = 1 << 1, // include symbol name
	File      = 1 << 2, // include source file name
	Line      = 1 << 3, // include source line number
	Multiline = 1 << 4, // allow formatting on multiple lines

	ModuleAndSymbol         = Module + Symbol,
	ModuleSymbolFileAndLine = Module + Symbol + File + Line,
	FileAndLine             = File + Line,
};
ENUM_CLASS_FLAGS(EStackFrameFormatFlags);

FText GetCallstackNotAvailableString();
FText GetNoCallstackString();
FText GetEmptyCallstackString();

void FormatStackFrame(const TraceServices::FStackFrame& Frame, FStringBuilderBase& OutString, EStackFrameFormatFlags FormatFlags);

} // namespace UE::Insights
