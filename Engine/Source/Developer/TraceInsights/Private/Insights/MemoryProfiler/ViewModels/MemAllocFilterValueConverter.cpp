// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemAllocFilterValueConverter.h"

#include "Internationalization/Internationalization.h"
#include "Misc/Char.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemAllocNode"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemoryFilterValueConverter
////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemoryFilterValueConverter::Convert(const FString& Input, int64& Output, FText& OutError) const
{
	FString TrimmedInput = Input.TrimStartAndEnd();

	if (TrimmedInput.IsNumeric())
	{
		Output = FCString::Atoi64(*TrimmedInput);
	}

	static const TCHAR* Prefixes = TEXT("KMGTPE");
	static const TCHAR* Suffix = TEXT("iB");
	for (int Index = 0; Index < TrimmedInput.Len(); ++Index)
	{
		TCHAR Character = TrimmedInput[Index];
		if (TChar<TCHAR>::IsDigit(Character) || TChar<TCHAR>::IsWhitespace(Character) || Character == '.')
		{
			continue;
		}
		else if (TChar<TCHAR>::IsAlpha(Character))
		{
			Character = TChar<TCHAR>::ToUpper(Character);
			const TCHAR* PrefixPos = FCString::Strchr(Prefixes, Character);
			if (PrefixPos == nullptr)
			{
				OutError = LOCTEXT("InvalidUnit", "Invalid unit. Expected one of KiB, MiB, GiB, TiB, PiB, EiB.");
				return false;
			}

			int EndIndex = Index + 1;
			while (EndIndex < TrimmedInput.Len() && TChar<TCHAR>::IsWhitespace(TrimmedInput[EndIndex]))
			{
				++EndIndex;
			}

			if (TrimmedInput.Mid(EndIndex).Compare(Suffix, ESearchCase::Type::IgnoreCase) != 0)
			{
				OutError = LOCTEXT("InvalidUnit", "Invalid unit. Expected one of KiB, MiB, GiB, TiB, PiB, EiB.");
				return false;
			}

			const int64 Unit = 1LL << ((PrefixPos - Prefixes + 1) * 10);
			double Value = FCString::Atod(*TrimmedInput.Left(Index));
			Output = static_cast<int64>(Value * static_cast<double>(Unit));
			break;
		}
		else
		{
			OutError = LOCTEXT("InvalidCharacters", "Invalid character.");
			return false;
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMemoryFilterValueConverter::GetTooltipText() const
{
	return LOCTEXT("MemoryTooltipText", "Enter a memory value, such as 100 or 4 KiB or 1 MiB");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMemoryFilterValueConverter::GetHintText() const
{
	return LOCTEXT("MemoryHintText", "Ex. 4 KiB");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
