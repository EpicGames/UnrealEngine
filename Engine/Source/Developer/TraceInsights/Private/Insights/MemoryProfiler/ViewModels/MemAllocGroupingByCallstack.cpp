// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemAllocGroupingByCallstack.h"

// TraceServices
#include "TraceServices/Model/Callstack.h"

// TraceInsightsCore
#include "InsightsCore/Common/AsyncOperationProgress.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/MemoryProfiler/ViewModels/CallstackFormatting.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemAllocNode"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FCallstackFrameGroupNode
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FCallstackFrameGroupNode)

////////////////////////////////////////////////////////////////////////////////////////////////////

FCallstackFrameGroupNode::FCallstackFrameGroupNode(TWeakPtr<FTable> InParentTable, const TraceServices::FStackFrame* InStackFrame)
	: FTableTreeNode(InParentTable)
	, DisplayName(FMemAllocGroupingByCallstack::GetDisplayName(InStackFrame))
	, StackFrame(InStackFrame)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FCallstackFrameGroupNode::GetTooltipText() const
{
	if (StackFrame)
	{
		TStringBuilder<1024> String;
		FormatStackFrame(*StackFrame, String, EStackFrameFormatFlags::ModuleSymbolFileAndLine | EStackFrameFormatFlags::Multiline);
		return FText::FromString(FString(String));
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* FCallstackFrameGroupNode::GetIcon() const
{
	return FInsightsStyle::GetBrush("Icons.Group.TreeItem");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FCallstackFrameGroupNode::GetIconColor() const
{
	return FLinearColor(0.5f, 0.75f, 1.0f, 1.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FCallstackFrameGroupNode::GetColor() const
{
	return FLinearColor(0.5f, 0.75f, 1.0f, 1.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemAllocGroupingByCallstack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FMemAllocGroupingByCallstack)

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemAllocGroupingByCallstack::FMemAllocGroupingByCallstack(const FText& InShortName, const FText& InTitleName, const FText& InDescription, const FSlateBrush* InIcon, bool bInIsAllocCallstack, bool bInIsInverted, bool bInIsGroupingByFunction)
	: FTreeNodeGrouping(InShortName, InTitleName, InDescription, InIcon)
	, bIsAllocCallstack(bInIsAllocCallstack)
	, bIsInverted(bInIsInverted)
	, bIsGroupingByFunction(bInIsGroupingByFunction)
	, bShouldSkipFilteredFrames(bIsInverted)
{
	SetColor(FLinearColor(0.5f, 0.75f, 1.0f, 1.0f));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemAllocGroupingByCallstack::FMemAllocGroupingByCallstack(bool bInIsAllocCallstack, bool bInIsInverted, bool bInIsGroupingByFunction)
	: FMemAllocGroupingByCallstack(
		bInIsAllocCallstack
			? (bInIsInverted ? LOCTEXT("Grouping_ByCallstack2_ShortName", "Inverted Alloc Callstack")
							 : LOCTEXT("Grouping_ByCallstack1_ShortName", "Alloc Callstack"))
			: (bInIsInverted ? LOCTEXT("Grouping_ByCallstack4_ShortName", "Inverted Free Callstack")
							 : LOCTEXT("Grouping_ByCallstack3_ShortName", "Free Callstack")),
		bInIsAllocCallstack
			? (bInIsInverted ? LOCTEXT("Grouping_ByCallstack2_TitleName", "By Inverted Alloc Callstack")
							 : LOCTEXT("Grouping_ByCallstack1_TitleName", "By Alloc Callstack"))
			: (bInIsInverted ? LOCTEXT("Grouping_ByCallstack4_TitleName", "By Inverted Free Callstack")
							 : LOCTEXT("Grouping_ByCallstack3_TitleName", "By Free Callstack")),
		LOCTEXT("Grouping_Callstack_Desc", "Creates a tree based on callstack of each allocation."),
		nullptr,
		bInIsAllocCallstack,
		bInIsInverted,
		bInIsGroupingByFunction)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemAllocGroupingByCallstack::~FMemAllocGroupingByCallstack()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemAllocGroupingByCallstack::GroupNodes(
	const TArray<FTableTreeNodePtr>& Nodes,
	FTableTreeNode& ParentGroup,
	TWeakPtr<FTable> InParentTable,
	IAsyncOperationProgress& InAsyncOperationProgress) const
{
	GroupNodesInternal(Nodes, ParentGroup, InParentTable, InAsyncOperationProgress, [](FTableTreeNodePtr& InTableTreeNodePtr) { return InTableTreeNodePtr; });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemAllocGroupingByCallstack::GroupNodesInternal(
	const TArray<FTableTreeNodePtr>& Nodes,
	FTableTreeNode& ParentGroup,
	TWeakPtr<FTable> InParentTable,
	IAsyncOperationProgress& InAsyncOperationProgress,
	TUniqueFunction<FTableTreeNodePtr(FTableTreeNodePtr&)> ProcessNode) const
{
	const bool bLocalIsGroupingByFunction = bIsGroupingByFunction;
	const bool bLocalShouldSkipFilteredFrames = bShouldSkipFilteredFrames;

	ParentGroup.ClearChildren();

	TArray<FCallstackGroup*> CallstackGroups;
	ON_SCOPE_EXIT
	{
		for (FCallstackGroup* Group : CallstackGroups)
		{
			delete Group;
		}
	};

	FCallstackGroup* Root = new FCallstackGroup();
	Root->Node = &ParentGroup;
	CallstackGroups.Add(Root);

	FTableTreeNode* UnsetGroupPtr = nullptr;
	FTableTreeNode* NoCallstackGroupPtr = nullptr;
	FTableTreeNode* EmptyCallstackGroupPtr = nullptr;
	TMap<const TraceServices::FCallstack*, FCallstackGroup*> GroupMapByCallstack; // Callstack* -> FCallstackGroup*

	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			return;
		}

		if (NodePtr->IsGroup())
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		if (!NodePtr->Is<FMemAllocNode>())
		{
			continue;
		}

		FCallstackGroup* GroupPtr = Root;
		bool bIsEmptyCallstack = false;
		bool bIsDefaultEmptyCallstack = false;

		const FMemAllocNode& MemAllocNode = NodePtr->As<FMemAllocNode>();
		const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
		if (Alloc)
		{
			const TraceServices::FCallstack* Callstack = bIsAllocCallstack ? Alloc->GetAllocCallstack() : Alloc->GetFreeCallstack();

			NodePtr = ProcessNode(NodePtr);
			if (!NodePtr.IsValid())
			{
				continue;
			}

			FCallstackGroup** FoundGroupPtrPtr = GroupMapByCallstack.Find(Callstack);
			if (FoundGroupPtrPtr)
			{
				GroupPtr = *FoundGroupPtrPtr;
			}
			else if (Callstack)
			{
				if (Callstack->Num() == 0)
				{
					bIsEmptyCallstack = true;
					bIsDefaultEmptyCallstack = (Callstack->GetEmptyId() == 0);
				}
				else
				{
					int32 NumFrames = static_cast<int32>(Callstack->Num());
					check(NumFrames > 0);
					check(NumFrames <= 256); // see Callstack->Frame(uint8)
					for (int32 FrameDepth = NumFrames - 1; FrameDepth >= 0; --FrameDepth)
					{
						const TraceServices::FStackFrame* Frame = Callstack->Frame(static_cast<uint8>(bIsInverted ? NumFrames - FrameDepth - 1 : FrameDepth));
						check(Frame != nullptr);

						if (bLocalShouldSkipFilteredFrames &&
							Frame->Symbol->FilterStatus.load() == TraceServices::EResolvedSymbolFilterStatus::Filtered)
						{
							continue;
						}

						uint64 GroupNameId = 0;
						if (bLocalIsGroupingByFunction)
						{
							// Merge groups by function name.
							GroupNameId = GetGroupNameId(Frame);

							// Merge with the parent group, if it has the same name id (i.e. same function name).
							if (GroupPtr->Parent != nullptr && GroupPtr->NameId == GroupNameId)
							{
								GroupPtr = GroupPtr->Parent;
							}
						}
						else
						{
							// Merge groups by unique callstack frame.
							GroupNameId = Frame->Addr;
						}

						FCallstackGroup** FoundGroup = GroupPtr->GroupMap.Find(GroupNameId);
						if (!FoundGroup)
						{
							GroupPtr = CreateGroup(CallstackGroups, GroupPtr, GroupNameId, InParentTable, Frame);
							check(GroupPtr->Parent != nullptr);
							GroupPtr->Parent->GroupMap.Add(GroupNameId, GroupPtr);
						}
						else
						{
							GroupPtr = *FoundGroup;
						}
					}

					GroupMapByCallstack.Add(Callstack, GroupPtr);
				}
			}
		}

		if (GroupPtr != Root)
		{
			GroupPtr->Node->AddChildAndSetParent(NodePtr);
		}
		else if (bIsEmptyCallstack)
		{
			if (bIsDefaultEmptyCallstack)
			{
				if (!NoCallstackGroupPtr)
				{
					NoCallstackGroupPtr = CreateNoCallstackGroup(InParentTable, ParentGroup);
				}
				NoCallstackGroupPtr->AddChildAndSetParent(NodePtr);
			}
			else
			{
				if (!EmptyCallstackGroupPtr)
				{
					EmptyCallstackGroupPtr = CreateEmptyCallstackGroup(InParentTable, ParentGroup);
				}
				EmptyCallstackGroupPtr->AddChildAndSetParent(NodePtr);
			}
		}
		else
		{
			if (!UnsetGroupPtr)
			{
				UnsetGroupPtr = CreateUnsetGroup(InParentTable, ParentGroup);
			}
			UnsetGroupPtr->AddChildAndSetParent(NodePtr);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint64 FMemAllocGroupingByCallstack::GetGroupNameId(const TraceServices::FStackFrame* Frame) const
{
	const TraceServices::ESymbolQueryResult Result = Frame->Symbol->GetResult();
	if (Result == TraceServices::ESymbolQueryResult::OK)
	{
		return uint64(Frame->Symbol->Name);
	}
	else
	{
		return Frame->Addr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMemAllocGroupingByCallstack::GetDisplayName(const TraceServices::FStackFrame* Frame)
{
	const TraceServices::ESymbolQueryResult Result = Frame->Symbol->GetResult();
	if (Result == TraceServices::ESymbolQueryResult::OK)
	{
		return FText::FromString(Frame->Symbol->Name);
	}
	else
	{
		TStringBuilder<1024> Str;
		Str.Append(Frame->Symbol->Module);
		Str.Appendf(TEXT("!0x%llX"), Frame->Addr);
		if (Result == TraceServices::ESymbolQueryResult::Pending)
		{
			Str.Append(TEXT(" [...]"));
		}
		return FText::FromStringView(Str.ToView());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemAllocGroupingByCallstack::FCallstackGroup* FMemAllocGroupingByCallstack::CreateGroup(
	TArray<FCallstackGroup*>& InOutAllCallstackGroup,
	FCallstackGroup* InParentGroup,
	uint64 InGroupNameId,
	TWeakPtr<FTable> InParentTable,
	const TraceServices::FStackFrame* InFrame) const
{
	FCallstackGroup* NewGroupPtr = new FCallstackGroup();
	NewGroupPtr->Parent = InParentGroup;
	NewGroupPtr->NameId = InGroupNameId;

	InOutAllCallstackGroup.Add(NewGroupPtr);

	auto Node = MakeShared<FCallstackFrameGroupNode>(InParentTable, InFrame);
	Node->SetExpansion(false);

	InParentGroup->Node->AddChildAndSetParent(Node);
	NewGroupPtr->Node = &Node.Get();

	return NewGroupPtr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTableTreeNode* FMemAllocGroupingByCallstack::CreateUnsetGroup(TWeakPtr<FTable> ParentTable, FTableTreeNode& Parent) const
{
	const FText DisplayName = GetCallstackNotAvailableString();
	auto GroupNode = MakeShared<FCustomTableTreeNode>(ParentTable, DisplayName);
	GroupNode->SetExpansion(false);
	Parent.AddChildAndSetParent(GroupNode);
	return &GroupNode.Get();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTableTreeNode* FMemAllocGroupingByCallstack::CreateNoCallstackGroup(TWeakPtr<FTable> ParentTable, FTableTreeNode& Parent) const
{
	const FText DisplayName = GetNoCallstackString();
	auto GroupNode = MakeShared<FCustomTableTreeNode>(ParentTable, DisplayName);
	GroupNode->SetExpansion(false);
	Parent.AddChildAndSetParent(GroupNode);
	return &GroupNode.Get();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTableTreeNode* FMemAllocGroupingByCallstack::CreateEmptyCallstackGroup(TWeakPtr<FTable> ParentTable, FTableTreeNode& Parent) const
{
	const FText DisplayName = GetEmptyCallstackString();
	auto GroupNode = MakeShared<FCustomTableTreeNode>(ParentTable, DisplayName);
	GroupNode->SetExpansion(false);
	Parent.AddChildAndSetParent(GroupNode);
	return &GroupNode.Get();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
