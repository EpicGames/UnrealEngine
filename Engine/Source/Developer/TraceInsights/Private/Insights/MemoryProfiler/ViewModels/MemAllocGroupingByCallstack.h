// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

namespace TraceServices { struct FStackFrame; }
namespace UE::Insights { class IAsyncOperationProgress; }

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FCallstackFrameGroupNode : public FTableTreeNode
{
	INSIGHTS_DECLARE_RTTI(FCallstackFrameGroupNode, FTableTreeNode)

public:
	/** Initialization constructor for the group node. */
	explicit FCallstackFrameGroupNode(TWeakPtr<FTable> InParentTable, const TraceServices::FStackFrame* InStackFrame);

	virtual ~FCallstackFrameGroupNode() = default;

	virtual const FText GetDisplayName() const override
	{
		return DisplayName;
	}

	virtual const FText GetTooltipText() const override;
	virtual const FSlateBrush* GetIcon() const override;
	virtual FLinearColor GetIconColor() const override;
	virtual FLinearColor GetColor() const override;

	/**
	 * @returns the stack frame.
	 */
	const TraceServices::FStackFrame* GetStackFrame() const
	{
		return StackFrame;
	}

private:
	FText DisplayName;
	const TraceServices::FStackFrame* StackFrame = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemAllocGroupingByCallstack : public FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FMemAllocGroupingByCallstack, FTreeNodeGrouping)

private:
	struct FCallstackGroup
	{
		FCallstackGroup* Parent = nullptr;
		uint64 NameId = 0;
		FTableTreeNode* Node = nullptr;
		const TraceServices::FStackFrame* Frame = nullptr;
		TMap<uint64, FCallstackGroup*> GroupMap; // Callstack Frame Address or Group Name Id -> FCallstackGroup*
	};

public:
	FMemAllocGroupingByCallstack(bool bInIsAllocCallstack, bool bInIsInverted, bool bInIsGroupingByFunction);
	virtual ~FMemAllocGroupingByCallstack();

	virtual void GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const override;

	bool IsInverted() const { return bIsInverted; }
	bool IsAllocCallstack() const { return bIsAllocCallstack; }

	bool IsGroupingByFunction() const { return bIsGroupingByFunction; }
	void SetGroupingByFunction(bool bOnOff) { bIsGroupingByFunction = bOnOff; }

	bool ShouldSkipFilteredFrames() const { return bShouldSkipFilteredFrames; }
	void SetSkipFilteredFrames(bool bOnOff) { bShouldSkipFilteredFrames = bOnOff; }

	static FText GetDisplayName(const TraceServices::FStackFrame* Frame);

protected:
	FMemAllocGroupingByCallstack(const FText& InShortName, const FText& InTitleName, const FText& InDescription, const FSlateBrush* InIcon, bool bInIsAllocCallstack, bool bInIsInverted, bool bInIsGroupingByFunction);
	void GroupNodesInternal(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress, TUniqueFunction<FTableTreeNodePtr(FTableTreeNodePtr&)> ProcessNode) const;

private:
	uint64 GetGroupNameId(const TraceServices::FStackFrame* Frame) const;

	FCallstackGroup* CreateGroup(TArray<FCallstackGroup*>& InOutAllCallstackGroup, FCallstackGroup* InParentGroup, uint64 InGroupNameId, TWeakPtr<FTable> InParentTable, const TraceServices::FStackFrame* InFrame) const;

	FTableTreeNode* CreateUnsetGroup(TWeakPtr<FTable> ParentTable, FTableTreeNode& Parent) const;
	FTableTreeNode* CreateNoCallstackGroup(TWeakPtr<FTable> ParentTable, FTableTreeNode& Parent) const;
	FTableTreeNode* CreateEmptyCallstackGroup(TWeakPtr<FTable> ParentTable, FTableTreeNode& Parent) const;

private:
	bool bIsAllocCallstack;
	bool bIsInverted;
	std::atomic<bool> bIsGroupingByFunction;
	std::atomic<bool> bShouldSkipFilteredFrames;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
