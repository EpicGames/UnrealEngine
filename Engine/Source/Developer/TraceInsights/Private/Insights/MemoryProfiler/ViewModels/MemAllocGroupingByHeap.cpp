// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemAllocGroupingByHeap.h"

#include "Internationalization/Internationalization.h"

// TraceServices
#include "Common/ProviderLock.h"

// TraceInsightsCore
#include "InsightsCore/Common/AsyncOperationProgress.h"

// TraceInsights
#include "Insights/MemoryProfiler/ViewModels/CallstackFormatting.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemAllocNode"

namespace UE::Insights::MemoryProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FMemAllocGroupingByHeap)

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemAllocGroupingByHeap::FMemAllocGroupingByHeap(const TraceServices::IAllocationsProvider& InAllocProvider)
	: FTreeNodeGrouping(
		LOCTEXT("Grouping_ByHeap_ShortName", "Heap"),
		LOCTEXT("Grouping_ByHeap_TitleName", "By Heap"),
		LOCTEXT("Grouping_ByHeap_Desc", "Creates a tree based on heap."),
		nullptr)
	, AllocProvider(InAllocProvider)
{
	SetColor(FLinearColor(1.0f, 0.45f, 0.6f, 1.0f));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemAllocGroupingByHeap::~FMemAllocGroupingByHeap()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemHeapTreeNode : public FTableTreeNode
{
	INSIGHTS_DECLARE_RTTI(FMemHeapTreeNode, FTableTreeNode)

public:
	/** Initialization constructor for the group node. */
	explicit FMemHeapTreeNode(const FName InName, TWeakPtr<FTable> InParentTable)
		: FTableTreeNode(InName, InParentTable)
	{
	}

	virtual ~FMemHeapTreeNode()
	{
	}

	virtual FLinearColor GetIconColor() const override final
	{
		return FLinearColor(1.0f, 0.45f, 0.6f, 1.0f);
	}

	virtual FLinearColor GetColor() const override final
	{
		return FLinearColor(1.0f, 0.45f, 0.6f, 1.0f);
	}
};

INSIGHTS_IMPLEMENT_RTTI(FMemHeapTreeNode)

////////////////////////////////////////////////////////////////////////////////////////////////////

FTableTreeNodePtr MakeGroupNodeHierarchy(const TraceServices::IAllocationsProvider::FHeapSpec& Spec, TWeakPtr<FTable>& InParentTable, TArray<FTableTreeNodePtr>& NodeTable)
{
	auto HeapGroup = MakeShared<FMemHeapTreeNode>(FName(Spec.Name), InParentTable);
	if (int32(Spec.Id) >= NodeTable.Num())
	{
		NodeTable.AddDefaulted(int32(Spec.Id) - NodeTable.Num() + 1);
	}

	FTableTreeNodePtr HeapsSubGroup;
	if (uint32(Spec.Flags) & uint32(EMemoryTraceHeapFlags::Root))
	{
		HeapsSubGroup = MakeShared<FTableTreeNode>(FName(TEXT("Heaps")), InParentTable);
		HeapGroup->AddChildAndSetParent(HeapsSubGroup);

		auto AllocsSubGroup = MakeShared<FTableTreeNode>(FName(TEXT("Allocs")), InParentTable);
		HeapGroup->AddChildAndSetParent(AllocsSubGroup);

		NodeTable[int32(Spec.Id)] = AllocsSubGroup;
	}
	else
	{
		HeapsSubGroup = HeapGroup;
		NodeTable[int32(Spec.Id)] = HeapGroup;
	}

	for (TraceServices::IAllocationsProvider::FHeapSpec* ChildSpec : Spec.Children)
	{
		auto ChildNode = MakeGroupNodeHierarchy(*ChildSpec, InParentTable, NodeTable);
		HeapsSubGroup->AddChildAndSetParent(ChildNode);
	}

	return HeapGroup;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemAllocGroupingByHeap::GroupNodes(
	const TArray<FTableTreeNodePtr>& Nodes,
	FTableTreeNode& ParentGroup,
	TWeakPtr<FTable> InParentTable,
	IAsyncOperationProgress& InAsyncOperationProgress) const
{
	ParentGroup.ClearChildren();

	// Build heap hierarchy
	TArray<FTableTreeNodePtr> HeapNodes;

	{
		TraceServices::FProviderReadScopeLock _(AllocProvider);
		AllocProvider.EnumerateRootHeaps([&](HeapId Id, const TraceServices::IAllocationsProvider::FHeapSpec& Spec)
		{
			FTableTreeNodePtr RootHeapGroup = MakeGroupNodeHierarchy(Spec, InParentTable, HeapNodes);
			ParentGroup.AddChildAndSetParent(RootHeapGroup);
		});
	}

	// Add allocations nodes to heaps
	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			return;
		}

		if (NodePtr->IsGroup())
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		const FMemAllocNode& MemAllocNode = static_cast<const FMemAllocNode&>(*NodePtr);
		const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();

		if (Alloc)
		{
			//TODO: Calculating the real HeapId when the allocation is first added to the provider was too expensive, so deferring that operation to here could make sense.
			//const HeapId Heap = AllocProvider.GetParentBlock(Alloc->GetAddress());
			const uint8 HeapId = static_cast<uint8>(Alloc->GetRootHeap());
			FTableTreeNodePtr GroupPtr = HeapNodes[HeapId];
			if (ensure(GroupPtr.IsValid()))
			{
				constexpr uint32 MaxRootHeaps = 16; // see TraceServices\Private\Model\AllocationsProvider.h
				if (Alloc->IsHeap() && HeapId < MaxRootHeaps)
				{
					GroupPtr->GetParent()->GetChildren()[0]->AddChildAndSetParent(NodePtr);
				}
				else
				{
					GroupPtr->AddChildAndSetParent(NodePtr);
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
