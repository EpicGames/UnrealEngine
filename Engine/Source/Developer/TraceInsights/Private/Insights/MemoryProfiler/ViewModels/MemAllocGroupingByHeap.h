// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

namespace TraceServices { class IAllocationsProvider; }

namespace UE::Insights { class IAsyncOperationProgress; }

namespace UE::Insights::MemoryProfiler
{

class FMemAllocGroupingByHeap : public FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FMemAllocGroupingByHeap, FTreeNodeGrouping)

public:
	FMemAllocGroupingByHeap(const TraceServices::IAllocationsProvider& AllocProvider);
	virtual ~FMemAllocGroupingByHeap() override;

	virtual void GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const override;

private:
	const TraceServices::IAllocationsProvider& AllocProvider;
};

} // namespace UE::Insights::MemoryProfiler
