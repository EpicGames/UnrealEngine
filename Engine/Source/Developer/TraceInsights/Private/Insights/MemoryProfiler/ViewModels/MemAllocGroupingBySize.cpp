// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemAllocGroupingBySize.h"

// TraceInsightsCore
#include "InsightsCore/Common/AsyncOperationProgress.h"

// TraceInsights
#include "Insights/MemoryProfiler/ViewModels/MemAllocNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemAllocNode"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemAllocGroupingBySize
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FMemAllocGroupingBySize)

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemAllocGroupingBySize::FMemAllocGroupingBySize()
	: FTreeNodeGrouping(
		LOCTEXT("Grouping_BySize_ShortName", "Size"),
		LOCTEXT("Grouping_BySize_TitleName", "By Size"),
		LOCTEXT("Grouping_BySize_Desc", "Group allocations based on their size."),
		nullptr)
{
	ResetThresholdsPow2();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemAllocGroupingBySize::~FMemAllocGroupingBySize()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemAllocGroupingBySize::ResetThresholdsPow2()
{
	bIsPow2 = true;

	constexpr uint32 NumThresholds = 65;

	FNumberFormattingOptions FormattingOptions;
	FormattingOptions.MinimumIntegralDigits = 2;

	FNumberFormattingOptions FormattingOptionsMem;
	FormattingOptionsMem.MaximumFractionalDigits = 1;

	Thresholds.Reset(65);

	Thresholds.Add({ 0, LOCTEXT("Grouping_BySize_0", "00 : 0 bytes") });
	Thresholds.Add({ 1, LOCTEXT("Grouping_BySize_1", "01 : 1 byte") });
	Thresholds.Add({ 3, LOCTEXT("Grouping_BySize_2", "02 : 2 or 3 bytes") });

	for (uint32 Index = 3; Index < 64; ++Index)
	{
		const uint64 Size1 = 1ULL << (Index - 1);
		const uint64 Size2 = 1ULL << Index;
		const FText Name = FText::Format(LOCTEXT("Grouping_BySize_Pow2_Fmt", "{0} : [{1} .. {2})"),
			FText::AsNumber(Index, &FormattingOptions),
			FText::AsMemory(Size1, &FormattingOptionsMem),
			FText::AsMemory(Size2, &FormattingOptionsMem));
		Thresholds.Add({ Size2 - 1ULL, Name });
	}

	const FText Name64 = FText::Format(LOCTEXT("Grouping_BySize_64_Fmt", "64 : [{0} .. MAX]"), FText::AsMemory(1ULL << 63, &FormattingOptionsMem));
	Thresholds.Add({ ~0ULL, Name64 });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemAllocGroupingBySize::GroupNodes(
	const TArray<FTableTreeNodePtr>& Nodes,
	FTableTreeNode& ParentGroup,
	TWeakPtr<FTable> InParentTable,
	IAsyncOperationProgress& InAsyncOperationProgress) const
{
	TArray<FTableTreeNodePtr> GroupMap; // Node for each ThresholdIndex
	GroupMap.AddDefaulted(Thresholds.Num());

	ParentGroup.ClearChildren();

	FTableTreeNodePtr UnsetGroupPtr = nullptr;
	static FName NotAvailableName(TEXT("N/A"));

	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			return;
		}

		if (NodePtr->IsGroup())
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		FTableTreeNodePtr GroupPtr = nullptr;

		const FMemAllocNode& MemAllocNode = static_cast<const FMemAllocNode&>(*NodePtr);
		const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
		if (Alloc)
		{
			const uint64 Size = FMath::Abs(Alloc->GetSize());

			uint32 ThresholdIndex;
			if (bIsPow2)
			{
				ThresholdIndex = 64u - static_cast<uint32>(FMath::CountLeadingZeros64(Size));
			}
			else
			{
				ThresholdIndex = Algo::UpperBoundBy(Thresholds, Size, &FThreshold::Size) - 1;
			}

			GroupPtr = GroupMap[ThresholdIndex];
			if (!GroupPtr)
			{
				const FThreshold& Threshold = Thresholds[ThresholdIndex];
				GroupPtr = MakeShared<FTableTreeNode>(FName(Threshold.Name.ToString(), 0), InParentTable);
				GroupPtr->SetExpansion(false);
				ParentGroup.AddChildAndSetParent(GroupPtr);
				GroupMap[ThresholdIndex] = GroupPtr;
			}
		}

		if (GroupPtr != nullptr)
		{
			GroupPtr->AddChildAndSetParent(NodePtr);
		}
		else
		{
			if (!UnsetGroupPtr)
			{
				UnsetGroupPtr = MakeShared<FTableTreeNode>(NotAvailableName, InParentTable);
				UnsetGroupPtr->SetExpansion(false);
				ParentGroup.AddChildAndSetParent(UnsetGroupPtr);
			}
			UnsetGroupPtr->AddChildAndSetParent(NodePtr);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
