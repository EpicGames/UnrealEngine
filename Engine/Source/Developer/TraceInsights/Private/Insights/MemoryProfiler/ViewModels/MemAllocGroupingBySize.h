// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

namespace UE::Insights { class IAsyncOperationProgress; }

namespace UE::Insights::MemoryProfiler
{

class FMemAllocGroupingBySize : public FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FMemAllocGroupingBySize, FTreeNodeGrouping)

public:
	struct FThreshold
	{
		uint64 Size; // inclusive upper limit
		FText Name; // group name
	};

public:
	FMemAllocGroupingBySize();
	virtual ~FMemAllocGroupingBySize();

	const TArray<FThreshold>& GetThresholds() const { return Thresholds; }
	TArray<FThreshold>& EditThresholds() { bIsPow2 = false; return Thresholds; }

	bool IsPow2() const { return bIsPow2; }
	void ResetThresholdsPow2();

	virtual void GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const override;

private:
	TArray<FThreshold> Thresholds;
	bool bIsPow2; // thresholds are automatically set as power of two
};

} // namespace UE::Insights::MemoryProfiler
