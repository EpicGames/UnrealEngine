// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemAllocGroupingBySwapPage.h"

#include "Internationalization/Internationalization.h"

// TraceServices
#include "Common/ProviderLock.h"

// TraceInsights
#include "Insights/MemoryProfiler/ViewModels/MemAllocNode.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocValueNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemAllocNode"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

struct TMemoryPageMapKeyFuncs : BaseKeyFuncs<TPair<uint64, FTableTreeNodePtr>, uint64, false>
{
	static FORCEINLINE uint64 GetSetKey(const TPair<uint64, FTableTreeNodePtr>& Element)
	{
		return Element.Key;
	}
	static FORCEINLINE uint32 GetKeyHash(const uint64 Key)
	{
		// memory page is always at least 4k aligned, so skip lower bytes
		return (uint32)(Key >> 12);
	}
	static FORCEINLINE bool Matches(const uint64 A, const uint64 B)
	{
		return A == B;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemSwapPageTreeNode
////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemSwapPageTreeNode : public FTableTreeNode
{
	INSIGHTS_DECLARE_RTTI(FMemSwapPageTreeNode, FTableTreeNode)

public:
	/** Initialization constructor for the group node. */
	explicit FMemSwapPageTreeNode(const FName InName, TWeakPtr<FTable> InParentTable)
		: FTableTreeNode(InName, InParentTable)
	{
	}

	virtual ~FMemSwapPageTreeNode()
	{
	}

	virtual FLinearColor GetIconColor() const override final
	{
		return FLinearColor(0.3f, 0.8f, 0.4f, 1.0f);
	}

	virtual FLinearColor GetColor() const override final
	{
		return FLinearColor(0.2f, 0.8f, 0.4f, 1.0f);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FMemSwapPageTreeNode)

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemSwappedAllocsTreeNode
////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemSwappedAllocsTreeNode : public FMemAllocValueNode
{
	INSIGHTS_DECLARE_RTTI(FMemSwappedAllocsTreeNode, FMemAllocValueNode)

	int64 SwappedSize;

public:
	/** Initialization constructor for the group node. */
	explicit FMemSwappedAllocsTreeNode(const FName InName, int64 InSwappedSize, TWeakPtr<FTable> InParentTable)
		: FMemAllocValueNode(InName, InParentTable)
		, SwappedSize(InSwappedSize)
	{
	}

	virtual ~FMemSwappedAllocsTreeNode()
	{
	}

	virtual FLinearColor GetIconColor() const override final
	{
		return FLinearColor(0.3f, 0.8f, 0.4f, 1.0f);
	}

	virtual FLinearColor GetColor() const override final
	{
		return FLinearColor(0.4f, 0.5f, 0.4f, 1.0f);
	}

	virtual int64 GetValueI64() const override
	{
		return SwappedSize;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FMemSwappedAllocsTreeNode)

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemAllocGroupingBySwapPage
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FMemAllocGroupingBySwapPage)

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemAllocGroupingBySwapPage::FMemAllocGroupingBySwapPage(const TraceServices::IAllocationsProvider& InAllocProvider, bool bInIsInverted, bool bInIsGroupingByFunction)
	: FMemAllocGroupingByCallstack(
		bInIsInverted ? LOCTEXT("Grouping_BySwapCallstack2_ShortName", "In-Swap Inverted Alloc Callstack")
					  : LOCTEXT("Grouping_BySwapCallstack1_ShortName", "In-Swap Alloc Callstack"),
		bInIsInverted ? LOCTEXT("Grouping_BySwapCallstack2_TitleName", "By In-Swap Inverted Alloc Callstack")
					  : LOCTEXT("Grouping_BySwapCallstack1_TitleName", "By In-Swap Alloc Callstack"),
		LOCTEXT("Grouping_SwapCallstack_Desc", "Creates a tree based on callstack of each allocation which is in swap memory."),
		nullptr,
		true /*bInIsAllocCallstack*/,
		bInIsInverted,
		bInIsGroupingByFunction)
	, AllocProvider(InAllocProvider)
{
	SetColor(FLinearColor(0.2f, 0.8f, 0.4f, 1.0f));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemAllocGroupingBySwapPage::~FMemAllocGroupingBySwapPage()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemAllocGroupingBySwapPage::GroupNodes(
	const TArray<FTableTreeNodePtr>& Nodes,
	FTableTreeNode& ParentGroup,
	TWeakPtr<FTable> InParentTable,
	IAsyncOperationProgress& InAsyncOperationProgress) const
{
	ParentGroup.ClearChildren();

	FSwapNodeMap SwapNodes;
	GatherSwapNodes(Nodes, SwapNodes, InParentTable, InAsyncOperationProgress);

	GroupNodesInternal(Nodes, ParentGroup, InParentTable, InAsyncOperationProgress, [this,InParentTable, SwapNodes](FTableTreeNodePtr& InTableTreeNodePtr) { return ProcessNodeForSwapInfo(InTableTreeNodePtr, InParentTable, SwapNodes); });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemAllocGroupingBySwapPage::GatherSwapNodes(const TArray<FTableTreeNodePtr>& Nodes, FSwapNodeMap& SwapNodeMap, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const
{
	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			return;
		}

		if (!NodePtr->Is<FMemAllocNode>())
		{
			continue;
		}
		const FMemAllocNode& MemAllocNode = NodePtr->As<FMemAllocNode>();
		const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();

		if (Alloc && Alloc->IsSwap())
		{
			auto SwapEntry = MakeShared<FMemSwapPageTreeNode>(FName(FString::Printf(TEXT("0x%016llx"), Alloc->GetAddress())), InParentTable);
			NodePtr->SetExpansion(false);
			SwapEntry->AddChildAndSetParent(NodePtr);

			ensure(SwapNodeMap.Find(Alloc->GetAddress()) == nullptr);
			SwapNodeMap.Add(Alloc->GetAddress(), SwapEntry);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTableTreeNodePtr FMemAllocGroupingBySwapPage::ProcessNodeForSwapInfo(FTableTreeNodePtr& NodePtr, TWeakPtr<FTable> InParentTable, const FSwapNodeMap& SwapNodeMap) const
{
	if (!NodePtr->Is<FMemAllocNode>())
	{
		return FTableTreeNodePtr();
	}

	const FMemAllocNode& MemAllocNode = NodePtr->As<FMemAllocNode>();
	const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();

	const bool bIsInSwap = Alloc && Alloc->IsSwap();
	if (bIsInSwap || !Alloc)
	{
		return FTableTreeNodePtr(); // Remove the swap_xxx entries, they are added back underneath their respective alloc_xxxs
	}

	int64 SwappedSize = 0;

	const uint64 PageSize = AllocProvider.GetPlatformPageSize();
	const uint64 PageMask = ~(PageSize - 1);

	const uint64 AllocStart = Alloc->GetAddress();
	const uint64 AllocEnd = AllocStart + FMath::Max(Alloc->GetSize(), 1); // fake alloc to have at least 1 byte size to make page walking logic work
	const uint64 PageRangeStart = AllocStart & PageMask;
	const uint64 PageRangeEnd = (AllocEnd + PageSize - 1) & PageMask;
	const uint64 PageAmount = (PageRangeEnd - PageRangeStart) / PageSize;

	TArray<FMemSwapPageTreeNodePtr> SwapPagesEncountered;
	// go through every memory page and check if the alloc is in swap, and record each page.
	for (uint64 PageAddress = PageRangeStart; PageAddress < PageRangeEnd; PageAddress += PageSize)
	{
		const FMemSwapPageTreeNodePtr* SwapNodePtrPtr = SwapNodeMap.Find(PageAddress);
		if (!SwapNodePtrPtr)
		{
			continue;
		}
		uint64 AllocSizeInPage = PageSize;

		if (PageAddress < AllocStart)
		{
			// adjust size if allocation starts after page start
			AllocSizeInPage -= AllocStart - PageAddress;
		}
		if (PageAddress + PageSize > AllocEnd)
		{
			// adjust size if allocation ends before page end
			AllocSizeInPage -= PageAddress + PageSize - AllocEnd;
		}

		if (AllocSizeInPage)
		{
			SwapPagesEncountered.Add(*SwapNodePtrPtr);
		}

		SwappedSize += AllocSizeInPage;
	}

	if (SwapPagesEncountered.IsEmpty())
	{
		// remove allocs that are not in swap.
		return FTableTreeNodePtr();
	}

	// Wrap the alloc to provide Swap Size column info.
	if (SwappedSize)
	{
		ensure(SwappedSize <= FMath::Max(Alloc->GetSize(), 1));
		auto SwappedWrapper = MakeShared<FMemSwappedAllocsTreeNode>(FName(FString::Printf(TEXT("%s swap info"), *NodePtr->GetName().ToString())), SwappedSize, InParentTable);
		SwappedWrapper->AddChildAndSetParent(NodePtr);

		if (TSharedPtr<FTable> SharedTable = InParentTable.Pin())
		{
			auto SwapEventsGroup = MakeShared<FMemSwapPageTreeNode>(FName(FString::Printf(TEXT("Swap events"))), InParentTable);
			SwappedWrapper->AddChildAndSetParent(SwapEventsGroup);

			// for each alloc we create a swap group that contains all of the swap events.
			for (FMemSwapPageTreeNodePtr& SwapPage : SwapPagesEncountered)
			{
				TSharedPtr<FMemAllocTable> MemAllocTable = StaticCastSharedPtr<FMemAllocTable>(SharedTable);
				if (!MemAllocTable.IsValid())
				{
					continue;
				}

				SwapPage->EnumerateChildren([&SwapEventsGroup, &MemAllocTable](const FBaseTreeNodePtr& ChildNode) -> bool
					{
						// duplicate swap alloc nodes
						if (ChildNode->Is<FMemAllocNode>())
						{
							const FMemAllocNode& SwapAllocNode = ChildNode->As<FMemAllocNode>();
							FMemAllocNodePtr DupeSwapNodePtr = MakeShared<FMemAllocNode>(ChildNode->GetName(), MemAllocTable, SwapAllocNode.GetRowIndex());
							SwapEventsGroup->AddChildAndSetParent(DupeSwapNodePtr);
						}
						return true;
					});
			}
			NodePtr = SwappedWrapper;
		}
	}

	return NodePtr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
