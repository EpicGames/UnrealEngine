// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"
#include "MemAllocGroupingByCallstack.h"

namespace TraceServices 
{
	class IAllocationsProvider;
}
namespace UE::Insights { class IAsyncOperationProgress; }

namespace UE::Insights::MemoryProfiler
{
typedef TSharedPtr<class FMemSwapPageTreeNode> FMemSwapPageTreeNodePtr;

class FMemAllocGroupingBySwapPage : public FMemAllocGroupingByCallstack
{
	INSIGHTS_DECLARE_RTTI(FMemAllocGroupingBySwapPage, FMemAllocGroupingByCallstack)

public:
	FMemAllocGroupingBySwapPage(const TraceServices::IAllocationsProvider& AllocProvider, bool bInIsInverted, bool bInIsGroupingByFunction);
	virtual ~FMemAllocGroupingBySwapPage() override;

	virtual void GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const override;

private:
	struct TMemoryPageMapKeyFuncs : BaseKeyFuncs<TPair<uint64, FMemSwapPageTreeNodePtr>, uint64, false>
	{
		static FORCEINLINE bool Matches(const uint64 A, const uint64 B) { return A == B; }
		static FORCEINLINE uint64 GetSetKey(const TPair<uint64, FMemSwapPageTreeNodePtr>& Element) { return Element.Key; }
		static FORCEINLINE uint32 GetKeyHash(const uint64 Key)
		{
			// memory page is always at least 4k aligned, so skip lower bytes
			return (uint32)(Key >> 12);
		}
	};

	using FSwapNodeMap = TMap<uint64, FMemSwapPageTreeNodePtr, FDefaultSetAllocator, TMemoryPageMapKeyFuncs>;
	FTableTreeNodePtr ProcessNodeForSwapInfo(FTableTreeNodePtr& NodePtr, TWeakPtr<FTable> InParentTable, const FSwapNodeMap& SwapNodeMap) const;
	
	void GatherSwapNodes(const TArray<FTableTreeNodePtr>& Nodes, FSwapNodeMap& SwapNodeMap, TWeakPtr<FTable> ParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const;

	const TraceServices::IAllocationsProvider& AllocProvider;
};

} // namespace UE::Insights::MemoryProfiler
