// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemAllocGroupingByTag.h"

// TraceServices
#include "Common/ProviderLock.h"

// TraceInsightsCore
#include "InsightsCore/Common/AsyncOperationProgress.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocNode.h"
#include "Insights/MemoryProfiler/MemoryProfilerManager.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemAllocNode"

#define INSIGHTS_MERGE_MEM_TAGS_BY_NAME 1

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FMemAllocGroupingByTag)
INSIGHTS_IMPLEMENT_RTTI(FMemTagTableTreeNode)

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemAllocGroupingByTag
////////////////////////////////////////////////////////////////////////////////////////////////////

FMemAllocGroupingByTag::FMemAllocGroupingByTag(const TraceServices::IAllocationsProvider& InTagProvider)
	: FTreeNodeGrouping(
		LOCTEXT("Grouping_ByTag_ShortName", "Tag"),
		LOCTEXT("Grouping_ByTag_TitleName", "By Tag"),
		LOCTEXT("Grouping_Tag_Desc", "Creates a tree based on Tag hierarchy."),
		nullptr)
	, TagProvider(InTagProvider)
{
	SetColor(FLinearColor(0.75f, 0.5f, 1.0f, 1.0f));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemAllocGroupingByTag::GroupNodes(
	const TArray<FTableTreeNodePtr>& Nodes,
	FTableTreeNode& ParentGroup,
	TWeakPtr<FTable> InParentTable,
	IAsyncOperationProgress& InAsyncOperationProgress) const
{
	using TagIdType = TraceServices::TagIdType;

	ParentGroup.ClearChildren();

	constexpr TagIdType InvalidTagId = ~0u;
	struct FTagNode
	{
		FTagNode(const TCHAR* InName, TagIdType InId, TagIdType InParentId, FTableTreeNodePtr InTreeNode)
			: Name(InName), Id(InId), ParentId(InParentId), TreeNode(InTreeNode), Parent(nullptr), AllocCount(0)
		{}

		void MergeTagNodesByName(TMap<TagIdType, FTagNode*>& IdToNodeMap)
		{
			const int32 ChildrenCount = Children.Num();
			for (int32 Index = 0; Index < ChildrenCount; ++Index)
			{
				FTagNode* CurrentNode = Children[Index];
				for (int32 TargetIndex = 0; TargetIndex < Index; ++TargetIndex)
				{
					FTagNode* TargetNode = Children[TargetIndex];

					//if (FCString::Strcmp(TargetNode->Name, CurrentNode->Name) == 0)
					// Name is persistent string so we can compare pointers directly.
					if (TargetNode->Name == CurrentNode->Name)
					{
						// Move children from CurrentNode to TargetNode.
						for (FTagNode* Child : CurrentNode->Children)
						{
							TargetNode->Children.Add(Child);
							Child->Parent = TargetNode;
						}
						CurrentNode->Children.Reset();

						// Allocs with CurrentNode's tag id will go to TargetNode.
						IdToNodeMap.Emplace(CurrentNode->Id, TargetNode);

						break;
					}
				}
			}
			for (FTagNode* Child : Children)
			{
				Child->MergeTagNodesByName(IdToNodeMap);
			}
		};

		const TCHAR* Name; // persistent string (from Session's StringStore or static string)

		TagIdType Id;
		TagIdType ParentId;
		FTableTreeNodePtr TreeNode;

		FTagNode* Parent;
		TArray<FTagNode*> Children;
		uint64 AllocCount;
	};
	TArray<FTagNode*> TagNodes;
	TMap<TagIdType, FTagNode*> IdToNodeMap;

	class FScopedMemory
	{
	public:
		FScopedMemory(TArray<FTagNode*>& InTagNodes)
			: TagNodes(InTagNodes)
		{
		}
		~FScopedMemory()
		{
			for (FTagNode* TagNode : TagNodes)
			{
				delete TagNode;
			}
		}

	private:
		TArray<FTagNode*>& TagNodes;
	};
	FScopedMemory _(TagNodes);

	// Create tag nodes.
	{
		TraceServices::FProviderReadScopeLock TagProviderReadScopeLock(TagProvider);
		TagProvider.EnumerateTags([&](const TCHAR* Name, const TCHAR* FullName, TagIdType Id, TagIdType ParentId)
			{
				FTableTreeNodePtr TreeNode = MakeShared<FMemTagTableTreeNode>(InParentTable, FullName, Name);
				FTagNode* TagNode = new FTagNode(Name, Id, ParentId, TreeNode);
				TagNodes.Add(TagNode);
				IdToNodeMap.Add(Id, TagNode);
			});
	}

	// Create the Untagged node (if not already present).
	constexpr TagIdType UntaggedNodeId = 0;
	FTagNode* UntaggedNode = IdToNodeMap.FindRef(UntaggedNodeId);
	if (!UntaggedNode)
	{
		const TCHAR* UntaggedNodeName = TEXT("Untagged");
		FTableTreeNodePtr UntaggedTreeNode = MakeShared<FMemTagTableTreeNode>(InParentTable, UntaggedNodeName, UntaggedNodeName);
		UntaggedNode = new FTagNode(UntaggedNodeName, UntaggedNodeId, InvalidTagId, UntaggedTreeNode);
		TagNodes.Add(UntaggedNode);
		IdToNodeMap.Add(UntaggedNodeId, UntaggedNode);
	}

	// Create the Unknown tag node.
	constexpr TagIdType UnknownTagNodeId = ~0u - 1;
	FTagNode* UnknownTagNode = IdToNodeMap.FindRef(UnknownTagNodeId);
	if (!UnknownTagNode)
	{
		const TCHAR* UnknownNodeName = TEXT("Unknown");
		FTableTreeNodePtr UnknownTagTreeNode = MakeShared<FMemTagTableTreeNode>(InParentTable, UnknownNodeName, UnknownNodeName);
		UnknownTagNode = new FTagNode(UnknownNodeName, UnknownTagNodeId, InvalidTagId, UnknownTagTreeNode);
		TagNodes.Add(UnknownTagNode);
		IdToNodeMap.Add(UnknownTagNodeId, UnknownTagNode);
	}

	// Create the virtual tree of tag group nodes.
	const TCHAR* RootNodeName = TEXT("Root");
	FTagNode Root(RootNodeName, InvalidTagId, InvalidTagId, nullptr);
	for (FTagNode* TagNode : TagNodes)
	{
		if (TagNode->ParentId == InvalidTagId)
		{
			TagNode->Parent = &Root;
		}
		else
		{
			TagNode->Parent = IdToNodeMap.FindRef(TagNode->ParentId);
			if (!TagNode->Parent)
			{
				TagNode->Parent = UnknownTagNode;
			}
		}

		TagNode->Parent->Children.Add(TagNode);
	}

	if (InAsyncOperationProgress.ShouldCancelAsyncOp())
	{
		return;
	}

#if INSIGHTS_MERGE_MEM_TAGS_BY_NAME
	// Merge tag group nodes (with same parent) by name.
	Root.MergeTagNodesByName(IdToNodeMap);
#endif

	// Add nodes to the correct tag group.
	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			return;
		}

		if (NodePtr->IsGroup())
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		const FMemAllocNode& MemAllocNode = static_cast<const FMemAllocNode&>(*NodePtr);
		const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
		if (Alloc)
		{
			const TagIdType TagId = Alloc->GetTagId();
			FTagNode* TagNode = IdToNodeMap.FindRef(TagId);
			if (!TagNode)
			{
				TagNode = UnknownTagNode;
			}
			TagNode->TreeNode->AddChildAndSetParent(NodePtr);
			TagNode->AllocCount++;
		}
		else
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
		}
	}

	if (InAsyncOperationProgress.ShouldCancelAsyncOp())
	{
		return;
	}

	for (FTagNode* TagNode : TagNodes)
	{
		FTagNode* Parent = TagNode->Parent;
		while (Parent)
		{
			Parent->AllocCount += TagNode->AllocCount;
			Parent = Parent->Parent;
		}
	}

	if (InAsyncOperationProgress.ShouldCancelAsyncOp())
	{
		return;
	}

	// Fixup parent relationships and filter out empty nodes.
	for (FTagNode* TagNode : TagNodes)
	{
		if (TagNode->AllocCount > 0)
		{
			check(TagNode->TreeNode);
			if (TagNode->Parent->TreeNode)
			{
				TagNode->Parent->TreeNode->AddChildAndSetParent(TagNode->TreeNode);
			}
			else
			{
				check(TagNode->Parent == &Root);
				ParentGroup.AddChildAndSetParent(TagNode->TreeNode);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemTagTableTreeNode
////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagTableTreeNode::UpdateLLMSize() const
{
	LLMSize = 0;

	TSharedPtr<FTable> Table = ParentTable.Pin();
	if (!Table.IsValid())
	{
		return;
	}

	TSharedPtr<FMemAllocTable> MemAllocTable = StaticCastSharedPtr<FMemAllocTable>(Table);
	if (!MemAllocTable.IsValid())
	{
		return;
	}

	double Time = MemAllocTable->GetTimeMarkerA();

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	FMemorySharedState* SharedState = FMemoryProfilerManager::Get()->GetSharedState();
	if (!SharedState)
	{
		return;
	}

	TraceServices::FMemoryTrackerId TrackerId = FMemoryTracker::InvalidTrackerId;
	TraceServices::FMemoryTagId TagId = FMemoryTag::InvalidTagId;

	const FMemoryTagList& TagList = SharedState->GetTagList();
	for (const FMemoryTag* MemTag : TagList.GetTags())
	{
		if (MemTag)
		{
			const FString& FullName = MemTag->GetStatFullName();
			if (FCString::Stricmp(*FullName, TagFullName) == 0)
			{
				TrackerId = MemTag->GetTrackerId();
				TagId = MemTag->GetId();
				break;
			}
		}
	}
	if (TrackerId == FMemoryTracker::InvalidTrackerId || TagId == FMemoryTag::InvalidTagId)
	{
		return;
	}

	const TraceServices::IMemoryProvider* MemoryProvider = TraceServices::ReadMemoryProvider(*Session.Get());
	if (MemoryProvider)
	{
		TraceServices::FProviderReadScopeLock _(*MemoryProvider);

		const uint64 TotalSampleCount = MemoryProvider->GetTagSampleCount(TrackerId, TagId);
		if (TotalSampleCount > 0)
		{
			int64 LocalLLMSize = INT64_MAX;
			MemoryProvider->EnumerateTagSamples(TrackerId, TagId, Time, Time, true,
				[&LocalLLMSize](double Time, double Duration, const TraceServices::FMemoryTagSample& Sample)
				{
					if (LocalLLMSize == INT64_MAX)
					{
						LocalLLMSize = Sample.Value;
					}
				});
			if (LocalLLMSize != INT64_MAX)
			{
				LLMSize = LocalLLMSize;
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef INSIGHTS_MERGE_MEM_TAGS_BY_NAME
#undef LOCTEXT_NAMESPACE
