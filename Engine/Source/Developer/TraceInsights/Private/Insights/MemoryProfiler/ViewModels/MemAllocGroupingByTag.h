// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"

// TraceServices
#include "TraceServices/Model/Memory.h"

// TraceInsightsCore
#include "InsightsCore/Common/SimpleRtti.h"
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

namespace TraceServices { class IAllocationsProvider; }

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemAllocGroupingByTag : public FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FMemAllocGroupingByTag, FTreeNodeGrouping);

public:
	FMemAllocGroupingByTag(const TraceServices::IAllocationsProvider& TagProvider);

	virtual void GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const override;

private:
	const TraceServices::IAllocationsProvider& TagProvider;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemTagTableTreeNode : public FTableTreeNode
{
	INSIGHTS_DECLARE_RTTI(FMemTagTableTreeNode, FTableTreeNode)

public:
	/** Initialization constructor for a table record node. */
	explicit FMemTagTableTreeNode(TWeakPtr<FTable> InParentTable, int32 InRowIndex, const TCHAR* InTagFullName, const TCHAR* InName, bool IsGroup)
		: FTableTreeNode(InParentTable, InRowIndex, IsGroup)
		, DisplayName(FText::FromString(InName))
		, TagFullName(InTagFullName)
	{
	}

	/** Initialization constructor for the group node. */
	explicit FMemTagTableTreeNode(TWeakPtr<FTable> InParentTable, const TCHAR* InTagFullName, const TCHAR* InName)
		: FTableTreeNode(InParentTable)
		, DisplayName(FText::FromString(InName))
		, TagFullName(InTagFullName)
	{
	}

	virtual ~FMemTagTableTreeNode() = default;

	virtual const FText GetDisplayName() const override
	{
		return DisplayName;
	}

	virtual const FText GetTooltipText() const
	{
		return FText::FromString(TagFullName);
	}

	virtual FLinearColor GetIconColor() const override
	{
		return FLinearColor(0.75f, 0.5f, 1.0f, 1.0f);
	}

	virtual FLinearColor GetColor() const override
	{
		return FLinearColor(0.75f, 0.5f, 1.0f, 1.0f);
	}

	int64 GetLLMSize() const
	{
		if (LLMSize == INT64_MAX)
		{
			UpdateLLMSize();
		}
		return LLMSize;
	}

private:
	void UpdateLLMSize() const;

private:
	FText DisplayName;
	const TCHAR* TagFullName = nullptr;
	mutable int64 LLMSize = INT64_MAX;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
