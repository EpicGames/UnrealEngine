// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemAllocNode.h"

// TraceServices
#include "TraceServices/Model/Callstack.h"

// TraceInsights
#include "Insights/MemoryProfiler/ViewModels/CallstackFormatting.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemAllocNode"

namespace UE::Insights::MemoryProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FMemAllocNode)

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FMemAllocNode::GetAllocCallstackId() const
{
	return IsValidMemAlloc() ? GetMemAllocChecked().GetAllocCallstackId() : 0u;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FMemAllocNode::GetFreeCallstackId() const
{
	return IsValidMemAlloc() ? GetMemAllocChecked().GetFreeCallstackId() : 0u;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMemAllocNode::GetFullCallstack(ECallstackType InCallstackType) const
{
	return GetFullCallstackOrSourceFiles(InCallstackType, (uint8)EStackFrameFormatFlags::ModuleSymbolFileAndLine);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMemAllocNode::GetFullCallstackSourceFiles(ECallstackType InCallstackType) const
{
	return GetFullCallstackOrSourceFiles(InCallstackType, (uint8)EStackFrameFormatFlags::File);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMemAllocNode::GetTopFunction(ECallstackType InCallstackType) const
{
	return GetTopFunctionOrSourceFile(InCallstackType, (uint8)EStackFrameFormatFlags::ModuleAndSymbol);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMemAllocNode::GetTopFunctionEx(ECallstackType InCallstackType) const
{
	return GetTopFunctionOrSourceFile(InCallstackType, (uint8)EStackFrameFormatFlags::ModuleSymbolFileAndLine);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMemAllocNode::GetTopSourceFile(ECallstackType InCallstackType) const
{
	return GetTopFunctionOrSourceFile(InCallstackType, (uint8)EStackFrameFormatFlags::File);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMemAllocNode::GetTopSourceFileEx(ECallstackType InCallstackType) const
{
	return GetTopFunctionOrSourceFile(InCallstackType, (uint8)EStackFrameFormatFlags::FileAndLine);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMemAllocNode::GetFullCallstackOrSourceFiles(ECallstackType InCallstackType, uint8 Flags) const
{
	if (!IsValidMemAlloc())
	{
		return FText();
	}

	const FMemoryAlloc& Alloc = GetMemAllocChecked();
	const TraceServices::FCallstack* Callstack =
		InCallstackType == ECallstackType::AllocCallstack
			? Alloc.GetAllocCallstack()
			: Alloc.GetFreeCallstack();

	if (!Callstack)
	{
		return GetCallstackNotAvailableString();
	}

	if (Callstack->Num() == 0)
	{
		if (Callstack->GetEmptyId() == 0)
		{
			return GetNoCallstackString();
		}
		else
		{
			return GetEmptyCallstackString();
		}
	}

	TStringBuilder<1024> Tooltip;
	const uint32 NumCallstackFrames = Callstack->Num();
	check(NumCallstackFrames <= 256); // see Callstack->Frame(uint8)
	for (uint32 FrameIndex = 0; FrameIndex < NumCallstackFrames; ++FrameIndex)
	{
		if (FrameIndex != 0)
		{
			Tooltip << TEXT("\n");
		}
		const TraceServices::FStackFrame* Frame = Callstack->Frame(static_cast<uint8>(FrameIndex));
		check(Frame != nullptr);
		FormatStackFrame(*Frame, Tooltip, (EStackFrameFormatFlags)Flags);
	}
	return FText::FromString(FString(Tooltip));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMemAllocNode::GetTopFunctionOrSourceFile(ECallstackType InCallstackType, uint8 Flags) const
{
	if (!IsValidMemAlloc())
	{
		return FText();
	}

	const FMemoryAlloc& Alloc = GetMemAllocChecked();
	const TraceServices::FCallstack* Callstack =
		InCallstackType == ECallstackType::AllocCallstack
			? Alloc.GetAllocCallstack()
			: Alloc.GetFreeCallstack();

	if (!Callstack)
	{
		return GetCallstackNotAvailableString();
	}

	if (Callstack->Num() == 0)
	{
		if (Callstack->GetEmptyId() == 0)
		{
			return GetNoCallstackString();
		}
		else
		{
			return GetEmptyCallstackString();
		}
	}

	const TraceServices::FStackFrame* Frame = nullptr;
	const uint32 NumCallstackFrames = Callstack->Num();
	check(NumCallstackFrames <= 256); // see Callstack->Frame(uint8)
	for (uint32 FrameIndex = 0; FrameIndex < NumCallstackFrames; ++FrameIndex)
	{
		Frame = Callstack->Frame(static_cast<uint8>(FrameIndex));
		check(Frame != nullptr);

		if (Frame->Symbol &&
			Frame->Symbol->Name &&
			Frame->Symbol->FilterStatus != TraceServices::EResolvedSymbolFilterStatus::Filtered)
		{
			break;
		}
	}
	check(Frame != nullptr);

	TStringBuilder<1024> Str;
	FormatStackFrame(*Frame, Str, (EStackFrameFormatFlags)Flags);
	return FText::FromString(FString(Str));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
