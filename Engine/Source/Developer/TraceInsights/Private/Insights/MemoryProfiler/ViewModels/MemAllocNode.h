// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Templates/SharedPointer.h"

// TraceServices
#include "TraceServices/Model/Memory.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"

// TraceInsights
#include "Insights/MemoryProfiler/ViewModels/MemAllocTable.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryAlloc.h"
#include "Insights/MemoryProfiler/ViewModels/MemorySharedState.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemAllocNode;

/** Type definition for shared pointers to instances of FMemAllocNode. */
typedef TSharedPtr<class FMemAllocNode> FMemAllocNodePtr;

/** Type definition for shared references to instances of FMemAllocNode. */
typedef TSharedRef<class FMemAllocNode> FMemAllocNodeRef;

/** Type definition for shared references to const instances of FMemAllocNode. */
typedef TSharedRef<const class FMemAllocNode> FMemAllocNodeRefConst;

/** Type definition for weak references to instances of FMemAllocNode. */
typedef TWeakPtr<class FMemAllocNode> FMemAllocNodeWeak;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class used to store information about an allocation node (used in the SMemAllocTreeView).
 */
class FMemAllocNode : public FTableTreeNode
{
	INSIGHTS_DECLARE_RTTI(FMemAllocNode, FTableTreeNode)

public:
	enum ECallstackType : uint8
	{
		AllocCallstack,
		FreeCallstack
	};

	/** Initialization constructor for the MemAlloc node. */
	explicit FMemAllocNode(const FName InName, TWeakPtr<FMemAllocTable> InParentTable, int32 InRowIndex)
		: FTableTreeNode(InName, InParentTable, InRowIndex)
		, MemAllocTable(InParentTable.Pin().Get())
	{
	}

	/** Initialization constructor for the group node. */
	explicit FMemAllocNode(const FName InGroupName, TWeakPtr<FMemAllocTable> InParentTable)
		: FTableTreeNode(InGroupName, InParentTable)
		, MemAllocTable(InParentTable.Pin().Get())
	{
	}

	FMemAllocTable& GetMemTableChecked() const { return *MemAllocTable; }
	FMemAllocTable* GetMemTable() const { return MemAllocTable; }

	bool IsValidMemAlloc() const { return GetMemTableChecked().IsValidRowIndex(GetRowIndex()); }
	const FMemoryAlloc* GetMemAlloc() const { return GetMemTableChecked().GetMemAlloc(GetRowIndex()); }
	const FMemoryAlloc& GetMemAllocChecked() const { return GetMemTableChecked().GetMemAllocChecked(GetRowIndex()); }

	uint32 GetAllocCallstackId() const;
	uint32 GetFreeCallstackId() const;

	FText GetFullCallstack(ECallstackType InCallstackType) const;
	FText GetFullCallstackSourceFiles(ECallstackType InCallstackType) const;
	FText GetTopFunction(ECallstackType InCallstackType) const;
	FText GetTopFunctionEx(ECallstackType InCallstackType) const;
	FText GetTopSourceFile(ECallstackType InCallstackType) const;
	FText GetTopSourceFileEx(ECallstackType InCallstackType) const;

private:
	FText GetFullCallstackOrSourceFiles(ECallstackType InCallstackType, uint8 Flags) const;
	FText GetTopFunctionOrSourceFile(ECallstackType InCallstackType, uint8 Flags) const;

private:
	FMemAllocTable* MemAllocTable;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
