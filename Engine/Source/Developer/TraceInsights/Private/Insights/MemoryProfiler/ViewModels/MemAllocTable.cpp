// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemAllocTable.h"

#include "Styling/StyleColors.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

// TraceServices
#include "TraceServices/Model/Callstack.h"

// TraceInsightsCore
#include "InsightsCore/Filter/ViewModels/TimeFilterValueConverter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/MemoryProfiler/ViewModels/CallstackFormatting.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocFilterValueConverter.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocGroupingByTag.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocNode.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocValueNode.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocTable.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemAllocTable"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Column identifiers

const FName FMemAllocTableColumns::StartEventIndexColumnId(TEXT("StartEventIndex"));
const FName FMemAllocTableColumns::EndEventIndexColumnId(TEXT("EndEventIndex"));
const FName FMemAllocTableColumns::EventDistanceColumnId(TEXT("EventDistance"));
const FName FMemAllocTableColumns::StartTimeColumnId(TEXT("StartTime"));
const FName FMemAllocTableColumns::EndTimeColumnId(TEXT("EndTime"));
const FName FMemAllocTableColumns::DurationColumnId(TEXT("Duration"));
const FName FMemAllocTableColumns::AllocThreadColumnId(TEXT("AllocThread"));
const FName FMemAllocTableColumns::FreeThreadColumnId(TEXT("FreeThread"));
const FName FMemAllocTableColumns::AddressColumnId(TEXT("Address"));
const FName FMemAllocTableColumns::MemoryPageColumnId(TEXT("MemoryPage"));
const FName FMemAllocTableColumns::CountColumnId(TEXT("Count"));
const FName FMemAllocTableColumns::CompressedSwapSizeColumnId(TEXT("CompressedSwapSize"));
const FName FMemAllocTableColumns::SwapSizeColumnId(TEXT("SwapSize"));
const FName FMemAllocTableColumns::SizeColumnId(TEXT("Size"));
const FName FMemAllocTableColumns::LLMSizeColumnId(TEXT("LLMSize"));
const FName FMemAllocTableColumns::LLMDeltaSizeColumnId(TEXT("LLMDeltaSize"));
const FName FMemAllocTableColumns::TagColumnId(TEXT("Tag"));
const FName FMemAllocTableColumns::AssetColumnId(TEXT("Asset"));
const FName FMemAllocTableColumns::PackageColumnId(TEXT("Package"));
const FName FMemAllocTableColumns::ClassNameColumnId(TEXT("ClassName"));
const FName FMemAllocTableColumns::AllocFunctionColumnId(TEXT("AllocFunction"));
const FName FMemAllocTableColumns::AllocSourceFileColumnId(TEXT("AllocSourceFile"));
const FName FMemAllocTableColumns::AllocCallstackIdColumnId(TEXT("AllocCallstackId"));
const FName FMemAllocTableColumns::AllocCallstackSizeColumnId(TEXT("AllocCallstackSize"));
const FName FMemAllocTableColumns::FreeFunctionColumnId(TEXT("FreeFunction"));
const FName FMemAllocTableColumns::FreeSourceFileColumnId(TEXT("FreeSourceFile"));
const FName FMemAllocTableColumns::FreeCallstackIdColumnId(TEXT("FreeCallstackId"));
const FName FMemAllocTableColumns::FreeCallstackSizeColumnId(TEXT("FreeCallstackSize"));

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemAllocTable
////////////////////////////////////////////////////////////////////////////////////////////////////

FMemAllocTable::FMemAllocTable()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemAllocTable::~FMemAllocTable()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemAllocTable::Reset()
{
	FTable::Reset();

	AddDefaultColumns();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemAllocTable::AddDefaultColumns()
{
	class FAggregatedValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const
		{
			return TOptional<FTableCellValue>();
		}

		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			if (Node.IsGroup() && Node.Is<FTableTreeNode>())
			{
				const FTableTreeNode& TableTreeNode = Node.As<FTableTreeNode>();
				if (TableTreeNode.HasAggregatedValue(Column.GetId()))
				{
					return TableTreeNode.GetAggregatedValue(Column.GetId());
				}
			}
			return GetNodeValue(Node);
		}
	};

	//////////////////////////////////////////////////
	// Hierarchy Column
	{
		const int32 HierarchyColumnIndex = -1;
		const TCHAR* HierarchyColumnName = nullptr;
		AddHierarchyColumn(HierarchyColumnIndex, HierarchyColumnName);

		FTableColumn& HierarchyColumn = *GetColumns()[0];
		HierarchyColumn.SetInitialWidth(200.0f);
		HierarchyColumn.SetShortName(LOCTEXT("AllocationColumnName", "Hierarchy"));
		HierarchyColumn.SetTitleName(LOCTEXT("AllocationColumnTitle", "Allocation Hierarchy"));
		HierarchyColumn.SetDescription(LOCTEXT("AllocationColumnDesc", "Hierarchy of the allocation's tree"));
	}

	int32 ColumnIndex = 0;

	//////////////////////////////////////////////////
	// Start Event Index Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::StartEventIndexColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("StartEventIndexColumnName", "Start Index"));
		Column.SetTitleName(LOCTEXT("StartEventIndexColumnTitle", "Start Event Index"));
		Column.SetDescription(LOCTEXT("StartEventIndexColumnDesc", "The event index when the allocation was allocated."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemAllocStartEventIndexValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(Alloc->GetStartEventIndex());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemAllocStartEventIndexValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Min);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// End Event Index Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::EndEventIndexColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("EndEventIndexColumnName", "End Index"));
		Column.SetTitleName(LOCTEXT("EndEventIndexColumnTitle", "End Event Index"));
		Column.SetDescription(LOCTEXT("EndEventIndexColumnDesc", "The event index when the allocation was freed."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemAllocEndEventIndexValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(Alloc->GetEndEventIndex());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemAllocEndEventIndexValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsUInt32InfiniteNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Max);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Event Distance Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::EventDistanceColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("EventDistanceColumnName", "Event Distance"));
		Column.SetTitleName(LOCTEXT("EventDistanceColumnTitle", "Event Distance"));
		Column.SetDescription(LOCTEXT("EventDistanceColumnDesc", "The event distance (index of free event minus index of alloc event)."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemAllocEventDistanceValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(Alloc->GetEventDistance());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemAllocEventDistanceValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsUInt32InfiniteNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Max);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Start Time Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::StartTimeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("StartTimeColumnName", "Start Time"));
		Column.SetTitleName(LOCTEXT("StartTimeColumnTitle", "Start Time"));
		Column.SetDescription(LOCTEXT("StartTimeColumnDesc", "The time when the allocation was allocated."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		class FMemAllocStartTimeValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(Alloc->GetStartTime());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemAllocStartTimeValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		TSharedRef<IFilterValueConverter> Converter = MakeShared<FTimeFilterValueConverter>();
		Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Min);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// End Time Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::EndTimeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("EndTimeColumnName", "End Time"));
		Column.SetTitleName(LOCTEXT("EndTimeColumnTitle", "End Time"));
		Column.SetDescription(LOCTEXT("EndTimeColumnDesc", "The time when the allocation was freed."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		class FMemAllocEndTimeValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(Alloc->GetEndTime());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemAllocEndTimeValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		TSharedRef<IFilterValueConverter> Converter = MakeShared<FTimeFilterValueConverter>();
		Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Max);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Duration Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::DurationColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("DurationColumnName", "Duration"));
		Column.SetTitleName(LOCTEXT("DurationColumnTitle", "Duration"));
		Column.SetDescription(LOCTEXT("DurationColumnDesc", "The duration of the allocation's life."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		class FMemAllocDurationValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(Alloc->GetDuration());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemAllocDurationValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		TSharedRef<IFilterValueConverter> Converter = MakeShared<FTimeFilterValueConverter>();
		Column.SetValueConverter(Converter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Alloc Thread Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::AllocThreadColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("AllocThreadColumnName", "Alloc Thread"));
		Column.SetTitleName(LOCTEXT("AllocThreadColumnTitle", "Alloc Thread"));
		Column.SetDescription(LOCTEXT("AllocThreadColumnDesc", "The thread the allocation was made on"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(80.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FAllocThreadValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(static_cast<int64>(Alloc->GetAllocThreadId()));
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAllocThreadValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Free Thread Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::FreeThreadColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("FreeThreadColumnName", "Free Thread"));
		Column.SetTitleName(LOCTEXT("FreeThreadColumnTitle", "Free Thread"));
		Column.SetDescription(LOCTEXT("FreeThreadColumnDesc", "The thread the allocation was freed on"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(80.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FAllocThreadValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(static_cast<int64>(Alloc->GetFreeThreadId()));
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAllocThreadValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Address Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::AddressColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("AddressColumnName", "Address"));
		Column.SetTitleName(LOCTEXT("AddressColumnTitle", "Address"));
		Column.SetDescription(LOCTEXT("AddressColumnDesc", "Address of allocation"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(120.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemAllocAddressValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(static_cast<int64>(Alloc->GetAddress()));
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemAllocAddressValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsHex64>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Memory Page Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::MemoryPageColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("MemoryPageColumnName", "Memory Page"));
		Column.SetTitleName(LOCTEXT("MemoryPageColumnTitle", "Memory Page"));
		Column.SetDescription(LOCTEXT("MemoryPageColumnDesc", "Memory Page of allocation"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(120.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemAllocPageValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						const FMemAllocTable& MemAllocTable = MemAllocNode.GetMemTableChecked();
						return FTableCellValue(static_cast<int64>(MemAllocTable.GetAddressPage(Alloc->GetAddress())));
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemAllocPageValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsHex64>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Alloc Count Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::CountColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("CountColumnName", "Count"));
		Column.SetTitleName(LOCTEXT("CountColumnTitle", "Allocation Count"));
		Column.SetDescription(LOCTEXT("CountColumnDesc", "Number of allocations"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemAllocCountValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(static_cast<int64>(1));
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemAllocCountValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Compressed Swap Size
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::CompressedSwapSizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("CompressedSwapSizeColumnName", "Compressed Swap Size"));
		Column.SetTitleName(LOCTEXT("CompressedSwapSizeColumnTitle", "Compressed Swap Size"));
		Column.SetDescription(LOCTEXT("CompressedSwapSizeColumnDesc", "Shows compressed size of swap page."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FSwapCompressedSizeColumnValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc && Alloc->IsSwap())
					{
						return FTableCellValue(static_cast<int64>(Alloc->GetSize()));
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FSwapCompressedSizeColumnValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Swap Size Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::SwapSizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SwapSizeColumnName", "Swap Size"));
		Column.SetTitleName(LOCTEXT("SwapSizeColumnTitle", "Swap Size"));
		Column.SetDescription(LOCTEXT("SwapSizeColumnDesc", "Shows (partial) size of allocation which is stored in the corresponding swap page."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FSizeInSwapColumnValueGetter : public FAggregatedValueGetter
		{
			public:
				virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
				{
 					if (Node.Is<FMemAllocValueNode>())
					{
						const FMemAllocValueNode& SwappedNode = Node.As<FMemAllocValueNode>();
						return FTableCellValue(SwappedNode.GetValueI64());
					}
					else
					{
 						return FAggregatedValueGetter::GetValue(Column, Node);
 					}
				}

				virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
				{
					if (Node.Is<FMemAllocNode>())
					{
						const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
						const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
						if (Alloc && Alloc->IsSwap())
						{
							return FTableCellValue(static_cast<int64>(Alloc->GetSize()));
						}
					}
					return TOptional<FTableCellValue>();
				}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FSizeInSwapColumnValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Size Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::SizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SizeColumnName", "Size"));
		Column.SetTitleName(LOCTEXT("SizeColumnTitle", "Size"));
		Column.SetDescription(LOCTEXT("SizeColumnDesc", "Size of allocation"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemAllocSizeValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						if (Alloc->IsSwap())
						{
							//Do not give swap nodes a size. The size is aggregated with allocations and pollutes the column.
							return FTableCellValue(0LL);
							// const FMemAllocTable& MemAllocTable = MemAllocNode.GetMemTableChecked();
							// return FTableCellValue(static_cast<int64>(MemAllocTable.GetPlatformPageSize()));
						}
						else
						{
							return FTableCellValue(static_cast<int64>(Alloc->GetSize()));
						}
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemAllocSizeValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// LLM Size Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::LLMSizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("LLMSizeColumnName", "LLM Size"));
		Column.SetTitleName(LOCTEXT("LLMSizeColumnTitle", "LLM Size"));
		Column.SetDescription(LOCTEXT("LLMSizeColumnDesc", "Size reported by LLM (only for Mem Tag group nodes)"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FLLMSizeValueGetter : public FTableCellValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.IsGroup())
				{
					if (Node.Is<FMemTagTableTreeNode>())
					{
						const FMemTagTableTreeNode& MemTagNode = Node.As<FMemTagTableTreeNode>();
						int64 LLMSize = MemTagNode.GetLLMSize();
						return FTableCellValue(LLMSize);
					}
					else if (Node.Is<FTableTreeNode>())
					{
						const FTableTreeNode& TableTreeNode = Node.As<FTableTreeNode>();
						if (TableTreeNode.HasAggregatedValue(Column.GetId()))
						{
							return TableTreeNode.GetAggregatedValue(Column.GetId());
						}
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FLLMSizeValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// LLM Delta Size Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::LLMDeltaSizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("LLMDeltaSizeColumnName", "LLM Delta"));
		Column.SetTitleName(LOCTEXT("LLMDeltaSizeColumnTitle", "LLM Delta Size"));
		Column.SetDescription(LOCTEXT("LLMDeltaSizeColumnDesc", "Difference from size reported by LLM (only for Mem Tag group nodes)"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FLLMDeltaSizeValueGetter : public FTableCellValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.IsGroup())
				{
					if (Node.Is<FMemTagTableTreeNode>())
					{
						const FMemTagTableTreeNode& MemTagNode = Node.As<FMemTagTableTreeNode>();
						int64 LLMSize = MemTagNode.GetLLMSize();
						int64 Size = 0;
						if (MemTagNode.HasAggregatedValue(FMemAllocTableColumns::SizeColumnId))
						{
							const FTableCellValue& Value = MemTagNode.GetAggregatedValue(FMemAllocTableColumns::SizeColumnId);
							if (Value.DataType == ETableCellDataType::Int64)
							{
								Size = Value.AsInt64();
							}
						}
						return FTableCellValue(Size - LLMSize);
					}
					else if (Node.Is<FTableTreeNode>())
					{
						const FTableTreeNode& TableTreeNode = Node.As<FTableTreeNode>();
						if (TableTreeNode.HasAggregatedValue(Column.GetId()))
						{
							return TableTreeNode.GetAggregatedValue(Column.GetId());
						}
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FLLMDeltaSizeValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Ascending);

		TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// LLM Tag Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::TagColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TagColumnName", "LLM Tag"));
		Column.SetTitleName(LOCTEXT("TagColumnTitle", "LLM Tag"));
		Column.SetDescription(LOCTEXT("TagColumnDesc", "LLM tag of allocation"));

		// This column is filtered with a custom filter with suggestions
		// so we do not mark it here as CanBeFiltered to prevent it from having a default string filter set.
		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(120.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FMemTagValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(Alloc->GetTag());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemTagValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Package Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::PackageColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("PackageColumnName", "Package"));
		Column.SetTitleName(LOCTEXT("PackageColumnTitle", "Package"));
		Column.SetDescription(LOCTEXT("PackageColumnDesc", "Asset package associated with allocation"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(320.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FPackageMetadataValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(Alloc->GetPackage());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPackageMetadataValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Asset Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::AssetColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("AssetColumnName", "Asset"));
		Column.SetTitleName(LOCTEXT("AssetColumnTitle", "Asset"));
		Column.SetDescription(LOCTEXT("AssetColumnDesc", "Asset associated with allocation"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(320.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FAssetMetadataValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(Alloc->GetAsset());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAssetMetadataValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Class Name Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::ClassNameColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("ClassNameColumnName", "Class Name"));
		Column.SetTitleName(LOCTEXT("ClassNameColumnTitle", "Class Name"));
		Column.SetDescription(LOCTEXT("ClassNameColumnDesc", "Class of asset associated with allocation"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(120.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FClassNameMetadataValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(Alloc->GetClassName());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FClassNameMetadataValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////

	class FCallstackValueFormatter : public FTableCellValueFormatter
	{
	public:
		FCallstackValueFormatter(FMemAllocNode::ECallstackType InCallstackType)
			: CallstackType(InCallstackType)
		{
		}

		virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override
		{
			return InValue.IsSet() ? FText::AsNumber(InValue.GetValue().Int64) : FText::GetEmpty();
		}

		FText GetHeaderText(const FMemAllocNode& MemAllocNode) const
		{
			FText HeaderText;
			if (CallstackType == FMemAllocNode::ECallstackType::AllocCallstack)
			{
				int64 CallstackSize = 0;
				const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
				if (Alloc)
				{
					const TraceServices::FCallstack* Callstack = Alloc->GetAllocCallstack();
					if (Callstack && (Callstack->Num() != 1 || Callstack->Addr(0) != 0))
					{
						CallstackSize = Callstack->Num();
					}
				}
				HeaderText = FText::Format(LOCTEXT("AllocCallstackHeaderFmt", "AllocCallstackId={0} ({1} frames)"),
					FText::AsNumber(MemAllocNode.GetAllocCallstackId(), &FNumberFormattingOptions::DefaultNoGrouping()),
					FText::AsNumber(CallstackSize));
			}
			else
			{
				int64 CallstackSize = 0;
				const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
				if (Alloc)
				{
					const TraceServices::FCallstack* Callstack = Alloc->GetFreeCallstack();
					if (Callstack && (Callstack->Num() != 1 || Callstack->Addr(0) != 0))
					{
						CallstackSize = Callstack->Num();
					}
				}
				HeaderText = FText::Format(LOCTEXT("FreeCallstackHeaderFmt", "FreeCallstackId={0} ({1} frames)"),
					FText::AsNumber(MemAllocNode.GetFreeCallstackId(), &FNumberFormattingOptions::DefaultNoGrouping()),
					FText::AsNumber(CallstackSize));
			}
			return HeaderText;
		}

		virtual TSharedPtr<IToolTip> GetCustomTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			if (!Node.Is<FMemAllocNode>())
			{
				return FTableCellValueFormatter::GetCustomTooltip(Column, Node);
			}
			const FMemAllocNode& MemAllocNode = Node.As<const FMemAllocNode>();

			FText HeaderText = GetHeaderText(MemAllocNode);
			return SNew(SToolTip)
				.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&FTableCellValueFormatter::GetTooltipVisibility)))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(HeaderText)
						.ColorAndOpacity(FSlateColor(EStyleColor::White25))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(&MemAllocNode, &FMemAllocNode::GetFullCallstack, CallstackType)
					]
				];
		}

		virtual FText FormatValueForGrouping(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			return FormatValue(Column.GetValue(Node));
		}

		virtual FText CopyTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			if (Node.Is<FMemAllocNode>())
			{
				const FMemAllocNode& MemAllocNode = Node.As<const FMemAllocNode>();
				return MemAllocNode.GetFullCallstack(CallstackType);
			}
			else
			{
				return FTableCellValueFormatter::CopyTooltip(Column, Node);
			}
		}

	protected:
		FMemAllocNode::ECallstackType CallstackType;
	};

	//////////////////////////////////////////////////

	class FFunctionValueFormatter : public FTextValueFormatter
	{
	public:
		FFunctionValueFormatter(FMemAllocNode::ECallstackType InCallstackType)
			: CallstackType(InCallstackType)
		{
		}

		virtual TSharedPtr<IToolTip> GetCustomTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			if (!Node.Is<FMemAllocNode>())
			{
				return FTextValueFormatter::GetCustomTooltip(Column, Node);
			}
			const FMemAllocNode& MemAllocNode = Node.As<const FMemAllocNode>();

			return SNew(SToolTip)
				.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&FTableCellValueFormatter::GetTooltipVisibility)))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(&MemAllocNode, &FMemAllocNode::GetTopFunction, CallstackType)
						.ColorAndOpacity(FSlateColor(EStyleColor::AccentBlue))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.0f, 6.0f, 2.0f, 2.0f)
					[
						SNew(STextBlock)
						.Text(&MemAllocNode, &FMemAllocNode::GetFullCallstack, CallstackType)
					]
				];
		}

		virtual FText FormatValueForTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			const FMemAllocNode& MemAllocNode = static_cast<const FMemAllocNode&>(Node);
			return MemAllocNode.GetTopFunction(CallstackType);
		}

	protected:
		FMemAllocNode::ECallstackType CallstackType;
	};

	//////////////////////////////////////////////////

	class FSourceValueFormatter : public FTextValueFormatter
	{
	public:
		FSourceValueFormatter(FMemAllocNode::ECallstackType InCallstackType)
			: CallstackType(InCallstackType)
		{
		}

		virtual TSharedPtr<IToolTip> GetCustomTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			if (!Node.Is<FMemAllocNode>())
			{
				return FTextValueFormatter::GetCustomTooltip(Column, Node);
			}
			const FMemAllocNode& MemAllocNode = Node.As<const FMemAllocNode>();

			return SNew(SToolTip)
				.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&FTableCellValueFormatter::GetTooltipVisibility)))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(&MemAllocNode, &FMemAllocNode::GetTopSourceFileEx, CallstackType)
						.ColorAndOpacity(FSlateColor(EStyleColor::AccentBlue))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.0f, 6.0f, 2.0f, 2.0f)
					[
						SNew(STextBlock)
						.Text(&MemAllocNode, &FMemAllocNode::GetFullCallstackSourceFiles, CallstackType)
					]
				];
		}

		virtual FText FormatValueForTooltip(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			const FMemAllocNode& MemAllocNode = static_cast<const FMemAllocNode&>(Node);
			return MemAllocNode.GetTopSourceFileEx(CallstackType);
		}

	protected:
		FMemAllocNode::ECallstackType CallstackType;
	};

	//////////////////////////////////////////////////
	// Top Function (Alloc Callstack) Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::AllocFunctionColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("FunctionColumnName_Alloc", "Function (Alloc)"));
		Column.SetTitleName(LOCTEXT("FunctionColumnTitle_Alloc", "Top Function (Alloc Callstack)"));
		Column.SetDescription(LOCTEXT("FunctionColumnDesc_Alloc", "Resolved top function from the callstack of allocation"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered | ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(550.0f);

		Column.SetDataType(ETableCellDataType::Text);

		class FFunctionValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const uint64 CallstackId = MemAllocNode.GetAllocCallstackId();
					const FText CallstackText = MemAllocNode.GetTopFunction(FMemAllocNode::ECallstackType::AllocCallstack);
					return FTableCellValue(CallstackText, CallstackId);
				}
				return TOptional<FTableCellValue>();
			}
			virtual uint64 GetValueId(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					return MemAllocNode.GetAllocCallstackId();
				}
				return 0;
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FFunctionValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FFunctionValueFormatter>(FMemAllocNode::ECallstackType::AllocCallstack);
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByTextValueWithId>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Top Source File (Alloc Callstack) Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::AllocSourceFileColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SourceFileColumnName_Alloc", "Source (Alloc)"));
		Column.SetTitleName(LOCTEXT("SourceFileColumnTitle_Alloc", "Top Source File (Alloc Callstack)"));
		Column.SetDescription(LOCTEXT("SourceFileColumnDesc_Alloc", "Source file of the top function from the callstack of allocation"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered | ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(550.0f);

		Column.SetDataType(ETableCellDataType::Text);

		class FSourceValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const uint64 CallstackId = MemAllocNode.GetAllocCallstackId();
					const FText CallstackText = MemAllocNode.GetTopSourceFile(FMemAllocNode::ECallstackType::AllocCallstack);
					return FTableCellValue(CallstackText, CallstackId);
				}
				return TOptional<FTableCellValue>();
			}
			virtual uint64 GetValueId(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					return MemAllocNode.GetAllocCallstackId();
				}
				return 0;
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FSourceValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FSourceValueFormatter>(FMemAllocNode::ECallstackType::AllocCallstack);
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByTextValueWithId>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Alloc Callstack Id Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::AllocCallstackIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("CallstackIdColumnName_Alloc", "Alloc Callstack"));
		Column.SetTitleName(LOCTEXT("CallstackIdColumnTitle_Alloc", "Alloc Callstack Id"));
		Column.SetDescription(LOCTEXT("CallstackIdColumnDesc_Alloc", "Id of alloc callstack.\nTooltip shows the entire alloc callstack."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FCallstackIdValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(int64(Alloc->GetAllocCallstackId()));
					}
				}
				return TOptional<FTableCellValue>();
			}
			virtual uint64 GetValueId(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					return MemAllocNode.GetAllocCallstackId();
				}
				return 0;
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FCallstackIdValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCallstackValueFormatter>(FMemAllocNode::ECallstackType::AllocCallstack);
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Alloc Callstack Size Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::AllocCallstackSizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("CallstackSizeColumnName_Alloc", "Alloc Callstack Size"));
		Column.SetTitleName(LOCTEXT("CallstackSizeColumnTitle_Alloc", "Alloc Callstack Size"));
		Column.SetDescription(LOCTEXT("CallstackSizeColumnDesc_Alloc", "Number of alloc callstack frames.\nTooltip shows the entire alloc callstack."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FCallstackSizeValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						const TraceServices::FCallstack* Callstack = Alloc->GetAllocCallstack();
						const int64 CallstackSize = (Callstack && (Callstack->Num() != 1 || Callstack->Addr(0) != 0)) ? Callstack->Num() : 0;
						return FTableCellValue(CallstackSize);
					}
				}
				return TOptional<FTableCellValue>();
			}
			virtual uint64 GetValueId(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					return MemAllocNode.GetAllocCallstackId();
				}
				return 0;
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FCallstackSizeValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCallstackValueFormatter>(FMemAllocNode::ECallstackType::AllocCallstack);
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Top Function (Free Callstack) Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::FreeFunctionColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("FunctionColumnName_Free", "Function (Free)"));
		Column.SetTitleName(LOCTEXT("FunctionColumnTitle_Free", "Top Function (Free Callstack)"));
		Column.SetDescription(LOCTEXT("FunctionColumnDesc_Free", "Resolved top function from the callstack of freeing an allocation"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered | ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(550.0f);

		Column.SetDataType(ETableCellDataType::Text);

		class FFunctionValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const uint64 CallstackId = MemAllocNode.GetFreeCallstackId();
					const FText CallstackText = MemAllocNode.GetTopFunction(FMemAllocNode::ECallstackType::FreeCallstack);
					return FTableCellValue(CallstackText, CallstackId);
				}
				return TOptional<FTableCellValue>();
			}
			virtual uint64 GetValueId(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					return MemAllocNode.GetFreeCallstackId();
				}
				return 0;
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FFunctionValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FFunctionValueFormatter>(FMemAllocNode::ECallstackType::FreeCallstack);
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByTextValueWithId>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Top Source File (Free Callstack) Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::FreeSourceFileColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SourceFileColumnName_Free", "Source (Free)"));
		Column.SetTitleName(LOCTEXT("SourceFileColumnTitle_Free", "Top Source File (Free Callstack)"));
		Column.SetDescription(LOCTEXT("SourceFileColumnDesc_Free", "Source file of the top function from the callstack of freeing an allocation"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered | ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(550.0f);

		Column.SetDataType(ETableCellDataType::Text);

		class FSourceValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const uint64 CallstackId = MemAllocNode.GetFreeCallstackId();
					const FText CallstackText = MemAllocNode.GetTopSourceFile(FMemAllocNode::ECallstackType::FreeCallstack);
					return FTableCellValue(CallstackText, CallstackId);
				}
				return TOptional<FTableCellValue>();
			}
			virtual uint64 GetValueId(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					return MemAllocNode.GetFreeCallstackId();
				}
				return 0;
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FSourceValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FSourceValueFormatter>(FMemAllocNode::ECallstackType::FreeCallstack);
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByTextValueWithId>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Free Callstack Id Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::FreeCallstackIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("CallstackIdColumnName_Free", "Free Callstack"));
		Column.SetTitleName(LOCTEXT("CallstackIdColumnTitle_Free", "Free Callstack Id"));
		Column.SetDescription(LOCTEXT("CallstackIdColumnDesc_Free", "Id of free callstack.\nTooltip shows the entire alloc callstack."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FCallstackIdValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						return FTableCellValue(int64(Alloc->GetFreeCallstackId()));
					}
				}
				return TOptional<FTableCellValue>();
			}
			virtual uint64 GetValueId(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					return MemAllocNode.GetFreeCallstackId();
				}
				return 0;
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FCallstackIdValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCallstackValueFormatter>(FMemAllocNode::ECallstackType::FreeCallstack);
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Free Callstack Size Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemAllocTableColumns::FreeCallstackSizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("CallstackSizeColumnName_Free", "Free Callstack Size"));
		Column.SetTitleName(LOCTEXT("CallstackSizeColumnTitle_Free", "Free Callstack Size"));
		Column.SetDescription(LOCTEXT("CallstackSizeColumnDesc_Free", "Number of free callstack frames.\nTooltip shows the entire free callstack."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FCallstackSizeValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
					if (Alloc)
					{
						const TraceServices::FCallstack* Callstack = Alloc->GetFreeCallstack();
						const int64 CallstackSize = (Callstack && (Callstack->Num() != 1 || Callstack->Addr(0) != 0)) ? Callstack->Num() : 0;
						return FTableCellValue(CallstackSize);
					}
				}
				return TOptional<FTableCellValue>();
			}
			virtual uint64 GetValueId(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemAllocNode>())
				{
					const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
					return MemAllocNode.GetFreeCallstackId();
				}
				return 0;
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FCallstackSizeValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCallstackValueFormatter>(FMemAllocNode::ECallstackType::FreeCallstack);
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
