// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "UObject/NameTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/Table.h"

// TraceInsights
#include "Insights/MemoryProfiler/ViewModels/MemoryAlloc.h"

namespace UE::Insights { class FTableColumn; }

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

// Column identifiers
struct FMemAllocTableColumns
{
	static const FName StartEventIndexColumnId;
	static const FName EndEventIndexColumnId;
	static const FName EventDistanceColumnId;
	static const FName StartTimeColumnId;
	static const FName EndTimeColumnId;
	static const FName DurationColumnId;
	static const FName AllocThreadColumnId;
	static const FName FreeThreadColumnId;
	static const FName AddressColumnId;
	static const FName MemoryPageColumnId;
	static const FName CountColumnId;
	static const FName CompressedSwapSizeColumnId;
	static const FName SwapSizeColumnId;
	static const FName SizeColumnId;
	static const FName LLMSizeColumnId;
	static const FName LLMDeltaSizeColumnId;
	static const FName TagColumnId;
	static const FName AssetColumnId;
	static const FName PackageColumnId;
	static const FName ClassNameColumnId;
	static const FName AllocFunctionColumnId;
	static const FName AllocSourceFileColumnId;
	static const FName AllocCallstackIdColumnId;
	static const FName AllocCallstackSizeColumnId;
	static const FName FreeFunctionColumnId;
	static const FName FreeSourceFileColumnId;
	static const FName FreeCallstackIdColumnId;
	static const FName FreeCallstackSizeColumnId;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemAllocTable : public FTable
{
public:
	FMemAllocTable();
	virtual ~FMemAllocTable();

	virtual void Reset();

	TArray<FMemoryAlloc>& GetAllocs() { return Allocs; }
	const TArray<FMemoryAlloc>& GetAllocs() const { return Allocs; }

	bool IsValidRowIndex(int32 InIndex) const { return InIndex >= 0 && InIndex < Allocs.Num(); }
	const FMemoryAlloc* GetMemAlloc(int32 InIndex) const { return IsValidRowIndex(InIndex) ? &Allocs[InIndex] : nullptr; }
	const FMemoryAlloc& GetMemAllocChecked(int32 InIndex) const { check(IsValidRowIndex(InIndex)); return Allocs[InIndex]; }

	double GetTimeMarkerA() const { return TimeA; }
	void SetTimeMarkerA(double InTime) { TimeA = InTime; }

	uint64 GetAddressPage(uint64 Address) const { return Address & ~(PlatformPageSize - 1); }

	void SetPlatformPageSize(uint64 InPlatformPageSize) { PlatformPageSize = InPlatformPageSize; }
	uint64 GetPlatformPageSize() const { return PlatformPageSize; }

private:
	void AddDefaultColumns();

private:
	TArray<FMemoryAlloc> Allocs;
	double TimeA = 0.0;
	uint64 PlatformPageSize = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
