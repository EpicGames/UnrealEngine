// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemAllocTableViewPresets.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"
#include "InsightsCore/Table/Widgets/STableTreeView.h"

// TraceInsights
#include "Insights/MemoryProfiler/ViewModels/MemAllocGroupingByCallstack.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocGroupingByHeap.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocGroupingBySize.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocGroupingBySwapPage.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocGroupingByTag.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocTable.h"
#include "Insights/MemoryProfiler/Widgets/SMemAllocTableTreeView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::SMemAllocTableTreeView"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Default View

TSharedRef<ITableTreeViewPreset> FMemAllocTableViewPresets::CreateDefaultViewPreset(SMemAllocTableTreeView& TableTreeView)
{
	class FDefaultViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("Default_PresetName", "Default");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("Default_PresetToolTip", "Default View\nConfigure the tree view to show default allocation info.");
		}
		virtual FName GetSortColumn() const override
		{
			return FTable::GetHierarchyColumnId();
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Ascending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),                    true, 200.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::CountColumnId,              true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::SizeColumnId,               true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::TagColumnId,                true, 120.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AllocFunctionColumnId,      true, 550.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AllocCallstackSizeColumnId, true, 100.0f });
		}
	};
	return MakeShared<FDefaultViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Detailed View

TSharedRef<ITableTreeViewPreset> FMemAllocTableViewPresets::CreateDetailedViewPreset(SMemAllocTableTreeView& TableTreeView)
{
	class FDetailedViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("Detailed_PresetName", "Detailed");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("Detailed_PresetToolTip", "Detailed View\nConfigure the tree view to show detailed allocation info.");
		}
		virtual FName GetSortColumn() const override
		{
			return FTable::GetHierarchyColumnId();
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Ascending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),                    true, 200.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::StartEventIndexColumnId,    true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::EndEventIndexColumnId,      true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::EventDistanceColumnId,      true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::StartTimeColumnId,          true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::EndTimeColumnId,            true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::DurationColumnId,           true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AddressColumnId,            true, 120.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::MemoryPageColumnId,         true, 120.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::CountColumnId,              true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::SizeColumnId,               true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::TagColumnId,                true, 120.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AllocFunctionColumnId,      true, 550.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AllocSourceFileColumnId,    true, 550.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AllocCallstackSizeColumnId, true, 100.0f });
		}
	};
	return MakeShared<FDetailedViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Heap Breakdown View

TSharedRef<ITableTreeViewPreset> FMemAllocTableViewPresets::CreateHeapViewPreset(SMemAllocTableTreeView& TableTreeView)
{
	class FHeapViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("Heap_PresetName", "Heap");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("Heap_PresetToolTip", "Heap Breakdown View\nConfigure the tree view to show a breakdown of allocations by their parent heap type.");
		}
		virtual FName GetSortColumn() const override
		{
			return FMemAllocTableColumns::SizeColumnId;
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Descending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			//check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			//InOutCurrentGroupings.Add(InAvailableGroupings[0]);

			const TSharedPtr<FTreeNodeGrouping>* HeapGrouping = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FMemAllocGroupingByHeap>();
				});
			if (HeapGrouping)
			{
				InOutCurrentGroupings.Add(*HeapGrouping);
			}
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),               true, 400.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::CountColumnId,         true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::SizeColumnId,          true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::TagColumnId,           true, 200.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AllocFunctionColumnId, true, 200.0f });
		}
	};
	return MakeShared<FHeapViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Size Breakdown View

TSharedRef<ITableTreeViewPreset> FMemAllocTableViewPresets::CreateSizeViewPreset(SMemAllocTableTreeView& TableTreeView)
{
	class FSizeViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("Size_PresetName", "Size");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("Size_PresetToolTip", "Size Breakdown View\nConfigure the tree view to show a breakdown of allocations by their size.");
		}
		virtual FName GetSortColumn() const override
		{
			return FMemAllocTableColumns::SizeColumnId;
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Descending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);

			const TSharedPtr<FTreeNodeGrouping>* SizeGrouping = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FMemAllocGroupingBySize>();
				});
			if (SizeGrouping)
			{
				InOutCurrentGroupings.Add(*SizeGrouping);
			}
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),               true, 200.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AddressColumnId,       true, 120.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::CountColumnId,         true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::SizeColumnId,          true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::TagColumnId,           true, 120.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AllocFunctionColumnId, true, 400.0f });
		}
	};
	return MakeShared<FSizeViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Tag Breakdown View

TSharedRef<ITableTreeViewPreset> FMemAllocTableViewPresets::CreateTagViewPreset(SMemAllocTableTreeView& TableTreeView)
{
	class FTagViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("Tag_PresetName", "Tags");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("Tag_PresetToolTip", "Tag Breakdown View\nConfigure the tree view to show a breakdown of allocations by their LLM tag.");
		}
		virtual FName GetSortColumn() const override
		{
			return FTable::GetHierarchyColumnId();
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Ascending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);

			const TSharedPtr<FTreeNodeGrouping>* TagGrouping = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FMemAllocGroupingByTag>();
				});
			if (TagGrouping)
			{
				InOutCurrentGroupings.Add(*TagGrouping);
			}
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),               true, 200.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::CountColumnId,         true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::SizeColumnId,          true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AllocFunctionColumnId, true, 400.0f });
		}
	};
	return MakeShared<FTagViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Asset Breakdown View

TSharedRef<ITableTreeViewPreset> FMemAllocTableViewPresets::CreateAssetViewPreset(SMemAllocTableTreeView& TableTreeView)
{
	class FAssetViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("Asset_PresetName", "Asset (Package)");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("Asset_PresetToolTip", "Asset (Package) Breakdown View\nConfigure the tree view to show a breakdown of allocations by Package and Asset Name metadata.");
		}
		virtual FName GetSortColumn() const override
		{
			return FTable::GetHierarchyColumnId();
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Ascending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);

			const TSharedPtr<FTreeNodeGrouping>* PackageGrouping = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FTreeNodeGroupingByPathBreakdown>() &&
						Grouping->As<FTreeNodeGroupingByPathBreakdown>().GetColumnId() == FMemAllocTableColumns::PackageColumnId;
				});
			if (PackageGrouping)
			{
				InOutCurrentGroupings.Add(*PackageGrouping);
			}

			const TSharedPtr<FTreeNodeGrouping>* AssetGrouping = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FTreeNodeGroupingByUniqueValueCString>() &&
						Grouping->As<FTreeNodeGroupingByUniqueValueCString>().GetColumnId() == FMemAllocTableColumns::AssetColumnId;
				});
			if (AssetGrouping)
			{
				InOutCurrentGroupings.Add(*AssetGrouping);
			}
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),               true, 200.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::CountColumnId,         true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::SizeColumnId,          true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::TagColumnId,           true, 120.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::ClassNameColumnId,     true, 120.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AllocFunctionColumnId, true, 300.0f });
		}
	};
	return MakeShared<FAssetViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Class Name Breakdown View

TSharedRef<ITableTreeViewPreset> FMemAllocTableViewPresets::CreateClassNameViewPreset(SMemAllocTableTreeView& TableTreeView)
{
	class FClassNameViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("ClassName_PresetName", "Class Name");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("ClassName_PresetToolTip", "Class Name Breakdown View\nConfigure the tree view to show a breakdown of allocations by Asset's Class Name metadata.");
		}
		virtual FName GetSortColumn() const override
		{
			return FTable::GetHierarchyColumnId();
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Ascending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);

			const TSharedPtr<FTreeNodeGrouping>* ClassNameGrouping = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FTreeNodeGroupingByUniqueValueCString>() &&
						Grouping->As<FTreeNodeGroupingByUniqueValueCString>().GetColumnId() == FMemAllocTableColumns::ClassNameColumnId;
				});
			if (ClassNameGrouping)
			{
				InOutCurrentGroupings.Add(*ClassNameGrouping);
			}

			const TSharedPtr<FTreeNodeGrouping>* PackageGrouping = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FTreeNodeGroupingByUniqueValueCString>() &&
						Grouping->As<FTreeNodeGroupingByUniqueValueCString>().GetColumnId() == FMemAllocTableColumns::PackageColumnId;
				});
			if (PackageGrouping)
			{
				InOutCurrentGroupings.Add(*PackageGrouping);
			}

			const TSharedPtr<FTreeNodeGrouping>* AssetGrouping = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FTreeNodeGroupingByUniqueValueCString>() &&
						Grouping->As<FTreeNodeGroupingByUniqueValueCString>().GetColumnId() == FMemAllocTableColumns::AssetColumnId;
				});
			if (AssetGrouping)
			{
				InOutCurrentGroupings.Add(*AssetGrouping);
			}
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),               true, 200.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::CountColumnId,         true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::SizeColumnId,          true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::TagColumnId,           true, 120.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AllocFunctionColumnId, true, 400.0f });
		}
	};
	return MakeShared<FClassNameViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Callstack Breakdown View

TSharedRef<ITableTreeViewPreset> FMemAllocTableViewPresets::CreateCallstackViewPreset(SMemAllocTableTreeView& TableTreeView, bool bIsInverted, bool bIsAlloc)
{
	class FCallstackViewPreset : public ITableTreeViewPreset
	{
	public:
		FCallstackViewPreset(bool bIsInverted, bool bIsAlloc)
			: bIsInvertedCallstack(bIsInverted)
			, bIsAllocCallstack(bIsAlloc)
		{
		}

		virtual FText GetName() const override
		{
			return
				bIsAllocCallstack
				? (bIsInvertedCallstack ?
					LOCTEXT("InvertedCallstack_Alloc_PresetName", "Inverted Alloc Callstack") :
					LOCTEXT("Callstack_Alloc_PresetName", "Alloc Callstack"))
				: (bIsInvertedCallstack ?
					LOCTEXT("InvertedCallstack_Free_PresetName", "Inverted Free Callstack") :
					LOCTEXT("Callstack_Free_PresetName", "Free Callstack"));
		}
		virtual FText GetToolTip() const override
		{
			return
				bIsAllocCallstack
				? (bIsInvertedCallstack ?
					LOCTEXT("InvertedCallstack_Alloc_PresetToolTip", "Inverted Alloc Callstack Breakdown View\nConfigure the tree view to show a breakdown of allocations by inverted callstack.") :
					LOCTEXT("Callstack_Alloc_PresetToolTip", "Alloc Callstack Breakdown View\nConfigure the tree view to show a breakdown of allocations by callstack."))
				: (bIsInvertedCallstack ?
					LOCTEXT("InvertedCallstack_Free_PresetToolTip", "Inverted Free Callstack Breakdown View\nConfigure the tree view to show a breakdown of allocations by inverted callstack.") :
					LOCTEXT("Callstack_Free_PresetToolTip", "Free Callstack Breakdown View\nConfigure the tree view to show a breakdown of allocations by callstack."));
		}
		virtual FName GetSortColumn() const override
		{
			return FMemAllocTableColumns::SizeColumnId;
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Descending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);

			const bool bIsInverted = bIsInvertedCallstack;
			const bool bIsAlloc = bIsAllocCallstack;
			const TSharedPtr<FTreeNodeGrouping>* CallstackGrouping = InAvailableGroupings.FindByPredicate(
				[bIsInverted, bIsAlloc](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FMemAllocGroupingByCallstack>() &&
						Grouping->As<FMemAllocGroupingByCallstack>().IsInverted() == bIsInverted &&
						Grouping->As<FMemAllocGroupingByCallstack>().IsAllocCallstack() == bIsAlloc;
				});
			if (CallstackGrouping)
			{
				InOutCurrentGroupings.Add(*CallstackGrouping);
			}
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),       true, 400.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::CountColumnId, true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::SizeColumnId,  true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::TagColumnId,   true, 200.0f });
			if (bIsAllocCallstack)
			{
				InOutConfigSet.Add({ FMemAllocTableColumns::AllocFunctionColumnId, true, 200.0f });
			}
			else
			{
				InOutConfigSet.Add({ FMemAllocTableColumns::FreeFunctionColumnId, true, 200.0f });
			}
		}

	private:
		bool bIsInvertedCallstack;
		bool bIsAllocCallstack;
	};
	return MakeShared<FCallstackViewPreset>(bIsInverted, bIsAlloc);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Address (Platform Page) Breakdown View

TSharedRef<ITableTreeViewPreset> FMemAllocTableViewPresets::CreatePlatformPageViewPreset(SMemAllocTableTreeView& TableTreeView)
{
	class FPageViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("Page_PresetName", "Address (Platform Page)");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("Page_PresetToolTip", "Platform Page Breakdown View\nConfigure the tree view to show a breakdown of allocations by their address.\nIt groups allocs into platform page size aligned memory pages.");
		}
		virtual FName GetSortColumn() const override
		{
			return FTable::GetHierarchyColumnId();
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Ascending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);

			const TSharedPtr<FTreeNodeGrouping>* MemoryPageGrouping = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FTreeNodeGroupingByUniqueValueInt64>() &&
						Grouping->As<FTreeNodeGroupingByUniqueValueInt64>().GetColumnId() == FMemAllocTableColumns::MemoryPageColumnId;
				});
			if (MemoryPageGrouping)
			{
				InOutCurrentGroupings.Add(*MemoryPageGrouping);
			}
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),               true, 200.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AddressColumnId,       true, 120.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::CountColumnId,         true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::SizeColumnId,          true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::TagColumnId,           true, 120.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AllocFunctionColumnId, true, 400.0f });
		}
	};
	return MakeShared<FPageViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Swap Breakdown View

TSharedRef<ITableTreeViewPreset> FMemAllocTableViewPresets::CreateSwapViewPreset(SMemAllocTableTreeView& TableTreeView, bool bIsInverted )
{
	class FSwapViewPreset : public ITableTreeViewPreset
	{
	public:
		FSwapViewPreset(bool bIsInverted)
			: bIsInvertedCallstack(bIsInverted)
		{
		}
		virtual FText GetName() const override
		{
			return bIsInvertedCallstack ? LOCTEXT("InvertedSwap_PresetName", "Inverted In-Swap Callstack") : LOCTEXT("Swap_PresetName", "In-Swap Callstack");
		}
		virtual FText GetToolTip() const override
		{
			return bIsInvertedCallstack ?
				LOCTEXT("InvertedSwap_PresetToolTip", "Inverted Swap Usage Breakdown View\nConfigure the tree view to show a breakdown of allocations by their swap page.\nIt shows allocs by inverted callstack in which some or all of the allocation was moved into swap pages.") :
				LOCTEXT("Swap_PresetToolTip", "Swap Usage Breakdown View\nConfigure the tree view to show a breakdown of allocations by their swap page.\nIt shows allocs by callstack in which some or all of the allocation was moved into swap pages.");
		}
		virtual FName GetSortColumn() const override
		{
			return FMemAllocTableColumns::SwapSizeColumnId;
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Descending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();
			const bool bIsInverted = bIsInvertedCallstack;

			const TSharedPtr<FTreeNodeGrouping>* HeapGrouping = InAvailableGroupings.FindByPredicate(
				[bIsInverted](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FMemAllocGroupingBySwapPage>() && 
 						Grouping->As<FMemAllocGroupingBySwapPage>().IsInverted() == bIsInverted;
				});
			if (HeapGrouping)
			{
				InOutCurrentGroupings.Add(*HeapGrouping);
			}
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),                    true, 400.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::StartTimeColumnId,          true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::EndTimeColumnId,            true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AddressColumnId,            true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::CountColumnId,              true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::SwapSizeColumnId,           true, 120.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::SizeColumnId,               true, 100.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::TagColumnId,                true, 130.0f });
			InOutConfigSet.Add({ FMemAllocTableColumns::AllocFunctionColumnId,      true, 200.0f });
		}
	private:
		bool bIsInvertedCallstack;
	};
	return MakeShared<FSwapViewPreset>(bIsInverted);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
