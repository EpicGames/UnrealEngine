// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Templates/SharedPointer.h"

namespace UE::Insights { class ITableTreeViewPreset; }

namespace UE::Insights::MemoryProfiler
{

class SMemAllocTableTreeView;

class FMemAllocTableViewPresets
{
public:
	static TSharedRef<ITableTreeViewPreset> CreateDefaultViewPreset(SMemAllocTableTreeView& TableTreeView);
	static TSharedRef<ITableTreeViewPreset> CreateDetailedViewPreset(SMemAllocTableTreeView& TableTreeView);
	static TSharedRef<ITableTreeViewPreset> CreateHeapViewPreset(SMemAllocTableTreeView& TableTreeView);
	static TSharedRef<ITableTreeViewPreset> CreateSizeViewPreset(SMemAllocTableTreeView& TableTreeView);
	static TSharedRef<ITableTreeViewPreset> CreateTagViewPreset(SMemAllocTableTreeView& TableTreeView);
	static TSharedRef<ITableTreeViewPreset> CreateAssetViewPreset(SMemAllocTableTreeView& TableTreeView);
	static TSharedRef<ITableTreeViewPreset> CreateClassNameViewPreset(SMemAllocTableTreeView& TableTreeView);
	static TSharedRef<ITableTreeViewPreset> CreateCallstackViewPreset(SMemAllocTableTreeView& TableTreeView, bool bIsInverted, bool bIsAlloc);
	static TSharedRef<ITableTreeViewPreset> CreatePlatformPageViewPreset(SMemAllocTableTreeView& TableTreeView);
	static TSharedRef<ITableTreeViewPreset> CreateSwapViewPreset(SMemAllocTableTreeView& TableTreeView, bool bIsInverted);
};

} // namespace UE::Insights::MemoryProfiler
