// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemAllocValueNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemAllocValueNode"

namespace UE::Insights::MemoryProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FMemAllocValueNode)

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
