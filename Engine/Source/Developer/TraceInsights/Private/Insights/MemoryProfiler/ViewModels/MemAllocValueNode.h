// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Templates/SharedPointer.h"

// TraceServices
#include "TraceServices/Model/Memory.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::MemoryProfiler
{
////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class used to store information about an allocation node (used in the SMemAllocTreeView).
 */
class FMemAllocValueNode : public FTableTreeNode
{
	INSIGHTS_DECLARE_RTTI(FMemAllocValueNode, FTableTreeNode)
public:
	/** Initialization constructor for the group node. */
	explicit FMemAllocValueNode(const FName InName, TWeakPtr<FTable> InParentTable)
		: FTableTreeNode(InName, InParentTable)
	{
	}

	virtual ~FMemAllocValueNode() = default;

	virtual int64 GetValueI64() const = 0;
};
////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
