// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemTagBudget.h"

#include "Math/UnitConversion.h"
#include "Misc/Optional.h"
#include "XmlParser.h"

// TraceInsights
#include "Insights/MemoryProfiler/MemoryProfilerManager.h"

namespace UE::Insights::MemoryProfiler
{

const FString FMemTagBudget::STR_Name(TEXT("name"));
const FString FMemTagBudget::STR_Id(TEXT("id"));
const FString FMemTagBudget::STR_Budget(TEXT("budget"));
const FString FMemTagBudget::STR_Set(TEXT("set"));
const FString FMemTagBudget::STR_Tag(TEXT("tag"));
const FString FMemTagBudget::STR_TagMemMax(TEXT("mem-max"));
const FString FMemTagBudget::STR_Tracker(TEXT("tracker"));
const FString FMemTagBudget::STR_Platform(TEXT("platform"));
const FString FMemTagBudget::STR_Grouping(TEXT("grouping"));
const FString FMemTagBudget::STR_Group(TEXT("group"));
const FString FMemTagBudget::STR_Include(TEXT("include"));
const FString FMemTagBudget::STR_Exclude(TEXT("exclude"));

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemTagBudgetGrouping
////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagBudgetGrouping::Reset()
{
	for (FMemTagBudgetGroup* Group : Groups)
	{
		delete Group;
	}
	Groups.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagBudgetGrouping::EnumerateGroups(TFunctionRef<void(const TCHAR* InCachedGroupName, const FMemTagBudgetGroup& InGroup)> Callback) const
{
	for (const FMemTagBudgetGroup* Group : Groups)
	{
		Callback(Group->GetName(), *Group);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FMemTagBudgetGroup* FMemTagBudgetGrouping::FindGroup(const TCHAR* InCachedGroupName) const
{
	for (const FMemTagBudgetGroup* Group : Groups)
	{
		if (Group->GetName() == InCachedGroupName)
		{
			return Group;
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemTagBudgetGroup& FMemTagBudgetGrouping::GetOrAddGroup(const TCHAR* InCachedGroupName)
{
	for (FMemTagBudgetGroup* Group : Groups)
	{
		if (Group->GetName() == InCachedGroupName)
		{
			return *Group;
		}
	}
	FMemTagBudgetGroup* NewGroup = new FMemTagBudgetGroup(InCachedGroupName);
	Groups.Add(NewGroup);
	return *NewGroup;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemTagBudgetTracker
////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagBudgetTracker::Reset()
{
	Values.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemTagBudgetTagSet
////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagBudgetTagSet::Reset()
{
	for (auto& KV : Trackers)
	{
		delete KV.Value;
	}
	Trackers.Reset();

	if (Grouping)
	{
		delete Grouping;
		Grouping = nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagBudgetTagSet::EnumerateTrackers(TFunctionRef<void(const TCHAR* InCachedTrackerName, const FMemTagBudgetTracker& InTracker)> Callback) const
{
	for (const auto& KV : Trackers)
	{
		Callback((const TCHAR*)KV.Key, *KV.Value);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FMemTagBudgetTracker* FMemTagBudgetTagSet::FindTracker(const TCHAR* InCachedTrackerName) const
{
	FMemTagBudgetTracker* const* Found = Trackers.Find(InCachedTrackerName);
	return Found ? *Found : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemTagBudgetTracker& FMemTagBudgetTagSet::GetOrAddTracker(const TCHAR* InCachedTrackerName)
{
	FMemTagBudgetTracker** Found = Trackers.Find(InCachedTrackerName);
	if (Found)
	{
		return **Found;
	}
	FMemTagBudgetTracker* Tracker = new FMemTagBudgetTracker(InCachedTrackerName, this);
	Trackers.Add(InCachedTrackerName, Tracker);
	return *Tracker;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemTagBudgetGrouping& FMemTagBudgetTagSet::GetOrCreateGrouping()
{
	if (!Grouping)
	{
		Grouping = new FMemTagBudgetGrouping();
	}
	return *Grouping;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemTagBudgetPlatform
////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagBudgetPlatform::Reset()
{
	for (auto& KV : TagSets)
	{
		delete KV.Value;
	}
	TagSets.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagBudgetPlatform::EnumerateTagSets(TFunctionRef<void(const TCHAR* InCachedTagSetName, const FMemTagBudgetTagSet& InTagSet)> Callback) const
{
	for (const auto& KV : TagSets)
	{
		Callback((const TCHAR*)KV.Key, *KV.Value);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FMemTagBudgetTagSet* FMemTagBudgetPlatform::FindTagSet(const TCHAR* InCachedTagSetName) const
{
	FMemTagBudgetTagSet* const* Found = TagSets.Find(InCachedTagSetName);
	return Found ? *Found : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemTagBudgetTagSet& FMemTagBudgetPlatform::GetOrAddTagSet(const TCHAR* InCachedTagSetName)
{
	FMemTagBudgetTagSet** Found = TagSets.Find(InCachedTagSetName);
	if (Found)
	{
		return **Found;
	}
	FMemTagBudgetTagSet* TagSet = new FMemTagBudgetTagSet(InCachedTagSetName);
	TagSets.Add(InCachedTagSetName, TagSet);
	return *TagSet;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemTagBudgetMode
////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagBudgetMode::Reset()
{
	DefaultPlatform.Reset();
	for (auto& KV : PlatformOverrides)
	{
		delete KV.Value;
	}
	PlatformOverrides.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagBudgetMode::EnumeratePlatforms(TFunctionRef<void(const TCHAR* InCachedPlatformName, const FMemTagBudgetPlatform& InPlatform)> Callback) const
{
	Callback(DefaultPlatform.GetName(), DefaultPlatform);

	for (const auto& KV : PlatformOverrides)
	{
		Callback((const TCHAR*)KV.Key, *KV.Value);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagBudgetMode::EnumeratePlatformOverrides(TFunctionRef<void(const TCHAR* InCachedPlatformName, const FMemTagBudgetPlatform& InPlatform)> Callback) const
{
	for (const auto& KV : PlatformOverrides)
	{
		Callback((const TCHAR*)KV.Key, *KV.Value);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FMemTagBudgetPlatform* FMemTagBudgetMode::FindPlatformOverride(const TCHAR* InCachedPlatformName) const
{
	FMemTagBudgetPlatform*const* Found = PlatformOverrides.Find(InCachedPlatformName);
	return Found ? *Found : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemTagBudgetPlatform& FMemTagBudgetMode::GetOrAddPlatformOverride(const TCHAR* InCachedPlatformName)
{
	FMemTagBudgetPlatform** Found = PlatformOverrides.Find(InCachedPlatformName);
	if (Found)
	{
		return **Found;
	}
	FMemTagBudgetPlatform* Platform = new FMemTagBudgetPlatform(InCachedPlatformName);
	PlatformOverrides.Add(InCachedPlatformName, Platform);
	return *Platform;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemTagBudget
////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagBudget::Reset()
{
	for (const auto& KV : Modes)
	{
		delete KV.Value;
	}
	Modes.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FMemTagBudget::FindString(const FStringView& InString) const
{
	return StringStore ? StringStore->Find(InString) : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FMemTagBudget::StoreString(const FStringView& InString)
{
	return StringStore ? StringStore->Store(InString) : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagBudget::EnumerateModes(TFunctionRef<void(const TCHAR* InCachedModeName, const FMemTagBudgetMode& InMode)> Callback) const
{
	for (const auto& KV : Modes)
	{
		Callback((const TCHAR*)KV.Key, *KV.Value);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FMemTagBudgetMode* FMemTagBudget::FindMode(const TCHAR* InCachedModeName) const
{
	FMemTagBudgetMode*const* Found = Modes.Find(InCachedModeName);
	return Found ? *Found : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemTagBudgetMode& FMemTagBudget::GetOrAddMode(const TCHAR* InCachedModeName)
{
	FMemTagBudgetMode** Found = Modes.Find(InCachedModeName);
	if (Found)
	{
		return **Found;
	}
	FMemTagBudgetMode* Mode = new FMemTagBudgetMode(InCachedModeName);
	Modes.Add(InCachedModeName, Mode);
	return *Mode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FMemTagBudgetMode* FMemTagBudget::FindMode(const FString& InModeName) const
{
	return FindMode(FindString(InModeName));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemTagBudgetMode& FMemTagBudget::GetOrAddMode(const FString& InModeName)
{
	return GetOrAddMode(StoreString(InModeName));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemTagBudget::LoadFromFile(const FString& InFilePath)
{
	Reset();
	FilePath = InFilePath;
	return ReLoadFromFile();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemTagBudget::ReLoadFromFile()
{
	DefaultCachedPlatformName = StoreString(FString(TEXT("Default")));

	DefaultCachedTrackerName = StoreString(FString(TEXT("Default")));
	PlatformCachedTrackerName = StoreString(FString(TEXT("Platform")));

	SystemCachedTagSetName = StoreString(FString(TEXT("Systems")));
	AssetCachedTagSetName = StoreString(FString(TEXT("Assets")));
	AssetClassCachedTagSetName = StoreString(FString(TEXT("AssetClasses")));

	FXmlFile XmlFile;
	if (!XmlFile.LoadFile(FilePath, EConstructMethod::ConstructFromFile))
	{
		UE_LOGF(LogMemoryProfiler, Warning, "Could not load memory budget file \"%ls\".", *FilePath);
		return false;
	}

	FXmlNode* RootNode = XmlFile.GetRootNode();
	if (RootNode == nullptr)
	{
		UE_LOGF(LogMemoryProfiler, Warning, "Could not parse memory budget file as an XML file; make sure it has only one root node.");
		return false;
	}

	const TArray<FXmlNode*>& ChildNodes = RootNode->GetChildrenNodes();
	for (FXmlNode* ChildNode : ChildNodes)
	{
		if (ChildNode->GetTag() == STR_Budget)
		{
			ProcessBudgetNode(ChildNode);
		}
		else
		{
			UE_LOGF(LogMemoryProfiler, Warning, "Ignoring unknown XML child node <%ls>.", *ChildNode->GetTag());
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemTagBudget::ProcessBudgetNode(FXmlNode* BudgetNode)
{
	FString ModeName = BudgetNode->GetAttribute(STR_Name);
	FMemTagBudgetMode& BudgetMode = GetOrAddMode(ModeName);

	BudgetMode.GetDefaultPlatform().SetName(DefaultCachedPlatformName);

	const TArray<FXmlNode*>& ChildNodes = BudgetNode->GetChildrenNodes();
	for (FXmlNode* ChildNode : ChildNodes)
	{
		if (ChildNode->GetTag() == STR_Set)
		{
			ProcessTagSetNode(ChildNode, BudgetMode.GetDefaultPlatform());
		}
		else if (ChildNode->GetTag() == STR_Platform)
		{
			ProcessPlatformNode(ChildNode, BudgetMode);
		}
		else
		{
			UE_LOGF(LogMemoryProfiler, Warning, "Ignoring unknown XML child node <%ls> in <%ls>.", *ChildNode->GetTag(), *STR_Budget);
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemTagBudget::ProcessPlatformNode(FXmlNode* PlatformNode, FMemTagBudgetMode& BudgetMode)
{
	FString PlatformName = PlatformNode->GetAttribute(STR_Name);
	const TCHAR* CachedPlatformName = StoreString(PlatformName);

	FMemTagBudgetPlatform& BudgetPlatform = BudgetMode.GetOrAddPlatformOverride(CachedPlatformName);

	const TArray<FXmlNode*>& ChildNodes = PlatformNode->GetChildrenNodes();
	for (FXmlNode* ChildNode : ChildNodes)
	{
		if (ChildNode->GetTag() == STR_Set)
		{
			ProcessTagSetNode(ChildNode, BudgetPlatform);
		}
		else
		{
			UE_LOGF(LogMemoryProfiler, Warning, "Ignoring unknown XML child node <%ls> in <%ls>.", *ChildNode->GetTag(), *STR_Platform);
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemTagBudget::ProcessTagSetNode(FXmlNode* TagSetNode, FMemTagBudgetPlatform& BudgetPlatform)
{
	FString TagSetName = TagSetNode->GetAttribute(STR_Name);
	const TCHAR* CachedTagSetName = nullptr;
	if (!TagSetName.IsEmpty())
	{
		if (!FCString::Stricmp(*TagSetName, TEXT("Asset")) ||
			!FCString::Stricmp(*TagSetName, TEXT("Assets")))
		{
			CachedTagSetName = AssetCachedTagSetName;
		}
		else
		if (!FCString::Stricmp(*TagSetName, TEXT("AssetClass")) ||
			!FCString::Stricmp(*TagSetName, TEXT("AssetClasses")))
		{
			CachedTagSetName = AssetClassCachedTagSetName;
		}
		else
		if (!FCString::Stricmp(*TagSetName, TEXT("System")) ||
			!FCString::Stricmp(*TagSetName, TEXT("Systems")))
		{
			CachedTagSetName = SystemCachedTagSetName;
		}
#if 0
		else
		{
			CachedTagSetName = StoreString(TagSetName);
			UE_LOGF(LogMemoryProfiler, Warning, "Unknown tag set \"%ls\".", *TagSetName);
		}
#endif
	}
	if (!CachedTagSetName)
	{
		UE_LOGF(LogMemoryProfiler, Warning, "Ignoring unknown tag set \"%ls\".", *TagSetName);
		return false;
	}

	FMemTagBudgetTagSet& BudgetTagSet = BudgetPlatform.GetOrAddTagSet(CachedTagSetName);

	const TArray<FXmlNode*>& ChildNodes = TagSetNode->GetChildrenNodes();
	for (FXmlNode* ChildNode : ChildNodes)
	{
		if (ChildNode->GetTag() == STR_Tracker)
		{
			ProcessTrackerNode(ChildNode, BudgetTagSet);
		}
		else if (ChildNode->GetTag() == STR_Grouping)
		{
			ProcessGroupingNode(ChildNode, BudgetTagSet);
		}
		else
		{
			UE_LOGF(LogMemoryProfiler, Warning, "Ignoring unknown XML child node <%ls> in <%ls>.", *ChildNode->GetTag(), *STR_Set);
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemTagBudget::ProcessTrackerNode(FXmlNode* TrackerNode, FMemTagBudgetTagSet& BudgetTagSet)
{
	FString TrackerName = TrackerNode->GetAttribute(STR_Name);
	const TCHAR* CachedTrackerName = nullptr;
	if (!TrackerName.IsEmpty())
	{
		if (!FCString::Stricmp(*TrackerName, DefaultCachedTrackerName))
		{
			CachedTrackerName = DefaultCachedTrackerName;
		}
		else if (!FCString::Stricmp(*TrackerName, PlatformCachedTrackerName))
		{
			CachedTrackerName = PlatformCachedTrackerName;
		}
#if 0
		else
		{
			CachedTrackerName = StoreString(TrackerName);
			UE_LOGF(LogMemoryProfiler, Warning, "Unknown tracker \"%ls\" in tag set \"%ls\".", *TrackerName, BudgetTagSet.GetName());
		}
#endif
	}
	if (!CachedTrackerName)
	{
		UE_LOGF(LogMemoryProfiler, Warning, "Ignoring unknown tracker \"%ls\" in tag set \"%ls\".", *TrackerName, BudgetTagSet.GetName());
		return false;
	}

	FMemTagBudgetTracker& BudgetTracker = BudgetTagSet.GetOrAddTracker(CachedTrackerName);

	const TArray<FXmlNode*>& ChildNodes = TrackerNode->GetChildrenNodes();
	for (FXmlNode* ChildNode : ChildNodes)
	{
		if (ChildNode->GetTag() == STR_Tag)
		{
			ProcessTagNode(ChildNode, BudgetTracker);
		}
		else
		{
			UE_LOGF(LogMemoryProfiler, Warning, "Ignoring unknown XML child node <%ls> in <%ls>.", *ChildNode->GetTag(), *STR_Tracker);
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemTagBudget::ProcessTagNode(FXmlNode* TagNode, FMemTagBudgetTracker& BudgetTracker)
{
	FString TagName = TagNode->GetAttribute(STR_Name);
	if (TagName.IsEmpty())
	{
		UE_LOGF(LogMemoryProfiler, Warning, "Ignoring budget value for empty tag.");
		return false;
	}
	const TCHAR* CachedTagName = StoreString(TagName);
	check(CachedTagName != nullptr);

	//////////////////////////////////////////////////
	// Value

	if (BudgetTracker.FindValue(CachedTagName) != nullptr)
	{
		UE_LOGF(LogMemoryProfiler, Warning, "Ignoring budget value for tag \"%ls\" (defined multiple times).", *TagName);
		return false;
	}

	int64 Value;
	if (!ReadMemValue(TagNode, STR_TagMemMax, Value))
	{
		UE_LOGF(LogMemoryProfiler, Warning, "Cannot parse value for tag \"%ls\".", *TagName);
		return false;
	}

	BudgetTracker.AddValue(CachedTagName, Value);
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemTagBudget::ProcessGroupingNode(FXmlNode* TrackerNode, FMemTagBudgetTagSet& BudgetTagSet)
{
	const TArray<FXmlNode*>& ChildNodes = TrackerNode->GetChildrenNodes();
	for (FXmlNode* ChildNode : ChildNodes)
	{
		if (ChildNode->GetTag() == STR_Group)
		{
			ProcessGroupNode(ChildNode, BudgetTagSet);
		}
		else
		{
			UE_LOGF(LogMemoryProfiler, Warning, "Ignoring unknown XML child node <%ls> in <%ls>.", *ChildNode->GetTag(), *STR_Grouping);
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemTagBudget::ProcessGroupNode(FXmlNode* GroupNode, FMemTagBudgetTagSet& BudgetTagSet)
{
	FString GroupName = GroupNode->GetAttribute(STR_Name);
	if (GroupName.IsEmpty())
	{
		UE_LOGF(LogMemoryProfiler, Warning, "Ignoring unnamed group.");
		return false;
	}
	const TCHAR* CachedGroupName = StoreString(GroupName);
	check(CachedGroupName != nullptr);

	FString Include = GroupNode->GetAttribute(STR_Include);
	FString Exclude = GroupNode->GetAttribute(STR_Exclude);

	int64 Value;
	if (!ReadMemValue(GroupNode, STR_TagMemMax, Value))
	{
		UE_LOGF(LogMemoryProfiler, Warning, "Cannot parse value for group \"%ls\".", *GroupName);
		return false;
	}

	FMemTagBudgetGrouping& Grouping = BudgetTagSet.GetOrCreateGrouping();
	FMemTagBudgetGroup& Group = Grouping.GetOrAddGroup(CachedGroupName);

	if (!Include.IsEmpty())
	{
		Group.SetInclude(Include);
	}

	if (!Exclude.IsEmpty())
	{
		Group.SetExclude(Exclude);
	}

	Group.SetMemMax(Value);
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS
bool FMemTagBudget::ReadMemValue(FXmlNode* TagNode, const FString& Attribute, int64& OutValue)
{
	FString Value = TagNode->GetAttribute(Attribute);

	TValueOrError<FNumericUnit<double>, FText> Result = FNumericUnit<double>::TryParseExpression(*Value, EUnit::Bytes, 0);
	if (!Result.IsValid())
	{
		UE_LOGF(LogMemoryProfiler, Warning, "Failed to parse value. %ls", *(Result.GetError().ToString()));
		return false;
	}

	TOptional<FNumericUnit<double>> MemoryValueInBytes = Result.GetValue().ConvertTo(EUnit::Bytes);
	if (!MemoryValueInBytes.IsSet())
	{
		UE_LOGF(LogMemoryProfiler, Warning, "Failed to parse value.");
		return false;
	}

	OutValue = (int64)MemoryValueInBytes.GetValue().Value;
	return true;
}
PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
