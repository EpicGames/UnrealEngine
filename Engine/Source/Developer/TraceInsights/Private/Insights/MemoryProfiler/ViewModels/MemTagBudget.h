// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceServices
#include "TraceServices/Containers/Allocators.h" // for IStringStore

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"

class FXmlNode;

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemTagBudgetGroup
{
public:
	FMemTagBudgetGroup(const TCHAR* InCachedGroupName)
		: CachedGroupName(InCachedGroupName)
	{
	}

	~FMemTagBudgetGroup() = default;

	const TCHAR* GetName() const { return CachedGroupName; }
	void SetName(const TCHAR* InCachedGroupName) { CachedGroupName = InCachedGroupName; }

	const FString& GetInclude() const { return Include; }
	void SetInclude(const FString& InInclude) { Include = InInclude; }

	const FString& GetExclude() const { return Exclude; }
	void SetExclude(const FString& InExclude) { Exclude = InExclude; }

	int64 GetMemMax() const { return MemMax; }
	void SetMemMax(int64 InMemMax) { MemMax = InMemMax; }

private:
	const TCHAR* CachedGroupName = nullptr;
	FString Include;
	FString Exclude;
	int64 MemMax = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemTagBudgetGrouping
{
public:
	FMemTagBudgetGrouping()
	{
	}
	~FMemTagBudgetGrouping()
	{
		Reset();
	}

	void Reset();

	int32 GetNumGroups() const { return Groups.Num(); }
	void EnumerateGroups(TFunctionRef<void(const TCHAR* InCachedGroupName, const FMemTagBudgetGroup& InGroup)> Callback) const;
	const FMemTagBudgetGroup* FindGroup(const TCHAR* InCachedGroupName) const;
	FMemTagBudgetGroup& GetOrAddGroup(const TCHAR* InCachedGroupName);

private:
	TArray<FMemTagBudgetGroup*> Groups;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemTagBudgetTagSet;

class FMemTagBudgetTracker
{
public:
	FMemTagBudgetTracker(const TCHAR* InCachedTrackerName, FMemTagBudgetTagSet* InParentTagSet)
		: CachedTrackerName(InCachedTrackerName)
		, ParentTagSet(InParentTagSet)
	{
	}
	~FMemTagBudgetTracker()
	{
		Reset();
	}

	void Reset();

	const TCHAR* GetName() const { return CachedTrackerName; }
	FMemTagBudgetTagSet* GetParentTagSet() const { return ParentTagSet; }

	const int64* FindValue(const TCHAR* InCachedTagName) const
	{
		return Values.Find(InCachedTagName);
	}

	void AddValue(const TCHAR* InCachedTagName, int64 Value)
	{
		Values.Add(InCachedTagName, Value);
	}

private:
	const TCHAR* CachedTrackerName = nullptr;
	FMemTagBudgetTagSet* ParentTagSet = nullptr;
	TMap<const void*, int64> Values;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemTagBudgetTagSet
{
public:
	FMemTagBudgetTagSet(const TCHAR* InCachedTagSetName)
		: CachedTagSetName(InCachedTagSetName)
	{
	}
	~FMemTagBudgetTagSet()
	{
		Reset();
	}

	void Reset();

	const TCHAR* GetName() const { return CachedTagSetName; }
	void SetName(const TCHAR* InCachedTagSetName) { CachedTagSetName = InCachedTagSetName; }

	void EnumerateTrackers(TFunctionRef<void(const TCHAR* InCachedTrackerName, const FMemTagBudgetTracker& InTracker)> Callback) const;
	const FMemTagBudgetTracker* FindTracker(const TCHAR* InCachedTrackerName) const;
	FMemTagBudgetTracker& GetOrAddTracker(const TCHAR* InCachedTrackerName);

	const FMemTagBudgetGrouping* GetGrouping() const { return Grouping; }
	FMemTagBudgetGrouping& GetOrCreateGrouping();

private:
	const TCHAR* CachedTagSetName = nullptr;
	TMap<const void*, FMemTagBudgetTracker*> Trackers;
	FMemTagBudgetGrouping* Grouping = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemTagBudgetPlatform
{
public:
	FMemTagBudgetPlatform(const TCHAR* InCachedPlatformName)
		: CachedPlatformName(InCachedPlatformName)
	{
	}

	~FMemTagBudgetPlatform()
	{
		Reset();
	}

	void Reset();

	const TCHAR* GetName() const { return CachedPlatformName; }
	void SetName(const TCHAR* InCachedPlatformName) { CachedPlatformName = InCachedPlatformName; }

	void EnumerateTagSets(TFunctionRef<void(const TCHAR* InCachedTagSetName, const FMemTagBudgetTagSet& InTagSet)> Callback) const;
	const FMemTagBudgetTagSet* FindTagSet(const TCHAR* InCachedTagSetName) const;
	FMemTagBudgetTagSet& GetOrAddTagSet(const TCHAR* InCachedTagSetName);

private:
	const TCHAR* CachedPlatformName = nullptr;
	TMap<const void*, FMemTagBudgetTagSet*> TagSets;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemTagBudgetMode
{
public:
	FMemTagBudgetMode(const TCHAR* InCachedModeName)
		: CachedModeName(InCachedModeName)
		, DefaultPlatform(nullptr)
	{
	}

	~FMemTagBudgetMode()
	{
		Reset();
	}

	void Reset();

	const TCHAR* GetName() const { return CachedModeName; }
	void SetName(const TCHAR* InCachedModeName) { CachedModeName = InCachedModeName; }

	void EnumeratePlatforms(TFunctionRef<void(const TCHAR* InCachedPlatformName, const FMemTagBudgetPlatform& InPlatform)> Callback) const;
	const FMemTagBudgetPlatform& GetDefaultPlatform() const { return DefaultPlatform; }
	void EnumeratePlatformOverrides(TFunctionRef<void(const TCHAR* InCachedPlatformName, const FMemTagBudgetPlatform& InPlatform)> Callback) const;
	const FMemTagBudgetPlatform* FindPlatformOverride(const TCHAR* InCachedPlatformName) const;
	FMemTagBudgetPlatform& GetDefaultPlatform() { return DefaultPlatform; }
	FMemTagBudgetPlatform& GetOrAddPlatformOverride(const TCHAR* InCachedPlatformName);

private:
	const TCHAR* CachedModeName = nullptr;
	FMemTagBudgetPlatform DefaultPlatform;
	TMap<const void*, FMemTagBudgetPlatform*> PlatformOverrides;
};
////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemTagBudget
{
public:
	FMemTagBudget(TraceServices::IStringStore* InStringStore)
		: StringStore(InStringStore)
	{
	}

	~FMemTagBudget()
	{
		Reset();
	}

	void Reset();

	const TCHAR* FindString(const FStringView& InString) const;
	const TCHAR* StoreString(const FStringView& InString);

	void EnumerateModes(TFunctionRef<void(const TCHAR* InCachedModeName, const FMemTagBudgetMode& InMode)> Callback) const;
	const FMemTagBudgetMode* FindMode(const TCHAR* InCachedModeName) const;
	FMemTagBudgetMode& GetOrAddMode(const TCHAR* InCachedModeName);

	const FMemTagBudgetMode* FindMode(const FString& InModeName) const;
	FMemTagBudgetMode& GetOrAddMode(const FString& InModeName);

	const FString& GetName() const { return Name; }
	void SetName(const FString& InName) { Name = InName; }

	const FString& GetFilePath() const { return FilePath; }
	bool LoadFromFile(const FString& InFilePath);
	bool ReLoadFromFile();

private:
	bool ProcessBudgetNode(FXmlNode* BudgetNode);
	bool ProcessPlatformNode(FXmlNode* PlatformNode, FMemTagBudgetMode& BudgetMode);
	bool ProcessTagSetNode(FXmlNode* TagSetNode, FMemTagBudgetPlatform& BudgetPlatform);
	bool ProcessTrackerNode(FXmlNode* TrackerNode, FMemTagBudgetTagSet& BudgetTagSet);
	bool ProcessTagNode(FXmlNode* TagNode, FMemTagBudgetTracker& BudgetTracker);
	bool ProcessGroupingNode(FXmlNode* TrackerNode, FMemTagBudgetTagSet& BudgetTagSet);
	bool ProcessGroupNode(FXmlNode* GroupNode, FMemTagBudgetTagSet& BudgetTagSet);
	bool ReadMemValue(FXmlNode* TagNode, const FString& Attribute, int64& OutValue);

private:
	TraceServices::IStringStore* StringStore = nullptr;
	FString Name;
	FString FilePath;
	TMap<const void*, FMemTagBudgetMode*> Modes;

	const TCHAR* DefaultCachedPlatformName;
	const TCHAR* DefaultCachedTrackerName;
	const TCHAR* PlatformCachedTrackerName;
	const TCHAR* SystemCachedTagSetName;
	const TCHAR* AssetCachedTagSetName;
	const TCHAR* AssetClassCachedTagSetName;

	static const FString STR_Name;
	static const FString STR_Id;
	static const FString STR_Budget;
	static const FString STR_Set;
	static const FString STR_Tag;
	static const FString STR_TagMemMax;
	static const FString STR_Tracker;
	static const FString STR_Platform;
	static const FString STR_Grouping;
	static const FString STR_Group;
	static const FString STR_Include;
	static const FString STR_Exclude;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
