// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemTagBudgetGrouping.h"

#include "Internationalization/Regex.h"

// TraceInsightsCore
#include "InsightsCore/Common/AsyncOperationProgress.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/MemoryProfiler/MemoryProfilerManager.h"
#include "Insights/MemoryProfiler/ViewModels/MemTagBudget.h"
#include "Insights/MemoryProfiler/ViewModels/MemTagNode.h"
#include "Insights/MemoryProfiler/Widgets/SMemTagTreeView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemTagNode"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FMemTagBudgetNodeGrouping)
INSIGHTS_IMPLEMENT_RTTI(FMemTagBudgetGroupNode)

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemTagBudgetNodeGrouping
////////////////////////////////////////////////////////////////////////////////////////////////////

FMemTagBudgetNodeGrouping::FMemTagBudgetNodeGrouping(TSharedPtr<SMemTagTreeView> InMemTagTreeView)
	: FTreeNodeGrouping(
		LOCTEXT("Grouping_Budget_ShortName", "Budget"),
		LOCTEXT("Grouping_Budget_TitleName", "Budget Groups"),
		LOCTEXT("Grouping_Budget_Desc", "Groups tags based on rules specified in the budget xml file."),
		nullptr)
	, MemTagTreeView(InMemTagTreeView.ToWeakPtr())
{
	SetColor(FLinearColor(0.75f, 0.5f, 1.0f, 1.0f));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagBudgetNodeGrouping::GroupNodes(
	const TArray<FTableTreeNodePtr>& Nodes,
	FTableTreeNode& ParentGroup,
	TWeakPtr<FTable> InParentTable,
	IAsyncOperationProgress& InAsyncOperationProgress) const
{
	ParentGroup.ClearChildren();

	class FGroupRule
	{
	public:
		FGroupRule(const TCHAR* InCachedGroupName, int64 InSizeBudget, const FString& InInclude, const FString& InExclude)
			: CachedGroupName(InCachedGroupName)
			, SizeBudget(InSizeBudget)
			, Include(InInclude)
			, Exclude(InExclude)
			, bHasExclude(!InExclude.IsEmpty())
		{
		}

		const TCHAR* GetGroupName() const { return CachedGroupName; }
		int64 GetSizeBudget() const { return SizeBudget; }
		void SetSizeBudget(int64 InSizeBudget) { SizeBudget = InSizeBudget; }

		bool Match(const FMemoryTag& MemTag)
		{
			FRegexMatcher IncludeMatcher(Include, MemTag.GetStatFullName());
			if (IncludeMatcher.FindNext())
			{
				FRegexMatcher ExcludeMatcher(Exclude, MemTag.GetStatFullName());
				if (!bHasExclude || !ExcludeMatcher.FindNext())
				{
					return true;
				}
			}
			return false;
		}

	private:
		const TCHAR* CachedGroupName = nullptr;
		int64 SizeBudget = 0;
		FRegexPattern Include;
		FRegexPattern Exclude;
		bool bHasExclude = false;
	};

	TArray<FGroupRule> GroupRules;

	const FMemTagBudgetGrouping* BudgetGrouping = nullptr;
	const FMemTagBudgetGrouping* BudgetGroupingOverride = nullptr;
	auto MemTagTreeViewPtr = MemTagTreeView.Pin();
	if (MemTagTreeViewPtr.IsValid())
	{
		MemTagTreeViewPtr->GetBudgetGrouping(BudgetGrouping, BudgetGroupingOverride);
	}
	if (BudgetGrouping)
	{
		GroupRules.Reserve(BudgetGrouping->GetNumGroups());
		BudgetGrouping->EnumerateGroups([&GroupRules](const TCHAR* InCachedGroupName, const FMemTagBudgetGroup& InGroup)
			{
				GroupRules.Emplace(InCachedGroupName, InGroup.GetMemMax(), InGroup.GetInclude(), InGroup.GetExclude());
			});
		if (BudgetGroupingOverride)
		{
			BudgetGroupingOverride->EnumerateGroups([&GroupRules](const TCHAR* InCachedGroupName, const FMemTagBudgetGroup& InGroup)
				{
					FGroupRule* GroupRule = GroupRules.FindByPredicate([InCachedGroupName](const FGroupRule& Rule) { return Rule.GetGroupName() == InCachedGroupName; });
					if (GroupRule)
					{
						GroupRule->SetSizeBudget(InGroup.GetMemMax());
					}
				});
		}
	}

	TMap<const void*, TSharedPtr<FMemTagBudgetGroupNode>> Groups;
	TSharedPtr<FTableTreeNode> UngroupedTagsGroupNode;

	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			return;
		}

		if (NodePtr->IsGroup())
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		if (NodePtr->Is<FMemTagNode>())
		{
			FMemTagNode& MemTagNode = NodePtr->As<FMemTagNode>();
			check(MemTagNode.GetMemTag() != nullptr);
			FMemoryTag& MemTag = *MemTagNode.GetMemTag();

			const TCHAR* CachedGroupName = nullptr;
			int64 SizeBudget = 0;
			for (FGroupRule& Rule : GroupRules)
			{
				if (Rule.Match(MemTag))
				{
					CachedGroupName = Rule.GetGroupName();
					SizeBudget = Rule.GetSizeBudget();
					break;
				}
			}
			if (CachedGroupName)
			{
				TSharedPtr<FMemTagBudgetGroupNode> GroupNode = Groups.FindRef(CachedGroupName);
				if (!GroupNode)
				{
					FName GroupName(CachedGroupName);
					GroupNode = MakeShared<FMemTagBudgetGroupNode>(GroupName, InParentTable, CachedGroupName);
					GroupNode->SetSizeBudget(SizeBudget);
					ParentGroup.AddChildAndSetParent(GroupNode);
					Groups.Add(CachedGroupName, GroupNode);
				}
				GroupNode->AddChildAndSetParent(NodePtr);
				continue;
			}
		}

		if (!UngroupedTagsGroupNode)
		{
			static FName UngroupedTagsGroupName("Ungrouped");
			UngroupedTagsGroupNode = MakeShared<FTableTreeNode>(UngroupedTagsGroupName, InParentTable);
			ParentGroup.AddChildAndSetParent(UngroupedTagsGroupNode);
		}
		UngroupedTagsGroupNode->AddChildAndSetParent(NodePtr);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
