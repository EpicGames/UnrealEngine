// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"

// TraceInsightsCore
#include "InsightsCore/Common/SimpleRtti.h"
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

namespace UE::Insights::MemoryProfiler
{

class SMemTagTreeView;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemTagBudgetNodeGrouping : public FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FMemTagBudgetNodeGrouping, FTreeNodeGrouping);

public:
	FMemTagBudgetNodeGrouping(TSharedPtr<SMemTagTreeView> InMemTagTreeView);

	virtual void GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const override;

private:
	TWeakPtr<SMemTagTreeView> MemTagTreeView;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemTagBudgetGroupNode : public FTableTreeNode
{
	INSIGHTS_DECLARE_RTTI(FMemTagBudgetGroupNode, FTableTreeNode)

public:
	/** Initialization constructor for a table record node. */
	explicit FMemTagBudgetGroupNode(const FName InName, TWeakPtr<FTable> InParentTable, int32 InRowIndex, const TCHAR* InCachedBudgetGroupName, bool IsGroup)
		: FTableTreeNode(InName, InParentTable, InRowIndex, IsGroup)
		, CachedBudgetGroupName(InCachedBudgetGroupName)
	{
	}

	/** Initialization constructor for the group node. */
	explicit FMemTagBudgetGroupNode(const FName InName, TWeakPtr<FTable> InParentTable, const TCHAR* InCachedBudgetGroupName)
		: FTableTreeNode(InName, InParentTable)
		, CachedBudgetGroupName(InCachedBudgetGroupName)
	{
	}

	virtual ~FMemTagBudgetGroupNode() = default;

	virtual FLinearColor GetIconColor() const override
	{
		return FLinearColor(0.75f, 0.5f, 1.0f, 1.0f);
	}

	virtual FLinearColor GetColor() const override
	{
		return FLinearColor(0.75f, 0.5f, 1.0f, 1.0f);
	}

	const TCHAR* GetBudgetGroupName() const
	{
		return CachedBudgetGroupName;
	}

	bool HasSizeBudget() const
	{
		return SizeBudget != 0;
	}

	void ResetSizeBudget()
	{
		SizeBudget = 0;
	}

	int64 GetSizeBudget() const
	{
		return SizeBudget;
	}

	void SetSizeBudget(int64 InValue)
	{
		SizeBudget = InValue;
	}

private:
	const TCHAR* CachedBudgetGroupName = nullptr;
	int64 SizeBudget = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
