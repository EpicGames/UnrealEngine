// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemTagNode.h"

#include "HAL/UnrealMemory.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/MemoryProfiler/MemoryProfilerManager.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemTagNode"

namespace UE::Insights::MemoryProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FMemTagNode)
INSIGHTS_IMPLEMENT_RTTI(FSystemMemTagNode)
INSIGHTS_IMPLEMENT_RTTI(FAssetMemTagNode)
INSIGHTS_IMPLEMENT_RTTI(FClassMemTagNode)

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMemTagNode::GetTagText() const
{
	if (MemTag)
	{
		return FText::FromString(MemTag->GetStatFullName());
	}

	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FMemTagNode::GetTagName() const
{
	if (MemTag)
	{
		return *MemTag->GetStatFullName();
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMemTagNode::GetTrackerText() const
{
	FMemorySharedState* SharedState = FMemoryProfilerManager::Get()->GetSharedState();
	if (SharedState)
	{
		FMemoryTrackerId TrackerId = GetMemTrackerId();
		const FMemoryTracker* Tracker = SharedState->GetTrackerById(TrackerId);
		if (Tracker)
		{
			return FText::FromString(Tracker->GetName());
		}
	}

	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FMemTagNode::GetTrackerName() const
{
	FMemorySharedState* SharedState = FMemoryProfilerManager::Get()->GetSharedState();
	if (SharedState)
	{
		FMemoryTrackerId TrackerId = GetMemTrackerId();
		const FMemoryTracker* Tracker = SharedState->GetTrackerById(TrackerId);
		if (Tracker)
		{
			return *Tracker->GetName();
		}
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMemTagNode::GetTagSetText() const
{
	FMemorySharedState* SharedState = FMemoryProfilerManager::Get()->GetSharedState();
	if (SharedState)
	{
		FMemoryTagSetId TagSetId = GetMemTagSetId();
		const FMemoryTagSet* TagSet = SharedState->GetTagSetById(TagSetId);
		if (TagSet)
		{
			return FText::FromString(TagSet->GetName());
		}
	}

	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FMemTagNode::GetTagSetName() const
{
	FMemorySharedState* SharedState = FMemoryProfilerManager::Get()->GetSharedState();
	if (SharedState)
	{
		FMemoryTagSetId TagSetId = GetMemTagSetId();
		const FMemoryTagSet* TagSet = SharedState->GetTagSetById(TagSetId);
		if (TagSet)
		{
			return *TagSet->GetName();
		}
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FMemTagNode::GetDisplayName() const
{
	return GetTagText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FMemTagNode::GetExtraDisplayName() const
{
	FMemorySharedState* SharedState = FMemoryProfilerManager::Get()->GetSharedState();
	if (SharedState)
	{
		if (MemTag && MemTag->GetTrackerId() != SharedState->GetDefaultTrackerId())
		{
			return FText::Format(LOCTEXT("MemTagNodeExtraDisplayNameFmt", "({0})"), GetTrackerText());
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemTagNode::HasExtraDisplayName() const
{
	FMemorySharedState* SharedState = FMemoryProfilerManager::Get()->GetSharedState();
	if (SharedState)
	{
		if (MemTag && MemTag->GetTrackerId() != SharedState->GetDefaultTrackerId())
		{
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FMemTagNode::GetTooltipText() const
{
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* FMemTagNode::GetIcon() const
{
	if (MemTag && MemTag->IsAddedToGraph())
	{
		return FInsightsStyle::GetBrush("Icons.HasGraph.TreeItem");
	}
	else
	{
		return FInsightsStyle::GetBrush("Icons.MemTag.TreeItem");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FMemTagNode::GetIconColor() const
{
	if (MemTag && MemTag->IsAddedToGraph())
	{
		return MemTag->GetColor();
	}
	else
	{
		return GetDefaultColor(IsGroup());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FMemTagNode::GetColor() const
{
	return GetDefaultColor(IsGroup());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagNode::ResetAggregatedStats()
{
	FMemory::Memzero<FMemTagStats>(Stats);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* FSystemMemTagNode::GetIcon() const
{
	if (GetMemTag() && GetMemTag()->IsAddedToGraph())
	{
		return FInsightsStyle::GetBrush("Icons.HasGraph.TreeItem");
	}
	else
	{
		return FInsightsStyle::GetBrush("Icons.SystemMemTag.TreeItem");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* FAssetMemTagNode::GetIcon() const
{
	if (GetMemTag() && GetMemTag()->IsAddedToGraph())
	{
		return FInsightsStyle::GetBrush("Icons.HasGraph.TreeItem");
	}
	else
	{
		return FInsightsStyle::GetBrush("Icons.AssetMemTag.TreeItem");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* FClassMemTagNode::GetIcon() const
{
	if (GetMemTag() && GetMemTag()->IsAddedToGraph())
	{
		return FInsightsStyle::GetBrush("Icons.HasGraph.TreeItem");
	}
	else
	{
		return FInsightsStyle::GetBrush("Icons.ClassMemTag.TreeItem");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
