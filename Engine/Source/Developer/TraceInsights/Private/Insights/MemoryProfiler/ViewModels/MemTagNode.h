// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// TraceServices
#include "TraceServices/Model/Memory.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

// TraceInsights
#include "Insights/MemoryProfiler/ViewModels/MemorySharedState.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryTag.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryTracker.h"
#include "Insights/MemoryProfiler/ViewModels/MemTagTable.h"

namespace TraceServices
{
	struct FMemoryProfilerAggregatedStats
	{
		uint32 Type;
		uint32 InstanceCount = 0U;
		uint64 Min = 0U;
		uint64 Max = 0U;
		uint64 Average = 0U;
		//uint64 Median = 0U;
	};
}

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FMemTagStats
{
	int64 SizeA = 0; // memory size, in [bytes], for current LLM tag, at time marker A
	int64 SizeB = 0; // memory size, in [bytes], for current LLM tag, at time marker B
	int64 SizeBudget = 0;
	int64 SampleCount = 0;
	int64 SizeMin = 0;
	int64 SizeMax = 0;
	int64 SizeAverage = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class used to store information about an LLM tag node (used in the SMemTagTreeView).
 */
class FMemTagNode : public FTableTreeNode
{
	INSIGHTS_DECLARE_RTTI(FMemTagNode, FTableTreeNode)

public:
	/** Initialization constructor for the MemTag node. */
	explicit FMemTagNode(TWeakPtr<FMemTagTable> InParentTable, FMemoryTag& InMemTag)
		: FTableTreeNode(FName("tag", InMemTag.GetIndex() + 1), InParentTable, InMemTag.GetIndex())
		, MemTag(&InMemTag)
	{
	}

	/** Initialization constructor for the group node. */
	explicit FMemTagNode(TWeakPtr<FMemTagTable> InParentTable, const FName InGroupName)
		: FTableTreeNode(InGroupName, InParentTable)
		, MemTag(nullptr)
	{
	}

	bool IsValidMemTag() const { return MemTag != nullptr; }
	FMemoryTag* GetMemTag() const { return MemTag; }

	FMemoryTagId GetMemTagId() const { return MemTag ? MemTag->GetId() : FMemoryTag::InvalidTagId; }
	FText GetTagText() const;
	const TCHAR* GetTagName() const;

	FMemoryTrackerId GetMemTrackerId() const { return MemTag ? MemTag->GetTrackerId() : FMemoryTracker::InvalidTrackerId; }
	FText GetTrackerText() const;
	const TCHAR* GetTrackerName() const;

	FMemoryTagSetId GetMemTagSetId() const { return MemTag ? MemTag->GetTagSetId() : FMemoryTagSet::InvalidTagSetId; }
	FText GetTagSetText() const;
	const TCHAR* GetTagSetName() const;

	virtual const FText GetDisplayName() const override;
	virtual const FText GetExtraDisplayName() const override;
	virtual bool HasExtraDisplayName() const override;
	virtual const FText GetTooltipText() const override;

	bool IsAddedToGraph() const { return MemTag && MemTag->IsAddedToGraph(); }

	virtual const FSlateBrush* GetIcon() const override;
	virtual FLinearColor GetIconColor() const override;
	virtual FLinearColor GetColor() const override;

	const FMemTagStats& GetStats() const { return Stats; }
	FMemTagStats& GetStats() { return Stats; }

	int64 GetSizeA() const { return Stats.SizeA; }
	int64 GetSizeB() const { return Stats.SizeB; }
	int64 GetSizeDiff() const { return Stats.SizeB - Stats.SizeA; }
	int64 GetSizeBudget() const { return Stats.SizeBudget; }
	int64 GetSampleCount() const { return Stats.SampleCount; }
	int64 GetSizeMin() const { return Stats.SizeMin; }
	int64 GetSizeMax() const { return Stats.SizeMax; }
	int64 GetSizeAverage() const { return Stats.SizeAverage; }

	bool HasSizeBudget() const { return Stats.SizeBudget != 0; }
	void ResetSizeBudget() { Stats.SizeBudget = 0; }
	void SetSizeBudget(int64 InSizeBudget) { Stats.SizeBudget = InSizeBudget; }
	void ResetAggregatedStats();

private:
	FMemoryTag* MemTag = nullptr;
	FMemTagStats Stats;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSystemMemTagNode : public FMemTagNode
{
	INSIGHTS_DECLARE_RTTI(FSystemMemTagNode, FMemTagNode)

public:
	/** Initialization constructor for the Asset MemTag node. */
	explicit FSystemMemTagNode(TWeakPtr<FMemTagTable> InParentTable, FMemoryTag& InMemTag)
		: FMemTagNode(InParentTable, InMemTag)
	{
	}

	virtual const FSlateBrush* GetIcon() const override;

	TSharedPtr<FMemTagNode> GetParentTagNode() const { return ParentTagNode; }
	FMemoryTag* GetParentMemTag() const { return ParentTagNode.IsValid() ? ParentTagNode->GetMemTag() : nullptr; }
	void SetParentTagNode(TSharedPtr<FMemTagNode> NodePtr) { ParentTagNode = NodePtr; }

private:
	TSharedPtr<FMemTagNode> ParentTagNode = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FAssetMemTagNode : public FMemTagNode
{
	INSIGHTS_DECLARE_RTTI(FAssetMemTagNode, FMemTagNode)

public:
	/** Initialization constructor for the Asset MemTag node. */
	explicit FAssetMemTagNode(TWeakPtr<FMemTagTable> InParentTable, FMemoryTag& InMemTag)
		: FMemTagNode(InParentTable, InMemTag)
	{
	}

	virtual const FSlateBrush* GetIcon() const override;

	int32 GetObjectSerial() const { return ObjectSerial; }
	void SetObjectSerial(int32 InObjectSerial) { ObjectSerial = InObjectSerial; }

	int64 GetInclusiveSizeA() const { return InclusiveSizeA; }
	void SetInclusiveSizeA(int64 Size) { InclusiveSizeA = Size; }

	int64 GetInclusiveSizeB() const { return InclusiveSizeB; }
	void SetInclusiveSizeB(int64 Size) { InclusiveSizeB = Size; }

private:
	int32 ObjectSerial = 0;
	int64 InclusiveSizeA = 0;
	int64 InclusiveSizeB = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FClassMemTagNode : public FMemTagNode
{
	INSIGHTS_DECLARE_RTTI(FClassMemTagNode, FMemTagNode)

public:
	/** Initialization constructor for the Asset MemTag node. */
	explicit FClassMemTagNode(TWeakPtr<FMemTagTable> InParentTable, FMemoryTag& InMemTag)
		: FMemTagNode(InParentTable, InMemTag)
	{
	}

	virtual const FSlateBrush* GetIcon() const override;

	void SetClassSerial(int32 InClassSerial) { ClassSerial = InClassSerial; }
	int32 GetClassSerial() const { return ClassSerial; }

private:
	int32 ClassSerial = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
