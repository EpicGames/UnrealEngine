// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemTagNodeGroupingAndSorting.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableColumn.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemTagNode"

#define INSIGHTS_ENSURE ensure
//#define INSIGHTS_ENSURE(...)

// Default pre-sorting (group nodes sorts above leaf nodes)
#define INSIGHTS_DEFAULT_PRESORTING_NODES(A, B) \
	{ \
		if (ShouldCancelSort()) \
		{ \
			return CancelSort(); \
		} \
		if (A->IsGroup() != B->IsGroup()) \
		{ \
			return A->IsGroup(); \
		} \
	}

// Sort by name (ascending).
#define INSIGHTS_DEFAULT_SORTING_NODES(A, B) return A->GetName().LexicalLess(B->GetName());
//#define INSIGHTS_DEFAULT_SORTING_NODES(A, B) return A->GetDefaultSortOrder() < B->GetDefaultSortOrder();

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorting by Tracker(s)
////////////////////////////////////////////////////////////////////////////////////////////////////

FMemTagNodeSortingByTracker::FMemTagNodeSortingByTracker(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByTracker")),
		LOCTEXT("Sorting_ByTracker_Name", "By Tracker"),
		LOCTEXT("Sorting_ByTracker_Title", "Sort By Tracker"),
		LOCTEXT("Sorting_ByTracker_Desc", "Sort by memory tracker."),
		InColumnRef)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagNodeSortingByTracker::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		NodesToSort.Sort([this](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_DEFAULT_PRESORTING_NODES(A, B);

			INSIGHTS_ENSURE(A.IsValid() && A->Is<FMemTagNode>());
			const TSharedPtr<FMemTagNode> MemTagNodeA = StaticCastSharedPtr<FMemTagNode, FBaseTreeNode>(A);

			INSIGHTS_ENSURE(B.IsValid() && B->Is<FMemTagNode>());
			const TSharedPtr<FMemTagNode> MemTagNodeB = StaticCastSharedPtr<FMemTagNode, FBaseTreeNode>(B);

			if (MemTagNodeA->GetMemTrackerId() == MemTagNodeB->GetMemTrackerId())
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by trackers (ascending).
				return int32(MemTagNodeA->GetMemTrackerId()) < int32(MemTagNodeB->GetMemTrackerId());
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		NodesToSort.Sort([this](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_DEFAULT_PRESORTING_NODES(A, B);

			INSIGHTS_ENSURE(A.IsValid() && A->Is<FMemTagNode>());
			const TSharedPtr<FMemTagNode> MemTagNodeA = StaticCastSharedPtr<FMemTagNode, FBaseTreeNode>(A);

			INSIGHTS_ENSURE(B.IsValid() && B->Is<FMemTagNode>());
			const TSharedPtr<FMemTagNode> MemTagNodeB = StaticCastSharedPtr<FMemTagNode, FBaseTreeNode>(B);

			if (MemTagNodeA->GetMemTrackerId() == MemTagNodeB->GetMemTrackerId())
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by trackers (descending).
				return int32(MemTagNodeB->GetMemTrackerId()) < int32(MemTagNodeA->GetMemTrackerId());
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef INSIGHTS_DEFAULT_SORTING_NODES
#undef INSIGHTS_DEFAULT_PRESORTING_NODES
#undef INSIGHTS_ENSURE
#undef LOCTEXT_NAMESPACE
