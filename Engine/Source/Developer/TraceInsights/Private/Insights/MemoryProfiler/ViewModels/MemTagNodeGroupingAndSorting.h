// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

// TraceInsights
#include "Insights/MemoryProfiler/ViewModels/MemTagNode.h"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemTagNodeSortingByTracker : public FTableCellValueSorter
{
public:
	FMemTagNodeSortingByTracker(TSharedRef<FTableColumn> InColumnRef);

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
