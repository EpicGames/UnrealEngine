// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemTagTable.h"

#include "Styling/StyleColors.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/MemoryProfiler/ViewModels/MemTagBudgetGrouping.h"
#include "Insights/MemoryProfiler/ViewModels/MemTagNode.h"
#include "Insights/MemoryProfiler/ViewModels/MemTagNodeGroupingAndSorting.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemTagTable"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Column identifiers

const FName FMemTagTableColumns::TagNameColumnId("TagName");
const FName FMemTagTableColumns::TagIdColumnId("TagId");
const FName FMemTagTableColumns::SizeAColumnId("SizeA");
const FName FMemTagTableColumns::SizeBColumnId("SizeB");
const FName FMemTagTableColumns::SizeDiffColumnId("SizeDiff");
const FName FMemTagTableColumns::SampleCountColumnId("SampleCount");
const FName FMemTagTableColumns::SizeMinColumnId("SizeMin");
const FName FMemTagTableColumns::SizeMaxColumnId("SizeMax");
const FName FMemTagTableColumns::SizeAverageColumnId("SizeAverage");
const FName FMemTagTableColumns::SizeBudgetColumnId("SizeBudget");
const FName FMemTagTableColumns::TrackerColumnId("Tracker");
const FName FMemTagTableColumns::TagSetColumnId("TagSet");

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemTagTable
////////////////////////////////////////////////////////////////////////////////////////////////////

FMemTagTable::FMemTagTable()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemTagTable::~FMemTagTable()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagTable::Reset()
{
	FTable::Reset();

	AddDefaultColumns();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemTagTable::AddDefaultColumns()
{
	class FAggregatedValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const
		{
			return TOptional<FTableCellValue>();
		}

		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			if (Node.IsGroup() && Node.Is<FTableTreeNode>())
			{
				const FTableTreeNode& TableTreeNode = Node.As<FTableTreeNode>();
				if (TableTreeNode.HasAggregatedValue(Column.GetId()))
				{
					return TableTreeNode.GetAggregatedValue(Column.GetId());
				}
			}
			return GetNodeValue(Node);
		}
	};

	class FBudgetedMemoryValueFormatter : public FInt64ValueFormatterAsMemory
	{
	public:
		FBudgetedMemoryValueFormatter()
		{
			GetFormattingOptions().MaximumFractionalDigits = 2;
		}
		virtual ~FBudgetedMemoryValueFormatter() {}

		virtual TSharedPtr<SWidget> GenerateCustomWidget(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			return SNew(SBox)
				.ToolTip(GetCustomTooltip(Column, Node))
				.HAlign(Column.GetHorizontalAlignment())
				.VAlign(VAlign_Center)
				.Padding(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text_Lambda([this, &Column, &Node]()
						{
							return FormatValue(Column.GetValue(Node));
						})
					.ColorAndOpacity_Lambda([&Column, &Node]
						{
							if (Node.Is<FMemTagNode>())
							{
								const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
								if (MemTagNode.HasSizeBudget())
								{
									TOptional<FTableCellValue> Value = Column.GetValue(Node);
									if (Value.IsSet())
									{
										int64 SizeValue = Value.GetValue().AsInt64();
										if (SizeValue > MemTagNode.GetSizeBudget())
										{
											return FLinearColor(1.0f, 0.3f, 0.3f, 1.0f);
										}
										else
										{
											return FLinearColor(0.1f, 0.5f, 0.1f, 1.0f);
										}
									}
								}
								else
								{
									return FLinearColor(0.5f, 0.5f, 0.5f, 1.0f);
								}
							}
							else if (Node.Is<FMemTagBudgetGroupNode>())
							{
								const FMemTagBudgetGroupNode& GroupNode = Node.As<FMemTagBudgetGroupNode>();
								if (GroupNode.HasSizeBudget())
								{
									if (GroupNode.HasAggregatedValue(Column.GetId()))
									{
										int64 SizeValue = GroupNode.GetAggregatedValue(Column.GetId()).AsInt64();
										if (SizeValue > GroupNode.GetSizeBudget())
										{
											return FLinearColor(1.0f, 0.3f, 0.3f, 1.0f);
										}
										else
										{
											return FLinearColor(0.1f, 0.5f, 0.1f, 1.0f);
										}
									}
								}
							}
							return FLinearColor(1.0f, 0.7f, 0.3f, 1.0f);
						})
				];
		}
	};

	//////////////////////////////////////////////////
	// Hierarchy Column
	{
		const int32 HierarchyColumnIndex = -1;
		const TCHAR* HierarchyColumnName = nullptr;
		AddHierarchyColumn(HierarchyColumnIndex, HierarchyColumnName);

		FTableColumn& HierarchyColumn = *GetColumns()[0];
		HierarchyColumn.SetInitialWidth(200.0f);
		HierarchyColumn.SetShortName(LOCTEXT("HierarchyColumnName", "Hierarchy"));
		HierarchyColumn.SetTitleName(LOCTEXT("HierarchyColumnTitle", "LLM Tag Hierarchy"));
		HierarchyColumn.SetDescription(LOCTEXT("HierarchyColumnDesc", "Hierarchy of the LLM tag's tree"));
	}

	int32 ColumnIndex = 0;

	//////////////////////////////////////////////////
	// Tag Name Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemTagTableColumns::TagNameColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TagNameColumnName", "Name"));
		Column.SetTitleName(LOCTEXT("TagNameColumnTitle", "Tag Name"));
		Column.SetDescription(LOCTEXT("TagNameColumnDesc", "The name of the LLM tag"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(120.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FMemTagNameValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemTagNode>())
				{
					const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
					return FTableCellValue(MemTagNode.GetTagName());
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemTagNameValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);

		// Set the same "by name" sorter also for the hierarchy column.
		FTableColumn& HierarchyColumn = *GetColumns()[0];
		HierarchyColumn.SetValueSorter(Sorter);
	}
	//////////////////////////////////////////////////
	// Tag Id Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemTagTableColumns::TagIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TagIdColumnName", "Tag Id"));
		Column.SetTitleName(LOCTEXT("TagIdColumnTitle", "Tag Id"));
		Column.SetDescription(LOCTEXT("TagIdColumnDesc", "The id of the LLM tag"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(120.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemTagIdValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemTagNode>())
				{
					const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
					return FTableCellValue(int64(MemTagNode.GetMemTagId()));
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemTagIdValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsHex64>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		//Column.SetAggregation(ETableColumnAggregation::None);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Size (Time A) Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemTagTableColumns::SizeAColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SizeAColumnName", "Size A"));
		Column.SetTitleName(LOCTEXT("SizeAColumnTitle", "Size at TimeMarker A"));
		Column.SetDescription(LOCTEXT("SizeAColumnDesc", "The memory size (bytes) of the LLM tag at the time marker A"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
						ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered |
						ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemTagSizeAValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemTagNode>())
				{
					const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
					return FTableCellValue(MemTagNode.GetSizeA());
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemTagSizeAValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FBudgetedMemoryValueFormatter>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Size (Time B) Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemTagTableColumns::SizeBColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SizeBColumnName", "Size B"));
		Column.SetTitleName(LOCTEXT("SizeBColumnTitle", "Size at TimeMarker B"));
		Column.SetDescription(LOCTEXT("SizeBColumnDesc", "The memory size (bytes) of the LLM tag at the time marker B"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered |
						ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemTagSizeAValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemTagNode>())
				{
					const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
					return FTableCellValue(MemTagNode.GetSizeB());
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemTagSizeAValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FBudgetedMemoryValueFormatter>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Size Difference (B - A) Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemTagTableColumns::SizeDiffColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SizeDiffColumnName", "Diff (B - A)"));
		Column.SetTitleName(LOCTEXT("SizeDiffColumnTitle", "Size Difference (B - A)"));
		Column.SetDescription(LOCTEXT("SizeDiffColumnDesc", "The memory size variation (in bytes) of the LLM tag between time marker B and time marker A"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered |
						ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemTagSizeDiffValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemTagNode>())
				{
					const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
					return FTableCellValue(MemTagNode.GetSizeDiff());
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemTagSizeDiffValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<FInt64ValueFormatterAsMemory> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Formatter->GetFormattingOptions().MaximumFractionalDigits = 2;
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Instance Count Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemTagTableColumns::SampleCountColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SampleCountColumnName", "Samples"));
		Column.SetTitleName(LOCTEXT("SampleCountColumnTitle", "Sample Count"));
		Column.SetDescription(LOCTEXT("SampleCountColumnDesc", "The number of snapshots the LLM tag has in the selected time range"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered |
						ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemTagSampleCountValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemTagNode>())
				{
					const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
					return FTableCellValue(MemTagNode.GetSampleCount());
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemTagSampleCountValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		//Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Size Min Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemTagTableColumns::SizeMinColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SizeMinColumnName", "Min"));
		Column.SetTitleName(LOCTEXT("SizeMinColumnTitle", "Min Size"));
		Column.SetDescription(LOCTEXT("SizeMinColumnDesc", "The minimum size value (in bytes) the LLM tag has in the selected time range"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered |
						ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemTagSizeMinValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemTagNode>())
				{
					const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
					return FTableCellValue(MemTagNode.GetSizeMin());
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemTagSizeMinValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FBudgetedMemoryValueFormatter>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Min);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Size Max Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemTagTableColumns::SizeMaxColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SizeMaxColumnName", "Max"));
		Column.SetTitleName(LOCTEXT("SizeMaxColumnTitle", "Max Size"));
		Column.SetDescription(LOCTEXT("SizeMaxColumnDesc", "The maximum size value (in bytes) the LLM tag has in the selected time range"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered |
						ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemTagSizeMaxValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemTagNode>())
				{
					const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
					return FTableCellValue(MemTagNode.GetSizeMax());
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemTagSizeMaxValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FBudgetedMemoryValueFormatter>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Max);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Size Average Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemTagTableColumns::SizeAverageColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SizeAverageColumnName", "Avg."));
		Column.SetTitleName(LOCTEXT("SizeAverageColumnTitle", "Average Size"));
		Column.SetDescription(LOCTEXT("SizeAverageColumnDesc", "The average size value (in bytes) the LLM tag has in the selected time range"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered |
						ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemTagSizeAverageValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemTagNode>())
				{
					const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
					return FTableCellValue(MemTagNode.GetSizeAverage());
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemTagSizeAverageValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FBudgetedMemoryValueFormatter>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		//Column.SetAggregation(ETableColumnAggregation::Average);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Size Budget Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemTagTableColumns::SizeBudgetColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("BudgetColumnName", "Budget"));
		Column.SetTitleName(LOCTEXT("BudgetColumnTitle", "Budget"));
		Column.SetDescription(LOCTEXT("BudgetColumnDesc", "The budget (max size value, in bytes) of the LLM tag"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
						ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered |
						ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FMemTagBudgetValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FMemTagNode>())
				{
					const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
					if (MemTagNode.HasSizeBudget())
					{
						return FTableCellValue(MemTagNode.GetSizeBudget());
					}
				}
				else if (Node.Is<FMemTagBudgetGroupNode>())
				{
					const FMemTagBudgetGroupNode& GroupNode = Node.As<FMemTagBudgetGroupNode>();
					if (GroupNode.HasSizeBudget())
					{
						return FTableCellValue(GroupNode.GetSizeBudget());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemTagBudgetValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<FInt64ValueFormatterAsMemory> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Formatter->GetFormattingOptions().MaximumFractionalDigits = 2;
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		//Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// LLM Tracker Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemTagTableColumns::TrackerColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TrackerColumnName", "Tracker"));
		Column.SetTitleName(LOCTEXT("TrackerColumnTitle", "Tracker"));
		Column.SetDescription(LOCTEXT("TrackerColumnDesc", "The LLM tracker using the LLM tag"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Text);

		class FMemTagTrackerValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const
			{
				if (Node.Is<FMemTagNode>())
				{
					const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
					return FTableCellValue(MemTagNode.GetTrackerText());
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemTagTrackerValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FMemTagNodeSortingByTracker>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// LLM Tag Set Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FMemTagTableColumns::TagSetColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TagSetColumnName", "Tag Set"));
		Column.SetTitleName(LOCTEXT("TagSetColumnTitle", "Tag Set"));
		Column.SetDescription(LOCTEXT("TagSetColumnDesc", "The LLM tag set of the LLM tag"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FMemTagSetNameValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const
			{
				if (Node.Is<FMemTagNode>())
				{
					const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
					return FTableCellValue(MemTagNode.GetTagSetName());
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMemTagSetNameValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef); // TODO: FMemTagNodeSortingByTagSet
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
