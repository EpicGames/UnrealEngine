// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/NameTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/Table.h"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

// Column identifiers
struct FMemTagTableColumns
{
	static const FName TagNameColumnId;
	static const FName TagIdColumnId;
	static const FName SizeAColumnId;
	static const FName SizeBColumnId;
	static const FName SizeDiffColumnId;
	static const FName SizeBudgetColumnId;
	static const FName SampleCountColumnId;
	static const FName SizeMinColumnId;
	static const FName SizeMaxColumnId;
	static const FName SizeAverageColumnId;
	static const FName TrackerColumnId;
	static const FName TagSetColumnId;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemTagTable : public FTable
{
public:
	FMemTagTable();
	virtual ~FMemTagTable();

	virtual void Reset();

	//TArray<FMemoryTag>& GetTags() { return Tags; }
	//const TArray<FMemoryTag>& GetTags() const { return Tags; }

	//bool IsValidRowIndex(int32 InIndex) const { return InIndex >= 0 && InIndex < Tags.Num(); }
	//const FMemoryTag* GetMemTag(int32 InIndex) const { return IsValidRowIndex(InIndex) ? &Tags[InIndex] : nullptr; }
	//const FMemoryTag& GetMemTagChecked(int32 InIndex) const { check(IsValidRowIndex(InIndex)); return Tags[InIndex]; }

private:
	void AddDefaultColumns();

private:
	//TArray<FMemoryTag> Tags;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
