// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemTagTableViewPresets.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"
#include "InsightsCore/Table/Widgets/STableTreeView.h"

// TraceInsights
#include "Insights/MemoryProfiler/ViewModels/MemTagTable.h"
#include "Insights/MemoryProfiler/Widgets/SMemoryProfilerWindow.h"
#include "Insights/MemoryProfiler/Widgets/SMemTagTreeView.h"
#include "Insights/TimingProfiler/ViewModels/TimeMarker.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::SMemTagTableTreeView"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Default View

TSharedRef<ITableTreeViewPreset> FMemTagTableViewPresets::CreateDefaultViewPreset(SMemTagTreeView& TableTreeView)
{
	class FDefaultViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("Default_PresetName", "Time A");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("Default_PresetToolTip", "Time A View\nConfigure the tree view to show the LLM tags and their values at time A.");
		}
		virtual FName GetSortColumn() const override
		{
			return FMemTagTableColumns::TagNameColumnId;
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Ascending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),          true, 400.0f });
			InOutConfigSet.Add({ FMemTagTableColumns::SizeAColumnId,      true, 100.0f });
			InOutConfigSet.Add({ FMemTagTableColumns::SizeBudgetColumnId, true, 100.0f });
		}
		virtual void OnAppliedToView(STableTreeView& TableTreeView) const override
		{
			SMemTagTreeView& MemTagTreeView = (SMemTagTreeView&)TableTreeView;
			TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = MemTagTreeView.GetProfilerWindow();
			if (ProfilerWindow.IsValid())
			{
				const TSharedRef<TimingProfiler::FTimeMarker>& MarkerA = ProfilerWindow->GetCustomTimeMarker(0);
				MarkerA->SetVisibility(true);
				const TSharedRef<TimingProfiler::FTimeMarker>& MarkerB = ProfilerWindow->GetCustomTimeMarker(1);
				MarkerB->SetVisibility(false);
			}
		}
	};
	return MakeShared<FDefaultViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Difference View

TSharedRef<ITableTreeViewPreset> FMemTagTableViewPresets::CreateDiffViewPreset(SMemTagTreeView& TableTreeView)
{
	class FDiffViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("Diff_PresetName", "Diff B-A");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("Diff_PresetToolTip", "Difference View\nConfigure the tree view to investigate variation of LLM tags between two snapshots.");
		}
		virtual FName GetSortColumn() const override
		{
			return FMemTagTableColumns::TagNameColumnId;
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Ascending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),          true, 400.0f });
			InOutConfigSet.Add({ FMemTagTableColumns::SizeAColumnId,      true, 80.0f });
			InOutConfigSet.Add({ FMemTagTableColumns::SizeBColumnId,      true, 80.0f });
			InOutConfigSet.Add({ FMemTagTableColumns::SizeDiffColumnId,   true, 80.0f });
			InOutConfigSet.Add({ FMemTagTableColumns::SizeBudgetColumnId, true, 80.0f });
		}
		virtual void OnAppliedToView(STableTreeView& TableTreeView) const override
		{
			SMemTagTreeView& MemTagTreeView = (SMemTagTreeView&)TableTreeView;
			TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = MemTagTreeView.GetProfilerWindow();
			if (ProfilerWindow.IsValid())
			{
				const TSharedRef<TimingProfiler::FTimeMarker>& MarkerA = ProfilerWindow->GetCustomTimeMarker(0);
				MarkerA->SetVisibility(true);
				const TSharedRef<TimingProfiler::FTimeMarker>& MarkerB = ProfilerWindow->GetCustomTimeMarker(1);
				MarkerB->SetVisibility(true);
			}
		}
	};
	return MakeShared<FDiffViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Time Range View

TSharedRef<ITableTreeViewPreset> FMemTagTableViewPresets::CreateTimeRangeViewPreset(SMemTagTreeView& TableTreeView)
{
	class FTimeRangeViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("TimeRange_PresetName", "Time Range");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("TimeRange_PresetToolTip", "Time Range View\nConfigure the tree view to investigate the aggregated LLM stats for the selected time range.");
		}
		virtual FName GetSortColumn() const override
		{
			return FMemTagTableColumns::TagNameColumnId;
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Ascending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),           true, 400.0f });
			InOutConfigSet.Add({ FMemTagTableColumns::SizeMinColumnId,     true, 80.0f });
			InOutConfigSet.Add({ FMemTagTableColumns::SizeMaxColumnId,     true, 80.0f });
			InOutConfigSet.Add({ FMemTagTableColumns::SizeAverageColumnId, true, 80.0f });
			InOutConfigSet.Add({ FMemTagTableColumns::SizeBudgetColumnId,  true, 80.0f });
		}
		virtual void OnAppliedToView(STableTreeView& TableTreeView) const override
		{
			SMemTagTreeView& MemTagTreeView = (SMemTagTreeView&)TableTreeView;
			TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = MemTagTreeView.GetProfilerWindow();
			if (ProfilerWindow.IsValid())
			{
				const TSharedRef<TimingProfiler::FTimeMarker>& MarkerA = ProfilerWindow->GetCustomTimeMarker(0);
				MarkerA->SetVisibility(false);
				const TSharedRef<TimingProfiler::FTimeMarker>& MarkerB = ProfilerWindow->GetCustomTimeMarker(1);
				MarkerB->SetVisibility(false);
			}
		}
	};
	return MakeShared<FTimeRangeViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
