// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Templates/SharedPointer.h"

namespace UE::Insights { class ITableTreeViewPreset; }

namespace UE::Insights::MemoryProfiler
{

class SMemTagTreeView;

class FMemTagTableViewPresets
{
public:
	static TSharedRef<ITableTreeViewPreset> CreateDefaultViewPreset(SMemTagTreeView& TableTreeView);
	static TSharedRef<ITableTreeViewPreset> CreateDiffViewPreset(SMemTagTreeView& TableTreeView);
	static TSharedRef<ITableTreeViewPreset> CreateTimeRangeViewPreset(SMemTagTreeView& TableTreeView);
};

} // namespace UE::Insights::MemoryProfiler
