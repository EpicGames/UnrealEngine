// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceServices
#include "TraceServices/Model/AllocationsProvider.h" // for TagIdType

namespace TraceServices { struct FCallstack; }

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemoryAlloc
{
	friend class SMemAllocTableTreeView;

public:
	FMemoryAlloc();
	~FMemoryAlloc();

	int64 GetStartEventIndex() const { return int64(StartEventIndex); }
	int64 GetEndEventIndex() const { return int64(EndEventIndex); }
	int64 GetEventDistance() const { return EndEventIndex == ~0 ? int64(EndEventIndex) : int64(EndEventIndex) - int64(StartEventIndex); }

	double GetStartTime() const { return StartTime; }
	double GetEndTime() const { return EndTime; }
	double GetDuration() const { return EndTime - StartTime; }

	uint64 GetAddress() const { return Address; }

	int64 GetSize() const { return Size; }

	TraceServices::TagIdType GetTagId() const { return TagId; }
	const TCHAR* GetTag() const { return Tag; }

	const TCHAR* GetAsset() const { return Asset; }
	const TCHAR* GetPackage() const { return Package; }
	const TCHAR* GetClassName() const { return ClassName; }

	uint16 GetAllocThreadId() const { return AllocThreadId; }
	uint16 GetFreeThreadId() const { return FreeThreadId; }

	uint32 GetAllocCallstackId() const { return AllocCallstackId; }
	uint32 GetFreeCallstackId() const { return FreeCallstackId; }
	const TraceServices::FCallstack* GetAllocCallstack() const { return AllocCallstack; }
	const TraceServices::FCallstack* GetFreeCallstack() const { return FreeCallstack; }

	HeapId GetRootHeap() const { return RootHeap; }
	bool IsHeap() const { return bIsHeap; }
	bool IsSwap() const { return bIsSwap; }

	bool IsDecline() const { return bIsDecline; }

private:
	uint32 StartEventIndex = 0;
	uint32 EndEventIndex = 0;
	double StartTime = 0.0;
	double EndTime = 0.0;
	uint64 Address = 0;
	int64 Size = 0;
	uint16 AllocThreadId = 0;
	uint16 FreeThreadId = 0;
	TraceServices::TagIdType TagId = 0;
	const TCHAR* Tag = nullptr;
	const TCHAR* Asset = nullptr;
	const TCHAR* Package = nullptr;
	const TCHAR* ClassName = nullptr;
	uint32 AllocCallstackId = 0;
	uint32 FreeCallstackId = 0;
	const TraceServices::FCallstack* AllocCallstack = nullptr;
	const TraceServices::FCallstack* FreeCallstack = nullptr;
	HeapId RootHeap = 0;
	bool bIsHeap = false;
	bool bIsSwap = false;
	bool bIsDecline = false;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
