// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryExporter.h"

#include "Containers/Map.h"
#include "Containers/StringConv.h"
#include "Containers/Utf8String.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"
#include "Logging/LogMacros.h"
#include "Misc/Paths.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Bookmarks.h"
#include "TraceServices/Model/Callstack.h"
#include "TraceServices/Model/Definitions.h"
#include "TraceServices/Model/Strings.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"

// TraceInsights
#include "Insights/Common/Utf8FileWriter.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocTable.h"

#include <limits>

DEFINE_LOG_CATEGORY_STATIC(LogMemoryExporter, Log, All);

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemoryExporter
////////////////////////////////////////////////////////////////////////////////////////////////////

const FName FMemoryExporter::AllocCallstackColumnId("AllocCallstack");
const FName FMemoryExporter::FreeCallstackColumnId("FreeCallstack");

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryExporter::FMemoryExporter(const TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryExporter::~FMemoryExporter()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryExporter::InitAvailableColumns()
{
	if (!AvailableColumns.IsEmpty())
	{
		return;
	}

	AvailableColumns.Add(FMemAllocTableColumns::StartEventIndexColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::EndEventIndexColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::EventDistanceColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::StartTimeColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::EndTimeColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::DurationColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::AddressColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::MemoryPageColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::SizeColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::TagColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::AssetColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::ClassNameColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::PackageColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::AllocThreadColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::FreeThreadColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::AllocFunctionColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::FreeFunctionColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::AllocSourceFileColumnId);
	AvailableColumns.Add(FMemAllocTableColumns::FreeSourceFileColumnId);
	AvailableColumns.Add(FMemoryExporter::AllocCallstackColumnId);
	AvailableColumns.Add(FMemoryExporter::FreeCallstackColumnId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryExporter::InitDefaultColumns()
{
	InitAvailableColumns();

	if (DefaultColumns.IsEmpty())
	{
		DefaultColumns = AvailableColumns;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryExporter::MakeColumnList(const FString& InColumnsString, TArray<FName>& OutColumnList)
{
	InitAvailableColumns();

	TArray<FString> Columns;
	InColumnsString.ParseIntoArray(Columns, TEXT(","), true);

	// Provide some hints of malformed list of columns (the white space case)
	if (InColumnsString.StartsWith(TEXT(",")) || InColumnsString.EndsWith(TEXT(",")))
	{
		UE_LOGF(LogMemoryExporter, Warning, "Column list has leading/trailing comma: '%ls'", *InColumnsString);
		UE_LOGF(LogMemoryExporter, Warning,
			"Check you didn't separate the list of columns with white spaces or remove extra commas.");
	}

	for (const FString& ColumnWildcard : Columns)
	{
		FName ColumnName(*ColumnWildcard);
		if (AvailableColumns.Contains(ColumnName))
		{
			OutColumnList.Add(ColumnName);
		}
		else
		{
			for (const FName& Column : AvailableColumns)
			{
				if (Column.GetPlainNameString().MatchesWildcard(ColumnWildcard))
				{
					OutColumnList.Add(Column);
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FMemoryExporter::ExportMemoryAllocsAsText(FExportMemoryAllocsParams& Params)
{
	UE_LOGF(LogMemoryExporter, Log, "Memory Exporter Started");
	UE_LOGF(LogMemoryExporter, Log, "Rule: %ls", LexToString(Params.Rule));
	UE_LOGF(LogMemoryExporter, Log, "Output CSV file will be saved at: %ls", *Params.OutputPathName);
	UE_LOGF(LogMemoryExporter, Log, "Time Markers: A=%.2f, B=%.2f, C=%.2f, D=%.2f",
		Params.TimeA, Params.TimeB, Params.TimeC, Params.TimeD);

	UE_LOGF(LogMemoryExporter, Log, "Querying allocations...");
	TArray<const TraceServices::IAllocationsProvider::FAllocation*> Allocations;
	if (!QueryAllocations(Params, Allocations))
	{
		UE_LOGF(LogMemoryExporter, Error, "Failed to query allocations");
		return -1;
	}
	UE_LOGF(LogMemoryExporter, Log, "Found %d allocations", Allocations.Num());

	// Determine what columns to export
	TArray<FName> ExportColumns;
	if (Params.Columns.IsEmpty())
	{
		InitDefaultColumns();
		ExportColumns = DefaultColumns;
		UE_LOGF(LogMemoryExporter, Log, "Using all %d columns available by default", ExportColumns.Num());
	}
	else
	{
		MakeColumnList(Params.Columns, ExportColumns);
		UE_LOGF(LogMemoryExporter, Log, "Using the %d columns passed in -Columns: %ls",
			ExportColumns.Num(), *Params.Columns);
	}

	// Adding limits just in case user needs for investigation
	if (Params.MaxResults > 0 && Allocations.Num() > Params.MaxResults)
	{
		Allocations.SetNum(Params.MaxResults);
		UE_LOGF(LogMemoryExporter, Log, "Export limited by the user to %d allocations", Params.MaxResults);
	}

	// Write the CSV/TSV file
	if (!ExportAllocations(Params.OutputPathName, Allocations, ExportColumns))
	{
		UE_LOGF(LogMemoryExporter, Error, "Failed to write CSV");
		return -1;
	}

	return Allocations.Num();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemoryExporter::QueryAllocations(const FExportMemoryAllocsParams& Params, TArray<const TraceServices::IAllocationsProvider::FAllocation*>& OutAllocations)
{
	// Initialize required reading scope for sessions and providers

	TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);
	const TraceServices::IAllocationsProvider* AllocationsProvider = TraceServices::ReadAllocationsProvider(Session);
	if (!AllocationsProvider)
	{
		UE_LOGF(LogMemoryExporter, Error, "Nullptr: AllocationsProvider not available");
		return false;
	}
	TraceServices::FProviderReadScopeLock ProviderReadLock(*AllocationsProvider);
	if (!AllocationsProvider->IsInitialized())
	{
		UE_LOGF(LogMemoryExporter, Error, "AllocationsProvider exists but not initialized");
		return false;
	}

	// Setup the query
	TraceServices::IAllocationsProvider::FQueryParams QueryParams;
	QueryParams.Rule = Params.Rule;
	QueryParams.TimeA = Params.TimeA;
	QueryParams.TimeB = Params.TimeB;
	QueryParams.TimeC = Params.TimeC;
	QueryParams.TimeD = Params.TimeD;;

	// Start the query and poll results. The API is asynchronous and we have to collect results in batches from it's queue.
	TraceServices::IAllocationsProvider::FQueryHandle QueryHandle = AllocationsProvider->StartQuery(QueryParams);

	TraceServices::IAllocationsProvider::FQueryStatus Status;
	int BatchCount = 0;
	do
	{
		Status = AllocationsProvider->PollQuery(QueryHandle);
		if (Status.Status == TraceServices::IAllocationsProvider::EQueryStatus::Available)
		{
			TraceServices::IAllocationsProvider::FQueryResult Result = Status.NextResult();
			if (Result)
			{
				const int NumAllocations = Result->Num();
				UE_LOGF(LogMemoryExporter, Log, "Batch %d: %d allocations", BatchCount, NumAllocations);
				for (int i = 0; i < NumAllocations; ++i)
				{
					OutAllocations.Add(Result->Get(i));
				}
			}
		}

		++BatchCount;

		// Handle time out with an arbitrarily big number for BatchCount
		if (BatchCount > 10000)
		{
			UE_LOGF(LogMemoryExporter, Error, "Query timeout after %d batches iterated", BatchCount);
			AllocationsProvider->CancelQuery(QueryHandle);
			return false;
		}

		// To save some cpu
		FPlatformProcess::Sleep(0.001f);
	} while (Status.Status != TraceServices::IAllocationsProvider::EQueryStatus::Done);

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// This function appends in buffer in real time, because query may retrieve millions of allocations
// TODO: Evaluate using a Function table to avoid so many if/else if per row

void FMemoryExporter::WriteColumnValue(
	FUtf8FileWriter& Writer,
	const FName& ColumnId,
	const TraceServices::IAllocationsProvider::FAllocation* Alloc,
	const FMemoryExporter::FWriteColumnValueParams& Params)
{
	constexpr uint32 InfiniteEventIndex = std::numeric_limits<uint32>::max();

	if (ColumnId == FMemAllocTableColumns::StartEventIndexColumnId)
	{
		Writer.AppendValue(Alloc->GetStartEventIndex());
	}
	else if (ColumnId == FMemAllocTableColumns::EndEventIndexColumnId)
	{
		if (Alloc->GetEndEventIndex() == InfiniteEventIndex)
		{
			Writer.AppendValue(UTF8TEXTVIEW("inf"));
		}
		else
		{
			Writer.AppendValue(Alloc->GetEndEventIndex());
		}
	}
	else if (ColumnId == FMemAllocTableColumns::EventDistanceColumnId)
	{
		if (Alloc->GetEndEventIndex() == InfiniteEventIndex)
		{
			Writer.AppendValue(UTF8TEXTVIEW("inf"));
		}
		else
		{
			const uint32 StartEventIndex = Alloc->GetStartEventIndex();
			const uint32 EndEventIndex = Alloc->GetEndEventIndex();
			const uint32 EventDistance = EndEventIndex - StartEventIndex;
			Writer.AppendValue(EventDistance);
		}
	}
	else if (ColumnId == FMemAllocTableColumns::StartTimeColumnId)
	{
		Writer.AppendValueAsTimestamp(Alloc->GetStartTime());
	}
	else if (ColumnId == FMemAllocTableColumns::EndTimeColumnId)
	{
		Writer.AppendValueAsTimestamp(Alloc->GetEndTime());
	}
	else if (ColumnId == FMemAllocTableColumns::DurationColumnId)
	{
		Writer.AppendValueAsDuration(Alloc->GetEndTime() - Alloc->GetStartTime());
	}
	else if (ColumnId == FMemAllocTableColumns::AddressColumnId)
	{
		Writer.AppendValueAsHex64(Alloc->GetAddress());
	}
	else if (ColumnId == FMemAllocTableColumns::MemoryPageColumnId)
	{
		Writer.AppendValueAsHex64(Alloc->GetAddress() & ~(Params.PageSize - 1)); // see FMemAllocTable::GetAddressPage
	}
	else if (ColumnId == FMemAllocTableColumns::SizeColumnId)
	{
		Writer.AppendValue(Alloc->GetSize());
	}
	else if (ColumnId == FMemAllocTableColumns::TagColumnId)
	{
		const TCHAR* LLMTagPath = Params.AllocationsProvider->GetTagFullPath(Alloc->GetTag());
		if (LLMTagPath)
		{
			Writer.AppendValue(LLMTagPath);
		}
	}
	else if (ColumnId == FMemAllocTableColumns::AssetColumnId)
	{
		Writer.AppendValue(Params.AssetName);
	}
	else if (ColumnId == FMemAllocTableColumns::ClassNameColumnId)
	{
		Writer.AppendValue(Params.ClassName);
	}
	else if (ColumnId == FMemAllocTableColumns::PackageColumnId)
	{
		Writer.AppendValue(Params.PackageName);
	}
	else if (ColumnId == FMemAllocTableColumns::AllocThreadColumnId)
	{
		Writer.AppendValue(Alloc->GetAllocThreadId());
	}
	else if (ColumnId == FMemAllocTableColumns::FreeThreadColumnId)
	{
		Writer.AppendValue(Alloc->GetFreeThreadId());
	}
	else if (ColumnId == FMemAllocTableColumns::AllocFunctionColumnId)
	{
		Writer.AppendValue(GetTopFunction(Params.AllocCallstack));
	}
	else if (ColumnId == FMemAllocTableColumns::FreeFunctionColumnId)
	{
		Writer.AppendValue(GetTopFunction(Params.FreeCallstack));
	}
	else if (ColumnId == FMemAllocTableColumns::AllocSourceFileColumnId)
	{
		Writer.AppendValue(GetTopSourceFile(Params.AllocCallstack));
	}
	else if (ColumnId == FMemAllocTableColumns::FreeSourceFileColumnId)
	{
		Writer.AppendValue(GetTopSourceFile(Params.FreeCallstack));
	}
	else if (ColumnId == FMemoryExporter::AllocCallstackColumnId)
	{
		if (Writer.IsCSV())
		{
#if 0
			// Slower, but it would also escape the " char, if present.
			Writer.AppendValue(FormatCallstack(Params.AllocCallstack));
#else
			// Faster, but we assume the callstack string does not contains the " char
			// (so we can Append it directly without escaping " char).
			Writer.AppendDoubleQuotationMark();
			AppendCallstack(Writer, Params.AllocCallstack);
			Writer.AppendDoubleQuotationMark();
#endif
		}
		else
		{
			AppendCallstack(Writer, Params.AllocCallstack);
		}
	}
	else if (ColumnId == FMemoryExporter::FreeCallstackColumnId)
	{
		if (Writer.IsCSV())
		{
#if 0
			// Slower, but it would also escape the " char, if present.
			Writer.AppendValue(FormatCallstack(Params.FreeCallstack));
#else
			// Faster, but we assume the callstack string does not contains the " char
			// (so we can Append it directly without escaping " char).
			Writer.AppendDoubleQuotationMark();
			AppendCallstack(Writer, Params.FreeCallstack);
			Writer.AppendDoubleQuotationMark();
#endif
		}
		else
		{
			AppendCallstack(Writer, Params.FreeCallstack);
		}
	}
	else
	{
		//Writer.Append(TEXT(""));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryExporter::ExtractMetadata(
	const TraceServices::IAllocationsProvider::FAllocation* Alloc,
	FMemoryExporter::FWriteColumnValueParams& InOutParams)
{
	InOutParams.AssetName = nullptr;
	InOutParams.ClassName = nullptr;
	InOutParams.PackageName = nullptr;

	const uint32 MetadataId = Alloc->GetMetadataId();
	if (MetadataId == TraceServices::IMetadataProvider::InvalidMetadataId)
	{
		return;
	}

	check(InOutParams.MetadataProvider);
	check(InOutParams.DefinitionProvider);
	check(InOutParams.MetadataSchema);

	TraceServices::FProviderReadScopeLock MetadataProviderReadLock(*InOutParams.MetadataProvider);
	TraceServices::FProviderReadScopeLock DefinitionProviderReadLock(*InOutParams.DefinitionProvider);

	InOutParams.MetadataProvider->EnumerateMetadata(Alloc->GetAllocThreadId(), MetadataId,
		[&InOutParams]
		(uint32 StackDepth, uint16 Type, const void* Data, uint32 Size) -> bool
		{
			if (Type == InOutParams.AssetMetadataType)
			{
				const auto Reader = InOutParams.MetadataSchema->Reader();

				// Asset Name
				const auto AssetNameRef = Reader.GetValueAs<UE::Trace::FEventRef32>((uint8*)Data, 0);
				const auto AssetNameDef = InOutParams.DefinitionProvider->Get<TraceServices::FStringDefinition>(*AssetNameRef);
				if (AssetNameDef)
				{
					InOutParams.AssetName = AssetNameDef->Display;
				}

				// Class Name
				const auto ClassNameRef = Reader.GetValueAs<UE::Trace::FEventRef32>((uint8*)Data, 1);
				const auto ClassNameDef = InOutParams.DefinitionProvider->Get<TraceServices::FStringDefinition>(*ClassNameRef);
				if (ClassNameDef)
				{
					InOutParams.ClassName = ClassNameDef->Display;
				}

				// Package Name
				const auto PackageNameRef = Reader.GetValueAs<UE::Trace::FEventRef32>((uint8*)Data, 2);
				const auto PackageNameDef = InOutParams.DefinitionProvider->Get<TraceServices::FStringDefinition>(*PackageNameRef);
				if (PackageNameDef)
				{
					InOutParams.PackageName = PackageNameDef->Display;
				}

				return false;
			}
			return true;
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemoryExporter::ExportAllocations(
	const FString& FilePath,
	const TArray<const TraceServices::IAllocationsProvider::FAllocation*>& Allocations,
	const TArray<FName>& ExportColumns)
{
	const TraceServices::IAllocationsProvider* AllocationsProvider = TraceServices::ReadAllocationsProvider(Session);
	if (!AllocationsProvider)
	{
		UE_LOGF(LogMemoryExporter, Error, "AllocationsProvider not available");
		return false;
	}

	const TraceServices::ICallstacksProvider* CallstacksProvider = TraceServices::ReadCallstacksProvider(Session);
	const TraceServices::IMetadataProvider* MetadataProvider = TraceServices::ReadMetadataProvider(Session);
	const TraceServices::IDefinitionProvider* DefinitionProvider = TraceServices::ReadDefinitionProvider(Session);

	TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);
	TraceServices::FProviderReadScopeLock AllocProviderLock(*AllocationsProvider);

	// User requires data for Asset/Class/Package extraction. Setting up the Metadata schema here
	uint16 AssetMetadataType = TraceServices::IMetadataProvider::InvalidMetadataType;
	const TraceServices::FMetadataSchema* MetadataSchema = nullptr;
	if (MetadataProvider)
	{
		TraceServices::FProviderReadScopeLock MetadataProviderLock(*MetadataProvider);
		AssetMetadataType = MetadataProvider->GetRegisteredMetadataType(TEXT("Asset"));
		if (AssetMetadataType != TraceServices::IMetadataProvider::InvalidMetadataType)
		{
			MetadataSchema = MetadataProvider->GetRegisteredMetadataSchema(AssetMetadataType);
		}
		else
		{
			// If Asset metadata type not found, disable metadata provider
			MetadataProvider = nullptr;
		}
	}
	const bool bCanAccessMetadata = MetadataProvider && MetadataSchema && DefinitionProvider;

	FStopwatch Stopwatch;
	Stopwatch.Start();

	TUniquePtr<IFileHandle> ExportFileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*FilePath));
	if (!ExportFileHandle)
	{
		UE_LOGF(LogMemoryExporter, Error, "Failed to open file for writing: %ls", *FilePath);
		return false;
	}
	bool bIsCSV = FilePath.EndsWith(TEXT(".csv"));
	FUtf8FileWriter Writer(ExportFileHandle.Get(), bIsCSV);

	constexpr uint64 PageSize = 65536;

	FWriteColumnValueParams Params
	{
		PageSize,
		AllocationsProvider,
		MetadataProvider,
		MetadataSchema,
		DefinitionProvider,
		AssetMetadataType,
	};

	// Write header
	for (int32 ColumnIndex = 0; ColumnIndex < ExportColumns.Num(); ++ColumnIndex)
	{
		if (ColumnIndex > 0)
		{
			Writer.AppendSeparator();
		}
		Writer.Append(ExportColumns[ColumnIndex].ToString());
	}
	Writer.AppendLineEnd();

	const uint32 NumAllocations = Allocations.Num();
	uint32 NumWrittenAllocations = 0;
	uint32 StepNumWrittenAllocations = 0;
	int32 StepProgress = 0;
	FStopwatch StepProgressStopwatch;
	StepProgressStopwatch.Start();

	// Write values
	for (const TraceServices::IAllocationsProvider::FAllocation* Alloc : Allocations)
	{
		// Resolve callstacks
		if (CallstacksProvider)
		{
			Params.AllocCallstack = CallstacksProvider->GetCallstack(Alloc->GetAllocCallstackId());
			Params.FreeCallstack  = CallstacksProvider->GetCallstack(Alloc->GetFreeCallstackId());
		}

		// Resolve metadata
		if (bCanAccessMetadata)
		{
			ExtractMetadata(Alloc, Params);
		}

		for (int32 ColumnIndex = 0; ColumnIndex < ExportColumns.Num(); ++ColumnIndex)
		{
			if (ColumnIndex > 0)
			{
				Writer.AppendSeparator();
			}
			WriteColumnValue(Writer, ExportColumns[ColumnIndex], Alloc, Params);
		}
		Writer.AppendLineEnd();

		++NumWrittenAllocations;
		double Progress = ((double)NumWrittenAllocations / (double)NumAllocations) * 100.0f;
		int32 RoundedProgress = FMath::FloorToInt32(Progress);
		if (RoundedProgress != StepProgress)
		{
			StepProgressStopwatch.Stop();
			const double Duration = StepProgressStopwatch.GetAccumulatedTime();
			const double AllocsPerSecond = (double)(NumWrittenAllocations - StepNumWrittenAllocations) / Duration;
			UE_LOGF(LogMemoryExporter, Log, "Export progress: %i%% (%u / %u) -> %.0f allocs/s", RoundedProgress, NumWrittenAllocations, NumAllocations, AllocsPerSecond);
			StepNumWrittenAllocations = NumWrittenAllocations;
			StepProgress = RoundedProgress;
			StepProgressStopwatch.Restart();
		}
	}
	check(NumWrittenAllocations == NumAllocations);

	Writer.Flush();
	ExportFileHandle->Flush();
	ExportFileHandle.Reset();

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	const double TotalAllocsPerSecond = (double)NumWrittenAllocations / TotalTime;
	UE_LOGF(LogMemoryExporter, Log, "Exported %d allocations to file in %.3fs (\"%ls\") -> %.0f allocs/s", NumWrittenAllocations, TotalTime, *FilePath, TotalAllocsPerSecond);

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FUtf8String FMemoryExporter::GetTopFunction(const TraceServices::FCallstack* Callstack)
{
	if (!Callstack || Callstack->IsEmpty())
	{
		return UTF8TEXT("No Callstack Recorded");
	}

	if (Callstack->Num() > 0)
	{
		const TraceServices::FStackFrame* Frame = Callstack->Frame(0);
		if (Frame && Frame->Symbol)
		{
			TUtf8StringBuilder<512> Builder;

			// "Module!Function" format
			if (Frame->Symbol->Module)
			{
				Builder.Append(Frame->Symbol->Module);
				Builder.Append(UTF8TEXTVIEW("!"));
			}
			if (Frame->Symbol->Name)
			{
				Builder.Append(Frame->Symbol->Name);
			}
			else
			{
				// Address (as hex number) if no symbol name
				Builder.Appendf(UTF8TEXT("0x%llX"), Frame->Addr);
			}

			return Builder.ToString();
		}
	}

	return UTF8TEXT("Empty Callstack");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FUtf8String FMemoryExporter::GetTopSourceFile(const TraceServices::FCallstack* Callstack)
{
	if (!Callstack || Callstack->IsEmpty())
	{
		return UTF8TEXT("No Callstack Recorded");
	}

	if (Callstack->Num() > 0)
	{
		const TraceServices::FStackFrame* Frame = Callstack->Frame(0);
		if (Frame && Frame->Symbol && Frame->Symbol->File)
		{
			return FUtf8String(Frame->Symbol->File);
		}
	}

	return UTF8TEXT("Empty Callstack");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FUtf8String FMemoryExporter::FormatCallstack(const TraceServices::FCallstack* Callstack)
{
	if (!Callstack || Callstack->IsEmpty())
	{
		return UTF8TEXT("No Callstack Recorded");
	}

	TUtf8StringBuilder<4096> Builder;
	const uint32 NumFrames = Callstack->Num();
	check(NumFrames < 256);

	for (uint8 i = 0; i < NumFrames; ++i)
	{
		const TraceServices::FStackFrame* Frame = Callstack->Frame(i);
		if (Frame && Frame->Symbol)
		{
			if (i > 0)
			{
				Builder.Append(UTF8TEXTVIEW("/"));
			}

			// "Module!Function" format
			if (Frame->Symbol->Module)
			{
				Builder.Append(Frame->Symbol->Module);
				Builder.Append(UTF8TEXTVIEW("!"));
			}
			if (Frame->Symbol->Name)
			{
				Builder.Append(Frame->Symbol->Name);
			}
			else
			{
				// Address (as hex number) if no symbol name
				Builder.Appendf(UTF8TEXT("0x%llX"), Frame->Addr);
			}
		}
	}

	if (Builder.Len() == 0)
	{
		return UTF8TEXT("Empty Callstack");
	}

	return Builder.ToString();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryExporter::AppendCallstack(FUtf8FileWriter& Writer, const TraceServices::FCallstack* Callstack)
{
	if (!Callstack || Callstack->IsEmpty())
	{
		Writer.Append(UTF8TEXTVIEW("No Callstack Recorded"));
		return;
	}

	const uint32 NumFrames = Callstack->Num();
	check(NumFrames < 256);
	bool bIsEmpty = true;

	for (uint8 i = 0; i < NumFrames; ++i)
	{
		const TraceServices::FStackFrame* Frame = Callstack->Frame(i);
		if (Frame && Frame->Symbol)
		{
			if (i > 0)
			{
				Writer.Append(UTF8TEXTVIEW("/"));
			}

			// "Module!Function" format
			if (Frame->Symbol->Module)
			{
				Writer.Append(Frame->Symbol->Module);
				Writer.Append(UTF8TEXTVIEW("!"));
			}
			if (Frame->Symbol->Name)
			{
				Writer.Append(Frame->Symbol->Name);
			}
			else
			{
				// Address (as hex number) if no symbol name
				Writer.Appendf(UTF8TEXT("0x%llX"), Frame->Addr);
			}
			bIsEmpty = false;
		}
	}

	if (bIsEmpty)
	{
		Writer.Append(UTF8TEXTVIEW("Empty Callstack"));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemoryExporter::FindBookmarkByName(const FString& BookmarkName, double& OutTime) const
{
	const TraceServices::IBookmarkProvider& BookmarkProvider = TraceServices::ReadBookmarkProvider(Session);

	TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

	const uint64 TotalBookmarks = BookmarkProvider.GetBookmarkCount();
	if (TotalBookmarks == 0)
	{
		UE_LOGF(LogMemoryExporter, Warning, "No bookmarks found in trace");
		return false;
	}

	bool BookmarkExists = false;
	BookmarkProvider.EnumerateBookmarks(
		0.0,
		std::numeric_limits<double>::max(),
		[&BookmarkName, &OutTime, &BookmarkExists]
		(const TraceServices::FBookmark& Bookmark)
		{
			if (Bookmark.Text && FCString::Strcmp(Bookmark.Text, *BookmarkName) == 0)
			{
				OutTime = Bookmark.Time;
				BookmarkExists = true;
			}
		});

	if (BookmarkExists)
	{
		UE_LOGF(LogMemoryExporter, Log, "Found bookmark '%ls' at time %.3f", *BookmarkName, OutTime);
	}
	else
	{
		UE_LOGF(LogMemoryExporter, Warning, "Bookmark '%ls' not found", *BookmarkName);
	}

	return BookmarkExists;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
