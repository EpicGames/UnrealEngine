// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "UObject/NameTypes.h"

//TraceServices
#include "TraceServices/Model/AllocationsProvider.h"
#include "TraceServices/Model/MetadataProvider.h"

class IFileHandle;

namespace TraceServices
{
	class IDefinitionProvider;
	struct FCallstack;
}

namespace UE::Insights
{
	class FUtf8FileWriter;
}

namespace UE::Insights::MemoryProfiler
{

struct FExportMemoryLeakSnapshotParams;

class FMemoryExporter
{
public:
	/**
	* The list of parameters to export memory allocs
	*/
	struct FExportMemoryAllocsParams
	{
		FString OutputPathName;
		FString Columns;
		TraceServices::IAllocationsProvider::EQueryRule Rule;
		double TimeA, TimeB, TimeC, TimeD;
		int32 MaxResults;
	};

public:
	FMemoryExporter(const TraceServices::IAnalysisSession& InSession);
	virtual ~FMemoryExporter();

	//////////////////////////////////////////////////////////////////////
	// Exporters

	int32 ExportMemoryAllocsAsText(FExportMemoryAllocsParams& Params);

	//////////////////////////////////////////////////////////////////////
	// Utilities

	bool FindBookmarkByName(const FString& BookmarkName, double& OutTime) const;

private:
	void InitDefaultColumns();
	void InitAvailableColumns();
	void MakeColumnList(const FString& InColumnsString, TArray<FName>& OutColumnList);

	bool QueryAllocations(
		const FExportMemoryAllocsParams& Params,
		TArray<const TraceServices::IAllocationsProvider::FAllocation*>& OutAllocations);

	bool ExportAllocations(
		const FString& FilePath,
		const TArray<const TraceServices::IAllocationsProvider::FAllocation*>& Allocations,
		const TArray<FName>& ExportColumns);

	FUtf8String GetTopFunction(const TraceServices::FCallstack* Callstack);
	FUtf8String GetTopSourceFile(const TraceServices::FCallstack* Callstack);
	FUtf8String FormatCallstack(const TraceServices::FCallstack* Callstack);
	void AppendCallstack(FUtf8FileWriter& InWriter, const TraceServices::FCallstack* Callstack);

	struct FWriteColumnValueParams
	{
		uint64 PageSize = 0;
		const TraceServices::IAllocationsProvider* AllocationsProvider = nullptr;
		const TraceServices::IMetadataProvider* MetadataProvider = nullptr;
		const TraceServices::FMetadataSchema* MetadataSchema = nullptr;
		const TraceServices::IDefinitionProvider* DefinitionProvider = nullptr;
		uint16 AssetMetadataType = TraceServices::IMetadataProvider::InvalidMetadataType;
		const TraceServices::FCallstack* AllocCallstack = nullptr;
		const TraceServices::FCallstack* FreeCallstack = nullptr;
		const TCHAR* AssetName = nullptr;
		const TCHAR* ClassName = nullptr;
		const TCHAR* PackageName = nullptr;
	};

	void ExtractMetadata(
		const TraceServices::IAllocationsProvider::FAllocation* Alloc,
		FWriteColumnValueParams& InOutParams);

	void WriteColumnValue(
		FUtf8FileWriter& InWriter,
		const FName& InColumnId,
		const TraceServices::IAllocationsProvider::FAllocation* Alloc,
		const FWriteColumnValueParams& InParams);

private:
	const TraceServices::IAnalysisSession& Session;

	TArray<FName> AvailableColumns;
	TArray<FName> DefaultColumns;

	static const FName AllocCallstackColumnId;
	static const FName FreeCallstackColumnId;
};

} // namespace UE::Insights::MemoryProfiler
