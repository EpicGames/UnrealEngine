// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryFiltersValueConverters.h"

// TraceServices
#include "TraceServices/Model/Threads.h"
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsights
#include "Insights/InsightsManager.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::MemoryFilterConverters"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadFilterValueConverter::Convert(const FString& Input, int64& Output, FText& OutError) const
{
	if (Input.IsNumeric() && !Input.Contains(TEXT(".")))
	{
		Output = FCString::Atoi(*Input);
		return true;
	}

	int32 Pos = Input.Find(TEXT(" (id:"));
	FString ThreadName = Input;
	if (Pos > 0)
	{
		ThreadName = Input.Left(Pos);
	}

	bool bIsFound = false;
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(*Session.Get());

		ThreadProvider.EnumerateThreads([&ThreadName, &Output, &bIsFound](const TraceServices::FThreadInfo& ThreadInfo)
		{
			if (FCString::Strcmp(ThreadInfo.Name, *ThreadName) == 0)
			{
				Output = ThreadInfo.Id;
				bIsFound = true;
			}
		});
	}

	if (!bIsFound)
	{
		OutError = LOCTEXT("NoThreadFound", "No thread with this name was found!");
	}

	return bIsFound;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FThreadFilterValueConverter::GetTooltipText() const
{
	return LOCTEXT("FThreadConverterTooltipText", "Enter the name or the id of the thread.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FThreadFilterValueConverter::GetHintText() const
{
	return LOCTEXT("FThreadConverterHint", "Start typing or press arrow down or up to see options.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
