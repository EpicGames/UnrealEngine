// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemorySharedState.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Logging/MessageLog.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AllocationsProvider.h"
#include "TraceServices/Model/Memory.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/MemoryProfiler/MemoryProfilerManager.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryTag.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryTracker.h"
#include "Insights/MemoryProfiler/ViewModels/Report.h"
#include "Insights/MemoryProfiler/ViewModels/ReportXmlParser.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Insights/Widgets/STimingView.h"

#include <limits>

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FMemorySharedState"

namespace UE::Insights::MemoryProfiler
{

static_assert(FMemoryTracker::InvalidTrackerId == TraceServices::FMemoryTrackerInfo::InvalidTrackerId, "InvalidTrackerId");
static_assert(FMemoryTag::InvalidTagId == TraceServices::FMemoryTagInfo::InvalidTagId, "InvalidTagId");

const FName FQueryTargetWindowSpec::NewWindow = TEXT("New Window");

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemoryTimingViewCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryTimingViewCommands::FMemoryTimingViewCommands()
	: TCommands<FMemoryTimingViewCommands>(
		TEXT("MemoryTimingViewCommands"),
		NSLOCTEXT("Contexts", "MemoryTimingViewCommands", "Insights - Timing View - Memory"),
		NAME_None,
		FInsightsStyle::GetStyleSetName())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryTimingViewCommands::~FMemoryTimingViewCommands()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// UI_COMMAND takes long for the compiler to optimize
UE_DISABLE_OPTIMIZATION_SHIP
void FMemoryTimingViewCommands::RegisterCommands()
{
	UI_COMMAND(ShowHideAllMemoryTracks,
		"Memory Tracks",
		"Shows/hides the Memory tracks.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EModifierKey::Control, EKeys::M));
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemorySharedState
////////////////////////////////////////////////////////////////////////////////////////////////////

FMemorySharedState::FMemorySharedState()
{
	InitMemoryRules();

	CurrentQueryTarget = MakeShared<FQueryTargetWindowSpec>(FQueryTargetWindowSpec::NewWindow, LOCTEXT("NewWindow", "New Window"));
	QueryTargetSpecs.Add(CurrentQueryTarget);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemorySharedState::~FMemorySharedState()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::ResetMemoryTags()
{
	TagList.Reset();

	Trackers.Reset();
	DefaultTracker = nullptr;
	PlatformTracker = nullptr;

	TagSets.Reset();
	NumSyncedTagSets = 0;
	NumValidTagSets = 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::UpdateMemoryTags()
{
	TagList.Update();

	if (!DefaultTracker)
	{
		SyncTrackers();
	}

	UpdateTagSets();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::ResetTimingView()
{
	MainGraphTrack = nullptr;
	LiveAllocsGraphTrack = nullptr;
	AllocFreeGraphTrack = nullptr;
	SwapMemoryGraphTrack = nullptr;
	PageSwapGraphTrack = nullptr;
	AllTracks.Reset();
	for (FMemoryTag* TagPtr : TagList.GetTags())
	{
		TagPtr->RemoveAllTracks();
	}

	bShowHideAllMemoryTracks = true;

	CreatedDefaultTracks.Reset();
	LastTagCountForDefaultTracks = 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::OnBeginSession(Timing::ITimingViewSession& InSession)
{
	TimingProfiler::STimingView* TimingView = GetTimingView().Get();
	if (&InSession != TimingView)
	{
		return;
	}

	ResetTimingView();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::OnEndSession(Timing::ITimingViewSession& InSession)
{
	TimingProfiler::STimingView* TimingView = GetTimingView().Get();
	if (&InSession != TimingView)
	{
		return;
	}

	ResetTimingView();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::Tick(Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	TimingProfiler::STimingView* TimingView = GetTimingView().Get();
	if (&InSession != TimingView)
	{
		return;
	}

	bool bHasAllocationEvents = false;
	bool bHasSwapOpEvents = false;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::IAllocationsProvider* AllocationsProvider = TraceServices::ReadAllocationsProvider(*Session.Get());
		if (AllocationsProvider)
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*AllocationsProvider);
			bHasAllocationEvents = AllocationsProvider->IsInitialized() && AllocationsProvider->HasAllocationEvents();
			bHasSwapOpEvents = AllocationsProvider->IsInitialized() && AllocationsProvider->HasSwapOpEvents();
		}
	}

	constexpr int32 GraphTrackOrder = FTimingTrackOrder::First + FTimingTrackOrder::GroupRange / 2;

	if (!MainGraphTrack.IsValid())
	{
		MainGraphTrack = CreateMemoryGraphTrack();
		check(MainGraphTrack);

		MainGraphTrack->SetOrder(GraphTrackOrder);
		MainGraphTrack->SetName(TEXT("MAIN MEMORY GRAPH"));

		MainGraphTrack->AddTimelineSeries(FAllocationsGraphSeries::ETimeline::MaxTotalMem);
		MainGraphTrack->AddTimelineSeries(FAllocationsGraphSeries::ETimeline::MinTotalMem);

		MainGraphTrack->SetVisibilityFlag(bShowHideAllMemoryTracks);

		MainGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Small, 100.0f);
		MainGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Medium, 200.0f);
		MainGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Large, 400.0f);
		MainGraphTrack->SetCurrentTrackHeight(TrackHeightMode);

		TimingView->InvalidateScrollableTracksOrder();
	}

	if (!LiveAllocsGraphTrack.IsValid() && bHasAllocationEvents)
	{
		LiveAllocsGraphTrack = CreateMemoryGraphTrack();
		check(LiveAllocsGraphTrack);

		LiveAllocsGraphTrack->SetOrder(GraphTrackOrder + 1);
		LiveAllocsGraphTrack->SetName(TEXT("Live Allocation Count"));
		LiveAllocsGraphTrack->SetLabelUnit(EGraphTrackLabelUnit::Count, 0);

		LiveAllocsGraphTrack->AddTimelineSeries(FAllocationsGraphSeries::ETimeline::MaxLiveAllocs);
		LiveAllocsGraphTrack->AddTimelineSeries(FAllocationsGraphSeries::ETimeline::MinLiveAllocs);

		LiveAllocsGraphTrack->SetVisibilityFlag(bShowHideAllMemoryTracks);

		LiveAllocsGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Small, 50.0f);
		LiveAllocsGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Medium, 100.0f);
		LiveAllocsGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Large, 200.0f);
		LiveAllocsGraphTrack->SetCurrentTrackHeight(TrackHeightMode);

		TimingView->InvalidateScrollableTracksOrder();
	}

	if (!AllocFreeGraphTrack.IsValid() && bHasAllocationEvents)
	{
		AllocFreeGraphTrack = CreateMemoryGraphTrack();
		check(AllocFreeGraphTrack);

		AllocFreeGraphTrack->SetOrder(GraphTrackOrder + 2);
		AllocFreeGraphTrack->SetName(TEXT("Alloc/Free Event Count"));
		AllocFreeGraphTrack->SetLabelUnit(EGraphTrackLabelUnit::Count, 0);

		AllocFreeGraphTrack->AddTimelineSeries(FAllocationsGraphSeries::ETimeline::AllocEvents);
		AllocFreeGraphTrack->AddTimelineSeries(FAllocationsGraphSeries::ETimeline::FreeEvents);

		AllocFreeGraphTrack->SetVisibilityFlag(bShowHideAllMemoryTracks);

		AllocFreeGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Small, 50.0f);
		AllocFreeGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Medium, 100.0f);
		AllocFreeGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Large, 200.0f);
		AllocFreeGraphTrack->SetCurrentTrackHeight(TrackHeightMode);

		TimingView->InvalidateScrollableTracksOrder();
	}

	if (!SwapMemoryGraphTrack.IsValid() && bHasSwapOpEvents)
	{
		SwapMemoryGraphTrack = CreateMemoryGraphTrack();
		check(SwapMemoryGraphTrack);

		SwapMemoryGraphTrack->SetOrder(GraphTrackOrder + 3);
		SwapMemoryGraphTrack->SetName(TEXT("Swap Memory Graph"));

		SwapMemoryGraphTrack->AddTimelineSeries(FAllocationsGraphSeries::ETimeline::MaxSwapMem);
		SwapMemoryGraphTrack->AddTimelineSeries(FAllocationsGraphSeries::ETimeline::MinSwapMem);
		SwapMemoryGraphTrack->AddTimelineSeries(FAllocationsGraphSeries::ETimeline::MaxCompressedSwapMem);
		SwapMemoryGraphTrack->AddTimelineSeries(FAllocationsGraphSeries::ETimeline::MinCompressedSwapMem);
		SwapMemoryGraphTrack->SetLabelUnit(EGraphTrackLabelUnit::MiB, 1);

		SwapMemoryGraphTrack->SetVisibilityFlag(bShowHideAllMemoryTracks);

		SwapMemoryGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Small, 50.0f);
		SwapMemoryGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Medium, 100.0f);
		SwapMemoryGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Large, 200.0f);
		SwapMemoryGraphTrack->SetCurrentTrackHeight(TrackHeightMode);

		TimingView->InvalidateScrollableTracksOrder();
	}

	if (!PageSwapGraphTrack.IsValid() && bHasSwapOpEvents)
	{
		PageSwapGraphTrack = CreateMemoryGraphTrack();
		check(PageSwapGraphTrack);

		PageSwapGraphTrack->SetOrder(GraphTrackOrder + 4);
		PageSwapGraphTrack->SetName(TEXT("Page In/Out Event Count"));
		PageSwapGraphTrack->SetLabelUnit(EGraphTrackLabelUnit::Count, 0);

		PageSwapGraphTrack->AddTimelineSeries(FAllocationsGraphSeries::ETimeline::PageInEvents);
		PageSwapGraphTrack->AddTimelineSeries(FAllocationsGraphSeries::ETimeline::PageOutEvents);
		PageSwapGraphTrack->AddTimelineSeries(FAllocationsGraphSeries::ETimeline::SwapFreeEvents);

		PageSwapGraphTrack->SetVisibilityFlag(bShowHideAllMemoryTracks);

		PageSwapGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Small, 50.0f);
		PageSwapGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Medium, 100.0f);
		PageSwapGraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Large, 200.0f);
		PageSwapGraphTrack->SetCurrentTrackHeight(TrackHeightMode);

		TimingView->InvalidateScrollableTracksOrder();
	}

	CreateDefaultTracks();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::CreateDefaultTracks()
{
	if (!DefaultTracker)
	{
		return;
	}

	const auto Tags = TagList.GetTags();
	if (Tags.Num() == LastTagCountForDefaultTracks)
	{
		// Only attempt to create default tracks if new tags are added.
		return;
	}
	LastTagCountForDefaultTracks = Tags.Num();

	static const TCHAR* DefaultTags[] =
	{
		TEXT("Total"),
		TEXT("TrackedTotal"),
		TEXT("Untracked"),
		TEXT("Meshes"),
		TEXT("Textures"),
		TEXT("Physics"),
		TEXT("Audio"),
		TEXT("Animation"),
		TEXT("Lumen"),
		TEXT("Nanite"),
		TEXT("ProgramSize"),
		TEXT("RenderTargets"),
		TEXT("SceneRender"),
		TEXT("UObject")
	};
	constexpr int32 DefaultTagCount = UE_ARRAY_COUNT(DefaultTags);

	if (CreatedDefaultTracks.Num() != DefaultTagCount)
	{
		CreatedDefaultTracks.Init(false, DefaultTagCount);
	}

	const FMemoryTrackerId DefaultTrackerId = DefaultTracker->GetId();

	for (int32 DefaultTagIndex = 0; DefaultTagIndex < DefaultTagCount; ++DefaultTagIndex)
	{
		if (!CreatedDefaultTracks[DefaultTagIndex])
		{
			for (const FMemoryTag* Tag : Tags)
			{
				if (Tag->GetTrackerId() == DefaultTrackerId && // is it used by the default tracker?
					Tag->GetGraphTracks().Num() == 0 && // a graph isn't already added for this LLM tag?
					FCString::Stricmp(*Tag->GetStatName(), DefaultTags[DefaultTagIndex]) == 0) // is it one of the LLM tags to show as default?
				{
					CreateMemTagGraphTrack(DefaultTrackerId, Tag->GetId());
					CreatedDefaultTracks[DefaultTagIndex] = true;
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FMemorySharedState::TrackersToString(uint64 Flags, const TCHAR* Conjunction) const
{
	FString Str;
	if (Flags != 0)
	{
		for (const TSharedPtr<FMemoryTracker>& Tracker : Trackers)
		{
			const uint64 TrackerFlag = FMemoryTracker::AsFlag(Tracker->GetId());
			if ((Flags & TrackerFlag) != 0)
			{
				if (!Str.IsEmpty())
				{
					Str.Append(Conjunction);
				}
				Str.Append(Tracker->GetName());
				Flags &= ~TrackerFlag;
				if (Flags == 0)
				{
					break;
				}
			}
		}
	}
	return Str;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FMemoryTracker* FMemorySharedState::GetTrackerById(FMemoryTrackerId InMemTrackerId) const
{
	const TSharedPtr<FMemoryTracker>* TrackerPtr = Trackers.FindByPredicate([InMemTrackerId](TSharedPtr<FMemoryTracker>& Tracker) { return Tracker->GetId() == InMemTrackerId; });
	return TrackerPtr ? TrackerPtr->Get() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::SyncTrackers()
{
	DefaultTracker = nullptr;
	PlatformTracker = nullptr;
	Trackers.Reset();

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::IMemoryProvider* MemoryProvider = TraceServices::ReadMemoryProvider(*Session.Get());
		if (MemoryProvider)
		{
			{
				TraceServices::FProviderReadScopeLock _(*MemoryProvider);
				MemoryProvider->EnumerateTrackers([this](const TraceServices::FMemoryTrackerInfo& Tracker)
				{
					Trackers.Add(MakeShared<FMemoryTracker>(Tracker.Id, Tracker.Name));
				});
			}

			Trackers.Sort([](const TSharedPtr<FMemoryTracker>& A, const TSharedPtr<FMemoryTracker>& B) { return A->GetId() < B->GetId(); });
		}
	}

	if (Trackers.Num() > 0)
	{
		for (const TSharedPtr<FMemoryTracker>& Tracker : Trackers)
		{
			if (FCString::Stricmp(*Tracker->GetName(), TEXT("Default")) == 0)
			{
				DefaultTracker = Tracker;
			}
			if (FCString::Stricmp(*Tracker->GetName(), TEXT("Platform")) == 0)
			{
				PlatformTracker = Tracker;
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::UpdateTagSets()
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::IMemoryProvider* MemoryProvider = TraceServices::ReadMemoryProvider(*Session.Get());
		if (MemoryProvider)
		{
			TraceServices::FProviderReadScopeLock _(*MemoryProvider);

			const uint32 NumProviderTagSets = MemoryProvider->GetTagSetCount();
			if (NumProviderTagSets != NumSyncedTagSets)
			{
				NumSyncedTagSets = NumProviderTagSets;
				MemoryProvider->EnumerateTagSets([this](const TraceServices::FMemoryTagSetInfo& TagSet)
				{
					int32 Index = (int32)TagSet.Id;
					if (Index >= 0)
					{
						if (TagSets.Num() <= Index)
						{
							TagSets.AddDefaulted(Index - TagSets.Num() + 1);
						}
						if (!TagSets[Index].IsValid())
						{
							++NumValidTagSets;
						}
						TagSets[Index] = MakeShared<FMemoryTagSet>(static_cast<FMemoryTagSetId>(TagSet.Id), TagSet.Name);
					}
				});
			}
		}
	}
	else
	{
		TagSets.Reset();
		NumSyncedTagSets = 0;
		NumValidTagSets = 0;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::EnumerateTagSets(TFunctionRef<void(const FMemoryTagSet& InTagSet)> Callback) const
{
	for (const TSharedPtr<FMemoryTagSet>& TagSet : TagSets)
	{
		if (TagSet.IsValid())
		{
			Callback(*TagSet);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FMemoryTagSet* FMemorySharedState::GetTagSetById(FMemoryTagSetId InMemTagSetId) const
{
	return (int32)InMemTagSetId >= 0 && (int32)InMemTagSetId < TagSets.Num() ? TagSets[(int32)InMemTagSetId].Get() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::SetTrackHeightMode(EMemoryTrackHeightMode InTrackHeightMode)
{
	TrackHeightMode = InTrackHeightMode;

	for (TSharedPtr<FMemoryGraphTrack>& GraphTrack : AllTracks)
	{
		GraphTrack->SetCurrentTrackHeight(InTrackHeightMode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::ExtendOtherTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InOutMenuBuilder)
{
	TimingProfiler::STimingView* TimingView = GetTimingView().Get();
	if (&InSession != TimingView)
	{
		return;
	}

	InOutMenuBuilder.BeginSection("Memory", LOCTEXT("ContextMenu_Section_Memory", "Memory"));
	{
		InOutMenuBuilder.AddMenuEntry(FMemoryTimingViewCommands::Get().ShowHideAllMemoryTracks);
	}
	InOutMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::BindCommands()
{
	FMemoryTimingViewCommands::Register();

	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView)
	{
		return;
	}

	TSharedPtr<FUICommandList> CommandList = TimingView->GetCommandList();
	ensure(CommandList.IsValid());

	CommandList->MapAction(
		FMemoryTimingViewCommands::Get().ShowHideAllMemoryTracks,
		FExecuteAction::CreateSP(this, &FMemorySharedState::ShowHideAllMemoryTracks),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FMemorySharedState::IsAllMemoryTracksToggleOn));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::SetAllMemoryTracksToggle(bool bOnOff)
{
	bShowHideAllMemoryTracks = bOnOff;

	for (TSharedPtr<FMemoryGraphTrack>& GraphTrack : AllTracks)
	{
		GraphTrack->SetVisibilityFlag(bShowHideAllMemoryTracks);
	}

	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (TimingView)
	{
		TimingView->HandleTrackVisibilityChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FMemorySharedState::GetNextMemoryGraphTrackOrder()
{
	int32 Order = FTimingTrackOrder::Memory;
	for (const TSharedPtr<FMemoryGraphTrack>& GraphTrack : AllTracks)
	{
		Order = FMath::Max(Order, GraphTrack->GetOrder() + 1);
	}
	return Order;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FMemoryGraphTrack> FMemorySharedState::CreateMemoryGraphTrack()
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<FMemoryGraphTrack> GraphTrack = MakeShared<FMemoryGraphTrack>(*this);

	const int32 Order = GetNextMemoryGraphTrackOrder();
	GraphTrack->SetOrder(Order);
	GraphTrack->SetName(TEXT("Memory Graph"));
	GraphTrack->SetVisibilityFlag(bShowHideAllMemoryTracks);

	GraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Small, 100.0f);
	GraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Medium, 300.0f);
	GraphTrack->SetAvailableTrackHeight(EMemoryTrackHeightMode::Large, 600.0f);
	GraphTrack->SetCurrentTrackHeight(TrackHeightMode);

	GraphTrack->SetLabelUnit(EGraphTrackLabelUnit::MiB, 1);
	GraphTrack->EnableAutoZoom();

	TimingView->AddScrollableTrack(GraphTrack);
	AllTracks.Add(GraphTrack);

	return GraphTrack;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FMemorySharedState::RemoveMemoryGraphTrack(TSharedPtr<FMemoryGraphTrack> GraphTrack)
{
	if (!GraphTrack)
	{
		return 0;
	}

	if (GraphTrack == MainGraphTrack)
	{
		RemoveTrackFromMemTags(GraphTrack);
		GraphTrack->RemoveAllMemTagSeries();
		if (GraphTrack->GetSeries().Num() == 0)
		{
			GraphTrack->Hide();
			TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
			if (TimingView)
			{
				TimingView->HandleTrackVisibilityChanged();
			}
		}
		return -1;
	}

	if (AllTracks.Remove(GraphTrack))
	{
		RemoveTrackFromMemTags(GraphTrack);
		GraphTrack->RemoveAllMemTagSeries();
		TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
		if (TimingView)
		{
			TimingView->RemoveTrack(GraphTrack);
		}
		return 1;
	}

	return 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::RemoveTrackFromMemTags(TSharedPtr<FMemoryGraphTrack>& GraphTrack)
{
	for (TSharedPtr<FGraphSeries>& Series : GraphTrack->GetSeries())
	{
		if (Series->Is<FMemTagGraphSeries>())
		{
			FMemTagGraphSeries& MemTagSeries = Series->As<FMemTagGraphSeries>();
			FMemoryTag* TagPtr = TagList.GetTagById(MemTagSeries.GetTrackerId(), MemTagSeries.GetTagId());
			if (TagPtr)
			{
				TagPtr->RemoveTrack(GraphTrack);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FMemoryGraphTrack> FMemorySharedState::GetMemTagGraphTrack(FMemoryTrackerId InMemTrackerId, FMemoryTagId InMemTagId)
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return nullptr;
	}

	FMemoryTag* TagPtr = TagList.GetTagById(InMemTrackerId, InMemTagId);
	if (TagPtr)
	{
		for (TSharedPtr<FMemoryGraphTrack> MemoryGraph : TagPtr->GetGraphTracks())
		{
			if (MemoryGraph != MainGraphTrack &&
				MemoryGraph->GetSeries().Num() == 1 &&
				MemoryGraph->GetSeries()[0]->Is<FMemTagGraphSeries>())
			{
				return MemoryGraph;
			}
		}
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FMemoryGraphTrack> FMemorySharedState::CreateMemTagGraphTrack(FMemoryTrackerId InMemTrackerId, FMemoryTagId InMemTagId)
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return nullptr;
	}

	FMemoryTag* TagPtr = TagList.GetTagById(InMemTrackerId, InMemTagId);
	FMemoryTagSetId MemTagSetId = FMemoryTagSet::InvalidTagSetId;

	FString SeriesName;
	if (TagPtr)
	{
		const FMemoryTracker* Tracker = GetTrackerById(InMemTrackerId);
		if (Tracker && Tracker != DefaultTracker.Get())
		{
			SeriesName = FString::Printf(TEXT("LLM %s (%s)"), *TagPtr->GetStatFullName(), *Tracker->GetName());
		}
		else
		{
			SeriesName = FString::Printf(TEXT("LLM %s"), *TagPtr->GetStatFullName());
		}
		MemTagSetId = TagPtr->GetTagSetId();
	}
	else
	{
		SeriesName = FString::Printf(TEXT("Unknown LLM Tag (tag id: 0x%llX, tracker id: %i)"), uint64(InMemTagId), int32(InMemTrackerId));
	}

	const FLinearColor Color = TagPtr ? TagPtr->GetColor() : FLinearColor(0.5f, 0.5f, 0.5f, 1.0f);
	const FLinearColor BorderColor(FMath::Min(Color.R + 0.4f, 1.0f), FMath::Min(Color.G + 0.4f, 1.0f), FMath::Min(Color.B + 0.4f, 1.0f), 1.0f);

	// Also create a series in the MainGraphTrack.
	if (MainGraphTrack.IsValid())
	{
		TSharedPtr<FMemoryGraphSeries> Series = MainGraphTrack->AddMemTagSeries(InMemTrackerId, MemTagSetId, InMemTagId);
		Series->SetName(SeriesName);
		Series->SetColor(Color, BorderColor, Color.CopyWithNewOpacity(0.1f));
		Series->DisableAutoZoom();
		Series->SetScaleY(0.0000002);

		if (TagPtr)
		{
			TagPtr->AddTrack(MainGraphTrack);
		}

		MainGraphTrack->Show();
		TimingView->HandleTrackVisibilityChanged();
	}

	TSharedPtr<FMemoryGraphTrack> GraphTrackPtr = GetMemTagGraphTrack(InMemTrackerId, InMemTagId);

	if (!GraphTrackPtr.IsValid())
	{
		// Create a new Graph track.
		GraphTrackPtr = MakeShared<FMemoryGraphTrack>(*this);
		FMemoryGraphTrack& GraphTrack = *GraphTrackPtr;

		GraphTrack.SetVisibleOptions(GraphTrack.GetVisibleOptions() | EGraphOptions::AutoZoomIncludesBaseline | EGraphOptions::AutoZoomIncludesThresholds);
		GraphTrack.SetEditableOptions(GraphTrack.GetEditableOptions() | EGraphOptions::AutoZoomIncludesBaseline | EGraphOptions::AutoZoomIncludesThresholds);
		GraphTrack.SetEnabledOptions(GraphTrack.GetEnabledOptions() | EGraphOptions::ShowThresholds);

		const int32 Order = GetNextMemoryGraphTrackOrder();
		GraphTrack.SetOrder(Order);
		GraphTrack.SetName(SeriesName);
		GraphTrack.Show();

		GraphTrack.SetAvailableTrackHeight(EMemoryTrackHeightMode::Small, 32.0f);
		GraphTrack.SetAvailableTrackHeight(EMemoryTrackHeightMode::Medium, 100.0f);
		GraphTrack.SetAvailableTrackHeight(EMemoryTrackHeightMode::Large, 200.0f);
		GraphTrack.SetCurrentTrackHeight(TrackHeightMode);

		GraphTrack.EnableAutoZoom();

		// Create a new MemTag graph series.
		TSharedPtr<FMemTagGraphSeries> Series = GraphTrack.AddMemTagSeries(InMemTrackerId, MemTagSetId, InMemTagId);
		Series->SetName(SeriesName);
		Series->SetColor(Color, BorderColor);
		Series->SetBaselineY(GraphTrack.GetHeight() - 1.0f);
		Series->EnableAutoZoom();

		if (TagPtr)
		{
			TagPtr->AddTrack(GraphTrackPtr);
		}

		// Add the new Graph to the TimingView as a scrollable track.
		TimingView->AddScrollableTrack(GraphTrackPtr);

		AllTracks.Add(GraphTrackPtr);
	}
	else
	{
		GraphTrackPtr->Show();
		TimingView->HandleTrackVisibilityChanged();
	}

	return GraphTrackPtr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FMemorySharedState::RemoveMemTagGraphTrack(FMemoryTrackerId InMemTrackerId, FMemoryTagId InMemTagId)
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return -1;
	}

	int32 TrackCount = 0;

	FMemoryTag* TagPtr = TagList.GetTagById(InMemTrackerId, InMemTagId);
	if (TagPtr)
	{
		for (TSharedPtr<FMemoryGraphTrack> GraphTrack : TagPtr->GetGraphTracks())
		{
			GraphTrack->RemoveMemTagSeries(InMemTrackerId, TagPtr->GetTagSetId(), InMemTagId);
			if (GraphTrack->GetSeries().Num() == 0)
			{
				if (GraphTrack == MainGraphTrack)
				{
					GraphTrack->Hide();
					TimingView->HandleTrackVisibilityChanged();
				}
				else
				{
					++TrackCount;
					AllTracks.Remove(GraphTrack);
					TimingView->RemoveTrack(GraphTrack);
				}
			}
		}
		TagPtr->RemoveAllTracks();
	}

	return TrackCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FMemorySharedState::RemoveAllMemTagGraphTracks()
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return -1;
	}

	int32 TrackCount = 0;

	TArray<TSharedPtr<FMemoryGraphTrack>> TracksToRemove;
	for (TSharedPtr<FMemoryGraphTrack>& GraphTrack : AllTracks)
	{
		GraphTrack->RemoveAllMemTagSeries();
		if (GraphTrack->GetSeries().Num() == 0)
		{
			if (GraphTrack == MainGraphTrack)
			{
				GraphTrack->Hide();
				TimingView->HandleTrackVisibilityChanged();
			}
			else
			{
				++TrackCount;
				TimingView->RemoveTrack(GraphTrack);
				TracksToRemove.Add(GraphTrack);
			}
		}
	}
	for (TSharedPtr<FMemoryGraphTrack>& GraphTrack : TracksToRemove)
	{
		AllTracks.Remove(GraphTrack);
	}

	for (FMemoryTag* TagPtr : TagList.GetTags())
	{
		TagPtr->RemoveAllTracks();
	}

	return TrackCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FMemoryGraphSeries> FMemorySharedState::ToggleMemTagGraphSeries(TSharedPtr<FMemoryGraphTrack> InGraphTrack, FMemoryTrackerId InMemTrackerId, FMemoryTagId InMemTagId)
{
	if (!InGraphTrack.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return nullptr;
	}

	FMemoryTag* TagPtr = TagList.GetTagById(InMemTrackerId, InMemTagId);
	FMemoryTagSetId TagSetId = TagPtr ? TagPtr->GetTagSetId() : FMemoryTagSet::InvalidTagSetId;

	TSharedPtr<FMemoryGraphSeries> Series = InGraphTrack->GetMemTagSeries(InMemTrackerId, TagSetId, InMemTagId);
	if (Series.IsValid())
	{
		// Remove existing series.
		InGraphTrack->RemoveMemTagSeries(InMemTrackerId, TagSetId, InMemTagId);
		InGraphTrack->SetDirtyFlag();
		TimingView->HandleTrackVisibilityChanged();

		if (TagPtr)
		{
			TagPtr->RemoveTrack(InGraphTrack);
		}

		return nullptr;
	}
	else
	{
		// Add new series.
		Series = InGraphTrack->AddMemTagSeries(InMemTrackerId, TagSetId, InMemTagId);
		Series->DisableAutoZoom();

		if (TagPtr)
		{
			TagPtr->AddTrack(InGraphTrack);
		}

		InGraphTrack->SetDirtyFlag();
		InGraphTrack->Show();
		TimingView->HandleTrackVisibilityChanged();

		return Series;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// Create graphs from LLMReportTypes.xml file
void FMemorySharedState::CreateTracksFromReport(const FString& Filename)
{
	FReportConfig ReportConfig;

	FReportXmlParser ReportXmlParser;

	ReportXmlParser.LoadReportTypesXML(ReportConfig, Filename);
	if (ReportXmlParser.GetStatus() != FReportXmlParser::EStatus::Completed)
	{
		FMessageLog ReportMessageLog(FMemoryProfilerManager::Get()->GetLogListingName());
		ReportMessageLog.AddMessages(ReportXmlParser.GetErrorMessages());
		ReportMessageLog.Notify();
	}

	CreateTracksFromReport(ReportConfig);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::CreateTracksFromReport(const FReportConfig& ReportConfig)
{
	for (const FReportTypeConfig& ReportTypeConfig : ReportConfig.ReportTypes)
	{
		CreateTracksFromReport(ReportTypeConfig);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::CreateTracksFromReport(const FReportTypeConfig& ReportTypeConfig)
{
	int32 Order = GetNextMemoryGraphTrackOrder();
	int32 NumAddedTracks = 0;

	const bool bIsPlatformTracker = ReportTypeConfig.Name.StartsWith(TEXT("LLMPlatform"));

	for (const FReportTypeGraphConfig& ReportTypeGraphConfig : ReportTypeConfig.Graphs)
	{
		TSharedPtr<FMemoryGraphTrack> GraphTrack = CreateGraphTrack(ReportTypeGraphConfig, bIsPlatformTracker);
		if (GraphTrack)
		{
			GraphTrack->SetOrder(Order++);
			++NumAddedTracks;
		}
	}

	if (NumAddedTracks > 0)
	{
		TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
		if (TimingView)
		{
			TimingView->InvalidateScrollableTracksOrder();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FMemoryGraphTrack> FMemorySharedState::CreateGraphTrack(const FReportTypeGraphConfig& ReportTypeGraphConfig, bool bIsPlatformTracker)
{
	if (ReportTypeGraphConfig.GraphConfig == nullptr)
	{
		// Invalid graph config.
		return nullptr;
	}

	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return nullptr;
	}

	const FGraphConfig& GraphConfig = *ReportTypeGraphConfig.GraphConfig;

	int32 CharIndex;
	const TCHAR* DelimStr;

	if (GraphConfig.StatString.FindChar(TEXT(','), CharIndex))
	{
		DelimStr = TEXT(",");
	}
	else if (GraphConfig.StatString.FindChar(TEXT(';'), CharIndex))
	{
		DelimStr = TEXT(";");
	}
	else
	{
		DelimStr = TEXT(" ");
	}
	TArray<FString> IncludeStats;
	GraphConfig.StatString.ParseIntoArray(IncludeStats, DelimStr);

	if (IncludeStats.Num() == 0)
	{
		// No stats specified!?
		return nullptr;
	}

	if (GraphConfig.IgnoreStats.FindChar(TEXT(';'), CharIndex))
	{
		DelimStr = TEXT(";");
	}
	else if (GraphConfig.IgnoreStats.FindChar(TEXT(','), CharIndex))
	{
		DelimStr = TEXT(",");
	}
	else
	{
		DelimStr = TEXT(" ");
	}
	TArray<FString> IgnoreStats;
	GraphConfig.IgnoreStats.ParseIntoArray(IgnoreStats, DelimStr);

	TArray<FMemoryTag*> Tags;
	TagList.FilterTags(IncludeStats, IgnoreStats, Tags);

	FMemoryTrackerId MemTrackerId = bIsPlatformTracker ?
		(PlatformTracker ? PlatformTracker->GetId() : FMemoryTracker::InvalidTrackerId) :
		(DefaultTracker ? DefaultTracker->GetId() : FMemoryTracker::InvalidTrackerId);

	TSharedPtr<FMemoryGraphTrack> GraphTrack = CreateMemoryGraphTrack();
	if (GraphTrack)
	{
		if (GraphConfig.Height > 0.0f)
		{
			constexpr float MinGraphTrackHeight = 32.0f;
			constexpr float MaxGraphTrackHeight = 600.0f;
			GraphTrack->SetHeight(FMath::Clamp(GraphConfig.Height, MinGraphTrackHeight, MaxGraphTrackHeight));
		}

		GraphTrack->SetName(ReportTypeGraphConfig.Title);

		const double MinValue = GraphConfig.MinY * 1024.0 * 1024.0;
		const double MaxValue = GraphConfig.MaxY * 1024.0 * 1024.0;
		GraphTrack->SetDefaultValueRange(MinValue, MaxValue);

		UE_LOGF(LogMemoryProfiler, Log, "[Memory] Created graph \"%ls\" (H=%.1f%ls, MainStat=%ls, Stats=%ls)",
			*ReportTypeGraphConfig.Title,
			GraphTrack->GetHeight(),
			GraphConfig.bStacked ? TEXT(", stacked") : TEXT(""),
			*GraphConfig.MainStat,
			*GraphConfig.StatString);

		TSharedPtr<FMemoryGraphSeries> MainSeries;

		for (FMemoryTag* TagPtr : Tags)
		{
			FMemoryTag& Tag = *TagPtr;

			TSharedPtr<FMemoryGraphSeries> Series = GraphTrack->AddMemTagSeries(MemTrackerId, Tag.GetTagSetId(), Tag.GetId());
			Series->SetName(FText::FromString(FString::Printf(TEXT("LLM %s"), *Tag.GetStatFullName())));
			const FLinearColor Color = Tag.GetColor();
			const FLinearColor BorderColor(FMath::Min(Color.R + 0.4f, 1.0f), FMath::Min(Color.G + 0.4f, 1.0f), FMath::Min(Color.B + 0.4f, 1.0f), 1.0f);
			Series->SetColor(Color, BorderColor);

			Tag.AddTrack(MainGraphTrack);

			if (GraphConfig.MainStat == Tag.GetStatName())
			{
				MainSeries = Series;
			}
		}

		if (GraphConfig.bStacked)
		{
			GraphTrack->SetStacked(true);
			GraphTrack->SetMainSeries(MainSeries);
		}
	}

	return GraphTrack;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::InitMemoryRules()
{
	using ERule = TraceServices::IAllocationsProvider::EQueryRule;

	MemoryRules.Reset();

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::aAf, 1,
		LOCTEXT("MemRule_aAf_Short", "*A*"),
		LOCTEXT("MemRule_aAf_Verbose", "Active Allocs"),
		LOCTEXT("MemRule_aAf_Desc", "Identifies active allocations at time A.\n(a ≤ A ≤ f)")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::afA, 1,
		LOCTEXT("MemRule_afA_Short", "**A"),
		LOCTEXT("MemRule_afA_Verbose", "Before"),
		LOCTEXT("MemRule_afA_Desc", "Identifies allocations allocated and freed before time A.\n(a ≤ f ≤ A)")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::Aaf, 1,
		LOCTEXT("MemRule_Aaf_Short", "A**"),
		LOCTEXT("MemRule_Aaf_Verbose", "After"),
		LOCTEXT("MemRule_Aaf_Desc", "Identifies allocations allocated after time A.\n(A ≤ a ≤ f)")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::aAfB, 2,
		LOCTEXT("MemRule_aAfB_Short", "*A*B"),
		LOCTEXT("MemRule_aAfB_Verbose", "Decline"),
		LOCTEXT("MemRule_aAfB_Desc", "Identifies allocations allocated before time A and freed between time A and time B.\n(a ≤ A ≤ f ≤ B)")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::AaBf, 2,
		LOCTEXT("MemRule_AaBf_Short", "A*B*"),
		LOCTEXT("MemRule_AaBf_Verbose", "Growth"),
		LOCTEXT("MemRule_AaBf_Desc", "Identifies allocations allocated between time A and time B and not freed until at least time B.\n(A ≤ a ≤ B ≤ f)")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::aAfaBf, 2,
		LOCTEXT("MemRule_aAfaBf_Short", "A*B*/*A*B"),
		LOCTEXT("MemRule_aAfaBf_Verbose", "Growth vs. Decline"),
		LOCTEXT("MemRule_aAfaBf_Desc", "Identifies \"growth\" allocations, allocated between time A and time B and not freed until at least time B (A ≤ a ≤ B ≤ f)\nand \"decline\" allocations, allocated before time A and freed between time A and time B (a ≤ A ≤ f ≤ B).\nThe \"decline\" allocations are changed to have negative size, so the size aggregation shows variation between A and B.")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::AfB, 2,
		LOCTEXT("MemRule_AfB_Short", "*A**B"),
		LOCTEXT("MemRule_AfB_Verbose", "Free Events"),
		LOCTEXT("MemRule_AfB_Desc", "Identifies allocations freed between time A and time B.\n(A ≤ f ≤ B)")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::AaB, 2,
		LOCTEXT("MemRule_AaB_Short", "A**B*"),
		LOCTEXT("MemRule_AaB_Verbose", "Alloc Events"),
		LOCTEXT("MemRule_AaB_Desc", "Identifies allocations allocated between time A and time B.\n(A ≤ a ≤ B)")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::AafB, 2,
		LOCTEXT("MemRule_AafB_Short", "A**B"),
		LOCTEXT("MemRule_AafB_Verbose", "Short Living Allocs"),
		LOCTEXT("MemRule_AafB_Desc", "Identifies allocations allocated and freed between time A and time B.\n(A ≤ a ≤ f ≤ B)")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::aABf, 2,
		LOCTEXT("MemRule_aABf_Short", "*A B*"),
		LOCTEXT("MemRule_aABf_Verbose", "Long Living Allocs"),
		LOCTEXT("MemRule_aABf_Desc", "Identifies allocations allocated before time A and not freed until at least time B.\n(a ≤ A ≤ B ≤ f)")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::AaBCf, 3,
		LOCTEXT("MemRule_AaBCf_Short", "A*B C*"),
		LOCTEXT("MemRule_AaBCf_Verbose", "Memory Leaks"),
		LOCTEXT("MemRule_AaBCf_Desc", "Identifies allocations allocated between time A and time B and not freed until at least time C.\n(A ≤ a ≤ B ≤ C ≤ f)")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::AaBfC, 3,
		LOCTEXT("MemRule_AaBfC_Short", "A*B*C"),
		LOCTEXT("MemRule_AaBfC_Verbose", "Limited Lifetime"),
		LOCTEXT("MemRule_AaBfC_Desc", "Identifies allocations allocated between time A and time B and freed between time B and time C.\n(A ≤ a ≤ B ≤ f ≤ C)")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::aABfC, 3,
		LOCTEXT("MemRule_aABfC_Short", "*A B*C"),
		LOCTEXT("MemRule_aABfC_Verbose", "Decline of Long Living Allocs"),
		LOCTEXT("MemRule_aABfC_Desc", "Identifies allocations allocated before time A and freed between time B and time C.\n(a ≤ A ≤ B ≤ f ≤ C)")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::AaBCfD, 4,
		LOCTEXT("MemRule_AaBCfD_Short", "A*B C*D"),
		LOCTEXT("MemRule_AaBCfD_Verbose", "Specific Lifetime"),
		LOCTEXT("MemRule_AaBCfD_Desc", "Identifies allocations allocated between time A and time B and freed between time C and time D.\n(A ≤ a ≤ B ≤ C ≤ f ≤ D)")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::AoB, 2,
		LOCTEXT("MemRule_AoB_Short", "A ↓ B"),
		LOCTEXT("MemRule_AoB_Verbose", "Paged-Out Allocs"),
		LOCTEXT("MemRule_AoB_Desc", "Identifies allocations paged-out (swapped-out) between time A and time B.\n(A ≤ page-out ≤ B)")));

	MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
		ERule::AiB, 2,
		LOCTEXT("MemRule_AiB_Short", "A ↑ B"),
		LOCTEXT("MemRule_AiB_Verbose", "Paged-In Allocs"),
		LOCTEXT("MemRule_AiB_Desc", "Identifies allocations paged-in (swapped-in) between time A and time B.\n(A ≤ page-in ≤ B)")));

	//TODO
	//MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
	//	ERule::A_vs_B, 2,
	//	LOCTEXT("MemRule_A_vs_B_Short", "*A* + *B*"),
	//	LOCTEXT("MemRule_A_vs_B_Verbose", "Compare A vs. B"),
	//	LOCTEXT("MemRule_A_vs_B_Desc", "Compares live allocations at time A with live allocations at time B.\n(*A* vs. *B*)")));

	//TODO
	//MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
	//	ERule::A_or_B, 2,
	//	LOCTEXT("MemRule_A_or_B_Short", "*A* | *B*"),
	//	LOCTEXT("MemRule_A_or_B_Verbose", "A or B"),
	//	LOCTEXT("MemRule_A_or_B_Desc", "Identifies allocations live at time A or at time B.\n(a ≤ A ≤ f OR a ≤ B ≤ f)\n{*A*} U {*B*}")));

	//TODO
	//MemoryRules.Add(MakeShared<FMemoryRuleSpec>(
	//	ERule::A_xor_B, 2,
	//	LOCTEXT("MemRule_A_xor_B_Short", "*A* ^ *B*"),
	//	LOCTEXT("MemRule_A_xor_B_Verbose", "A xor B"),
	//	LOCTEXT("MemRule_A_xor_B_Desc", "Identifies allocations live either at time A or at time B (but not both).\n(a ≤ A ≤ f XOR a ≤ B ≤ f)\n({*A*} U {*B*}) \\ {*AB*}")));

	CurrentMemoryRule = MemoryRules[0];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::OnMemoryRuleChanged()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemorySharedState::AddQueryTarget(TSharedPtr<FQueryTargetWindowSpec> InPtr)
{
	QueryTargetSpecs.Add(InPtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////


void FMemorySharedState::RemoveQueryTarget(TSharedPtr<FQueryTargetWindowSpec> InPtr)
{
	QueryTargetSpecs.Remove(InPtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
