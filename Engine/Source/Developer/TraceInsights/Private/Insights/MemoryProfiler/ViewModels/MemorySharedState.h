// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Framework/Commands/Commands.h"

// TraceServices
#include "TraceServices/Model/AllocationsProvider.h"

// TraceInsights
#include "Insights/ITimingViewExtender.h"
#include "Insights/MemoryProfiler/Tracks/MemoryGraphTrack.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryTag.h"

namespace UE::Insights::TimingProfiler { class STimingView; }

namespace UE::Insights::MemoryProfiler
{

struct FReportConfig;
struct FReportTypeConfig;
struct FReportTypeGraphConfig;
class FMemoryTracker;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemoryRuleSpec
{
public:
	typedef TraceServices::IAllocationsProvider::EQueryRule ERule;

public:
	FMemoryRuleSpec(ERule InValue, uint32 InNumTimeMarkers, const FText& InShortName, const FText& InVerboseName, const FText& InDescription)
		: Value(InValue)
		, NumTimeMarkers(InNumTimeMarkers)
		, ShortName(InShortName)
		, VerboseName(InVerboseName)
		, Description(InDescription)
	{}

	ERule GetValue() const { return Value; }
	uint32 GetNumTimeMarkers() const { return NumTimeMarkers; }
	FText GetShortName() const { return ShortName; }
	FText GetVerboseName() const { return VerboseName; }
	FText GetDescription() const { return Description; }

private:
	ERule Value;           // ex.: ERule::AafB
	uint32 NumTimeMarkers; // ex.: 2
	FText ShortName;       // ex.: "A**B"
	FText VerboseName;     // ex.: "Short Living Allocations"
	FText Description;     // ex.: "Allocations allocated and freed between time A and time B (A <= a <= f <= B)."
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FQueryTargetWindowSpec
{
public:
	FQueryTargetWindowSpec(const FName& InName, const FText& InText)
		: Text(InText)
		, Name(InName)
	{}

	FText GetText() const { return Text; }
	FName GetName() const { return Name; }

public:
	static const FName NewWindow;

private:
	FText Text;
	FName Name;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemoryTimingViewCommands : public TCommands<FMemoryTimingViewCommands>
{
public:
	FMemoryTimingViewCommands();
	virtual ~FMemoryTimingViewCommands();
	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> ShowHideAllMemoryTracks;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemorySharedState : public Timing::ITimingViewExtender, public TSharedFromThis<FMemorySharedState>
{
public:
	FMemorySharedState();
	virtual ~FMemorySharedState();

	//////////////////////////////////////////////////
	// Memory Tags

	void ResetMemoryTags();
	void UpdateMemoryTags();

	const FMemoryTagList& GetTagList() { return TagList; }

	const TArray<TSharedPtr<FMemoryTracker>>& GetTrackers() const { return Trackers; }
	FString TrackersToString(uint64 Flags, const TCHAR* Conjunction) const;
	const FMemoryTracker* GetTrackerById(FMemoryTrackerId InMemTrackerId) const;
	FMemoryTrackerId GetDefaultTrackerId() const
	{
		return DefaultTracker.IsValid() ? DefaultTracker->GetId() : FMemoryTracker::InvalidTrackerId;
	}
	FMemoryTrackerId GetPlatformTrackerId() const
	{
		return PlatformTracker.IsValid() ? PlatformTracker->GetId() : FMemoryTracker::InvalidTrackerId;
	}

	uint32 GetNumTagSets() const { return NumValidTagSets; }
	void EnumerateTagSets(TFunctionRef<void(const FMemoryTagSet& InTagSet)> Callback) const;
	const FMemoryTagSet* GetTagSetById(FMemoryTagSetId InMemTagSetId) const;

	//////////////////////////////////////////////////
	// ITimingViewExtender interface

	virtual void OnBeginSession(Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(Timing::ITimingViewSession& InSession) override;
	virtual void Tick(Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;
	virtual void ExtendOtherTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InOutMenuBuilder) override;

	//////////////////////////////////////////////////
	// Timing View

	void ResetTimingView();

	TSharedPtr<TimingProfiler::STimingView> GetTimingView() const
	{
		return TimingViewWeak.Pin();
	}
	void SetTimingView(TWeakPtr<TimingProfiler::STimingView> InTimingView)
	{
		TimingViewWeak = InTimingView;
		ResetTimingView();
	}

	TSharedPtr<FMemoryGraphTrack> GetMainGraphTrack() const { return MainGraphTrack; }

	EMemoryTrackHeightMode GetTrackHeightMode() const { return TrackHeightMode; }
	void SetTrackHeightMode(EMemoryTrackHeightMode InTrackHeightMode);

	bool IsAllMemoryTracksToggleOn() const { return bShowHideAllMemoryTracks; }
	void SetAllMemoryTracksToggle(bool bOnOff);
	void ShowAllMemoryTracks() { SetAllMemoryTracksToggle(true); }
	void HideAllMemoryTracks() { SetAllMemoryTracksToggle(false); }
	void ShowHideAllMemoryTracks() { SetAllMemoryTracksToggle(!IsAllMemoryTracksToggleOn()); }

	void CreateDefaultTracks();

	TSharedPtr<FMemoryGraphTrack> CreateMemoryGraphTrack();
	int32 RemoveMemoryGraphTrack(TSharedPtr<FMemoryGraphTrack> GraphTrack);

	TSharedPtr<FMemoryGraphTrack> GetMemTagGraphTrack(FMemoryTrackerId InMemTrackerId, FMemoryTagId InMemTagId);
	TSharedPtr<FMemoryGraphTrack> CreateMemTagGraphTrack(FMemoryTrackerId InMemTrackerId, FMemoryTagId InMemTagId);
	void RemoveTrackFromMemTags(TSharedPtr<FMemoryGraphTrack>& GraphTrack);
	int32 RemoveMemTagGraphTrack(FMemoryTrackerId InMemTrackerId, FMemoryTagId InMemTagId);
	int32 RemoveAllMemTagGraphTracks();
	void ResetAllMemTagGraphTracks();

	TSharedPtr<FMemoryGraphSeries> ToggleMemTagGraphSeries(TSharedPtr<FMemoryGraphTrack> InGraphTrack, FMemoryTrackerId InMemTrackerId, FMemoryTagId InMemTagId);

	//////////////////////////////////////////////////
	// Memory Investigation

	const TArray<TSharedPtr<FMemoryRuleSpec>>& GetMemoryRules() const { return MemoryRules; }

	TSharedPtr<FMemoryRuleSpec> GetCurrentMemoryRule() const { return CurrentMemoryRule; }
	void SetCurrentMemoryRule(TSharedPtr<FMemoryRuleSpec> InRule) { CurrentMemoryRule = InRule; OnMemoryRuleChanged(); }

	const TArray<TSharedPtr<FQueryTargetWindowSpec>>& GetQueryTargets() const { return QueryTargetSpecs; }

	TSharedPtr<FQueryTargetWindowSpec> GetCurrentQueryTarget() const { return CurrentQueryTarget; }
	void SetCurrentQueryTarget(TSharedPtr<FQueryTargetWindowSpec> InTarget) { CurrentQueryTarget = InTarget; }
	void AddQueryTarget(TSharedPtr<FQueryTargetWindowSpec> InPtr);
	void RemoveQueryTarget(TSharedPtr<FQueryTargetWindowSpec> InPtr);

	//////////////////////////////////////////////////
	// Misc

	void BindCommands();

	void CreateTracksFromReport(const FString& Filename);
	void CreateTracksFromReport(const FReportConfig& ReportConfig);
	void CreateTracksFromReport(const FReportTypeConfig& ReportTypeConfig);

	//////////////////////////////////////////////////

private:
	void SyncTrackers();
	void UpdateTagSets();
	int32 GetNextMemoryGraphTrackOrder();
	TSharedPtr<FMemoryGraphTrack> CreateGraphTrack(const FReportTypeGraphConfig& ReportTypeGraphConfig, bool bIsPlatformTracker);
	void InitMemoryRules();
	void OnMemoryRuleChanged();

private:
	//////////////////////////////////////////////////
	// Memory Tags

	FMemoryTagList TagList;

	TArray<TSharedPtr<FMemoryTracker>> Trackers;
	TSharedPtr<FMemoryTracker> DefaultTracker;
	TSharedPtr<FMemoryTracker> PlatformTracker;

	TArray<TSharedPtr<FMemoryTagSet>> TagSets;
	uint32 NumSyncedTagSets = 0;
	uint32 NumValidTagSets = 0;

	//////////////////////////////////////////////////
	// Timing View

	TWeakPtr<TimingProfiler::STimingView> TimingViewWeak;

	TSharedPtr<FMemoryGraphTrack> MainGraphTrack; // the Main Memory Graph track; also hosts the Total Allocated Memory series
	TSharedPtr<FMemoryGraphTrack> LiveAllocsGraphTrack; // the graph track for the Live Allocation Count series
	TSharedPtr<FMemoryGraphTrack> AllocFreeGraphTrack; // the graph track for the Alloc Event Count and the Free Event Count series
	TSharedPtr<FMemoryGraphTrack> SwapMemoryGraphTrack; // the swap memory graph for Total Swap Memory and Total Compressed Swap Memory series
	TSharedPtr<FMemoryGraphTrack> PageSwapGraphTrack; // the graph track for the Page In Event Count and the Page Out Event Count series
	TSet<TSharedPtr<FMemoryGraphTrack>> AllTracks;

	EMemoryTrackHeightMode TrackHeightMode = EMemoryTrackHeightMode::Medium;

	bool bShowHideAllMemoryTracks = false;

	TBitArray<> CreatedDefaultTracks;
	uint32 LastTagCountForDefaultTracks = 0;

	//////////////////////////////////////////////////
	// Memory Investigation

	TArray<TSharedPtr<FMemoryRuleSpec>> MemoryRules;
	TSharedPtr<FMemoryRuleSpec> CurrentMemoryRule;

	TSharedPtr<FQueryTargetWindowSpec> CurrentQueryTarget;
	TArray<TSharedPtr<FQueryTargetWindowSpec>> QueryTargetSpecs;

	//////////////////////////////////////////////////
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
