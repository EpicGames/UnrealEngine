// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryTag.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/Memory.h"

// TraceInsights
#include "Insights/InsightsManager.h"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemoryTag
////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryTag::SetColorAuto()
{
	uint32 Hash = 49;
	for (const TCHAR* c = *StatName; *c; ++c)
	{
		Hash = ((Hash << 5) + Hash + *c) * 0xfb23618f;
	}

	const uint8 H = Hash & 0xFF;
	const uint8 S = 155 + ((Hash >> 8) & 0xFF) * (255 - 155) / 255;
	const uint8 V = 128 + ((Hash >> 16) & 0x7F);
	Color = FLinearColor::MakeFromHSV8(H, S, V);
	Color.A = 1.0f;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryTag::SetRandomColor()
{
	uint64 Time = FPlatformTime::Cycles64();
	uint32 Hash = (Time & 0xFFFFFFFF) ^ (Time >> 32);
	Hash = ((Hash << 5) + Hash) * 0xfb23618f;

	const uint8 H = Hash & 0xFF;
	const uint8 S = 128 + ((Hash >> 8) & 0x7F);
	const uint8 V = 128 + ((Hash >> 16) & 0x7F);
	Color = FLinearColor::MakeFromHSV8(H, S, V);
	Color.A = 1.0f;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemoryTag::MatchesWildcard(const FString& FullName) const
{
	return StatFullName.MatchesWildcard(FullName);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMemoryTag::MatchesWildcard(const TArray<FString>& FullNames) const
{
	for (const FString& FullName : FullNames)
	{
		if (StatFullName.MatchesWildcard(FullName))
		{
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemoryTagList
////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryTagList::FMemoryTagList()
	: LastTraceSerialNumber(0)
	, SerialNumber(0)
	, NextUpdateTimestamp(0)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryTagList::~FMemoryTagList()
{
	Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryTag* FMemoryTagList::GetTagById(FMemoryTrackerId InTrackerId, FMemoryTagId InTagId) const
{
	const TMap<FMemoryTagId, FMemoryTag*>* TagsMapPtr = TrackersAndTagsMap.Find(InTrackerId);
	if (TagsMapPtr)
	{
		return TagsMapPtr->FindRef(InTagId);
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryTagList::Reset()
{
	for (FMemoryTag* TagPtr : Tags)
	{
		delete TagPtr;
	}
	Tags.Reset();
	TrackersAndTagsMap.Reset();

	LastTraceSerialNumber = 0;
	SerialNumber = 0;
	NextUpdateTimestamp = 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryTagList::Update()
{
	// We need to check if the list of LLM tags has changed.
	// But, ensure we do not check too often.
	const uint64 Time = FPlatformTime::Cycles64();
	if (Time > NextUpdateTimestamp)
	{
		UpdateInternal();

		// 1000 tags --> check each 150ms
		// 10000 tags --> check each 600ms
		// 100000 tags --> check each 5.1s
		const double WaitTimeSec = 0.1 + Tags.Num() / 20000.0;
		const uint64 WaitTime = static_cast<uint64>(WaitTimeSec / FPlatformTime::GetSecondsPerCycle64());
		NextUpdateTimestamp = Time + WaitTime;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryTagList::UpdateInternal()
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	const TraceServices::IMemoryProvider* MemoryProvider = TraceServices::ReadMemoryProvider(*Session.Get());
	if (!MemoryProvider)
	{
		return;
	}

	TraceServices::FProviderReadScopeLock _(*MemoryProvider);

	const int32 TraceSerialNumber = MemoryProvider->GetTagSerial();
	if (LastTraceSerialNumber == TraceSerialNumber)
	{
		return;
	}
	LastTraceSerialNumber = TraceSerialNumber;

	uint32 AddedOrUpdatedTags = 0;

	MemoryProvider->EnumerateTags([this, &AddedOrUpdatedTags](const TraceServices::FMemoryTagInfo& TraceTag)
	{
		static_assert(FMemoryTag::InvalidTagId == static_cast<FMemoryTagId>(TraceServices::FMemoryTagInfo::InvalidTagId), "Memory TagId type mismatch!");
		FMemoryTagId TagId = static_cast<FMemoryTagId>(TraceTag.Id);

		uint64 TrackerFlags = TraceTag.Trackers;
		int32 TrackerId = 0;
		while (TrackerFlags != 0)
		{
			if (TrackerFlags & 1)
			{
				TMap<FMemoryTagId, FMemoryTag*>& TagsMap = TrackersAndTagsMap.FindOrAdd(FMemoryTrackerId(TrackerId));
				FMemoryTag* TagPtr = TagsMap.FindRef(TagId);
				if (!TagPtr)
				{
					TagPtr = new FMemoryTag();

					FMemoryTag& Tag = *TagPtr;
					Tag.Index = Tags.Num();
					Tag.Id = TagId;
					Tag.ParentId = static_cast<FMemoryTagId>(TraceTag.ParentId);
					Tag.StatName = TraceTag.Name;
					Tag.StatFullName = TraceTag.Name;
					Tag.TrackerId = static_cast<FMemoryTrackerId>(TrackerId);
					Tag.TagSetId = static_cast<FMemoryTagSetId>(TraceTag.TagSetId);
					Tag.SetColorAuto();

					// Skip the parent prefix if it is already included in the name.
					int32 CharIndex;
					if (Tag.StatName.FindLastChar(TEXT('/'), CharIndex))
					{
						Tag.StatName = Tag.StatName.RightChop(CharIndex + 1);
					}

					Tags.Add(TagPtr);
					TagsMap.Add(TagId, TagPtr);
					++AddedOrUpdatedTags;
				}
			}
			TrackerFlags >>= 1;
			++TrackerId;
		}
	});

	if (AddedOrUpdatedTags > 0)
	{
		++SerialNumber;

		// Update Parent and StatFullName for each tag.
		for (FMemoryTag* TagPtr : Tags)
		{
			UpdateParentAndStatFullName(*TagPtr);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMemoryTagList::UpdateParentAndStatFullName(FMemoryTag& Tag)
{
	if (Tag.ParentId == FMemoryTag::InvalidTagId || Tag.Parent != nullptr)
	{
		// Tag has no Parent or the Parent and the StatFullName are already set.
		return;
	}

	for (FMemoryTag* ParentTagPtr : Tags)
	{
		FMemoryTag& ParentTag = *ParentTagPtr;
		if (ParentTag.Id == Tag.ParentId)
		{
			// Ensure StatFullName for parent tag is updated first.
			UpdateParentAndStatFullName(ParentTag);

			ParentTag.Children.Add(&Tag);

			Tag.Parent = ParentTagPtr;
			Tag.StatFullName = ParentTag.StatFullName + TEXT("/") + Tag.StatName;

			break;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FMemoryTagList::FilterTags(const TArray<FString>& InIncludeStats, const TArray<FString>& InIgnoreStats, TArray<FMemoryTag*>& OutTags) const
{
	int32 AddedTags = 0;

	for (const FString& IncludeStat : InIncludeStats)
	{
		if (IncludeStat == TEXT("*"))
		{
			for (FMemoryTag* TagPtr : Tags)
			{
				if (!TagPtr->MatchesWildcard(InIgnoreStats))
				{
					OutTags.Add(TagPtr);
					++AddedTags;
				}
			}
		}
		else
		{
			for (FMemoryTag* TagPtr : Tags)
			{
				if (TagPtr->MatchesWildcard(IncludeStat) && !TagPtr->MatchesWildcard(InIgnoreStats))
				{
					OutTags.Add(TagPtr);
					++AddedTags;
				}
			}
		}
	}

	return AddedTags;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
