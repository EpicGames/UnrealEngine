// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"

#include "Insights/MemoryProfiler/ViewModels/MemoryTracker.h"

namespace UE::Insights::MemoryProfiler
{

class FMemoryGraphTrack;

typedef int64 FMemoryTagSetId;
typedef int64 FMemoryTagId;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemoryTagSet
{
public:
	static const FMemoryTagSetId InvalidTagSetId = -1;
	static const FMemoryTagSetId DefaultTagSetId = 0;

public:
	FMemoryTagSet(FMemoryTagSetId InId, const FString& InName) : Id(InId), Name(InName) {}
	virtual ~FMemoryTagSet() = default;

	int32 GetIndex() const { return (int32)Id; }
	FMemoryTagSetId GetId() const { return Id; }
	const FString& GetName() const { return Name; }

private:
	FMemoryTagSetId Id = InvalidTagSetId;
	FString Name;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemoryTag
{
	friend class FMemoryTagList;

public:
	static const FMemoryTagId InvalidTagId = 0;

public:
	//FMemoryTag();
	//~FMemoryTag();

	int32 GetIndex() const { return Index; }
	FMemoryTagId GetId() const { return Id; }
	FMemoryTagId GetParentId() const { return ParentId; }

	const FString& GetStatName() const { return StatName; }
	const FString& GetStatFullName() const { return StatFullName; }

	bool MatchesWildcard(const FString& FullName) const;
	bool MatchesWildcard(const TArray<FString>& FullNames) const;

	FMemoryTrackerId GetTrackerId() const { return TrackerId; }
	FMemoryTagSetId GetTagSetId() const { return TagSetId; }

	const FLinearColor& GetColor() const { return Color; }
	void SetColor(FLinearColor InColor) { Color = InColor; }
	void SetColorAuto();
	void SetRandomColor();

	FMemoryTag* GetParent() const { return Parent; }
	const TSet<FMemoryTag*>& GetChildren() const { return Children; }

	bool IsAddedToGraph() const { return Tracks.Num() > 0; }
	const TSet<TSharedPtr<FMemoryGraphTrack>>& GetGraphTracks() const { return Tracks; }
	void AddTrack(TSharedPtr<FMemoryGraphTrack> InTrack) { Tracks.Add(InTrack); }
	void RemoveTrack(TSharedPtr<FMemoryGraphTrack> InTrack) { Tracks.Remove(InTrack); }
	void RemoveAllTracks() { Tracks.Reset(); }

private:
	int32 Index = -1;
	FMemoryTagId Id = InvalidTagId; // LLM tag id
	FMemoryTagId ParentId = InvalidTagId;
	FString StatName;
	FString StatFullName; // includes the parent prefix
	FMemoryTrackerId TrackerId = FMemoryTracker::InvalidTrackerId;
	FMemoryTagSetId TagSetId = FMemoryTagSet::InvalidTagSetId;

	FLinearColor Color;
	FMemoryTag* Parent = nullptr;
	TSet<FMemoryTag*> Children;
	TSet<TSharedPtr<FMemoryGraphTrack>> Tracks; // tracks using this LLM tag
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemoryTagList
{
public:
	FMemoryTagList();
	~FMemoryTagList();

	uint32 GetSerialNumber() const { return SerialNumber; }
	const TArray<FMemoryTag*>& GetTags() const { return Tags; }

	FMemoryTag* GetTagById(FMemoryTrackerId InTrackerId, FMemoryTagId InTagId) const;

	void Reset();
	void Update();

	/**
	 * Filters the list of tags using wildcard matching (on tag's full name).
	 * Returns the number of tags added to the output tags array.
	 */
	int32 FilterTags(const TArray<FString>& InIncludeStats, const TArray<FString>& InIgnoreStats, TArray<FMemoryTag*>& OutTags) const;

private:
	void UpdateInternal();
	void UpdateParentAndStatFullName(FMemoryTag& Tag);

private:
	TArray<FMemoryTag*> Tags; // the list of memory tags; owns the allocated memory
	TMap<FMemoryTrackerId, TMap<FMemoryTagId, FMemoryTag*>> TrackersAndTagsMap;
	uint32 LastTraceSerialNumber;
	uint32 SerialNumber;
	uint64 NextUpdateTimestamp;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
