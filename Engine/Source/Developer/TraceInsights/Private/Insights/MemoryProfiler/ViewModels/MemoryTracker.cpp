// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryTracker.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryTag.h"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMemoryTracker
////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryTracker::FMemoryTracker(FMemoryTrackerId InTrackerId, const FString InTrackerName)
	: Id(InTrackerId)
	, Name(InTrackerName)
{
	//TODO: static_assert(FMemoryTracker::InvalidTrackerId == static_cast<FMemoryTracker>(FMemoryTrackerDesc::InvalidTrackerId), "Memory TrackerId type mismatch!");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMemoryTracker::~FMemoryTracker()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
