// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"

namespace UE::Insights::MemoryProfiler
{

typedef int32 FMemoryTrackerId;
class FMemoryTag;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMemoryTracker
{
	friend class FMemorySharedState;

public:
	static const FMemoryTrackerId InvalidTrackerId = -1;

	static bool IsValidTrackerId(FMemoryTrackerId InTrackerId) { return (uint32(InTrackerId) & ~0x3F) == 0; } // valid ids: [0 .. 63]
	static uint64 AsFlag(FMemoryTrackerId InTrackerId) { return IsValidTrackerId(InTrackerId) ? (1ULL << static_cast<uint32>(InTrackerId)) : 0; }

public:
	FMemoryTracker(FMemoryTrackerId InTrackerId, const FString InTrackerName);
	~FMemoryTracker();

	FMemoryTrackerId GetId() const { return Id; }
	const FString& GetName() const { return Name; }

private:
	FMemoryTrackerId Id; // the tracker's id
	FString Name; // the tracker's name
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
