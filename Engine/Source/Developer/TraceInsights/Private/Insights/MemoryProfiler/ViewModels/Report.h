// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::Insights::MemoryProfiler
{

struct FGraphConfig
{
	FString Title; // the name of the graph
	FString StatString; // the stats to include
	FString IgnoreStats; // the stats to exclude
	FString HideStatPrefix; // the graph series will be named excluding the specified stat prefix
	FString MainStat; // the main stat (for stacked graph)
	FString ShowEvents; // ??

	uint32 MaxHierarchyDepth = 4; // ??

	bool bStacked = false; // if graph series are stacked or not
	bool bRequiresDetailedStats = false; // ??
	bool bShowAverages = false;
	bool bSmooth = false; // ??
	bool bVSync = false; // ??

	double LegendAverageThreshold = 0.0; // ??
	double SmoothKernelSize = -1.0; // ??
	double SmoothKernelPercent = 1.0; // ??
	double Thickness = 100.0; // ??
	double Compression = 0.05; // ??

	float Width = 1800.0f; // the width of the graph
	float Height = 600.0f; // the height of the graph

	double MinY = 0.0; // min vertical value
	double MaxY = 0.0; // max vertical value
	double Budget = 0.0; // budget
};

struct FGraphGroupConfig
{
	FString Name;
	TArray<FGraphConfig> Graphs;
};

struct FReportSummaryTableConfig
{
	FString Name;
	FString RowSort;
	FString Filter;
};

struct FReportTypeSummaryConfig
{
	FString Type;
	//TODO: float Fps;
	//TODO: float HitchThreshold;
	//TODO: bool bUseEngineHitchMetric;
	FString Stats; // <stats>
	//TODO: <colourThresholds>[]
	const FReportSummaryTableConfig* SummaryTable = nullptr; // valid only until the FReportConfig is further changed
};

struct FReportTypeGraphConfig
{
	FString Title;
	double Budget = 0.0;
	bool bInSummary = false; // ??
	bool bInMainSummary = false; // ??
	const FGraphConfig* GraphConfig = nullptr; // valid only until the FReportConfig is further changed
};

struct FReportTypeConfig
{
	FString Name;
	FString Title;
	FString IgnoreList; // ??
	bool bVSync = false; // ??
	//TODO: <autodetection>
	FString MetadataToShow; // ??
	TArray<FReportTypeSummaryConfig> Summaries;
	TArray<FReportTypeGraphConfig> Graphs;

	FReportTypeConfig() = default;

private:
	FReportTypeConfig(const FReportTypeConfig&) = delete;
};

struct FReportConfig
{
	//TODO: <statDisplayNameMappings>
	//TODO: <csvEventsToStrip>
	TArray<FReportSummaryTableConfig> SummaryTables;
	TArray<FGraphGroupConfig> GraphGroups;
	TArray<FReportTypeConfig> ReportTypes;

	FReportConfig() = default;

private:
	FReportConfig(const FReportConfig&) = delete;
};

} // namespace UE::Insights::MemoryProfiler
