// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReportXmlParser.h"

#include "Logging/TokenizedMessage.h"
#include "Misc/Paths.h"
#include "XmlNode.h"
#include "XmlFile.h"

// TraceInsights
#include "Insights/Log.h"
#include "Insights/MemoryProfiler/ViewModels/Report.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::FReportXmlParser"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

void FReportXmlParser::LoadReportGraphsXML(FReportConfig& ReportConfig, FString Path)
{
	Status = EStatus::Completed;
	ErrorMessages.Empty();

	LoadReportGraphsXML_Internal(ReportConfig, Path);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FReportXmlParser::LoadReportGraphsXML_Internal(FReportConfig& ReportConfig, FString Path)
{
	UE_LOGF(TraceInsights, Log, "[Report] Loading Report Graphs from \"%ls\"...", *Path);

	FXmlFile XmlFile;
	if (XmlFile.LoadFile(Path))
	{
		FXmlNode* RootNode = XmlFile.GetRootNode();
		if (RootNode)
		{
			const FString& RootNodeTag = RootNode->GetTag();
			if (RootNodeTag == TEXT("graphGroups"))
			{
				// The first set of base settings, declared inside the root "graphGroups" xml node.
				FGraphConfig BaseSettings1;

				for (const FXmlNode* Node : RootNode->GetChildrenNodes())
				{
					const FString& NodeTag = Node->GetTag();
					if (NodeTag == TEXT("baseSettings"))
					{
						ParseGraph(BaseSettings1, Node);
					}
					else if (NodeTag == TEXT("graphGroup"))
					{
						// The second set of base settings, declared inside each "graphGroup" xml node.
						// Will inherit from the base settings declared in the parent "graphGroups" xml node.
						FGraphConfig BaseSettings2 = BaseSettings1;

						FGraphGroupConfig& GraphGroupConfig = ReportConfig.GraphGroups.AddDefaulted_GetRef();

						GraphGroupConfig.Name = Node->GetAttribute(TEXT("name"));

						for (const FXmlNode* GraphNode : Node->GetChildrenNodes())
						{
							const FString& GraphNodeTag = GraphNode->GetTag();
							if (GraphNodeTag == TEXT("baseSettings"))
							{
								ParseGraph(BaseSettings2, GraphNode);
							}
							else if (GraphNodeTag == TEXT("graph"))
							{
								FGraphConfig GraphConfig = BaseSettings2;
								ParseGraph(GraphConfig, GraphNode);
								GraphGroupConfig.Graphs.Add(MoveTemp(GraphConfig));
							}
							else
							{
								UnknownXmlNode(GraphNode, Node);
							}
						}
					}
					else
					{
						UnknownXmlNode(Node, RootNode);
					}
				}
			}
			else
			{
				UnknownXmlNode(RootNode, nullptr);
			}
		}
		else
		{
			FText DetailMsg = FText::Format(LOCTEXT("FailedToLoad_ReportGraphsFmt", "Failed to load Report Graphs from \"{0}\".{1}No root xml node."),
				FText::FromString(Path),
				FText::FromString(LINE_TERMINATOR));

			LogError(DetailMsg);
		}
	}
	else
	{
		FText DetailMsg = FText::Format(LOCTEXT("FailedToLoad_ReportGraphsFmt2", "Failed to load Report Graphs from \"{0}\".{1}{2}"),
			FText::FromString(Path),
			FText::FromString(LINE_TERMINATOR),
			FText::FromString(XmlFile.GetLastError()));

		LogError(DetailMsg);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FReportXmlParser::ParseGraph(FGraphConfig& GraphConfig, const FXmlNode* XmlNode)
{
	for (const FXmlAttribute& XmlAttribute : XmlNode->GetAttributes())
	{
		const FString& Tag = XmlAttribute.GetTag();
		const FString& Value = XmlAttribute.GetValue();

		if (Tag == TEXT("title"))
		{
			GraphConfig.Title = Value;
		}
		else if (Tag == TEXT("statString"))
		{
			GraphConfig.StatString = Value;
		}
		else if (Tag == TEXT("ignoreStats"))
		{
			GraphConfig.IgnoreStats = Value;
		}
		else if (Tag == TEXT("hideStatPrefix"))
		{
			GraphConfig.HideStatPrefix = Value;
		}
		else if (Tag == TEXT("mainStat"))
		{
			GraphConfig.MainStat = Value;
		}
		else if (Tag == TEXT("showEvents"))
		{
			GraphConfig.ShowEvents = Value;
		}
		else if (Tag == TEXT("maxHierarchyDepth"))
		{
			GraphConfig.MaxHierarchyDepth = FCString::Atoi(*Value);
		}
		else if (Tag == TEXT("stacked"))
		{
			GraphConfig.bStacked = FCString::Atoi(*Value) != 0;
		}
		else if (Tag == TEXT("requiresDetailedStats"))
		{
			GraphConfig.bRequiresDetailedStats = FCString::Atoi(*Value) != 0;
		}
		else if (Tag == TEXT("showAverages"))
		{
			GraphConfig.bShowAverages = FCString::Atoi(*Value) != 0;
		}
		else if (Tag == TEXT("smooth"))
		{
			GraphConfig.bSmooth = FCString::Atoi(*Value) != 0;
		}
		else if (Tag == TEXT("vsync"))
		{
			GraphConfig.bVSync = FCString::Atoi(*Value) != 0;
		}
		else if (Tag == TEXT("legendAverageThreshold"))
		{
			GraphConfig.LegendAverageThreshold = FCString::Atod(*Value);
		}
		else if (Tag == TEXT("smoothKernelSize"))
		{
			GraphConfig.SmoothKernelSize = FCString::Atod(*Value);
		}
		else if (Tag == TEXT("smoothKernelPercent"))
		{
			GraphConfig.SmoothKernelPercent = FCString::Atod(*Value);
		}
		else if (Tag == TEXT("thickness"))
		{
			GraphConfig.Thickness = FCString::Atod(*Value);
		}
		else if (Tag == TEXT("compression"))
		{
			GraphConfig.Compression = FCString::Atod(*Value);
		}
		else if (Tag == TEXT("width"))
		{
			GraphConfig.Width = FCString::Atof(*Value);
		}
		else if (Tag == TEXT("height"))
		{
			GraphConfig.Height = FCString::Atof(*Value);
		}
		else if (Tag == TEXT("miny"))
		{
			GraphConfig.MinY = FCString::Atod(*Value);
		}
		else if (Tag == TEXT("maxy"))
		{
			GraphConfig.MaxY = FCString::Atod(*Value);
		}
		else if (Tag == TEXT("budget"))
		{
			GraphConfig.Budget = FCString::Atod(*Value);
		}
		else
		{
			UnknownXmlAttribute(XmlNode, XmlAttribute);
		}
	}

	for (const FXmlNode* ChildXmlNode : XmlNode->GetChildrenNodes())
	{
		const FString& Tag = ChildXmlNode->GetTag();
		if (Tag == TEXT("statString"))
		{
			GraphConfig.StatString = ChildXmlNode->GetContent();
		}
		else
		{
			UnknownXmlNode(ChildXmlNode, XmlNode);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FReportXmlParser::LoadReportTypesXML(FReportConfig& ReportConfig, FString Path)
{
	Status = EStatus::Completed;
	ErrorMessages.Empty();

	LoadReportTypesXML_Internal(ReportConfig, Path);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FReportXmlParser::LoadReportTypesXML_Internal(FReportConfig& ReportConfig, FString Path)
{
	UE_LOGF(TraceInsights, Log, "[Report] Loading Report Types from \"%ls\"...", *Path);

	FXmlFile XmlFile;
	if (XmlFile.LoadFile(Path))
	{
		FXmlNode* RootNode = XmlFile.GetRootNode();
		if (RootNode)
		{
			const FString& RootNodeTag = RootNode->GetTag();

			if (RootNodeTag == TEXT("root"))
			{
				for (const FXmlNode* Node : RootNode->GetChildrenNodes())
				{
					const FString& NodeTag = Node->GetTag();
					if (NodeTag == TEXT("statDisplayNameMappings"))
					{
						//TODO
					}
					else if (NodeTag == TEXT("csvEventsToStrip"))
					{
						//TODO
					}
					else if (NodeTag == TEXT("summaryTables"))
					{
						for (const FXmlNode* SummaryTableNode : Node->GetChildrenNodes())
						{
							const FString& SummaryTableNodeTag = SummaryTableNode->GetTag();
							if (SummaryTableNodeTag == TEXT("summaryTable"))
							{
								FReportSummaryTableConfig& ReportSummaryTable = ReportConfig.SummaryTables.AddDefaulted_GetRef();
								ParseReportSummaryTable(ReportSummaryTable, SummaryTableNode);
							}
							else
							{
								UnknownXmlNode(SummaryTableNode, Node);
							}
						}
					}
					else if (NodeTag == TEXT("reporttypes"))
					{
						FString ReportGraphsFile = Node->GetAttribute(TEXT("reportGraphsFile"));
						LoadReportGraphsXML_Internal(ReportConfig, FPaths::GetPath(Path) / ReportGraphsFile);

						for (const FXmlNode* ReportTypeNode : Node->GetChildrenNodes())
						{
							const FString& ReportTypeNodeTag = ReportTypeNode->GetTag();
							if (ReportTypeNodeTag == TEXT("reporttype"))
							{
								FReportTypeConfig& ReportType = ReportConfig.ReportTypes.AddDefaulted_GetRef();
								ParseReportType(ReportType, ReportTypeNode);
							}
							else
							{
								UnknownXmlNode(ReportTypeNode, Node);
							}
						}
					}
					else
					{
						UnknownXmlNode(Node, RootNode);
					}
				}
			}
			else
			{
				UnknownXmlNode(RootNode, nullptr);
			}
		}
		else
		{
			FText DetailMsg = FText::Format(LOCTEXT("FailedToLoad_ReportTypesFmt", "Failed to load Report Types from \"{0}\".{1}No root xml node."),
				FText::FromString(Path),
				FText::FromString(LINE_TERMINATOR));

			LogError(DetailMsg);
		}
	}
	else
	{
		FText DetailMsg = FText::Format(LOCTEXT("FailedToLoad_ReportTypesFmt2", "Failed to load Report Types from \"{0}\".{1}{2}"),
			FText::FromString(Path),
			FText::FromString(LINE_TERMINATOR),
			FText::FromString(XmlFile.GetLastError()));

		LogError(DetailMsg);
	}

	// Resolve GraphConfig pointers.
	for (FReportTypeConfig& ReportType : ReportConfig.ReportTypes)
	{
		for (FReportTypeGraphConfig& ReportTypeGraph : ReportType.Graphs)
		{
			bool bGraphFound = false;
			for (FGraphGroupConfig& GraphGroup : ReportConfig.GraphGroups)
			{
				for (FGraphConfig& Graph : GraphGroup.Graphs)
				{
					if (Graph.Title == ReportTypeGraph.Title)
					{
						ReportTypeGraph.GraphConfig = &Graph;
						bGraphFound = true;
						break;
					}
				}
				if (bGraphFound)
				{
					break;
				}
			}
			if (!bGraphFound)
			{
				FText DetailMsg = FText::Format(LOCTEXT("ReportGraphNotFoundFmt", "Report graph \"{0}\" not found (referenced in report type \"{1}\")!"),
					FText::FromString(*ReportTypeGraph.Title),
					FText::FromString(*ReportType.Name));

				LogWarning(DetailMsg);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FReportXmlParser::ParseReportSummaryTable(FReportSummaryTableConfig& ReportSummaryTable, const FXmlNode* XmlNode)
{
	//TODO
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FReportXmlParser::ParseReportType(FReportTypeConfig& ReportType, const FXmlNode* XmlNode)
{
	for (const FXmlAttribute& XmlAttribute : XmlNode->GetAttributes())
	{
		const FString& Tag = XmlAttribute.GetTag();
		const FString& Value = XmlAttribute.GetValue();

		if (Tag == TEXT("name"))
		{
			ReportType.Name = Value;
		}
		else if (Tag == TEXT("title"))
		{
			ReportType.Title = Value;
		}
		else if (Tag == TEXT("ignoreList"))
		{
			ReportType.IgnoreList = Value;
		}
		else if (Tag == TEXT("vsync"))
		{
			ReportType.bVSync = (FCString::Atoi(*Value) != 0);
		}
		else
		{
			UnknownXmlAttribute(XmlNode, XmlAttribute);
		}
	}

	for (const FXmlNode* ChildXmlNode : XmlNode->GetChildrenNodes())
	{
		const FString& ChildXmlNodeTag = ChildXmlNode->GetTag();
		if (ChildXmlNodeTag == TEXT("autodetection"))
		{
			//TODO
		}
		else if (ChildXmlNodeTag == TEXT("metadataToShow"))
		{
			ReportType.MetadataToShow = ChildXmlNode->GetContent();
		}
		else if (ChildXmlNodeTag == TEXT("summary"))
		{
			//TODO
		}
		else if (ChildXmlNodeTag == TEXT("graph"))
		{
			FReportTypeGraphConfig& ReportTypeGraph = ReportType.Graphs.AddDefaulted_GetRef();

			for (const FXmlAttribute& XmlAttribute : ChildXmlNode->GetAttributes())
			{
				const FString& Tag = XmlAttribute.GetTag();
				const FString& Value = XmlAttribute.GetValue();

				if (Tag == TEXT("title"))
				{
					ReportTypeGraph.Title = Value;
				}
				else if (Tag == TEXT("budget"))
				{
					ReportTypeGraph.Budget = FCString::Atod(*Value);
				}
				else if (Tag == TEXT("inSummary"))
				{
					ReportTypeGraph.bInSummary = (FCString::Atoi(*Value) != 0);
				}
				else if (Tag == TEXT("inMainSummary"))
				{
					ReportTypeGraph.bInMainSummary = (FCString::Atoi(*Value) != 0);
				}
				else
				{
					UnknownXmlAttribute(ChildXmlNode, XmlAttribute);
				}
			}
		}
		else
		{
			UnknownXmlNode(ChildXmlNode, XmlNode);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FReportXmlParser::UnknownXmlNode(const FXmlNode* XmlChildNode, const FXmlNode* XmlParentNode)
{
	if (XmlParentNode != nullptr)
	{
		FText DetailMsg = FText::Format(LOCTEXT("UnknownXmlChildNodeFmt", "Unknown XML child node <{0}> in <{1}> node."),
			FText::FromString(*XmlChildNode->GetTag()),
			FText::FromString(*XmlParentNode->GetTag()));

		LogWarning(DetailMsg);
	}
	else
	{
		LogError(LOCTEXT("FailedToLoad_NoRootXmlNode", "Failed to load Report. No root xml node."));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FReportXmlParser::UnknownXmlAttribute(const FXmlNode* XmlNode, const FXmlAttribute& XmlAttribute)
{
	FText DetailMsg = FText::Format(LOCTEXT("UnknownXmlAttributeFmt", "Unknown XML attribute {0}=\"{1}\" in <{2}> node."),
		FText::FromString(*XmlAttribute.GetTag()),
		FText::FromString(*XmlAttribute.GetValue()),
		FText::FromString(*XmlNode->GetTag()));

	LogWarning(DetailMsg);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FReportXmlParser::AutoLoadLLMReportXML(FReportConfig& ReportConfig)
{
	//FString ReportGraphsFilename(TEXT("Engine/Binaries/DotNET/CsvTools/LLMReportGraphs.xml"));
	//LoadReportGraphsXML(ReportConfig, FPaths::RootDir() / ReportGraphsFilename);

	FString ReportTypesFilename(TEXT("Engine/Binaries/DotNET/CsvTools/LLMReportTypes.xml"));
	LoadReportTypesXML(ReportConfig, FPaths::RootDir() / ReportTypesFilename);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FReportXmlParser::LogError(const FText& Text)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
	Message->AddToken(FDynamicTextToken::Create(Text));

	ErrorMessages.Add(Message);
	Status = EStatus::Error;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FReportXmlParser::LogWarning(const FText& Text)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning);
	Message->AddToken(FDynamicTextToken::Create(Text));

	ErrorMessages.Add(Message);

	if (Status == EStatus::Completed)
	{
		Status = EStatus::CompletedWithWarnings;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
