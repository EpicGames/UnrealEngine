// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FXmlNode;
class FXmlAttribute;
class FTokenizedMessage;

namespace UE::Insights::MemoryProfiler
{

struct FGraphConfig;
struct FReportSummaryTableConfig;
struct FReportTypeConfig;
struct FReportConfig;

class FReportXmlParser
{
public:
	enum class EStatus: uint8
	{
		Completed,
		CompletedWithWarnings,
		Error
	};

public:
	void LoadReportGraphsXML(FReportConfig& ReportConfig, FString Filename);
	void LoadReportTypesXML(FReportConfig& ReportConfig, FString Filename);
	void AutoLoadLLMReportXML(FReportConfig& ReportConfig);
	const TArray<TSharedRef<FTokenizedMessage>> GetErrorMessages() { return ErrorMessages; }
	EStatus GetStatus() { return Status; }

private:
	void LoadReportGraphsXML_Internal(FReportConfig& ReportConfig, FString Filename);
	void LoadReportTypesXML_Internal(FReportConfig& ReportConfig, FString Filename);
	void ParseGraph(FGraphConfig& GraphConfig, const FXmlNode* XmlNode);
	void ParseReportSummaryTable(FReportSummaryTableConfig& ReportSummaryTable, const FXmlNode* XmlNode);
	void ParseReportType(FReportTypeConfig& ReportType, const FXmlNode* XmlNode);
	void UnknownXmlNode(const FXmlNode* XmlNode, const FXmlNode* XmlParentNode = nullptr);
	void UnknownXmlAttribute(const FXmlNode* XmlNode, const FXmlAttribute& XmlAttribute);
	void LogError(const FText& Text);
	void LogWarning(const FText& Text);

private:
	TArray<TSharedRef<FTokenizedMessage>> ErrorMessages;
	EStatus Status = EStatus::Completed;
};

} // namespace UE::Insights::MemoryProfiler
