// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMemAllocTableTreeView.h"

#include "Containers/Set.h"
#include "DesktopPlatformModule.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformFileManager.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"
#include "Logging/MessageLog.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SToolTip.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AllocationsProvider.h"
#include "TraceServices/Model/Callstack.h"
#include "TraceServices/Model/Definitions.h"
#include "TraceServices/Model/MetadataProvider.h"
#include "TraceServices/Model/Modules.h"
#include "TraceServices/Model/Strings.h"
#include "TraceServices/Model/Threads.h"

// TraceInsightsCore
#include "InsightsCore/Filter/ViewModels/FilterConfigurator.h"
#include "InsightsCore/Filter/ViewModels/Filters.h"
#include "InsightsCore/Table/ViewModels/TableCellValue.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/Common/Utf8FileWriter.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/MemoryProfiler/Common/SymbolSearchPathsHelper.h"
#include "Insights/MemoryProfiler/MemoryProfilerManager.h"
#include "Insights/MemoryProfiler/ViewModels/CallstackFormatting.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocGroupingByCallstack.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocGroupingByHeap.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocGroupingBySize.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocGroupingBySwapPage.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocGroupingByTag.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocNode.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocTableViewPresets.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryFiltersValueConverters.h"
#include "Insights/MemoryProfiler/ViewModels/MemorySharedState.h"

#include <memory>

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::SMemAllocTableTreeView"

namespace UE::Insights::MemoryProfiler
{

const int32 SMemAllocTableTreeView::FullCallStackIndex = 0x0000FFFFF;
const int32 SMemAllocTableTreeView::LLMFilterIndex = 0x0000FFFFE;
const int32 SMemAllocTableTreeView::AllocThreadFilterIndex = 0x0000FFFFD;
const int32 SMemAllocTableTreeView::FreeThreadFilterIndex = 0x0000FFFFC;

////////////////////////////////////////////////////////////////////////////////////////////////////

SMemAllocTableTreeView::SMemAllocTableTreeView()
{
	bRunInAsyncMode = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SMemAllocTableTreeView::~SMemAllocTableTreeView()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::Construct(const FArguments& InArgs, TSharedPtr<FMemAllocTable> InTablePtr)
{
	ConstructWidget(InTablePtr);

	// Apply the default preset.
	ApplyViewPreset(*(*GetAvailableViewPresets())[0]);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> SMemAllocTableTreeView::ConstructToolbar()
{
	TSharedPtr<SHorizontalBox> Box = SNew(SHorizontalBox);

	ConstructViewPreset(Box);

	Box->AddSlot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		[
			ConstructFunctionToggleButton()
		];

	return Box;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> SMemAllocTableTreeView::ConstructFooter()
{
	return
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.Padding(2.0f, 2.0f, 0.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(this, &SMemAllocTableTreeView::GetQueryInfo)
			.ToolTipText(this, &SMemAllocTableTreeView::GetQueryInfoTooltip)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.Padding(0.0f, 2.0f, 0.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("FooterSeparator", " : "))
			.ColorAndOpacity(FSlateColor(EStyleColor::White25))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.Padding(0.0f, 2.0f, 8.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(this, &SMemAllocTableTreeView::GetFooterLeftText)
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.HAlign(HAlign_Center)
		.Padding(2.0f, 2.0f, 2.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(this, &SMemAllocTableTreeView::GetFooterCenterText)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.Padding(8.0f, 2.0f, 2.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(this, &SMemAllocTableTreeView::GetSymbolResolutionStatus)
			.ToolTipText(this, &SMemAllocTableTreeView::GetSymbolResolutionTooltip)
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::InternalCreateGroupings()
{
	STableTreeView::InternalCreateGroupings();

	AvailableGroupings.RemoveAll(
		[](TSharedPtr<FTreeNodeGrouping>& Grouping)
		{
			if (Grouping->Is<FTreeNodeGroupingByUniqueValue>())
			{
				const FName ColumnId = Grouping->As<FTreeNodeGroupingByUniqueValue>().GetColumnId();
				if (ColumnId == FMemAllocTableColumns::StartEventIndexColumnId ||
					ColumnId == FMemAllocTableColumns::EndEventIndexColumnId ||
					ColumnId == FMemAllocTableColumns::CountColumnId)
				{
					return true;
				}
			}
			else if (Grouping->Is<FTreeNodeGroupingByPathBreakdown>())
			{
				const FName ColumnId = Grouping->As<FTreeNodeGroupingByPathBreakdown>().GetColumnId();
				if (ColumnId == FMemAllocTableColumns::AllocFunctionColumnId ||
					ColumnId == FMemAllocTableColumns::FreeFunctionColumnId ||
					ColumnId == FMemAllocTableColumns::ClassNameColumnId)
				{
					return true;
				}
			}
			return false;
		});

	int32 Index = 1; // after the Flat ("All") grouping

	AvailableGroupings.Insert(MakeShared<FMemAllocGroupingBySize>(), Index++);

	const TraceServices::IAllocationsProvider* AllocationsProvider = Session.IsValid() ? TraceServices::ReadAllocationsProvider(*Session.Get()) : nullptr;

	if (AllocationsProvider)
	{
		AvailableGroupings.Insert(MakeShared<FMemAllocGroupingByTag>(*AllocationsProvider), Index++);
	}

	AvailableGroupings.Insert(MakeShared<FMemAllocGroupingByCallstack>(true, false, bIsCallstackGroupingByFunction), Index++);
	AvailableGroupings.Insert(MakeShared<FMemAllocGroupingByCallstack>(true, true, bIsCallstackGroupingByFunction), Index++);
	AvailableGroupings.Insert(MakeShared<FMemAllocGroupingByCallstack>(false, false, bIsCallstackGroupingByFunction), Index++);
	AvailableGroupings.Insert(MakeShared<FMemAllocGroupingByCallstack>(false, true, bIsCallstackGroupingByFunction), Index++);

	if (AllocationsProvider)
	{
		AvailableGroupings.Insert(MakeShared<FMemAllocGroupingByHeap>(*AllocationsProvider), Index++);
		AvailableGroupings.Insert(MakeShared<FMemAllocGroupingBySwapPage>(*AllocationsProvider, true, bIsCallstackGroupingByFunction), Index++);
		AvailableGroupings.Insert(MakeShared<FMemAllocGroupingBySwapPage>(*AllocationsProvider, false, bIsCallstackGroupingByFunction), Index++);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::InitAvailableViewPresets()
{
	AvailableViewPresets.Add(FMemAllocTableViewPresets::CreateDefaultViewPreset(*this));
	AvailableViewPresets.Add(FMemAllocTableViewPresets::CreateDetailedViewPreset(*this));
	AvailableViewPresets.Add(FMemAllocTableViewPresets::CreateHeapViewPreset(*this));
	AvailableViewPresets.Add(FMemAllocTableViewPresets::CreateSizeViewPreset(*this));
	AvailableViewPresets.Add(FMemAllocTableViewPresets::CreateTagViewPreset(*this));
	AvailableViewPresets.Add(FMemAllocTableViewPresets::CreateAssetViewPreset(*this));
	AvailableViewPresets.Add(FMemAllocTableViewPresets::CreateClassNameViewPreset(*this));
	AvailableViewPresets.Add(FMemAllocTableViewPresets::CreateCallstackViewPreset(*this, false, true));
	AvailableViewPresets.Add(FMemAllocTableViewPresets::CreateCallstackViewPreset(*this, true, true));
	AvailableViewPresets.Add(FMemAllocTableViewPresets::CreateCallstackViewPreset(*this, false, false));
	AvailableViewPresets.Add(FMemAllocTableViewPresets::CreateCallstackViewPreset(*this, true, false));
	AvailableViewPresets.Add(FMemAllocTableViewPresets::CreatePlatformPageViewPreset(*this));
	AvailableViewPresets.Add(FMemAllocTableViewPresets::CreateSwapViewPreset(*this, true));
	AvailableViewPresets.Add(FMemAllocTableViewPresets::CreateSwapViewPreset(*this, false));

	SelectedViewPreset = AvailableViewPresets[0];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::Reset()
{
	//...
	STableTreeView::Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	STableTreeView::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bHasPendingQueryReset && !bIsUpdateRunning)
	{
		ResetAndStartQuery();
		bHasPendingQueryReset = false;
	}

	if (!bIsUpdateRunning)
	{
		RebuildTree(false);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::RebuildTree(bool bResync)
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	FStopwatch SyncStopwatch;
	SyncStopwatch.Start();

	if (bResync)
	{
		TableRowNodes.Empty();
	}

	const int32 PreviousNodeCount = TableRowNodes.Num();

	TSharedPtr<FMemAllocTable> MemAllocTable = GetMemAllocTable();

	if (Session.IsValid() && MemAllocTable.IsValid())
	{
		TraceServices::IAllocationsProvider::EQueryStatus QueryStatus;
		UpdateQuery(QueryStatus);

		if (QueryStatus == TraceServices::IAllocationsProvider::EQueryStatus::Done)
		{
			UE_LOGF(LogMemoryProfiler, Log, "[MemAlloc] Rebuilding tree...");
			TArray<FMemoryAlloc>& Allocs = MemAllocTable->GetAllocs();

			const int32 TotalAllocCount = Allocs.Num();
			if (TotalAllocCount != TableRowNodes.Num())
			{
				UE_LOGF(LogMemoryProfiler, Log, "[MemAlloc] Creating nodes (%d nodes --> %d allocs)...", TableRowNodes.Num(), TotalAllocCount);

				if (TableRowNodes.Num() > TotalAllocCount)
				{
					TableRowNodes.Empty();
				}
				TableRowNodes.Reserve(TotalAllocCount);

				uint32 HeapAllocCount = 0;
				uint32 SwapAllocCount = 0;
				const FName BaseNodeName(TEXT("alloc"));
				const FName BaseHeapName(TEXT("heap"));
				const FName BaseSwapName(TEXT("swap"));
				for (int32 AllocIndex = TableRowNodes.Num(); AllocIndex < TotalAllocCount; ++AllocIndex)
				{
					const FMemoryAlloc* Alloc = MemAllocTable->GetMemAlloc(AllocIndex);

					if (Alloc->bIsHeap)
					{
						++HeapAllocCount;
						if (!bIncludeHeapAllocs)
						{
							continue;
						}
					}

					if (Alloc->bIsSwap)
					{
						++SwapAllocCount;
						if (!bIncludeSwapAllocs)
						{
							continue;
						}
					}

					if (!PackageFilter.IsEmpty())
					{
						const TCHAR* PackageString = Alloc->GetPackage();
						if (!PackageString || !PackageFilter.Contains(PackageString))
						{
							continue;
						}
					}

					FName NodeName(Alloc->bIsHeap ? BaseHeapName : (Alloc->bIsSwap ? BaseSwapName : BaseNodeName), static_cast<int32>(Alloc->GetStartEventIndex() + 1));
					FMemAllocNodePtr NodePtr = MakeShared<FMemAllocNode>(NodeName, MemAllocTable, AllocIndex);
					TableRowNodes.Add(NodePtr);
				}
				
				// ensure only valid when no package filter is active
				if (PackageFilter.IsEmpty())
				{
					ensure(TableRowNodes.Num() == TotalAllocCount - (bIncludeHeapAllocs ? 0 : HeapAllocCount) - (bIncludeSwapAllocs ? 0 : SwapAllocCount));
				}
			}
		}
	}

	SyncStopwatch.Stop();

	if (bResync || TableRowNodes.Num() != PreviousNodeCount)
	{
		// Save selection.
		TArray<FTableTreeNodePtr> SelectedItems;
		TreeView->GetSelectedItems(SelectedItems);

		UpdateTree();

		TreeView->RebuildList();

		// Restore selection.
		if (SelectedItems.Num() > 0)
		{
			TreeView->ClearSelection();
			for (FTableTreeNodePtr& NodePtr : SelectedItems)
			{
				NodePtr = GetNodeByTableRowIndex(NodePtr->GetRowIndex());
			}
			SelectedItems.RemoveAll([](const FTableTreeNodePtr& NodePtr) { return !NodePtr.IsValid(); });
			if (SelectedItems.Num() > 0)
			{
				TreeView->SetItemSelection(SelectedItems, true);
				TreeView->RequestScrollIntoView(SelectedItems.Last());
			}
		}
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.01)
	{
		const double SyncTime = SyncStopwatch.GetAccumulatedTime();
		UE_LOGF(LogMemoryProfiler, Log, "[MemAlloc] Tree view rebuilt in %.4fs (sync: %.4fs + update: %.4fs) --> %d nodes (%d added)",
			TotalTime, SyncTime, TotalTime - SyncTime, TableRowNodes.Num(), TableRowNodes.Num() - PreviousNodeCount);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::OnQueryInvalidated()
{
	CancelQuery();

	if (bIsUpdateRunning)
	{
		bHasPendingQueryReset = true;
	}
	else
	{
		ResetAndStartQuery();
	}

	UpdateQueryInfo();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::ResetAndStartQuery()
{
	// Apply the "Default" or the "Heap" preset.
	const TArray<TSharedRef<ITableTreeViewPreset>>* ViewPresets = GetAvailableViewPresets();
	if (ViewPresets && ViewPresets->Num() > 3)
	{
		constexpr int32 DefaultPresetIndex = 0;
		constexpr int32 HeapPresetIndex = 2;
		SelectViewPreset((*ViewPresets)[HasHeapAllocs() ? HeapPresetIndex : DefaultPresetIndex].ToSharedPtr());
	}

	TableRowNodes.Reset();

	TSharedPtr<FMemAllocTable> MemAllocTable = GetMemAllocTable();
	if (MemAllocTable)
	{
		MemAllocTable->SetTimeMarkerA(TimeMarkers[0]); // to be used by LLM Size and LLM Delta Size columns

		TArray<FMemoryAlloc>& Allocs = MemAllocTable->GetAllocs();
		Allocs.Reset(10 * 1024 * 1024);
	}

	StartQuery();

	RebuildTree(true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::StartQuery()
{
	check(Query == 0);

	if (!Rule)
	{
		UE_LOGF(LogMemoryProfiler, Warning, "[MemAlloc] Invalid query rule!");
		return;
	}

	if (!Session.IsValid())
	{
		UE_LOGF(LogMemoryProfiler, Warning, "[MemAlloc] Invalid analysis session!");
		return;
	}

	const TraceServices::IAllocationsProvider* AllocationsProvider = TraceServices::ReadAllocationsProvider(*Session.Get());
	if (!AllocationsProvider)
	{
		UE_LOGF(LogMemoryProfiler, Warning, "[MemAlloc] Invalid allocations provider!");
		return;
	}

	{
		const TraceServices::IAllocationsProvider& Provider = *AllocationsProvider;
		TraceServices::FProviderReadScopeLock _(Provider);

		TSharedPtr<FMemAllocTable> MemAllocTable = GetMemAllocTable();
		if (MemAllocTable)
		{
			MemAllocTable->SetPlatformPageSize(Provider.GetPlatformPageSize());
		}

		TraceServices::IAllocationsProvider::FQueryParams Params = { Rule->GetValue(), TimeMarkers[0], TimeMarkers[1], TimeMarkers[2], TimeMarkers[3] };
		Query = Provider.StartQuery(Params);
	}

	if (Query == 0)
	{
		UE_LOGF(LogMemoryProfiler, Error, "[MemAlloc] Unsupported query rule (%ls)!", *Rule->GetShortName().ToString());
	}
	else
	{
		QueryStopwatch.Reset();
		QueryStopwatch.Start();
	}

	//TODO: update window title
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::UpdateQuery(TraceServices::IAllocationsProvider::EQueryStatus& OutStatus)
{
	if (Query == 0)
	{
		OutStatus = TraceServices::IAllocationsProvider::EQueryStatus::Unknown;
		return;
	}

	if (!Session.IsValid())
	{
		UE_LOGF(LogMemoryProfiler, Warning, "[MemAlloc] Invalid analysis session!");
		return;
	}

	const TraceServices::IAllocationsProvider* AllocationsProvider = TraceServices::ReadAllocationsProvider(*Session.Get());
	if (!AllocationsProvider)
	{
		UE_LOGF(LogMemoryProfiler, Warning, "[MemAlloc] Invalid allocations provider!");
		return;
	}
	const TraceServices::IAllocationsProvider& Provider = *AllocationsProvider;

	const TraceServices::ICallstacksProvider* CallstacksProvider = TraceServices::ReadCallstacksProvider(*Session.Get());

	const TraceServices::IMetadataProvider* MetadataProvider = TraceServices::ReadMetadataProvider(*Session.Get());

	uint16 AssetMetadataType = 0;
	const TraceServices::FMetadataSchema* Schema = nullptr;
	if (MetadataProvider)
	{
		TraceServices::FProviderReadScopeLock MetadataProviderReadLock(*MetadataProvider);
		AssetMetadataType = MetadataProvider->GetRegisteredMetadataType(TEXT("Asset"));
		if (AssetMetadataType == TraceServices::IMetadataProvider::InvalidMetadataId)
		{
			// If AssetMetadataType is not valid then we do not need to further check the Asset metadata for each allocation.
			MetadataProvider = nullptr;
		}
		else
		{
			Schema = MetadataProvider->GetRegisteredMetadataSchema(AssetMetadataType);
		}
	}

	const TraceServices::IDefinitionProvider* DefinitionProvider = TraceServices::ReadDefinitionProvider(*Session.Get());

	constexpr double MaxPollTime = 0.03; // Stop getting results after 30 ms so we don't tank the frame rate too much.
	FStopwatch TotalStopwatch;
	TotalStopwatch.Start();

	do
	{
		TraceServices::IAllocationsProvider::FQueryStatus Status = Provider.PollQuery(Query);
		OutStatus = Status.Status;

		if (Status.Status <= TraceServices::IAllocationsProvider::EQueryStatus::Done)
		{
			UE_LOGF(LogMemoryProfiler, Log, "[MemAlloc] Query completed.");
			Query = 0;
			QueryStopwatch.Stop();
			return;
		}

		if (Status.Status == TraceServices::IAllocationsProvider::EQueryStatus::Working)
		{
			break;
		}

		check(Status.Status == TraceServices::IAllocationsProvider::EQueryStatus::Available);

		TSharedPtr<FMemAllocTable> MemAllocTable = GetMemAllocTable();
		if (MemAllocTable)
		{
			TraceServices::FProviderReadScopeLock _(Provider);

			TArray<FMemoryAlloc>& Allocs = MemAllocTable->GetAllocs();

			FStopwatch ResultStopwatch;
			FStopwatch PageStopwatch;
			ResultStopwatch.Start();
			uint32 PageCount = 0;
			uint32 TotalAllocCount = 0;

			// Multiple 'pages' of results will be returned. No guarantees are made
			// about the order of pages or the allocations they report.
			TraceServices::IAllocationsProvider::FQueryResult Result = Status.NextResult();
			while (Result.IsValid())
			{
				UE_LOGF(LogMemoryProfiler, Log, "[MemAlloc] Page with %u allocs...", Result->Num());

				++PageCount;
				PageStopwatch.Restart();

				const uint32 AllocCount = Result->Num();
				TotalAllocCount += AllocCount;

				int32 AllocsDestIndex = Allocs.Num();
				Allocs.AddUninitialized(AllocCount);

				for (uint32 AllocIndex = 0; AllocIndex < AllocCount; ++AllocIndex, ++AllocsDestIndex)
				{
					const TraceServices::IAllocationsProvider::FAllocation* Allocation = Result->Get(AllocIndex);
					FMemoryAlloc& Alloc = Allocs[AllocsDestIndex];

					Alloc.StartEventIndex = Allocation->GetStartEventIndex();
					Alloc.EndEventIndex = Allocation->GetEndEventIndex();

					Alloc.StartTime = Allocation->GetStartTime();
					Alloc.EndTime = Allocation->GetEndTime();

					Alloc.Address = Allocation->GetAddress();
					Alloc.Size = int64(Allocation->GetSize());

					Alloc.TagId = Allocation->GetTag();
					Alloc.Tag = Provider.GetTagFullPath(Allocation->GetTag());

					Alloc.Asset = nullptr;
					Alloc.ClassName = nullptr;
					Alloc.Package = nullptr;

					const uint32 MetadataId = Allocation->GetMetadataId();
					if (MetadataId != TraceServices::IMetadataProvider::InvalidMetadataId && MetadataProvider && DefinitionProvider)
					{
						TraceServices::FProviderReadScopeLock MetadataProviderReadLock(*MetadataProvider);
						MetadataProvider->EnumerateMetadata(Allocation->GetAllocThreadId(), MetadataId,
							[AssetMetadataType, Schema, &Alloc, DefinitionProvider](uint32 StackDepth, uint16 Type, const void* Data, uint32 Size) -> bool
							{
								if (Type == AssetMetadataType)
								{
									TraceServices::FProviderReadScopeLock DefinitionProviderReadLock(*DefinitionProvider);
									const auto Reader = Schema->Reader();
									const auto AssetNameRef = Reader.GetValueAs<UE::Trace::FEventRef32>((uint8*)Data, 0);
									const auto AssetName = DefinitionProvider->Get<TraceServices::FStringDefinition>(*AssetNameRef);
									if (AssetName)
									{
										Alloc.Asset = AssetName->Display;
									}
									const auto ClassNameRef = Reader.GetValueAs<UE::Trace::FEventRef32>((uint8*)Data, 1);
									const auto ClassName = DefinitionProvider->Get<TraceServices::FStringDefinition>(*ClassNameRef);
									if (ClassName)
									{
										Alloc.ClassName = ClassName->Display;
									}
									const auto PackageNameRef = Reader.GetValueAs<UE::Trace::FEventRef32>((uint8*)Data, 2);
									const auto PackageName = DefinitionProvider->Get<TraceServices::FStringDefinition>(*PackageNameRef);
									if (PackageName)
									{
										Alloc.Package = PackageName->Display;
									}
									return false;
								}
								return true;
							});
					}

					Alloc.AllocThreadId = uint16(Allocation->GetAllocThreadId());
					Alloc.FreeThreadId = uint16(Allocation->GetFreeThreadId());

					Alloc.AllocCallstackId = Allocation->GetAllocCallstackId();
					Alloc.FreeCallstackId = Allocation->GetFreeCallstackId();

					if (CallstacksProvider)
					{
						Alloc.AllocCallstack = CallstacksProvider->GetCallstack(Allocation->GetAllocCallstackId());
						check(Alloc.AllocCallstack != nullptr);

						Alloc.FreeCallstack = CallstacksProvider->GetCallstack(Allocation->GetFreeCallstackId());
						check(Alloc.FreeCallstack != nullptr);
					}
					else
					{
						Alloc.AllocCallstack = nullptr;
						Alloc.FreeCallstack = nullptr;
					}

					Alloc.RootHeap = Allocation->GetRootHeap();
					Alloc.bIsHeap = Allocation->IsHeap();
					Alloc.bIsSwap = Allocation->IsSwap();

					Alloc.bIsDecline = false;
					if (Rule->GetValue() == TraceServices::IAllocationsProvider::EQueryRule::aAfaBf)
					{
						if (Alloc.StartTime <= TimeMarkers[0] && Alloc.EndTime <= TimeMarkers[1]) // decline
						{
							Alloc.Size = -Alloc.Size;
							Alloc.bIsDecline = true;
						}
					}
				}

				PageStopwatch.Stop();
				const double PageTime = PageStopwatch.GetAccumulatedTime();
				if (PageTime > 0.01)
				{
					const double Speed = (PageTime * 1000000.0) / AllocCount;
					UE_LOGF(LogMemoryProfiler, Log, "[MemAlloc] Query result for page %u (%u allocs, slack=%u) retrieved in %.3fs (speed: %.3f seconds per 1M allocs).", PageCount, AllocCount, Allocs.GetSlack(), PageTime, Speed);
				}

				Result = Status.NextResult();
			}

			ResultStopwatch.Stop();
			const double TotalTime = ResultStopwatch.GetAccumulatedTime();
			if (TotalTime > 0.01)
			{
				const double Speed = (TotalTime * 1000000.0) / TotalAllocCount;
				UE_LOGF(LogMemoryProfiler, Log, "[MemAlloc] Query results (%u pages, %u allocs, slack=%u) retrieved in %.3fs (speed: %.3f seconds per 1M allocs).", PageCount, TotalAllocCount, Allocs.GetSlack(), TotalTime, Speed);
			}
		}

		TotalStopwatch.Update();
	}
	while (OutStatus == TraceServices::IAllocationsProvider::EQueryStatus::Available && TotalStopwatch.GetAccumulatedTime() < MaxPollTime);

	TotalStopwatch.Stop();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::CancelQuery()
{
	if (Query != 0)
	{
		if (Session.IsValid())
		{
			const TraceServices::IAllocationsProvider* AllocationsProvider = TraceServices::ReadAllocationsProvider(*Session.Get());
			if (AllocationsProvider)
			{
				AllocationsProvider->CancelQuery(Query);
				UE_LOGF(LogMemoryProfiler, Log, "[MemAlloc] Query canceled.");
			}
		}

		Query = 0;
		QueryStopwatch.Stop();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemAllocTableTreeView::IsRunning() const
{
	return Query != 0 || STableTreeView::IsRunning();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double SMemAllocTableTreeView::GetAllOperationsDuration()
{
	if (Query != 0)
	{
		QueryStopwatch.Update();
		return QueryStopwatch.GetAccumulatedTime();
	}

	return STableTreeView::GetAllOperationsDuration();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SMemAllocTableTreeView::GetCurrentOperationName() const
{
	if (Query != 0)
	{
		return LOCTEXT("CurrentOperationName", "Running Query");
	}

	return STableTreeView::GetCurrentOperationName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::UpdateQueryInfo()
{
	if (Rule.IsValid())
	{
		FText TimeMarkersText;

		const int NumMarkers = Rule->GetNumTimeMarkers();
		switch (NumMarkers)
		{
		case 1:
			TimeMarkersText = FText::Format(LOCTEXT("OneTimeMarkersFmt", "A={0}"), FText::AsNumber(TimeMarkers[0]));
			break;
		case 2:
			TimeMarkersText = FText::Format(LOCTEXT("TwoTimeMarkersFmt", "A={0}  B={1}"), FText::AsNumber(TimeMarkers[0]), FText::AsNumber(TimeMarkers[1]));
			break;
		case 3:
			TimeMarkersText = FText::Format(LOCTEXT("ThreeTimeMarkersFmt", "A={0}  B={1}  C={2}"), FText::AsNumber(TimeMarkers[0]), FText::AsNumber(TimeMarkers[1]), FText::AsNumber(TimeMarkers[2]));
			break;
		case 4:
			TimeMarkersText = FText::Format(LOCTEXT("FourTimeMarkersFmt", "A={0}  B={1}  C={2}  D={3}"), FText::AsNumber(TimeMarkers[0]), FText::AsNumber(TimeMarkers[1]), FText::AsNumber(TimeMarkers[2]), FText::AsNumber(TimeMarkers[3]));
			break;
		default:
			// Unhandled value
			check(false);
		}

		QueryInfo = FText::Format(LOCTEXT("QueryInfoFmt", "{0} ({1})"), Rule->GetVerboseName(), TimeMarkersText);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SMemAllocTableTreeView::GetQueryInfo() const
{
	return QueryInfo;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SMemAllocTableTreeView::GetQueryInfoTooltip() const
{
	return Rule.IsValid() ? Rule->GetDescription() : FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SMemAllocTableTreeView::GetFooterLeftText() const
{
	if (!Rule.IsValid())
	{
		return FText::GetEmpty();
	}

	if (Query != 0)
	{
		return LOCTEXT("FooterLeftTextRunningQuery", "running query...");
	}

	if (FilteredNodesPtr->Num() == TableRowNodes.Num())
	{
		return FText::Format(LOCTEXT("FooterLeftTextFmt1", "{0} {0}|plural(one=alloc,other=allocs)"), FText::AsNumber(TableRowNodes.Num()));
	}
	else
	{
		return FText::Format(LOCTEXT("FooterLeftTextFmt2", "{0} / {1} {1}|plural(one=alloc,other=allocs)"), FilteredNodesPtr->Num(), FText::AsNumber(TableRowNodes.Num()));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::UpdateSelectionStatsText()
{
	TArray<FTableTreeNodePtr> SelectedNodes;
	const int32 NumSelectedNodes = TreeView->GetSelectedItems(SelectedNodes);

	if (NumSelectedNodes > 0)
	{
		int64 TotalAllocCount = 0;
		int64 TotalAllocSize = 0;

		TSharedRef<FTableColumn> CountColumn = Table->FindColumnChecked(FMemAllocTableColumns::CountColumnId);
		TSharedRef<FTableColumn> SizeColumn = Table->FindColumnChecked(FMemAllocTableColumns::SizeColumnId);

		for (const FTableTreeNodePtr& Node : SelectedNodes)
		{
			const TOptional<FTableCellValue> CountValue = CountColumn->GetValue(*Node.Get());
			if (CountValue.IsSet())
			{
				TotalAllocCount += CountValue.GetValue().AsInt64();
			}

			const TOptional<FTableCellValue> SizeValue = SizeColumn->GetValue(*Node.Get());
			if (SizeValue.IsSet())
			{
				TotalAllocSize += SizeValue.GetValue().AsInt64();
			}
		}

		FNumberFormattingOptions FormattingOptionsMem;
		FormattingOptionsMem.MaximumFractionalDigits = 2;

		SelectionStatsText = FText::Format(LOCTEXT("SelectionStatsFmt", "{0} selected {0}|plural(one=item,other=items) ({1} {1}|plural(one=alloc,other=allocs), {2})"),
			FText::AsNumber(NumSelectedNodes), FText::AsNumber(TotalAllocCount), FText::AsMemory(TotalAllocSize, &FormattingOptionsMem));
	}
	else
	{
		SelectionStatsText = FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SMemAllocTableTreeView::GetFooterCenterText() const
{
	return SelectionStatsText;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SMemAllocTableTreeView::GetSymbolResolutionStatus() const
{
	if (Session.IsValid())
	{
		const TraceServices::IModuleProvider* ModuleProvider = ReadModuleProvider(*Session.Get());
		if (ModuleProvider)
		{
			TraceServices::IModuleProvider::FStats Stats;
			ModuleProvider->GetStats(&Stats);

			const int32 SymbolsPending = Stats.SymbolsDiscovered - (Stats.SymbolsCached + Stats.SymbolsResolved + Stats.SymbolsFailed);
			if (SymbolsPending != 0)
			{
				return FText::Format(LOCTEXT("SymbolsResolved1", "Resolving {0} / {1} symbols ({2} cached + {3} resolved + {4} failed)"),
					SymbolsPending, Stats.SymbolsDiscovered, Stats.SymbolsCached, Stats.SymbolsResolved, Stats.SymbolsFailed);
			}
			else
			{
				return FText::Format(LOCTEXT("SymbolsResolved2", "{0} symbols ({1} cached + {2} resolved + {3} failed)"),
					Stats.SymbolsDiscovered, Stats.SymbolsCached, Stats.SymbolsResolved, Stats.SymbolsFailed);
			}
		}
	}

	return LOCTEXT("SymbolsResolutionNotPossible", "Symbol resolution was not possible.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SMemAllocTableTreeView::GetSymbolResolutionTooltip() const
{
	if (Session.IsValid())
	{
		const TraceServices::IModuleProvider* ModuleProvider = ReadModuleProvider(*Session.Get());
		if (ModuleProvider)
		{
			return FSymbolSearchPathsHelper::GetLocalizedSymbolSearchPathsText(ModuleProvider);
		}
	}
	return FText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::TreeView_OnSelectionChanged(FTableTreeNodePtr SelectedItem, ESelectInfo::Type SelectInfo)
{
	UpdateSelectionStatsText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::UpdateFilterContext(const FFilterConfigurator& InFilterConfigurator, const FTableTreeNode& InNode) const
{
	STableTreeView::UpdateFilterContext(InFilterConfigurator, InNode);

	if (InNode.Is<FMemAllocNode>())
	{
		const FMemAllocNode& MemNode = InNode.As<FMemAllocNode>();

		// GetFullCallstack is super heavy to compute. Validate that the filter has a use for this key before computing it.
		if (InFilterConfigurator.IsKeyUsed(FullCallStackIndex))
		{
			FilterContext.SetFilterData<FString>(FullCallStackIndex, MemNode.GetFullCallstack(FMemAllocNode::ECallstackType::AllocCallstack).ToString());
		}

		if (InFilterConfigurator.IsKeyUsed(LLMFilterIndex))
		{
			FilterContext.SetFilterData<FString>(LLMFilterIndex, MemNode.GetMemAlloc()->GetTag());
		}

		if (InFilterConfigurator.IsKeyUsed(AllocThreadFilterIndex))
		{
			FilterContext.SetFilterData<int64>(AllocThreadFilterIndex, MemNode.GetMemAlloc()->GetAllocThreadId());
		}

		if (InFilterConfigurator.IsKeyUsed(FreeThreadFilterIndex))
		{
			FilterContext.SetFilterData<int64>(FreeThreadFilterIndex, MemNode.GetMemAlloc()->GetFreeThreadId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::InitFilterConfigurator(FFilterConfigurator& InOutFilterConfigurator)
{
	STableTreeView::InitFilterConfigurator(InOutFilterConfigurator);

	TSharedRef<FFilter> FullCallStackFilter = MakeShared<FFilter>(
		FullCallStackIndex,
		LOCTEXT("FullCallstack", "Full Alloc Callstack"),
		LOCTEXT("SearchFullCallstack", "Search in all the alloc callstack frames"),
		EFilterDataType::String,
		nullptr,
		FFilterService::Get()->GetStringOperators());
	FilterContext.AddFilterData<FString>(FullCallStackIndex, FString());
	InOutFilterConfigurator.Add(FullCallStackFilter);

	TSharedRef<FFilterWithSuggestions> LLMTagFilter = MakeShared<FFilterWithSuggestions>(
		LLMFilterIndex,
		LOCTEXT("LLMTag", "LLM Tag"),
		LOCTEXT("LLMTag", "LLM Tag"),
		EFilterDataType::String,
		nullptr,
		FFilterService::Get()->GetStringOperators());
	FilterContext.AddFilterData<FString>(LLMFilterIndex, FString());
	LLMTagFilter->SetCallback([this](const FString& Text, TArray<FString>& OutSuggestions)
	{
		this->PopulateLLMTagSuggestionList(Text, OutSuggestions);
	});
	InOutFilterConfigurator.Add(LLMTagFilter);

	TSharedPtr<TArray<TSharedPtr<IFilterOperator>>> AllocThreadFilterOperators = MakeShared<TArray<TSharedPtr<IFilterOperator>>>();
	AllocThreadFilterOperators->Add(StaticCastSharedRef<IFilterOperator>(MakeShared<FFilterOperator<int64>>(EFilterOperator::Eq, TEXT("Is"), [](int64 lhs, int64 rhs) { return lhs == rhs; })));

	TSharedRef<FFilterWithSuggestions> AllocThreadFilter = MakeShared<FFilterWithSuggestions>(
		AllocThreadFilterIndex,
		LOCTEXT("AllocThread", "Alloc Thread"),
		LOCTEXT("AllocThreadDesc", "The allocation thread."),
		EFilterDataType::StringInt64Pair,
		MakeShared<FThreadFilterValueConverter>(),
		AllocThreadFilterOperators);
	FilterContext.AddFilterData<int64>(AllocThreadFilterIndex, 0);
	AllocThreadFilter->SetCallback([this](const FString& Text, TArray<FString>& OutSuggestions)
		{
			this->PopulateThreadSuggestionList(Text, OutSuggestions);
		});
	InOutFilterConfigurator.Add(AllocThreadFilter);

	TSharedRef<FFilterWithSuggestions> FreeThreadFilter = MakeShared<FFilterWithSuggestions>(
		FreeThreadFilterIndex,
		LOCTEXT("FreeThread", "Free Thread"),
		LOCTEXT("FreeThreadDesc", "The thread the allocation was freed on."),
		EFilterDataType::StringInt64Pair,
		MakeShared<FThreadFilterValueConverter>(),
		AllocThreadFilterOperators);
	FilterContext.AddFilterData<int64>(FreeThreadFilterIndex, 0);
	FreeThreadFilter->SetCallback([this](const FString& Text, TArray<FString>& OutSuggestions)
		{
			this->PopulateThreadSuggestionList(Text, OutSuggestions);
		});
	InOutFilterConfigurator.Add(FreeThreadFilter);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::PopulateLLMTagSuggestionList(const FString& Text, TArray<FString>& OutSuggestions) const
{
	const TraceServices::IAllocationsProvider* AllocationsProvider = TraceServices::ReadAllocationsProvider(*Session.Get());
	if (!AllocationsProvider)
	{
		return;
	}

	// Use a Set to avoid duplicate tag names.
	TSet<FString> Suggestions;

	{
		TraceServices::FProviderReadScopeLock _(*AllocationsProvider);
		AllocationsProvider->EnumerateTags([&Suggestions, &Text](const TCHAR* Display, const TCHAR* FullPath, TraceServices::TagIdType CurrentTag, TraceServices::TagIdType ParentTag)
		{
			if (Text.IsEmpty() || FCString::Stristr(FullPath, *Text))
			{
				Suggestions.Add(FullPath);
			}

			return true;
		});
	}

	OutSuggestions = Suggestions.Array();
	OutSuggestions.Sort();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::PopulateThreadSuggestionList(const FString& Text, TArray<FString>& OutSuggestions) const
{
	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
	const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(*Session.Get());

	bool bIsInteger = Text.IsNumeric() && !Text.Contains(TEXT("."));
	TStringBuilder<128> Builder;
	ThreadProvider.EnumerateThreads([&Text, &OutSuggestions, &Builder, bIsInteger](const TraceServices::FThreadInfo& ThreadInfo)
	{
		bool bAddSuggestion = false;
		if (bIsInteger)
		{
			if (ThreadInfo.Id == FCString::Atoi(*Text))
			{
				bAddSuggestion = true;
			}
		}
		if (FCString::Stristr(ThreadInfo.Name, *Text))
		{
			bAddSuggestion = true;
		}

		if (bAddSuggestion)
		{
			Builder.Appendf(TEXT("%s (id:%d)"), ThreadInfo.Name, ThreadInfo.Id);
			OutSuggestions.Add(Builder.ToString());
			Builder.Reset();
		}
	});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SMemAllocTableTreeView::ConstructFunctionToggleButton()
{
	TSharedRef<SWidget> Widget = SNew(SCheckBox)
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.HAlign(HAlign_Center)
		.Padding(FMargin(4.0f, 2.0f, 4.0f, 2.0f))
		.OnCheckStateChanged(this, &SMemAllocTableTreeView::CallstackGroupingByFunction_OnCheckStateChanged)
		.IsChecked(this, &SMemAllocTableTreeView::CallstackGroupingByFunction_IsChecked)
		.ToolTip(
			SNew(SToolTip)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CallstackGroupingByFunction_Tooltip_Title", "Callstack Grouping by Function Name"))
					.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f, 8.0f, 2.0f, 2.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CallstackGroupingByFunction_Tooltip_Content", "If enabled, the callstack grouping will create a single group node per function name.\nExample 1: When two callstack frames are located in same function, but at different line numbers; \nExample 2: When a function is called recursively.\nOtherwise it will create separate group nodes for each unique callstack frame."))
					.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f, 8.0f, 2.0f, 2.0f)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Top)
					.Padding(0.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CallstackGroupingByFunction_Warning", "Warning:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
						.ColorAndOpacity(FLinearColor(1.0f, 0.6f, 0.3f, 1.0f))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CallstackGroupingByFunction_Warning_Content", "When this option is enabled, the tree nodes that have merged multiple callstack frames\nwill show in their tooltips the source file name and the line number of an arbitrary\ncallstack frame from ones merged by respective tree node."))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					]
				]
			])
		[
			SNew(SImage)
			.Image(FInsightsStyle::GetBrush("Icons.Function"))
		];

	return Widget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::CallstackGroupingByFunction_OnCheckStateChanged(ECheckBoxState NewRadioState)
{
	PreChangeGroupings();
	bIsCallstackGroupingByFunction = (NewRadioState == ECheckBoxState::Checked);
	for (TSharedPtr<FTreeNodeGrouping>& Grouping : AvailableGroupings)
	{
		if (Grouping->Is<FMemAllocGroupingByCallstack>())
		{
			Grouping->As<FMemAllocGroupingByCallstack>().SetGroupingByFunction(bIsCallstackGroupingByFunction);
		}
	}
	PostChangeGroupings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState SMemAllocTableTreeView::CallstackGroupingByFunction_IsChecked() const
{
	return bIsCallstackGroupingByFunction ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FMemAllocNode> SMemAllocTableTreeView::GetSingleSelectedMemAllocNode() const
{
	if (TreeView->GetNumItemsSelected() == 1)
	{
		FTableTreeNodePtr TreeNode = TreeView->GetSelectedItems()[0];
		if (TreeNode.IsValid() && TreeNode->Is<FMemAllocNode>())
		{
			TSharedPtr<FMemAllocNode> SelectedTreeNode = StaticCastSharedPtr<FMemAllocNode>(TreeNode);
			if (SelectedTreeNode.IsValid() && !SelectedTreeNode->IsGroup())
			{
				return SelectedTreeNode;
			}
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FCallstackFrameGroupNode> SMemAllocTableTreeView::GetSingleSelectedCallstackFrameGroupNode() const
{
	if (TreeView->GetNumItemsSelected() == 1)
	{
		FTableTreeNodePtr TreeNode = TreeView->GetSelectedItems()[0];
		if (TreeNode.IsValid() && TreeNode->Is<FCallstackFrameGroupNode>())
		{
			TSharedPtr<FCallstackFrameGroupNode> SelectedTreeNode = StaticCastSharedPtr<FCallstackFrameGroupNode>(TreeNode);
			if (SelectedTreeNode.IsValid() && SelectedTreeNode->IsGroup())
			{
				return SelectedTreeNode;
			}
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::ExtendMenu(TSharedRef<FExtender> Extender)
{
	Extender->AddMenuExtension("Misc", EExtensionHook::Before, nullptr, FMenuExtensionDelegate::CreateSP(this, &SMemAllocTableTreeView::ExtendMenuAllocation));
	Extender->AddMenuExtension("Misc", EExtensionHook::Before, nullptr, FMenuExtensionDelegate::CreateSP(this, &SMemAllocTableTreeView::ExtendMenuCallstackFrame));
	Extender->AddMenuExtension("Misc", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateSP(this, &SMemAllocTableTreeView::ExtendMenuExportSnapshot));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::ExtendMenuAllocation(FMenuBuilder& MenuBuilder)
{
	FMemAllocNodePtr SingleSelectedMemAllocNode = GetSingleSelectedMemAllocNode();
	if (!SingleSelectedMemAllocNode.IsValid())
	{
		return;
	}

	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();

	MenuBuilder.BeginSection("Allocation", LOCTEXT("ContextMenu_Section_Allocation", "Allocation"));
	{
		FText ItemLabel;
		FText ItemToolTip;

		if (SourceCodeAccessor.CanAccessSourceCode())
		{
			ItemLabel = FText::Format(LOCTEXT("ContextMenu_OpenSourceAllocCallstack_SubMenu", "Open in {0} | Alloc Callstack"),
				SourceCodeAccessor.GetNameText());
			ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSourceAllocCallstack_SubMenu_Desc", "Opens the source file of the selected allocation callstack frame in {0}."),
				SourceCodeAccessor.GetNameText());
		}
		else
		{
			ItemLabel = LOCTEXT("ContextMenu_AllocCallstack_SubMenu", "Alloc Callstack");
			ItemToolTip = LOCTEXT("ContextMenu_SourceCodeAccessorNA", "Source Code Accessor is not available.");
		}

		// Alloc Callstack
		MenuBuilder.AddSubMenu
		(
			ItemLabel,
			ItemToolTip,
			FNewMenuDelegate::CreateSP(this, &SMemAllocTableTreeView::BuildOpenSourceSubMenu, true, ESourceSubMenuType::OpenInSourceEditor),
			false,
			FSlateIcon(SourceCodeAccessor.GetStyleSet(), SourceCodeAccessor.GetOpenIconName())
		);

		if (SourceCodeAccessor.CanAccessSourceCode())
		{
			ItemLabel = FText::Format(LOCTEXT("ContextMenu_OpenSourceFreeCallstack_SubMenu", "Open in {0} | Free Callstack"),
				SourceCodeAccessor.GetNameText());
			ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSourceFreeCallstack_SubMenu_Desc", "Opens the source file of the selected free callstack frame in {0}."),
				SourceCodeAccessor.GetNameText());
		}
		else
		{
			ItemLabel = LOCTEXT("ContextMenu_FreeCallstack_SubMenu", "Free Callstack");
			ItemToolTip = LOCTEXT("ContextMenu_SourceCodeAccessorNA", "Source Code Accessor is not available.");
		}

		// Free Callstack
		MenuBuilder.AddSubMenu
		(
			ItemLabel,
			ItemToolTip,
			FNewMenuDelegate::CreateSP(this, &SMemAllocTableTreeView::BuildOpenSourceSubMenu, false, ESourceSubMenuType::OpenInSourceEditor),
			false,
			FSlateIcon(SourceCodeAccessor.GetStyleSet(), SourceCodeAccessor.GetOpenIconName())
		);

		// Copy Callstack
		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_CopyAllocCallstack_SubMenu", "Copy to Clipboard | Alloc Callstack"),
			LOCTEXT("ContextMenu_CopyAllocCallstack_SubMenu_Desc", "Copy the source file path and line of the selected allocation callstack frame to the clipboard."),
			FNewMenuDelegate::CreateSP(this, &SMemAllocTableTreeView::BuildOpenSourceSubMenu, true, ESourceSubMenuType::CopyToClipboard),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy")
		);

		// Free Callstack
		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_CopyFreeCallstack_SubMenu", "Copy to Clipboard | Free Callstack"),
			LOCTEXT("ContextMenu_CopyFreeCallstack_SubMenu_Desc", "Copy the source file path and line of the selected free callstack frame to the clipboard."),
			FNewMenuDelegate::CreateSP(this, &SMemAllocTableTreeView::BuildOpenSourceSubMenu, false, ESourceSubMenuType::CopyToClipboard),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy")
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::ExtendMenuCallstackFrame(FMenuBuilder & MenuBuilder)
{
	TSharedPtr<FCallstackFrameGroupNode> SingleSelectedCallstackFrameGroupNode = GetSingleSelectedCallstackFrameGroupNode();
	if (!SingleSelectedCallstackFrameGroupNode.IsValid())
	{
		return;
	}

	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();

	MenuBuilder.BeginSection("CallstackFrame", LOCTEXT("ContextMenu_Section_CallstackFrame", "Callstack Frame"));
	{
		FText FileName = GetSelectedCallstackFrameFileName();
		if (SourceCodeAccessor.CanAccessSourceCode())
		{
			FText ItemLabel = FText::Format(LOCTEXT("ContextMenu_OpenSourceFile", "Open Source File in {0}"),
				SourceCodeAccessor.GetNameText());

			FText ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSourceFile_Desc", "Opens the source file of the selected callstack frame in {0}.\n{1}"),
				SourceCodeAccessor.GetNameText(),
				FileName);

			MenuBuilder.AddMenuEntry(
				ItemLabel,
				ItemToolTip,
				FSlateIcon(SourceCodeAccessor.GetStyleSet(), SourceCodeAccessor.GetOpenIconName()),
				FUIAction(
					FExecuteAction::CreateSP(this, &SMemAllocTableTreeView::OpenCallstackFrameSourceFileInIDE),
					FCanExecuteAction::CreateSP(this, &SMemAllocTableTreeView::CanOpenCallstackFrameSourceFileInIDE)));
		}
		else
		{
			FText ItemLabel = LOCTEXT("ContextMenu_OpenSourceFile_NoAccessor", "Open Source File");

			FText ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSourceFile_NoAccessor_Desc_Fmt", "{0}\nSource Code Accessor is not available."),
				FileName);

			MenuBuilder.AddMenuEntry(
				ItemLabel,
				ItemToolTip,
				FSlateIcon(SourceCodeAccessor.GetStyleSet(), SourceCodeAccessor.GetOpenIconName()),
				FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([]() { return false; })),
				NAME_None,
				EUserInterfaceActionType::None);
		}

		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ContextMenu_CopySourceFilePath_ToClipboard", "Copy Source File Path to Clipboard"),
				FileName,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
				FUIAction(
					FExecuteAction::CreateSP(this, &SMemAllocTableTreeView::CopySelectedFilePathToClipboard),
					FCanExecuteAction() ));
		}
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::ExtendMenuExportSnapshot(FMenuBuilder& MenuBuilder)
{
	const FText ItemLabel = LOCTEXT("ContextMenu_Export_SubMenu", "Export Snapshot...");
	const FText ItemToolTip = LOCTEXT("ContextMenu_Export_Desc_SubMenu", "Export memory snapshot to construct diff later.");

	MenuBuilder.AddMenuEntry(
		ItemLabel,
		ItemToolTip,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Save"),
		FUIAction(
			FExecuteAction::CreateSP(this, &SMemAllocTableTreeView::ExportMemorySnapshot),
			FCanExecuteAction::CreateSP(this, &SMemAllocTableTreeView::IsExportMemorySnapshotAvailable)),
		NAME_None,
		EUserInterfaceActionType::Button);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 SMemAllocTableTreeView::CountSourceFiles(FMemAllocNode& MemAllocNode)
{
	if (MemAllocNode.IsGroup())
	{
		return 0;
	}

	const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
	if (!Alloc || !Alloc->AllocCallstack)
	{
		return 0;
	}

	uint32 NumSourceFiles = 0;
	const uint32 NumCallstackFrames = Alloc->AllocCallstack->Num();
	check(NumCallstackFrames <= 256); // see Callstack->Frame(uint8)
	for (uint32 FrameIndex = 0; FrameIndex < NumCallstackFrames; ++FrameIndex)
	{
		const TraceServices::FStackFrame* Frame = Alloc->AllocCallstack->Frame(static_cast<uint8>(FrameIndex));
		if (Frame && Frame->Symbol && Frame->Symbol->File)
		{
			++NumSourceFiles;
		}
	}
	return NumSourceFiles;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemAllocTableTreeView::BuildOpenSourceSubMenuItems(FMenuBuilder& MenuBuilder, const TraceServices::FCallstack& Callstack, ESourceSubMenuType SourceSubMenuType)
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();

	uint32 NumSourceFiles = 0;
	const uint32 NumCallstackFrames = Callstack.Num();
	check(NumCallstackFrames <= 256); // see Callstack.Frame(uint8)
	for (uint32 FrameIndex = 0; FrameIndex < NumCallstackFrames; ++FrameIndex)
	{
		const TraceServices::FStackFrame* Frame = Callstack.Frame(static_cast<uint8>(FrameIndex));
		if (Frame && Frame->Symbol && Frame->Symbol->File)
		{
			FText ItemLabel;
			FText ItemToolTip;
			FString UsablePath;
			const bool bPathExists = FInsightsManager::Get()->GetSourceFilePathHelper().GetUsableFilePath(Frame->Symbol->File, UsablePath);

			if (Frame->Symbol->GetResult() == TraceServices::ESymbolQueryResult::OK)
			{
				FText FileName;
				constexpr int32 MaxFileNameLen = 120;
				const int32 FileNameLen = UsablePath.Len();
				if (FileNameLen > MaxFileNameLen)
				{
					FString FileNameStr = TEXT("...") + UsablePath.Right(MaxFileNameLen);
					FileName = FText::FromString(FileNameStr);
				}
				else
				{
					FileName = FText::FromString(UsablePath);
				}

				FText SymbolName;
				constexpr int32 MaxSymbolNameLen = 100;
				const int32 SymbolNameLen = FCString::Strlen(Frame->Symbol->Name);
				if (SymbolNameLen > MaxSymbolNameLen)
				{
					FString SymbolNameStr = TEXT("...") + FString::ConstructFromPtrSize(Frame->Symbol->Name + (SymbolNameLen - MaxSymbolNameLen), MaxSymbolNameLen);
					SymbolName = FText::FromString(SymbolNameStr);
				}
				else
				{
					SymbolName = FText::FromString(Frame->Symbol->Name);
				}

				ItemLabel = FText::Format(LOCTEXT("ContextMenu_OpenSource_Fmt1", "{0} ({1}) \u2192 {2}"),
					FileName,
					FText::AsNumber(Frame->Symbol->Line, &FNumberFormattingOptions::DefaultNoGrouping()),
					SymbolName);

				if (SourceCodeAccessor.CanAccessSourceCode())
				{
					ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSource_Desc_Fmt1", "Opens the source file of the selected callstack frame in {0}.\n{1} (line {2})\n\u2192 {3}"),
						SourceCodeAccessor.GetNameText(),
						FText::FromString(UsablePath),
						FText::AsNumber(Frame->Symbol->Line, &FNumberFormattingOptions::DefaultNoGrouping()),
						FText::FromString(Frame->Symbol->Name));
				}
				else
				{
					ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSource_NoAccessor_Desc_Fmt1", "{0} (line {1})\n\u2192 {2}\nSource Code Accessor is not available."),
						FText::FromString(UsablePath),
						FText::AsNumber(Frame->Symbol->Line, &FNumberFormattingOptions::DefaultNoGrouping()),
						FText::FromString(Frame->Symbol->Name));
				}
			}
			else
			{
				ItemLabel = FText::Format(LOCTEXT("ContextMenu_OpenSource_Fmt2", "{0} ({1}) \u2192 {2}"),
					FText::FromString(Frame->Symbol->Module),
					FText::FromString(FString::Printf(TEXT("0x%llX"), Frame->Addr)),
					FText::FromString(TraceServices::QueryResultToString(Frame->Symbol->GetResult())));

				if (SourceCodeAccessor.CanAccessSourceCode())
				{
					ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSource_Desc_Fmt2", "Opens the source file of the selected callstack frame in {0}."),
						SourceCodeAccessor.GetNameText());
				}
				else
				{
					ItemToolTip = LOCTEXT("ContextMenu_SourceCodeAccessorNA", "Source Code Accessor is not available.");
				}
			}

			const bool bCanOpenSource = bPathExists && SourceCodeAccessor.CanAccessSourceCode();
			switch (SourceSubMenuType)
			{
				case ESourceSubMenuType::OpenInSourceEditor:
				{
					MenuBuilder.AddMenuEntry(
						ItemLabel,
						ItemToolTip,
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateSP(this, &SMemAllocTableTreeView::OpenSourceFileInIDE, UsablePath, uint32(Frame->Symbol->Line)),
							FCanExecuteAction::CreateLambda([bCanOpenSource]() { return bCanOpenSource; })),
						NAME_None,
						EUserInterfaceActionType::Button);
					break;
				}
				case ESourceSubMenuType::CopyToClipboard:
				{
					MenuBuilder.AddMenuEntry(
						ItemLabel,
						LOCTEXT("ContextMenu_CopySourceCode_Desc", "Copy the source file path and line to the clipboard."),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateSP(this, &SMemAllocTableTreeView::CopySourceFileToClipboard, UsablePath, uint32(Frame->Symbol->Line)),
							FCanExecuteAction()),
						NAME_None,
						EUserInterfaceActionType::Button);
						break;
				}
			}

			++NumSourceFiles;
		}
	}

	return NumSourceFiles > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::BuildOpenSourceSubMenu(FMenuBuilder& MenuBuilder, bool bIsAllocCallstack, ESourceSubMenuType SourceSubMenuType)
{
	MenuBuilder.BeginSection("OpenSource");
	{
		bool bHasAnySourceFilesToOpen = false;
		FMemAllocNodePtr MemAllocNode = GetSingleSelectedMemAllocNode();
		if (MemAllocNode.IsValid())
		{
			const FMemoryAlloc* Alloc = MemAllocNode->GetMemAlloc();
			if (Alloc)
			{
				const TraceServices::FCallstack* Callstack = bIsAllocCallstack ? Alloc->AllocCallstack : Alloc->FreeCallstack;
				if (Callstack)
				{
					bHasAnySourceFilesToOpen = BuildOpenSourceSubMenuItems(MenuBuilder, *Callstack, SourceSubMenuType);
				}
			}
		}

		if (!bHasAnySourceFilesToOpen)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ContextMenu_OpenSourceNA", "Not Available"),
				TAttribute<FText>(),
				FSlateIcon(),
				FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([]() { return false; })),
				NAME_None,
				EUserInterfaceActionType::None);
		}
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::OpenSourceFileInIDE(FString InFile, uint32 Line) const
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	if (FPaths::FileExists(InFile))
	{
		ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();
		SourceCodeAccessor.OpenFileAtLine(InFile, Line);
	}
	else
	{
		SourceCodeAccessModule.OnOpenFileFailed().Broadcast(InFile);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::CopySourceFileToClipboard(FString File, uint32 Line) const
{
	FPlatformApplicationMisc::ClipboardCopy(*FString::Format(TEXT("{0} ({1})"), { *File, Line }));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::CopySelectedFilePathToClipboard() const
{
	FPlatformApplicationMisc::ClipboardCopy(*GetSelectedCallstackFrameFileName().ToString());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::ExportMemorySnapshot() const
{
	// 1. Choose file
	FString DefaultFile = TEXT("Table");
	if (Table.IsValid() && !Table->GetDisplayName().IsEmpty())
	{
		DefaultFile = Table->GetDisplayName().ToString();
		DefaultFile.RemoveSpacesInline();
	}

	TArray<FString> SaveFilenames;
	bool bDialogResult = false;

	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		const FString DefaultPath = FPaths::ProjectSavedDir();
		bDialogResult = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("ExportFileTitle", "Export Table").ToString(),
			DefaultPath,
			DefaultFile,
			TEXT("Comma-Separated Values (*.csv)|*.csv|Tab-Separated Values (*.tsv)|*.tsv|Text Files (*.txt)|*.txt|All Files (*.*)|*.*"),
			EFileDialogFlags::None,
			SaveFilenames
		);
	}

	if (!bDialogResult || SaveFilenames.Num() == 0)
	{
		return;
	}

	const FString& FilePath = SaveFilenames[0];

	// Building the CSV/TSV data
	TUniquePtr<IFileHandle> FileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*FilePath));
	if (!FileHandle)
	{
		FMessageLog ReportMessageLog((LogListingName != NAME_None) ? LogListingName : TEXT("Other"));
		ReportMessageLog.Error(LOCTEXT("FailedToOpenFile", "Export failed. Failed to open file for write."));
		ReportMessageLog.Notify();
		return;
	}
	bool bIsCSV = FilePath.EndsWith(TEXT(".csv"));
	FUtf8FileWriter Writer(FileHandle.Get(), bIsCSV);

	auto WriteColumnHeader =
		[](FUtf8FileWriter& Writer, const FTableColumn& Column)
		{
			FString Value = Column.GetShortName().ToString();
			Writer.AppendValue(Value);
		};
	auto WriteColumn =
		[](FUtf8FileWriter& Writer, const FTableColumn& Column, const FTableTreeNode& Node)
		{
			FString Value = Column.GetValue(Node)->AsString();
			Writer.AppendValue(Value);
		};
	auto WriteCallstackColumn =
		[](FUtf8FileWriter& Writer, const FTableTreeNode& Node, bool bIsAllocCallstack)
		{
			if (!Node.Is<FMemAllocNode>())
			{
				return;
			}
			const FMemAllocNode& MemAllocNode = Node.As<FMemAllocNode>();
			const FMemoryAlloc* Alloc = MemAllocNode.GetMemAlloc();
			if (Alloc)
			{
				const TraceServices::FCallstack* Callstack = bIsAllocCallstack ? Alloc->AllocCallstack : Alloc->FreeCallstack;

				if (!Callstack)
				{
					Writer.AppendValue(GetCallstackNotAvailableString().ToString());
					return;
				}

				if (Callstack->Num() == 0)
				{
					if (Callstack->GetEmptyId() == 0)
					{
						Writer.AppendValue(GetNoCallstackString().ToString());
					}
					else
					{
						Writer.AppendValue(GetEmptyCallstackString().ToString());
					}
					return;
				}

				TStringBuilder<1024> StringBuilder;
				const uint32 NumCallstackFrames = Callstack->Num();
				check(NumCallstackFrames <= 256); // see Callstack->Frame(uint8)
				for (uint32 Index = 0; Index < NumCallstackFrames; ++Index)
				{
					if (Index != 0)
					{
						StringBuilder.Append(TEXT("/"));
					}
					const TraceServices::FStackFrame* Frame = Callstack->Frame(static_cast<uint8>(Index));
					check(Frame != nullptr);
					FormatStackFrame(*Frame, StringBuilder, EStackFrameFormatFlags::ModuleAndSymbol);
				}
				Writer.AppendValue(StringBuilder.ToView());
			}
		};
	const FTableColumn& StartEventIndexColumn = *GetTable()->FindColumn(FMemAllocTableColumns::StartEventIndexColumnId);
	const FTableColumn& EndEventIndexColumn = *GetTable()->FindColumn(FMemAllocTableColumns::EndEventIndexColumnId);
	const FTableColumn& EventDistanceColumn = *GetTable()->FindColumn(FMemAllocTableColumns::EventDistanceColumnId);
	const FTableColumn& StartTimeColumn = *GetTable()->FindColumn(FMemAllocTableColumns::StartTimeColumnId);
	const FTableColumn& EndTimeColumn = *GetTable()->FindColumn(FMemAllocTableColumns::EndTimeColumnId);
	const FTableColumn& DurationColumn = *GetTable()->FindColumn(FMemAllocTableColumns::DurationColumnId);
	const FTableColumn& AddressColumn = *GetTable()->FindColumn(FMemAllocTableColumns::AddressColumnId);
	const FTableColumn& MemoryPageColumn = *GetTable()->FindColumn(FMemAllocTableColumns::MemoryPageColumnId);
	const FTableColumn& SizeColumn = *GetTable()->FindColumn(FMemAllocTableColumns::SizeColumnId);
	const FTableColumn& TagColumn = *GetTable()->FindColumn(FMemAllocTableColumns::TagColumnId);
	const FTableColumn& AssetColumn = *GetTable()->FindColumn(FMemAllocTableColumns::AssetColumnId);
	const FTableColumn& ClassNameColumn = *GetTable()->FindColumn(FMemAllocTableColumns::ClassNameColumnId);
	const FTableColumn& PackageColumn = *GetTable()->FindColumn(FMemAllocTableColumns::PackageColumnId);
	const FTableColumn& AllocThreadColumn = *GetTable()->FindColumn(FMemAllocTableColumns::AllocThreadColumnId);
	const FTableColumn& FreeThreadColumn = *GetTable()->FindColumn(FMemAllocTableColumns::FreeThreadColumnId);
	const FTableColumn& AllocFunctionColumn = *GetTable()->FindColumn(FMemAllocTableColumns::AllocFunctionColumnId);
	const FTableColumn& FreeFunctionColumn = *GetTable()->FindColumn(FMemAllocTableColumns::FreeFunctionColumnId);
	const FTableColumn& AllocSourceFileColumn = *GetTable()->FindColumn(FMemAllocTableColumns::AllocSourceFileColumnId);
	const FTableColumn& FreeSourceFileColumn = *GetTable()->FindColumn(FMemAllocTableColumns::FreeSourceFileColumnId);

	// 2. Iterate over TreeNodes
	bool IsFirstRow = true;
	for (const TSharedPtr<FTableTreeNode>& Node : *FilteredNodesPtr)
	{
		// Export only leaves
		if (Node->IsGroup()) continue;

		if (IsFirstRow)
		{
			WriteColumnHeader(Writer, StartEventIndexColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, EndEventIndexColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, EventDistanceColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, StartTimeColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, EndTimeColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, DurationColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, AddressColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, MemoryPageColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, SizeColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, TagColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, AssetColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, ClassNameColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, PackageColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, AllocThreadColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, FreeThreadColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, AllocFunctionColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, FreeFunctionColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, AllocSourceFileColumn); Writer.AppendSeparator();
			WriteColumnHeader(Writer, FreeSourceFileColumn); Writer.AppendSeparator();
			Writer.Append(UTF8TEXTVIEW("Alloc Callstack")); Writer.AppendSeparator();
			Writer.Append(UTF8TEXTVIEW("Free Callstack")); Writer.AppendLineEnd();

			IsFirstRow = false;
		}

		// 3. Export these column values as is:
		WriteColumn(Writer, StartEventIndexColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, EndEventIndexColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, EventDistanceColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, StartTimeColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, EndTimeColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, DurationColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, AddressColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, MemoryPageColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, SizeColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, TagColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, AssetColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, ClassNameColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, PackageColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, AllocThreadColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, FreeThreadColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, AllocFunctionColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, FreeFunctionColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, AllocSourceFileColumn, *Node); Writer.AppendSeparator();
		WriteColumn(Writer, FreeSourceFileColumn, *Node); Writer.AppendSeparator();
		WriteCallstackColumn(Writer, *Node, true); Writer.AppendSeparator();
		WriteCallstackColumn(Writer, *Node, false); Writer.AppendLineEnd();
	}

	Writer.Flush();
	FileHandle->Flush();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemAllocTableTreeView::IsExportMemorySnapshotAvailable() const
{
	return FilteredNodesPtr && !FilteredNodesPtr->IsEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemAllocTableTreeView::CanOpenCallstackFrameSourceFileInIDE() const
{
	if (TreeView->GetNumItemsSelected() == 1)
	{
		FTableTreeNodePtr TreeNode = TreeView->GetSelectedItems()[0];
		if (TreeNode.IsValid() && TreeNode->Is<FCallstackFrameGroupNode>())
		{
			const FCallstackFrameGroupNode& CallstackFrameNode = TreeNode->As<FCallstackFrameGroupNode>();
			return CallstackFrameNode.GetStackFrame() != nullptr;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemAllocTableTreeView::OpenCallstackFrameSourceFileInIDE()
{
	if (TreeView->GetNumItemsSelected() == 1)
	{
		FTableTreeNodePtr TreeNode = TreeView->GetSelectedItems()[0];
		if (TreeNode.IsValid() && TreeNode->Is<FCallstackFrameGroupNode>())
		{
			const FCallstackFrameGroupNode& CallstackFrameNode = TreeNode->As<FCallstackFrameGroupNode>();
			const TraceServices::FStackFrame* Frame = CallstackFrameNode.GetStackFrame();
			if (Frame && Frame->Symbol && Frame->Symbol->File)
			{
				FString UsablePath;
				const bool bPathExists = FInsightsManager::Get()->GetSourceFilePathHelper().GetUsableFilePath(Frame->Symbol->File, UsablePath);

				OpenSourceFileInIDE(UsablePath, Frame->Symbol->Line);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SMemAllocTableTreeView::GetSelectedCallstackFrameFileName() const
{
	if (TreeView->GetNumItemsSelected() == 1)
	{
		FTableTreeNodePtr TreeNode = TreeView->GetSelectedItems()[0];
		if (TreeNode.IsValid() && TreeNode->Is<FCallstackFrameGroupNode>())
		{
			const FCallstackFrameGroupNode& CallstackFrameNode = TreeNode->As<FCallstackFrameGroupNode>();
			const TraceServices::FStackFrame* Frame = CallstackFrameNode.GetStackFrame();
			if (Frame && Frame->Symbol && Frame->Symbol->File)
			{
				FString UsablePath;
				const bool bPathExists = FInsightsManager::Get()->GetSourceFilePathHelper().GetUsableFilePath(Frame->Symbol->File, UsablePath);
				FString SourceFileAndLine = FString::Printf(TEXT("%s(%d)"), *UsablePath, Frame->Symbol->Line);
				return FText::FromString(SourceFileAndLine);
			}
			else
			{
				return LOCTEXT("NoSourceFile", "(source file not available)");
			}
		}
	}
	return LOCTEXT("NoCallstackFrame", "(only for resolved callstack frames)");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
