// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceServices
#include "TraceServices/Model/AllocationsProvider.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"

// TraceInsights
#include "Insights/MemoryProfiler/ViewModels/MemAllocTable.h"
#include "Insights/Table/Widgets/SSessionTableTreeView.h"

namespace TraceServices
{
	struct FCallstack;
}

namespace UE::Insights::MemoryProfiler
{

class FCallstackFrameGroupNode;
class FMemAllocNode;
class FMemoryRuleSpec;

////////////////////////////////////////////////////////////////////////////////////////////////////

class SMemAllocTableTreeView : public SSessionTableTreeView
{
public:
	/** Default constructor. */
	SMemAllocTableTreeView();

	/** Virtual destructor. */
	virtual ~SMemAllocTableTreeView();

	SLATE_BEGIN_ARGS(SMemAllocTableTreeView) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<FMemAllocTable> InTablePtr);

	virtual TSharedPtr<SWidget> ConstructToolbar() override;
	virtual TSharedPtr<SWidget> ConstructFooter() override;

	TSharedPtr<FMemAllocTable> GetMemAllocTable()
	{
		return StaticCastSharedPtr<FMemAllocTable>(GetTable());
	}

	TSharedPtr<const FMemAllocTable> GetMemAllocTable() const
	{
		return StaticCastSharedPtr<const FMemAllocTable>(GetTable());
	}

	virtual void Reset();

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/**
	 * Rebuilds the tree (if necessary).
	 * @param bResync - If true, it forces a resync even if the list did not changed since last sync.
	 */
	virtual void RebuildTree(bool bResync);

	bool HasHeapAllocs() const { return bIncludeHeapAllocs; }
	bool HasSwapAllocs() const { return bIncludeSwapAllocs; }

	struct FQueryParams
	{
		TSharedPtr<FMemoryRuleSpec> Rule;
		double TimeMarkers[4] = { 0.0, 0.0, 0.0, 0.0 };
		bool bIncludeHeapAllocs = false;
		bool bIncludeSwapAllocs = false;
		TSet<FString> PackageFilter;
	};

	void SetQueryParams(const FQueryParams& InQueryParams)
	{
		Rule = InQueryParams.Rule;
		TimeMarkers[0] = InQueryParams.TimeMarkers[0];
		TimeMarkers[1] = InQueryParams.TimeMarkers[1];
		TimeMarkers[2] = InQueryParams.TimeMarkers[2];
		TimeMarkers[3] = InQueryParams.TimeMarkers[3];
		bIncludeHeapAllocs = InQueryParams.bIncludeHeapAllocs;
		bIncludeSwapAllocs = InQueryParams.bIncludeSwapAllocs;
		PackageFilter = InQueryParams.PackageFilter;
		OnQueryInvalidated();
	}

	int32 GetTabIndex() const { return TabIndex; }
	void SetTabIndex(int32 InTabIndex) { TabIndex = InTabIndex; }

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// IAsyncOperationStatusProvider implementation

	virtual bool IsRunning() const override;
	virtual double GetAllOperationsDuration() override;
	virtual FText GetCurrentOperationName() const override;

	////////////////////////////////////////////////////////////////////////////////////////////////////

protected:
	virtual void InternalCreateGroupings() override;

	TSharedPtr<FMemAllocNode> GetSingleSelectedMemAllocNode() const;
	TSharedPtr<FCallstackFrameGroupNode> GetSingleSelectedCallstackFrameGroupNode() const;
	uint32 CountSourceFiles(FMemAllocNode& MemAllocNode);

	virtual void ExtendMenu(TSharedRef<FExtender> Extender) override;
	enum class ESourceSubMenuType : uint8 { OpenInSourceEditor, CopyToClipboard };
	bool BuildOpenSourceSubMenuItems(FMenuBuilder& MenuBuilder, const TraceServices::FCallstack& Callstack, ESourceSubMenuType SourceSubMenuType);
	void BuildOpenSourceSubMenu(FMenuBuilder& MenuBuilder, bool bIsAllocCallstack, ESourceSubMenuType SourceSubMenuType);
	bool CanOpenCallstackFrameSourceFileInIDE() const;
	void OpenCallstackFrameSourceFileInIDE();
	FText GetSelectedCallstackFrameFileName() const;
	void OpenSourceFileInIDE(FString File, uint32 Line) const;
	void CopySourceFileToClipboard(FString File, uint32 Line) const;
	void CopySelectedFilePathToClipboard() const;

	void ExportMemorySnapshot() const;
	bool IsExportMemorySnapshotAvailable() const;

	virtual void TreeView_OnSelectionChanged(FTableTreeNodePtr SelectedItem, ESelectInfo::Type SelectInfo) override;

	virtual void UpdateFilterContext(const FFilterConfigurator& InFilterConfigurator, const FTableTreeNode& InNode) const override;
	virtual void InitFilterConfigurator(FFilterConfigurator& InOutFilterConfigurator) override;

private:
	void InitAvailableViewPresets();

	void ExtendMenuAllocation(FMenuBuilder& MenuBuilder);
	void ExtendMenuCallstackFrame(FMenuBuilder& MenuBuilder);
	void ExtendMenuExportSnapshot(FMenuBuilder& MenuBuilder);

	//////////////////////////////////////////////////
	// Queries

	void OnQueryInvalidated();
	void StartQuery();
	void UpdateQuery(TraceServices::IAllocationsProvider::EQueryStatus& OutStatus);
	void CancelQuery();
	void ResetAndStartQuery();

	//////////////////////////////////////////////////
	// Footer / Status Bar

	void UpdateQueryInfo();
	FText GetQueryInfo() const;
	FText GetQueryInfoTooltip() const;

	FText GetFooterLeftText() const;

	void UpdateSelectionStatsText();
	FText GetFooterCenterText() const;

	FText GetSymbolResolutionStatus() const;
	FText GetSymbolResolutionTooltip() const;

	//////////////////////////////////////////////////

	TSharedRef<SWidget> ConstructFunctionToggleButton();
	void CallstackGroupingByFunction_OnCheckStateChanged(ECheckBoxState NewRadioState);
	ECheckBoxState CallstackGroupingByFunction_IsChecked() const;

	void PopulateLLMTagSuggestionList(const FString& Text, TArray<FString>& OutSuggestions) const;
	void PopulateThreadSuggestionList(const FString& Text, TArray<FString>& OutSuggestions) const;

private:
	const static int32 FullCallStackIndex;
	const static int32 LLMFilterIndex;
	const static int32 AllocThreadFilterIndex;
	const static int32 FreeThreadFilterIndex;

	int32 TabIndex = -1;
	TSharedPtr<FMemoryRuleSpec> Rule = nullptr;
	double TimeMarkers[4];
	bool bIncludeHeapAllocs = false;
	bool bIncludeSwapAllocs = false;
	TSet<FString> PackageFilter;
	TraceServices::IAllocationsProvider::FQueryHandle Query = 0;
	FText QueryInfo;
	FText QueryInfoTooltip;
	FText SelectionStatsText;
	FStopwatch QueryStopwatch;
	bool bHasPendingQueryReset = false;
	bool bIsCallstackGroupingByFunction = true;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
