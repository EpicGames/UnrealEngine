// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMemInvestigationView.h"

#include "DesktopPlatformModule.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/MemoryProfiler/Common/SymbolSearchPathsHelper.h"
#include "Insights/MemoryProfiler/MemoryProfilerManager.h"
#include "Insights/MemoryProfiler/ViewModels/MemorySharedState.h"
#include "Insights/MemoryProfiler/Widgets/SMemAllocTableTreeView.h"
#include "Insights/MemoryProfiler/Widgets/SMemoryProfilerWindow.h"
#include "Insights/TimingProfiler/ViewModels/TimeMarker.h"
#include "Insights/TimingProfiler/Widgets/STimeMarkerEditBlock.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::SMemInvestigationView"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

SMemInvestigationView::SMemInvestigationView()
	: ProfilerWindowWeakPtr()
	, bIncludeHeapAllocs(false)
	, bIncludeSwapAllocs(false)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SMemInvestigationView::~SMemInvestigationView()
{
	// Remove ourselves from the Insights manager.
	if (FInsightsManager::Get().IsValid())
	{
		FInsightsManager::Get()->GetSessionChangedEvent().RemoveAll(this);
	}

	Session.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMemInvestigationView::Construct(const FArguments& InArgs, TSharedPtr<SMemoryProfilerWindow> InProfilerWindow)
{
	check(InProfilerWindow.IsValid());
	ProfilerWindowWeakPtr = InProfilerWindow;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
		.Padding(8.0f)
		[
			ConstructInvestigationWidgetArea()
		]
	];

	// Register ourselves with the Insights manager.
	FInsightsManager::Get()->GetSessionChangedEvent().AddSP(this, &SMemInvestigationView::InsightsManager_OnSessionChanged);

	// Update the Session (i.e. when analysis session was already started).
	InsightsManager_OnSessionChanged();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SMemInvestigationView::ConstructInvestigationWidgetArea()
{
	TSharedRef<SWidget> Widget =
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("QueryRuleText", "Rule:"))
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SAssignNew(QueryRuleComboBox, SComboBox<TSharedPtr<FMemoryRuleSpec>>)
				.ToolTipText(this, &SMemInvestigationView::QueryRule_GetTooltipText)
				.OptionsSource(GetAvailableQueryRules())
				.OnSelectionChanged(this, &SMemInvestigationView::QueryRule_OnSelectionChanged)
				.OnGenerateWidget(this, &SMemInvestigationView::QueryRule_OnGenerateWidget)
				[
					SNew(STextBlock)
					.Text(this, &SMemInvestigationView::QueryRule_GetSelectedText)
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ConstructTimeMarkerWidget(0)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ConstructTimeMarkerWidget(1)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ConstructTimeMarkerWidget(2)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ConstructTimeMarkerWidget(3)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 4.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(this, &SMemInvestigationView::QueryRule_GetTooltipText)
			.ColorAndOpacity(FLinearColor(0.3f, 0.3f, 0.3f, 1.0f))
			.AutoWrapText(true)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 4.0f, 0.0f, 0.0f)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([this]()
			{
				return bIncludeHeapAllocs ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
			{
				bIncludeHeapAllocs = (InCheckBoxState == ECheckBoxState::Checked);
			})
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("IncludeHeapAllocsText", "Include Heap Allocs (Advanced)"))
			]
			.ToolTipText(LOCTEXT("IncludeHeapAllocsToolTipText", "Include Heap Allocs (Advanced Feature)\n\nIf enabled, the table will also list the heap allocs (\"heap_XXXX\").\nThe heap allocs are memory blocks that can have child allocations.\n\nWarning:\tTotal aggregated sizes may be irrelevant when a heap is added with its child allocs.\n\t\t\t\t\tSome memory would be double counted."))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 4.0f, 0.0f, 0.0f)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([this]()
			{
				return bIncludeSwapAllocs ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
			{
				bIncludeSwapAllocs = (InCheckBoxState == ECheckBoxState::Checked);
			})
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("IncludeSwapAllocsText", "Include Swap Entries"))
			]
			.ToolTipText(LOCTEXT("IncludeSwapAllocsToolTipText", "Include swap entries."))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 4.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("QueryTargetWindow", "Target Window:"))
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SAssignNew(QueryTargetComboBox, SComboBox<TSharedPtr<FQueryTargetWindowSpec>>)
				.ToolTipText(LOCTEXT("QueryTargetWindowTooltip", "Select an existing or new window where the query results will be displayed"))
				.OptionsSource(GetAvailableQueryTargets())
				.OnSelectionChanged(this, &SMemInvestigationView::QueryTarget_OnSelectionChanged)
				.OnGenerateWidget(this, &SMemInvestigationView::QueryTarget_OnGenerateWidget)
				[
					SNew(STextBlock)
					.Text(this, &SMemInvestigationView::QueryTarget_GetSelectedText)
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 4.0f, 0.0f, 0.0f)
		.HAlign(HAlign_Right)
		[
			SNew(SButton)
			.Text(LOCTEXT("RunQueryBtnText", "Run Query"))
			.ToolTipText(LOCTEXT("RunQueryBtnToolTipText", "Run Memory Query.\nThe resulting list of allocations will be available in a tree view."))
			.OnClicked(this, &SMemInvestigationView::RunQuery)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(0.0f, 10.0f, 0.0f, 0.0f))
		.HAlign(HAlign_Fill)
		[
			SAssignNew(SymbolPathsTextBlock, STextBlock)
			.ColorAndOpacity(FLinearColor(0.3f, 0.3f, 0.3f, 1.0f))
			.AutoWrapText(true)
		]
	;

	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
		QueryRuleComboBox->SetSelectedItem(SharedState.GetCurrentMemoryRule());
	}

	return Widget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SMemInvestigationView::QueryRule_OnGenerateWidget(TSharedPtr<FMemoryRuleSpec> InRule)
{
	const FText QueryRuleText = FText::Format(LOCTEXT("QueryRuleComboBox_TextFmt", "{0} ({1})"), InRule->GetVerboseName(), InRule->GetShortName());

	TSharedRef<SHorizontalBox> Widget = SNew(SHorizontalBox);

	Widget->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(0.0f, 2.0f, 2.0f, 2.0f)
		[
			SNew(SImage)
			.Visibility_Lambda([Widget]() { return Widget->GetParentWidget()->IsHovered() ? EVisibility::Visible : EVisibility::Hidden; })
			.Image(FInsightsStyle::GetBrush("Icons.Hint.TreeItem"))
			.ToolTipText(InRule->GetDescription())
		];

	Widget->AddSlot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(QueryRuleText)
			.Margin(2.0f)
		];

	return Widget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SMemInvestigationView::ConstructTimeMarkerWidget(uint32 TimeMarkerIndex)
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (!ProfilerWindow.IsValid())
	{
		return SNew(SBox);
	}

	const uint32 NumTimeMarkers = ProfilerWindow->GetNumCustomTimeMarkers();
	if (TimeMarkerIndex >= NumTimeMarkers)
	{
		return SNew(SBox);
	}

	using FTimeMarker = TimingProfiler::FTimeMarker;

	const TSharedRef<FTimeMarker>& TimeMarker = ProfilerWindow->GetCustomTimeMarker(TimeMarkerIndex);
	TSharedPtr<FTimeMarker> PreviousTimeMarker;
	if (TimeMarkerIndex > 0)
	{
		PreviousTimeMarker = ProfilerWindow->GetCustomTimeMarker(TimeMarkerIndex - 1);
	}

	TSharedRef<SWidget> Widget =
		SNew(TimingProfiler::STimeMarkerEditBlock, TimeMarker)
		.Padding(FMargin(20.0f, 0.0f, 0.0f, 0.0f))
		.Visibility_Lambda(
			[TimeMarkerIndex, ProfilerWindowWeak = ProfilerWindowWeakPtr]()
			{
				TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = ProfilerWindowWeak.Pin();
				if (ProfilerWindow.IsValid())
				{
					FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
					TSharedPtr<FMemoryRuleSpec> Rule = SharedState.GetCurrentMemoryRule();
					if (Rule && TimeMarkerIndex < Rule->GetNumTimeMarkers())
					{
						return EVisibility::Visible;
					}
				}
				return EVisibility::Collapsed;
			})
		.PreviousTimeMarker(PreviousTimeMarker)
		.OnGetTimingView_Lambda(
			[ProfilerWindowWeak = ProfilerWindowWeakPtr](TSharedRef<FTimeMarker> TimeMarker) -> TSharedPtr<TimingProfiler::STimingView>
			{
				TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = ProfilerWindowWeak.Pin();
				if (ProfilerWindow.IsValid())
				{
					return ProfilerWindow->GetTimingView();
				}
				return nullptr;
			})
		.OnTimeMarkerChanged_Lambda(
			[ProfilerWindowWeak = ProfilerWindowWeakPtr](TSharedRef<FTimeMarker> TimeMarker)
			{
				TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = ProfilerWindowWeak.Pin();
				if (ProfilerWindow.IsValid())
				{
					ProfilerWindow->OnTimeMarkerChanged(Timing::ETimeChangedFlags::None, TimeMarker);
				}
			})
		;

	return Widget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemInvestigationView::InsightsManager_OnSessionChanged()
{
	TSharedPtr<const TraceServices::IAnalysisSession> NewSession = FInsightsManager::Get()->GetSession();
	if (NewSession != Session)
	{
		Session = NewSession;
		Reset();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemInvestigationView::Reset()
{
	SymbolPathsTextBlock->SetText(FText::GetEmpty());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemInvestigationView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	UpdateSymbolPathsText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Query Rules
////////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<TSharedPtr<FMemoryRuleSpec>>* SMemInvestigationView::GetAvailableQueryRules()
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
		return &SharedState.GetMemoryRules();
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemInvestigationView::QueryRule_OnSelectionChanged(TSharedPtr<FMemoryRuleSpec> InRule, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
		if (ProfilerWindow.IsValid() && InRule)
		{
			FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
			SharedState.SetCurrentMemoryRule(InRule);
			ProfilerWindow->OnMemoryRuleChanged();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SMemInvestigationView::QueryRule_GetSelectedText() const
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
		TSharedPtr<FMemoryRuleSpec> Rule = SharedState.GetCurrentMemoryRule();
		if (Rule)
		{
			return FText::Format(LOCTEXT("QueryRuleComboBox_TextFmt", "{0} ({1})"), Rule->GetVerboseName(), Rule->GetShortName());
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SMemInvestigationView::QueryRule_GetTooltipText() const
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
		TSharedPtr<FMemoryRuleSpec> Rule = SharedState.GetCurrentMemoryRule();
		if (Rule.IsValid())
		{
			return Rule->GetDescription();
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<TSharedPtr<FQueryTargetWindowSpec>>* SMemInvestigationView::GetAvailableQueryTargets()
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
		return &SharedState.GetQueryTargets();
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemInvestigationView::QueryTarget_OnSelectionChanged(TSharedPtr<FQueryTargetWindowSpec> InTarget, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Type::Direct)
	{
		TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
		if (ProfilerWindow.IsValid() && InTarget)
		{
			FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
			SharedState.SetCurrentQueryTarget(InTarget);
		}
	}
	else
	{
		QueryTargetComboBox->SetSelectedItem(InTarget);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemInvestigationView::UpdateSymbolPathsText() const
{
	if (SymbolPathsTextBlock->GetText().IsEmpty() && Session)
	{
		if (const TraceServices::IModuleProvider* ModuleProvider = ReadModuleProvider(*Session.Get()))
		{
			SymbolPathsTextBlock->SetText(FSymbolSearchPathsHelper::GetLocalizedSymbolSearchPathsText(ModuleProvider));
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SMemInvestigationView::QueryTarget_OnGenerateWidget(TSharedPtr<FQueryTargetWindowSpec> InTarget)
{
	return SNew(STextBlock)
		.Text(InTarget->GetText())
		.Margin(2.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SMemInvestigationView::QueryTarget_GetSelectedText() const
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
		TSharedPtr<FQueryTargetWindowSpec> Target = SharedState.GetCurrentQueryTarget();
		if (Target)
		{
			return Target->GetText();
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SMemInvestigationView::RunQuery()
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (!ProfilerWindow.IsValid())
	{
		UE_LOGF(LogMemoryProfiler, Error, "[MemQuery] Invalid Profiler Window!");
		return FReply::Handled();
	}

	FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
	TSharedPtr<FMemoryRuleSpec> Rule = SharedState.GetCurrentMemoryRule();
	if (!Rule)
	{
		UE_LOGF(LogMemoryProfiler, Error, "[MemQuery] Invalid Rule!");
		return FReply::Handled();
	}

	const uint32 NumTimeMarkers = ProfilerWindow->GetNumCustomTimeMarkers();
	const uint32 RuleNumTimeMarkers = Rule->GetNumTimeMarkers();
	if (RuleNumTimeMarkers > NumTimeMarkers)
	{
		UE_LOGF(LogMemoryProfiler, Error, "[MemQuery] Only %d time markers available. Current rule (%ls) requires %u time markers!",
			NumTimeMarkers, *Rule->GetShortName().ToString(), RuleNumTimeMarkers);
		return FReply::Handled();
	}

#if !NO_LOGGING
	TStringBuilder<1024> Builder;
	if (RuleNumTimeMarkers > 0)
	{
		Builder.Append(TEXT(" ("));
		for (uint32 TimeMarkerIndex = 0; TimeMarkerIndex < RuleNumTimeMarkers; ++TimeMarkerIndex)
		{
			if (TimeMarkerIndex != 0)
			{
				Builder.Append(TEXT(", "));
			}
			Builder.AppendChar((TCHAR)(TEXT('A') + TimeMarkerIndex));
			using FTimeMarker = TimingProfiler::FTimeMarker;
			const TSharedRef<FTimeMarker>& TimeMarker = ProfilerWindow->GetCustomTimeMarker(TimeMarkerIndex);
			Builder.Appendf(TEXT("=%.9f"), TimeMarker->GetTime());
		}
		Builder.Append(TEXT(")"));
	}
	UE_LOGF(LogMemoryProfiler, Log, "[MemQuery] Run Query %ls%ls...", *Rule->GetShortName().ToString(), Builder.ToString());
#endif

	TSharedPtr<SMemAllocTableTreeView> MemAllocTableTreeView = ProfilerWindow->ShowMemAllocTableTreeViewTab();
	if (MemAllocTableTreeView)
	{
		SMemAllocTableTreeView::FQueryParams QueryParams;
		QueryParams.Rule = Rule;
		QueryParams.TimeMarkers[0] = (RuleNumTimeMarkers > 0) ? ProfilerWindow->GetCustomTimeMarker(0)->GetTime() : 0.0;
		QueryParams.TimeMarkers[1] = (RuleNumTimeMarkers > 1) ? ProfilerWindow->GetCustomTimeMarker(1)->GetTime() : 0.0;
		QueryParams.TimeMarkers[2] = (RuleNumTimeMarkers > 2) ? ProfilerWindow->GetCustomTimeMarker(2)->GetTime() : 0.0;
		QueryParams.TimeMarkers[3] = (RuleNumTimeMarkers > 3) ? ProfilerWindow->GetCustomTimeMarker(3)->GetTime() : 0.0;
		QueryParams.bIncludeHeapAllocs = bIncludeHeapAllocs;
		QueryParams.bIncludeSwapAllocs = bIncludeSwapAllocs;
		MemAllocTableTreeView->SetQueryParams(QueryParams);
	}

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SMemInvestigationView::OnTimeMarkerLabelDoubleClicked(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent, uint32 TimeMarkerIndex)
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		const uint32 NumTimeMarkers = ProfilerWindow->GetNumCustomTimeMarkers();
		if (TimeMarkerIndex < NumTimeMarkers)
		{
			const TSharedRef<TimingProfiler::FTimeMarker>& TimeMarker = ProfilerWindow->GetCustomTimeMarker(TimeMarkerIndex);
			TSharedPtr<TimingProfiler::STimingView> TimingView = ProfilerWindow->GetTimingView();
			if (TimingView.IsValid())
			{
				// Move timer to the center of the timing view.
				const double Time = (TimingView->GetViewport().GetStartTime() + TimingView->GetViewport().GetEndTime()) / 2.0;
				TimeMarker->SetTime(Time);
			}
		}
	}
	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
