// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMenuBuilder;

namespace TraceServices
{
	class IAnalysisSession;
}

namespace UE::Insights::MemoryProfiler
{

class FMemoryRuleSpec;
class FQueryTargetWindowSpec;
class SMemoryProfilerWindow;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * A custom widget used to setup and run mem (allocations) queries.
 */
class SMemInvestigationView : public SCompoundWidget
{
public:
	/** Default constructor. */
	SMemInvestigationView();

	/** Virtual destructor. */
	virtual ~SMemInvestigationView();

	SLATE_BEGIN_ARGS(SMemInvestigationView) {}
	SLATE_END_ARGS()

	/**
	 * Converts profiler window weak pointer to a shared pointer and returns it.
	 * Make sure the returned pointer is valid before trying to dereference it.
	 */
	TSharedPtr<SMemoryProfilerWindow> GetProfilerWindow() const
	{
		return ProfilerWindowWeakPtr.Pin();
	}

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<SMemoryProfilerWindow> InProfilerWindow);

	void Reset();

	void QueryTarget_OnSelectionChanged(TSharedPtr<FQueryTargetWindowSpec> InRule, ESelectInfo::Type SelectInfo);

private:
	void UpdateSymbolPathsText() const;
	TSharedRef<SWidget> ConstructInvestigationWidgetArea();
	TSharedRef<SWidget> ConstructTimeMarkerWidget(uint32 TimeMarkerIndex);

	/** Called when the analysis session has changed. */
	void InsightsManager_OnSessionChanged();

	const TArray<TSharedPtr<FQueryTargetWindowSpec>>* GetAvailableQueryTargets();
	TSharedRef<SWidget> QueryTarget_OnGenerateWidget(TSharedPtr<FQueryTargetWindowSpec> InRule);
	FText QueryTarget_GetSelectedText() const;
	const TArray<TSharedPtr<FMemoryRuleSpec>>* GetAvailableQueryRules();
	void QueryRule_OnSelectionChanged(TSharedPtr<FMemoryRuleSpec> InRule, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> QueryRule_OnGenerateWidget(TSharedPtr<FMemoryRuleSpec> InRule);
	FText QueryRule_GetSelectedText() const;
	FText QueryRule_GetTooltipText() const;
	FReply RunQuery();

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	FReply OnTimeMarkerLabelDoubleClicked(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent, uint32 TimeMarkerIndex);

private:
	/** A weak pointer to the Memory Insights window. */
	TWeakPtr<SMemoryProfilerWindow> ProfilerWindowWeakPtr;

	/** The analysis session used to populate this widget. */
	TSharedPtr<const TraceServices::IAnalysisSession> Session;

	TSharedPtr<SComboBox<TSharedPtr<FMemoryRuleSpec>>> QueryRuleComboBox;

	bool bIncludeHeapAllocs;
	bool bIncludeSwapAllocs;

	TSharedPtr<SComboBox<TSharedPtr<FQueryTargetWindowSpec>>> QueryTargetComboBox;
	
	TSharedPtr<STextBlock> SymbolPathsTextBlock;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
