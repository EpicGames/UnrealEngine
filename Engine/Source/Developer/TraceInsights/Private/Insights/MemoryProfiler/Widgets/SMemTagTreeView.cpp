// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMemTagTreeView.h"

#include "DesktopPlatformModule.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/ConfigCacheIni.h"
#include "SlateOptMacros.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSegmentedControl.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/Memory.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Table/ViewModels/Table.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/MemoryProfiler/MemoryProfilerManager.h"
#include "Insights/MemoryProfiler/ViewModels/MemorySharedState.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryTag.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryTracker.h"
#include "Insights/MemoryProfiler/ViewModels/MemTagBudget.h"
#include "Insights/MemoryProfiler/ViewModels/MemTagBudgetGrouping.h"
#include "Insights/MemoryProfiler/ViewModels/MemTagNode.h"
#include "Insights/MemoryProfiler/ViewModels/MemTagTableViewPresets.h"
#include "Insights/MemoryProfiler/Widgets/SMemoryProfilerWindow.h"
#include "Insights/TimingProfiler/ViewModels/TimeMarker.h"
#include "Insights/TimingProfiler/Widgets/STimeMarkerEditBlock.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler::SMemTagTreeView"

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

SMemTagTreeView::SMemTagTreeView()
{
	bRunInAsyncMode = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SMemTagTreeView::~SMemTagTreeView()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::Construct(const FArguments& InArgs, TSharedPtr<SMemoryProfilerWindow> InProfilerWindow)
{
	check(InProfilerWindow.IsValid());
	ProfilerWindowWeakPtr = InProfilerWindow;

	TSharedRef<FMemTagTable> TablePtr = MakeShared<FMemTagTable>();
	TablePtr->Reset();
	TablePtr->SetDisplayName(FText::FromString(TEXT("Memory Tags")));

	ConstructWidget(TablePtr);

	// Apply the default preset.
	ApplyViewPreset(*(*GetAvailableViewPresets())[0]);

	UpdateSelectionStatsText();
	InitBudgetOptions();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::ConstructHeaderArea(TSharedRef<SVerticalBox> InHostBox)
{
	TSharedPtr<SWidget> TopSettingsWidget = ConstructTopSettings();
	if (TopSettingsWidget.IsValid())
	{
		InHostBox->AddSlot()
			.AutoHeight()
			.Padding(4.0f, 4.0f, 4.0f, 2.0f)
			[
				TopSettingsWidget.ToSharedRef()
			];
	}

	TSharedPtr<SWidget> TagSetAndViewPresetWidget = ConstructTagSetAndViewPreset();
	if (TagSetAndViewPresetWidget.IsValid())
	{
		InHostBox->AddSlot()
			.AutoHeight()
			.Padding(4.0f, 2.0f, 4.0f, 2.0f)
			[
				TagSetAndViewPresetWidget.ToSharedRef()
			];
	}

	TSharedPtr<SWidget> TimeMarkersWidget = ConstructTimeMarkers();
	if (TimeMarkersWidget.IsValid())
	{
		InHostBox->AddSlot()
			.AutoHeight()
			.Padding(0.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SBox)
				.Padding(4.0f, 1.0f, 2.0f, 1.0f)
				.Visibility_Lambda([this]
				{
					return bAreTimeMarkerSettingsVisible ? EVisibility::Visible : EVisibility::Collapsed;
				})
				[
					TimeMarkersWidget.ToSharedRef()
				]
			];
	}

	TSharedPtr<SWidget> FilterToolbarWidget = ConstructFilterToolbar();
	if (FilterToolbarWidget.IsValid())
	{
		InHostBox->AddSlot()
			.AutoHeight()
			.Padding(4.0f, 2.0f, 4.0f, 2.0f)
			[
				FilterToolbarWidget.ToSharedRef()
			];
	}

	TSharedPtr<SWidget> HierarchyBreadcrumbTrailWidget = ConstructHierarchyBreadcrumbTrail();
	if (HierarchyBreadcrumbTrailWidget.IsValid())
	{
		InHostBox->AddSlot()
			.AutoHeight()
			.Padding(4.0f, 2.0f, 4.0f, -2.0f)
			[
				HierarchyBreadcrumbTrailWidget.ToSharedRef()
			];
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::ConstructFooterArea(TSharedRef<SVerticalBox> InHostBox)
{
	TSharedPtr<SWidget> FooterWidget = ConstructFooter();
	if (FooterWidget.IsValid())
	{
		InHostBox->AddSlot()
			.AutoHeight()
			.Padding(0.0f, 6.0f, 0.0f, 0.0f)
			[
				FooterWidget.ToSharedRef()
			];
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> SMemTagTreeView::ConstructTopSettings()
{
	TSharedRef<SHorizontalBox> TopLine = SNew(SHorizontalBox)

	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(0.0f, 0.0f, 0.0f, 0.0f)
	.VAlign(VAlign_Center)
	[
		ConstructBudgetSettings()
	]

	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(4.0f, 0.0f, 0.0f, 0.0f)
	.VAlign(VAlign_Center)
	[
		SNew(SCheckBox)
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.Padding(FMargin(4.0f, 3.0f, 4.0f, 3.0f))
		.IsChecked_Lambda([this]()
			{
				return bAreTimeMarkerSettingsVisible ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
		.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
			{
				bAreTimeMarkerSettingsVisible = (InCheckBoxState == ECheckBoxState::Checked);
			})
		.Content()
		[
			SNew(SImage)
			.Image(FInsightsStyle::GetBrush("Icons.TimeMarkerSettings"))
		]
		.ToolTipText(LOCTEXT("TimeMarkerSettingsVisibilityToolTip", "Toggle visibility for the advanced Time Marker settings."))
	];

	return TopLine;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> SMemTagTreeView::ConstructTagSetAndViewPreset()
{
	UpdateAvailableTagSets();

	TSharedPtr<SHorizontalBox> Widget = SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 0.0f, 0.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			TagSetsSegmentedControl.ToSharedRef()
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.0);

	ConstructViewPreset(Widget);

	return Widget;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::UpdateAvailableTagSets()
{
	TSharedPtr<SHorizontalBox> ParentBox;
	if (TagSetsSegmentedControl.IsValid())
	{
		ParentBox = StaticCastSharedPtr<SHorizontalBox>(TagSetsSegmentedControl->GetParentWidget());
	}

	TagSetsSegmentedControl = SNew(SSegmentedControl<FMemoryTagSetId>)
		.SupportsMultiSelection(false)
		.OnValueChanged_Lambda([this](FMemoryTagSetId InValue)
			{
				TagSetFilter = InValue;
				OnNodeFilteringChanged();
				bShouldUpdateStats = true;
				bShouldUpdateBudgets = true;
			})
		.Value_Lambda([this]
			{
				return TagSetFilter;
			});

	for (TSharedPtr<FMemoryTagSetId> TagSetPtr : AvailableTagSets)
	{
		FMemoryTagSetId TagSet = *TagSetPtr;

		const FSlateBrush* TagSetIcon = nullptr;
		FText TagSetName;
		if (TagSet == SystemsTagSet)
		{
			TagSetIcon = FInsightsStyle::GetBrush("Icons.TagSet.Systems");
			TagSetName = FText::FromString("Systems");
		}
		else if (TagSet == AssetsTagSet)
		{
			TagSetIcon = FInsightsStyle::GetBrush("Icons.TagSet.Assets");
			TagSetName = FText::FromString("Assets");
		}
		else if (TagSet == AssetClassesTagSet)
		{
			TagSetIcon = FInsightsStyle::GetBrush("Icons.TagSet.AssetClasses");
			TagSetName = FText::FromString("AssetClasses");
		}
		else
		{
			TagSetName = FText::FromString("Unknown");
			TagSetIcon = FInsightsStyle::GetBrush("Icons.TagSet.Systems");
		}

		FText TagSetToolTip;
		const FText TagSetToolTipFmt = LOCTEXT("TagSetToolTipFmt", "'{0}' Tag Set\n\nThe tree view shows memory tags from the selected tag set.\n\nNote:\n\tTrace data is emitted by the Low-Level Memory Tracker system (LLM).\n\tSee documentation \n\tabout \"-llm -llmtagsets=assets,assetclasses\" command line parameters.");
		TagSetToolTip = FText::Format(TagSetToolTipFmt, TagSetName);

		TSharedRef<SWidget> Widget = SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(16.0f)
				.HeightOverride(16.0f)
				[
					SNew(SImage)
					.Image(TagSetIcon)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(TagSetName)
			];

		TagSetsSegmentedControl->AddSlot(TagSet)
			.ToolTip(TagSetToolTip)
			.AttachWidget(Widget);
	}

	if (ParentBox.IsValid())
	{
		ParentBox->GetSlot(0).AttachWidget(TagSetsSegmentedControl.ToSharedRef());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> SMemTagTreeView::ConstructFilterToolbar()
{
	TSharedRef<SHorizontalBox> FilterLine = SNew(SHorizontalBox)

	+ SHorizontalBox::Slot()
	.FillWidth(1.0f)
	.VAlign(VAlign_Center)
	[
		ConstructSearchBox()
	]

	+ SHorizontalBox::Slot()
	.AutoWidth()
	.MinSize(10.0f)
	.Padding(4.0f, 0.0f, 0.0f, 0.0f)
	.VAlign(VAlign_Center)
	[
		SNew(SComboButton)
		.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
		.OnGetMenuContent(this, &SMemTagTreeView::MakeTrackersMenu)
		.ToolTipText(LOCTEXT("TrackersMenuToolTip", "Filter the list of memory tags by LLM tracker."))
		.ContentPadding(FMargin(0.0f, 1.0f))
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SImage)
				.Image(FInsightsStyle::GetBrush("Icons.Filter.ToolBar"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TrackersMenuText", "Trackers"))
			]
		]
	]

	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(2.0f, 0.0f, 0.0f, 0.0f)
	.VAlign(VAlign_Center)
	[
		ConstructFilterConfiguratorButton()
	];

	return FilterLine;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> SMemTagTreeView::ConstructToolbar()
{
	return nullptr;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> SMemTagTreeView::ConstructTimeMarkers()
{
	TSharedPtr<SWidget> WidgetA = ConstructTimeMarkerA();
	TSharedPtr<SWidget> WidgetB = ConstructTimeMarkerB();

	if (!WidgetA.IsValid() && !WidgetB.IsValid())
	{
		return nullptr;
	}

	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	if (WidgetA.IsValid())
	{
		VerticalBox->AddSlot()
			.AutoHeight()
			[
				WidgetA.ToSharedRef()
			];
	}

	if (WidgetB.IsValid())
	{
		VerticalBox->AddSlot()
			.AutoHeight()
			[
				WidgetB.ToSharedRef()
			];
	}

	return VerticalBox;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> SMemTagTreeView::ConstructTimeMarkerA()
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (!ProfilerWindow.IsValid() || ProfilerWindow->GetNumCustomTimeMarkers() < 2)
	{
		return nullptr;
	}

	using FTimeMarker = TimingProfiler::FTimeMarker;
	TSharedRef<FTimeMarker> TimeMarkerA = ProfilerWindow->GetCustomTimeMarker(0);

	return SNew(TimingProfiler::STimeMarkerEditBlock, TimeMarkerA)
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
		.OnGetTimingView_Lambda(
			[ProfilerWindowWeak = ProfilerWindowWeakPtr](TSharedRef<FTimeMarker> TimeMarker) -> TSharedPtr<TimingProfiler::STimingView>
			{
				TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = ProfilerWindowWeak.Pin();
				if (ProfilerWindow.IsValid())
				{
					return ProfilerWindow->GetTimingView();
				}
				return nullptr;
			})
		.OnTimeMarkerChanged_Lambda(
			[ProfilerWindowWeak = ProfilerWindowWeakPtr](TSharedRef<FTimeMarker> TimeMarker)
			{
				TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = ProfilerWindowWeak.Pin();
				if (ProfilerWindow.IsValid())
				{
					ProfilerWindow->OnTimeMarkerChanged(Timing::ETimeChangedFlags::None, TimeMarker);
				}
			});
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> SMemTagTreeView::ConstructTimeMarkerB()
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (!ProfilerWindow.IsValid() || ProfilerWindow->GetNumCustomTimeMarkers() < 2)
	{
		return nullptr;
	}

	using FTimeMarker = TimingProfiler::FTimeMarker;
	TSharedRef<FTimeMarker> TimeMarkerA = ProfilerWindow->GetCustomTimeMarker(0);
	TSharedRef<FTimeMarker> TimeMarkerB = ProfilerWindow->GetCustomTimeMarker(1);

	return SNew(TimingProfiler::STimeMarkerEditBlock, TimeMarkerB)
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
		.PreviousTimeMarker(TimeMarkerA)
		.OnGetTimingView_Lambda(
			[ProfilerWindowWeak = ProfilerWindowWeakPtr](TSharedRef<FTimeMarker> TimeMarker) -> TSharedPtr<TimingProfiler::STimingView>
			{
				TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = ProfilerWindowWeak.Pin();
				if (ProfilerWindow.IsValid())
				{
					return ProfilerWindow->GetTimingView();
				}
				return nullptr;
			})
		.OnTimeMarkerChanged_Lambda(
			[ProfilerWindowWeak = ProfilerWindowWeakPtr](TSharedRef<FTimeMarker> TimeMarker)
			{
				TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = ProfilerWindowWeak.Pin();
				if (ProfilerWindow.IsValid())
				{
					ProfilerWindow->OnTimeMarkerChanged(Timing::ETimeChangedFlags::None, TimeMarker);
				}
			});
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SMemTagTreeView::MakeTrackersMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);
	MenuBuilder.SetSearchable(false);

	MenuBuilder.BeginSection("Trackers"/*, LOCTEXT("ContextMenu_Section_Trackers", "Trackers")*/);
	CreateTrackersMenuSection(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::CreateTrackersMenuSection(FMenuBuilder& MenuBuilder)
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
		const TArray<TSharedPtr<FMemoryTracker>>& Trackers = SharedState.GetTrackers();
		for (const TSharedPtr<FMemoryTracker>& Tracker : Trackers)
		{
			const FMemoryTrackerId TrackerId = Tracker->GetId();
			MenuBuilder.AddMenuEntry(
				FText::FromString(Tracker->GetName()),
				TAttribute<FText>(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SMemTagTreeView::ToggleTracker, TrackerId),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SMemTagTreeView::IsTrackerChecked, TrackerId)),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::ToggleTracker(FMemoryTrackerId InTrackerId)
{
	TrackersFilter ^= FMemoryTracker::AsFlag(InTrackerId);
	OnNodeFilteringChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::IsTrackerChecked(FMemoryTrackerId InTrackerId) const
{
	return (TrackersFilter & FMemoryTracker::AsFlag(InTrackerId)) != 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::TagSet_OnSelectionChanged(TSharedPtr<FMemoryTagSetId> NewTagSet, ESelectInfo::Type SelectInfo)
{
	if (NewTagSet.IsValid())
	{
		TagSetFilter = *NewTagSet;
		OnNodeFilteringChanged();
		bShouldUpdateStats = true;
		bShouldUpdateBudgets = true;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SMemTagTreeView::TagSet_OnGenerateWidget(TSharedPtr<FMemoryTagSetId> InTagSet) const
{
	FText TagSetName;
	if (InTagSet.IsValid())
	{
		TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
		if (ProfilerWindow.IsValid())
		{
			FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
			const FMemoryTagSet* TagSet = SharedState.GetTagSetById(*InTagSet);
			if (TagSet)
			{
				TagSetName = FText::FromString(TagSet->GetName());
			}
		}

		if (TagSetName.IsEmpty())
		{
			TagSetName = FText::AsNumber((int64)(*InTagSet));
		}
	}

	return SNew(STextBlock)
		.Text(TagSetName);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SMemTagTreeView::TagSet_GetSelectedText() const
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
		const FMemoryTagSet* TagSet = SharedState.GetTagSetById(TagSetFilter);
		if (TagSet)
		{
			return FText::FromString(TagSet->GetName());
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SButton> SMemTagTreeView::ConstructHideAllButton()
{
	return SNew(SButton)
		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
		.ToolTipText(LOCTEXT("RemoveAllGraphTracks_ToolTip", "Remove memory graph tracks (LLM graph series) for all memory tags."))
		.OnClicked(this, &SMemTagTreeView::HideAllTracks_OnClicked)
		.ContentPadding(FMargin(2.0f, 2.0f, 2.0f, 2.0f))
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FInsightsStyle::Get().GetBrush("Icons.RemoveMemTagGraphs"))
			]
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SButton> SMemTagTreeView::ConstructShowSelectedButton()
{
	return SNew(SButton)
		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
		.ToolTipText(LOCTEXT("CreateGraphTracksForSelectedMemTags_ToolTip", "Create memory graph tracks (LLM graph series) for the selected memory tags."))
		.OnClicked(this, &SMemTagTreeView::ShowSelectedTracks_OnClicked)
		.IsEnabled(this, &SMemTagTreeView::CanCreateGraphTracksForSelectedMemTags)
		.ContentPadding(FMargin(2.0f, 2.0f, 2.0f, 2.0f))
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FInsightsStyle::Get().GetBrush("Icons.AddMemTagGraphs"))
			]
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SMemTagTreeView::ConstructTrackHeightControls()
{
	return SNew(SSegmentedControl<uint32>)
		.OnValueChanged_Lambda([this](uint32 InValue)
			{
				TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
				if (ProfilerWindow.IsValid())
				{
					FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
					SharedState.SetTrackHeightMode((EMemoryTrackHeightMode)InValue);
				}
			})
		.Value_Lambda([this]
			{
				TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
				if (ProfilerWindow.IsValid())
				{
					FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
					return (uint32)SharedState.GetTrackHeightMode();
				}
				return (uint32)EMemoryTrackHeightMode::Medium;
			})

		+ SSegmentedControl<uint32>::Slot((uint32)EMemoryTrackHeightMode::Small)
		.Icon(FInsightsStyle::GetBrush("Icons.SizeSmall"))
		.ToolTip(LOCTEXT("SmallHeight_ToolTip", "Change height of all LLM Tag graph tracks to Small."))

		+ SSegmentedControl<uint32>::Slot((uint32)EMemoryTrackHeightMode::Medium)
		.Icon(FInsightsStyle::GetBrush("Icons.SizeMedium"))
		.ToolTip(LOCTEXT("MediumHeight_ToolTip", "Change height of all LLM Tag graph tracks to Medium."))

		+ SSegmentedControl<uint32>::Slot((uint32)EMemoryTrackHeightMode::Large)
		.Icon(FInsightsStyle::GetBrush("Icons.SizeLarge"))
		.ToolTip(LOCTEXT("LargeHeight_ToolTip", "Change height of all LLM Tag graph tracks to Large."));
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SMemTagTreeView::ConstructBudgetSettings()
{
	return SNew(SComboButton)
		.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
		.OnGetMenuContent(this, &SMemTagTreeView::MakeBudgetSettingsMenu)
		.ToolTipText(LOCTEXT("BudgetSettingsToolTip", "Budget Settings"))
		.ContentPadding(FMargin(0.0f, 1.0f))
		.ButtonContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SImage)
				.Image(FInsightsStyle::GetBrush("Icons.BudgetSettings"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("BudgetSettingsText", "Budget"))
			]
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> SMemTagTreeView::ConstructFooter()
{
	return
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, -2.0f, 0.0f, 2.0f)
		.VAlign(VAlign_Top)
		[
			ConstructShowSelectedButton()
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, -2.0f, 0.0f, 2.0f)
		.VAlign(VAlign_Top)
		[
			ConstructHideAllButton()
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, -3.0f, 0.0f, 3.0f)
		.VAlign(VAlign_Top)
		[
			ConstructTrackHeightControls()
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 2.0f, 0.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(this, &SMemTagTreeView::GetNumSelectedTagsText)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 2.0f, 0.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(this, &SMemTagTreeView::GetSelectedTagsText)
			.ColorAndOpacity(FSlateColor(EStyleColor::White25))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 2.0f, 0.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT(" (A: ")))
			.ColorAndOpacity(FSlateColor(EStyleColor::White25))
			.Visibility_Lambda([this]() { return !NumSelectedTagsText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed; })
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 2.0f, 0.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(this, &SMemTagTreeView::GetSelectionSizeAText)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 2.0f, 0.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT(", B: ")))
			.ColorAndOpacity(FSlateColor(EStyleColor::White25))
			.Visibility_Lambda([this]() { return !NumSelectedTagsText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed; })
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 2.0f, 0.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(this, &SMemTagTreeView::GetSelectionSizeBText)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 2.0f, 0.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT(", B-A: ")))
			.ColorAndOpacity(FSlateColor(EStyleColor::White25))
			.Visibility_Lambda([this]() { return !NumSelectedTagsText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed; })
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 2.0f, 0.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(this, &SMemTagTreeView::GetSelectionDiffText)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 2.0f, 0.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT(")")))
			.ColorAndOpacity(FSlateColor(EStyleColor::White25))
			.Visibility_Lambda([this]() { return !NumSelectedTagsText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed; })
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::UpdateSelectionStatsText()
{
	TArray<FTableTreeNodePtr> SelectedNodes;
	const int32 NumSelectedNodes = TreeView->GetSelectedItems(SelectedNodes);

	if (NumSelectedNodes > 0)
	{
		int64 TotalCount = 0;
		int64 TotalSizeA = 0;
		int64 TotalSizeB = 0;
		for (const FTableTreeNodePtr& SelectedNode : SelectedNodes)
		{
			if (SelectedNode->Is<FMemTagNode>())
			{
				const FMemTagNode& MemTagNode = SelectedNode->As<FMemTagNode>();
				++TotalCount;
				TotalSizeA += MemTagNode.GetStats().SizeA;
				TotalSizeB += MemTagNode.GetStats().SizeB;
			}
		}

		FNumberFormattingOptions FormattingOptionsMem;
		FormattingOptionsMem.MaximumFractionalDigits = 2;

		NumSelectedTagsText = FText::AsNumber(TotalCount);
		SelectedTagsText = FText::Format(LOCTEXT("SelectionStatsFmt", "selected {0}|plural(one=tag,other=tags)"), FText::AsNumber(TotalCount));
		SelectionSizeAText = TotalSizeA == 0 ? FText::FromString(TEXT("0")) : FText::AsMemory(TotalSizeA, &FormattingOptionsMem);
		SelectionSizeBText = TotalSizeB == 0 ? FText::FromString(TEXT("0")) : FText::AsMemory(TotalSizeB, &FormattingOptionsMem);
		SelectionDiffText = TotalSizeA == TotalSizeB ? FText::FromString(TEXT("0")) : FText::AsMemory(TotalSizeB - TotalSizeA, &FormattingOptionsMem);
	}
	else
	{
		NumSelectedTagsText = FText::GetEmpty();
		SelectedTagsText = LOCTEXT("NoSelectionStats", "No memory tag selected");
		SelectionSizeAText = FText::GetEmpty();
		SelectionSizeBText = FText::GetEmpty();
		SelectionDiffText = FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::TreeView_OnSelectionChanged(FTableTreeNodePtr SelectedItem, ESelectInfo::Type SelectInfo)
{
	UpdateSelectionStatsText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::TreeView_OnMouseButtonDoubleClick(FTableTreeNodePtr NodePtr)
{
	if (NodePtr->IsGroup())
	{
		const bool bIsGroupExpanded = TreeView->IsItemExpanded(NodePtr);
		TreeView->SetItemExpansion(NodePtr, !bIsGroupExpanded);
	}
	else if (NodePtr->Is<FMemTagNode>())
	{
		FMemTagNode& MemTagNode = NodePtr->As<FMemTagNode>();
		if (MemTagNode.IsValidMemTag())
		{
			TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
			if (ProfilerWindow.IsValid())
			{
				FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
				const FMemoryTrackerId MemTrackerId = MemTagNode.GetMemTrackerId();
				const FMemoryTagId MemTagId = MemTagNode.GetMemTagId();
				TSharedPtr<FMemoryGraphTrack> GraphTrack = SharedState.GetMemTagGraphTrack(MemTrackerId, MemTagId);
				if (!GraphTrack.IsValid())
				{
					GraphTrack = SharedState.CreateMemTagGraphTrack(MemTrackerId, MemTagId);
					if (GraphTrack.IsValid())
					{
						UpdateHighThreshold(MemTagNode, *GraphTrack);
					}
				}
				else
				{
					SharedState.RemoveMemTagGraphTrack(MemTrackerId, MemTagId);
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Grouping
////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::InternalCreateGroupings()
{
	STableTreeView::InternalCreateGroupings();

	AvailableGroupings.RemoveAll(
		[](TSharedPtr<FTreeNodeGrouping>& Grouping)
		{
			if (Grouping->Is<FTreeNodeGroupingByUniqueValue>())
			{
				const FName ColumnId = Grouping->As<FTreeNodeGroupingByUniqueValue>().GetColumnId();
				return (ColumnId != FMemTagTableColumns::SampleCountColumnId &&
						ColumnId != FMemTagTableColumns::TrackerColumnId);
			}
			else if (Grouping->Is<FTreeNodeGroupingByPathBreakdown>())
			{
				const FName ColumnId = Grouping->As<FTreeNodeGroupingByPathBreakdown>().GetColumnId();
				return (ColumnId != FMemTagTableColumns::TagNameColumnId);
			}
			return false;
		});

	TSharedPtr<FTreeNodeGrouping>* BySampleCountUnique = AvailableGroupings.FindByPredicate([](TSharedPtr<FTreeNodeGrouping>& Grouping)
		{
			return Grouping->Is<FTreeNodeGroupingByUniqueValue>() &&
				   Grouping->As<FTreeNodeGroupingByUniqueValue>().GetColumnId() == FMemTagTableColumns::SampleCountColumnId;
		});
	if (BySampleCountUnique != nullptr)
	{
		(*BySampleCountUnique)->SetColor(FLinearColor(0.6f, 0.5f, 0.1f, 1.0f));
	}

	int32 Index = 1; // after the Flat ("All") grouping

	AvailableGroupings.Insert(MakeShared<FMemTagBudgetNodeGrouping>(SharedThis(this)), Index++);

#if 0
	// Groups memory tags by their hierarchy.
	class FMemTagGroupingByHierarchy : public
	{
		{
			TMap<FName, FMemTagNodePtr> GroupNodeSet;
			for (const FMemTagNodePtr& NodePtr : MemTagNodes)
			{
				const FName GroupName = NodePtr->GetParentTagNode() ? NodePtr->GetParentTagNode()->GetName() : FName(TEXT("<LLM>"));
				FMemTagNodePtr GroupPtr = GroupNodeSet.FindRef(GroupName);
				if (!GroupPtr)
				{
					GroupPtr = GroupNodeSet.Add(GroupName, MakeShared<FMemTagNode>(GroupName));
				}
				GroupPtr->AddChildAndSetParent(NodePtr);
				TreeView->SetItemExpansion(GroupPtr, true);
			}
			GroupNodeSet.GenerateValueArray(GroupNodes);
		}
	};
	AvailableGroupings.Insert(MakeShared<FMemTagGroupingByHierarchy>(), Index++);
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::InitAvailableViewPresets()
{
	AvailableViewPresets.Add(FMemTagTableViewPresets::CreateDefaultViewPreset(*this));
	AvailableViewPresets.Add(FMemTagTableViewPresets::CreateDiffViewPreset(*this));
	AvailableViewPresets.Add(FMemTagTableViewPresets::CreateTimeRangeViewPreset(*this));

	SelectedViewPreset = AvailableViewPresets[0];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::Reset()
{
	StatsStartTime = 0.0;
	StatsEndTime = 0.0;

	STableTreeView::Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	STableTreeView::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	// We need to check if the list of memory tags has changed.
	// But, ensure we do not check too often.
	static uint64 NextTimestamp = 0;
	uint64 Time = FPlatformTime::Cycles64();
	if (Time > NextTimestamp && !bIsUpdateRunning)
	{
		RebuildTree(false);

		const int32 NumTags = GetTableRowNodes().Num();
		// 1000 tags --> check each 200ms
		// 10000 tags --> check each 1.1s
		// 100000 tags --> check each 10.1s
		const double WaitTimeSec = 0.1 + (double)NumTags / 10000.0;
		const uint64 WaitTime = static_cast<uint64>(WaitTimeSec / FPlatformTime::GetSecondsPerCycle64());
		NextTimestamp = Time + WaitTime;
	}

	bShouldUpdateStats = CheckIfShouldUpdateStats();
	if (bShouldUpdateStats)
	{
		bShouldUpdateStats = false;
		UpdateStats();
	}

	if (bShouldUpdateBudgets)
	{
		bShouldUpdateBudgets = false;
		ApplyBudgetToNodes();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::RebuildTree(bool bResync)
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	FStopwatch SyncStopwatch;
	SyncStopwatch.Start();

	bool bListHasChanged = false;

	if (bResync)
	{
		TableRowNodes.Empty();
		LastMemoryTagListSerialNumber = 0;
		MemTagNodesIdMap.Empty();
		bListHasChanged = true;
	}

	const int32 PreviousNodeCount = TableRowNodes.Num();

	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
		const FMemoryTagList& TagList = SharedState.GetTagList();

		if (LastMemoryTagListSerialNumber != TagList.GetSerialNumber())
		{
			LastMemoryTagListSerialNumber = TagList.GetSerialNumber();

			const TArray<FMemoryTag*>& MemTags = TagList.GetTags();
			const int32 MemTagCount = MemTags.Num();

			// Create the table row nodes...
			{
				TableRowNodes.Empty(MemTagCount);
				MemTagNodesIdMap.Empty(MemTagCount);
				bListHasChanged = true;

				TSharedPtr<FMemTagTable> ParentMemTagTable = GetMemTagTable();

				for (FMemoryTag* MemTagPtr : MemTags)
				{
					check(MemTagPtr);
					TSharedPtr<FMemTagNode> MemTagNodePtr;
					if (MemTagPtr->GetTagSetId() == SystemsTagSet)
					{
						MemTagNodePtr = MakeShared<FSystemMemTagNode>(ParentMemTagTable, *MemTagPtr);
					}
					else if (MemTagPtr->GetTagSetId() == AssetsTagSet)
					{
						MemTagNodePtr = MakeShared<FAssetMemTagNode>(ParentMemTagTable, *MemTagPtr);
					}
					else if (MemTagPtr->GetTagSetId() == AssetClassesTagSet)
					{
						MemTagNodePtr = MakeShared<FClassMemTagNode>(ParentMemTagTable, *MemTagPtr);
					}
					else
					{
						MemTagNodePtr = MakeShared<FMemTagNode>(ParentMemTagTable, *MemTagPtr);
					}
					TableRowNodes.Add(MemTagNodePtr);
					MemTagNodesIdMap.Add(MemTagPtr->GetId(), MemTagNodePtr);
				}
			}

			// Resolve pointers to parent tags.
			for (FTableTreeNodePtr& NodePtr : TableRowNodes)
			{
				if (NodePtr->Is<FSystemMemTagNode>())
				{
					FSystemMemTagNode& MemTagNode = NodePtr->As<FSystemMemTagNode>();
					check(MemTagNode.GetMemTag() != nullptr);
					FMemoryTag& MemTag = *MemTagNode.GetMemTag();
					TSharedPtr<FMemTagNode> ParentNodePtr = MemTagNodesIdMap.FindRef(MemTag.GetParentId());
					if (ParentNodePtr)
					{
						check(ParentNodePtr != NodePtr);
						MemTagNode.SetParentTagNode(ParentNodePtr);
					}
				}
			}

			bShouldUpdateBudgets = true;
		}

		if (AvailableTagSets.Num() != SharedState.GetNumTagSets())
		{
			AvailableTagSets.Reset();
			SharedState.EnumerateTagSets([this](const FMemoryTagSet& InTagSet)
				{
					AvailableTagSets.Add(MakeShared<FMemoryTagSetId>(InTagSet.GetId()));
				});
			UpdateAvailableTagSets();
		}
	}

	SyncStopwatch.Stop();

	if (bListHasChanged)
	{
		// Save selection.
		TArray<FTableTreeNodePtr> SelectedItems;
		TreeView->GetSelectedItems(SelectedItems);

		UpdateTree();
		UpdateStats();

		TreeView->RebuildList();

		// Restore selection.
		if (SelectedItems.Num() > 0)
		{
			TreeView->ClearSelection();
			for (FTableTreeNodePtr& NodePtr : SelectedItems)
			{
				if (!NodePtr->Is<FMemTagNode>())
				{
					NodePtr.Reset();
					continue;
				}
				NodePtr = GetMemTagNode(NodePtr->As<FMemTagNode>().GetMemTagId());
			}
			SelectedItems.RemoveAll([](const FTableTreeNodePtr& NodePtr) { return !NodePtr.IsValid(); });
			if (SelectedItems.Num() > 0)
			{
				TreeView->SetItemSelection(SelectedItems, true);
				TreeView->RequestScrollIntoView(SelectedItems.Last());
			}
		}
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.01)
	{
		const double SyncTime = SyncStopwatch.GetAccumulatedTime();
		UE_LOGF(LogMemoryProfiler, Log, "[MemTags] Tree view rebuilt in %.4fs (sync: %.4fs + update: %.4fs) --> %d tags (%d added)",
			TotalTime, SyncTime, TotalTime - SyncTime, TableRowNodes.Num(), TableRowNodes.Num() - PreviousNodeCount);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::CheckIfShouldUpdateStats() const
{
	if (bShouldUpdateStats)
	{
		return true;
	}

	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (!ProfilerWindow.IsValid())
	{
		return false;
	}
	FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();

	bool bUpdateWhileDragging = true;

	// Do not update while dragging the A or B time markers and analysis is still active (ex.: during live session).
	if (Session.IsValid())
	{
		const TraceServices::IMemoryProvider* MemoryProvider = TraceServices::ReadMemoryProvider(*Session.Get());
		if (MemoryProvider && !MemoryProvider->IsCompleted())
		{
			bUpdateWhileDragging = false;
		}
	}

	// Do not update while dragging the A or B time markers and current tag set is other than Systems.
	if (TagSetFilter != FMemoryTagSet::DefaultTagSetId)
	{
		bUpdateWhileDragging = false;
	}

	const TSharedRef<TimingProfiler::FTimeMarker>& MarkerA = ProfilerWindow->GetCustomTimeMarker(0);
	const TSharedRef<TimingProfiler::FTimeMarker>& MarkerB = ProfilerWindow->GetCustomTimeMarker(1);
	const bool bIsDragging = MarkerA->IsDragging() || MarkerB->IsDragging();

	if (bUpdateWhileDragging || !bIsDragging)
	{
		if (StatsTimeA != MarkerA->GetTime() ||
			StatsTimeB != MarkerB->GetTime())
		{
			return true;
		}
	}

	TSharedPtr<TimingProfiler::STimingView> TimingView = ProfilerWindow->GetTimingView();
	if (TimingView &&
		(bUpdateWhileDragging || !TimingView->IsSelecting()))
	{
		double SelectionStartTime = TimingView->GetSelectionStartTime();
		double SelectionEndTime = TimingView->GetSelectionEndTime();
		if (SelectionStartTime < SelectionEndTime)
		{
			if (StatsStartTime != SelectionStartTime ||
				StatsEndTime != SelectionEndTime)
			{
				return true;
			}
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::UpdateStats()
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		const TSharedRef<TimingProfiler::FTimeMarker>& MarkerA = ProfilerWindow->GetCustomTimeMarker(0);
		StatsTimeA = MarkerA->GetTime();

		const TSharedRef<TimingProfiler::FTimeMarker>& MarkerB = ProfilerWindow->GetCustomTimeMarker(1);
		StatsTimeB = MarkerB->GetTime();

		TSharedPtr<TimingProfiler::STimingView> TimingView = ProfilerWindow->GetTimingView();
		if (TimingView)
		{
			double SelectionStartTime = TimingView->GetSelectionStartTime();
			double SelectionEndTime = TimingView->GetSelectionEndTime();
			if (SelectionStartTime < SelectionEndTime)
			{
				StatsStartTime = SelectionStartTime;
				StatsEndTime = SelectionEndTime;
			}
		}
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	UpdateStatsInternal();

	bool bShouldUpdateTreeWithStats = false; // TODO: UI toggle (advanced setting)
	if (bShouldUpdateTreeWithStats)
	{
		UpdateTree();
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.01)
	{
		UE_LOGF(LogMemoryProfiler, Log, "[MemTags] Aggregated stats updated in %.4fs", TotalTime);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::UpdateStatsInternal()
{
	if (!Session)
	{
		return;
	}

	const TraceServices::IMemoryProvider* MemoryProvider = TraceServices::ReadMemoryProvider(*Session.Get());
	if (MemoryProvider)
	{
		TraceServices::FProviderReadScopeLock _(*MemoryProvider);

		for (const FTableTreeNodePtr& NodePtr : TableRowNodes)
		{
			if (!NodePtr->Is<FMemTagNode>())
			{
				continue;
			}
			FMemTagNode& MemTagNode = NodePtr->As<FMemTagNode>();

			FMemoryTag* MemTag = MemTagNode.GetMemTag();
			if (!MemTag)
			{
				MemTagNode.ResetAggregatedStats();
				continue;
			}

			if (MemTagNode.GetMemTagSetId() != TagSetFilter)
			{
				continue;
			}

			FMemTagStats CurrentStats;

			CurrentStats.SizeA = std::numeric_limits<int64>::max();
			MemoryProvider->EnumerateTagSamples(MemTag->GetTrackerId(), MemTag->GetId(), StatsTimeA, StatsTimeA, true,
				[&CurrentStats](double Time, double Duration, const TraceServices::FMemoryTagSample& Sample)
				{
					if (CurrentStats.SizeA == std::numeric_limits<int64>::max())
					{
						CurrentStats.SizeA = Sample.Value;
					}
				});
			if (CurrentStats.SizeA != std::numeric_limits<int64>::max())
			{
				MemTagNode.GetStats().SizeA = CurrentStats.SizeA;
			}
			else
			{
				MemTagNode.GetStats().SizeA = 0;
			}

			CurrentStats.SizeB = std::numeric_limits<int64>::max();
			MemoryProvider->EnumerateTagSamples(MemTag->GetTrackerId(), MemTag->GetId(), StatsTimeB, StatsTimeB, true,
				[&CurrentStats](double Time, double Duration, const TraceServices::FMemoryTagSample& Sample)
				{
					if (CurrentStats.SizeB == std::numeric_limits<int64>::max())
					{
						CurrentStats.SizeB = Sample.Value;
					}
				});
			if (CurrentStats.SizeB != std::numeric_limits<int64>::max())
			{
				MemTagNode.GetStats().SizeB = CurrentStats.SizeB;
			}
			else
			{
				MemTagNode.GetStats().SizeB = 0;
			}

			CurrentStats.SampleCount = 0;
			CurrentStats.SizeMin = std::numeric_limits<int64>::max();
			CurrentStats.SizeMax = std::numeric_limits<int64>::min();
			MemoryProvider->EnumerateTagSamples(MemTag->GetTrackerId(), MemTag->GetId(), StatsStartTime, StatsEndTime, false,
				[&CurrentStats](double Time, double Duration, const TraceServices::FMemoryTagSample& Sample)
				{
					CurrentStats.SampleCount++;
					if (Sample.Value < CurrentStats.SizeMin)
					{
						CurrentStats.SizeMin = Sample.Value;
					}
					if (Sample.Value > CurrentStats.SizeMax)
					{
						CurrentStats.SizeMax = Sample.Value;
					}
					CurrentStats.SizeAverage += Sample.Value;
				});
			if (CurrentStats.SampleCount != 0)
			{
				MemTagNode.GetStats().SampleCount = CurrentStats.SampleCount;
				MemTagNode.GetStats().SizeMin = CurrentStats.SizeMin;
				MemTagNode.GetStats().SizeMax = CurrentStats.SizeMax;
				MemTagNode.GetStats().SizeAverage = CurrentStats.SizeAverage / CurrentStats.SampleCount;
			}
			else
			{
				MemTagNode.GetStats().SampleCount = 0;
				MemTagNode.GetStats().SizeMin = 0;
				MemTagNode.GetStats().SizeMax = 0;
				MemTagNode.GetStats().SizeAverage = 0;
			}
		}
	}

	UpdateSelectionStatsText();
	UpdateAggregatedValuesRec(*Root);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::ExtendMenu(TSharedRef<FExtender> Extender)
{
	Extender->AddMenuExtension("Misc", EExtensionHook::Before, nullptr, FMenuExtensionDelegate::CreateSP(this, &SMemTagTreeView::ExtendMenuBeforeMisc));
	Extender->AddMenuExtension("Misc", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateSP(this, &SMemTagTreeView::ExtendMenuAfterMisc));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::ExtendMenuBeforeMisc(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Memory Tag", LOCTEXT("ContextMenu_Section_MemoryTag", "Memory Tag"));
	{
		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_CreateGraphTracks_SubMenu", "Create Graph Tracks"),
			LOCTEXT("ContextMenu_CreateGraphTracks_SubMenu_Desc", "Creates memory graph tracks (LLM graph series)."),
			FNewMenuDelegate::CreateSP(this, &SMemTagTreeView::ExtendMenuCreateGraphTracks),
			false,
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.AddMemTagGraphs")
		);

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_RemoveGraphTracks_SubMenu", "Remove Graph Tracks"),
			LOCTEXT("ContextMenu_RemoveGraphTracks_SubMenu_Desc", "Removes memory graph tracks (LLM graph series)."),
			FNewMenuDelegate::CreateSP(this, &SMemTagTreeView::ExtendMenuRemoveGraphTracks),
			false,
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.RemoveMemTagGraphs")
		);

		FUIAction Action_GenerateColorForSelectedMemTags
		(
			FExecuteAction::CreateSP(this, &SMemTagTreeView::GenerateColorForSelectedMemTags),
			FCanExecuteAction::CreateSP(this, &SMemTagTreeView::CanGenerateColorForSelectedMemTags)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_GenerateColorForSelectedMemTags", "Generate New Color"),
			LOCTEXT("ContextMenu_GenerateColorForSelectedMemTags_Desc", "Generates a new random color for the selected memory tags."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Refresh"),
			Action_GenerateColorForSelectedMemTags,
			NAME_None,
			EUserInterfaceActionType::Button
		);

		FUIAction Action_EditColorForSelectedMemTags
		(
			FExecuteAction::CreateSP(this, &SMemTagTreeView::EditColorForSelectedMemTags),
			FCanExecuteAction::CreateSP(this, &SMemTagTreeView::CanEditColorForSelectedMemTags)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_EditColorForSelectedMemTags", "Edit Color..."),
			LOCTEXT("ContextMenu_EditColorForSelectedMemTags_Desc", "Changes color for the selected memory tags."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.EyeDropper"),
			Action_EditColorForSelectedMemTags,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::ExtendMenuCreateGraphTracks(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.SetSearchable(false);

	MenuBuilder.BeginSection("CreateGraphTracks");
	{
		// Create memory graph tracks (LLM graph series) for the selected memory tags.
		FUIAction Action_CreateGraphTracksForSelectedMemTags
		(
			FExecuteAction::CreateSP(this, &SMemTagTreeView::CreateGraphTracksForSelectedMemTags),
			FCanExecuteAction::CreateSP(this, &SMemTagTreeView::CanCreateGraphTracksForSelectedMemTags)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_CreateGraphTracksForSelectedMemTags", "Selected"),
			LOCTEXT("ContextMenu_CreateGraphTracksForSelectedMemTags_Desc", "Creates memory graph tracks (LLM graph series) for the selected memory tags."),
			FSlateIcon(),
			Action_CreateGraphTracksForSelectedMemTags,
			NAME_None,
			EUserInterfaceActionType::Button
		);

		// Create memory graph tracks (LLM graph series) for the visible memory tags.
		FUIAction Action_CreateGraphTracksForVisibleMemTags
		(
			FExecuteAction::CreateSP(this, &SMemTagTreeView::CreateGraphTracksForVisibleMemTags),
			FCanExecuteAction::CreateSP(this, &SMemTagTreeView::CanCreateGraphTracksForVisibleMemTags)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_CreateGraphTracksForVisibleMemTags", "Visible"),
			LOCTEXT("ContextMenu_CreateGraphTracksForVisibleMemTags_Desc", "Creates memory graph tracks (LLM graph series) for the visible memory tags."),
			FSlateIcon(),
			Action_CreateGraphTracksForVisibleMemTags,
			NAME_None,
			EUserInterfaceActionType::Button
		);

		// Create memory graph tracks (LLM graph series) for all memory tags.
		FUIAction Action_CreateAllGraphTracks
		(
			FExecuteAction::CreateSP(this, &SMemTagTreeView::CreateAllGraphTracks),
			FCanExecuteAction::CreateSP(this, &SMemTagTreeView::CanCreateAllGraphTracks)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_CreateAllGraphTracks", "All"),
			LOCTEXT("ContextMenu_CreateAllGraphTracks_Desc", "Creates memory graph tracks (LLM graph series) for all memory tags."),
			FSlateIcon(),
			Action_CreateAllGraphTracks,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::ExtendMenuRemoveGraphTracks(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.SetSearchable(false);

	MenuBuilder.BeginSection("RemoveGraphTracks");
	{
		// Remove memory graph tracks (LLM graph series) for the selected memory tags.
		FUIAction Action_RemoveGraphTracksForSelectedMemTags
		(
			FExecuteAction::CreateSP(this, &SMemTagTreeView::RemoveGraphTracksForSelectedMemTags),
			FCanExecuteAction::CreateSP(this, &SMemTagTreeView::CanRemoveGraphTracksForSelectedMemTags)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_RemoveGraphTracksForSelectedMemTags", "Selected"),
			LOCTEXT("ContextMenu_RemoveGraphTracksForSelectedMemTags_Desc", "Removes memory graph tracks (LLM graph series) for the selected memory tags."),
			FSlateIcon(),
			Action_RemoveGraphTracksForSelectedMemTags,
			NAME_None,
			EUserInterfaceActionType::Button
		);

		// Remove memory graph tracks (LLM graph series) for all memory tags.
		FUIAction Action_RemoveAllGraphTracks
		(
			FExecuteAction::CreateSP(this, &SMemTagTreeView::RemoveAllGraphTracks),
			FCanExecuteAction::CreateSP(this, &SMemTagTreeView::CanRemoveAllGraphTracks)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_RemoveAllGraphTracks", "All"),
			LOCTEXT("ContextMenu_RemoveAllGraphTracks_Desc", "Removes memory graph tracks (LLM graph series) for all memory tags."),
			FSlateIcon(),
			Action_RemoveAllGraphTracks,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::ExtendMenuAfterMisc(FMenuBuilder& MenuBuilder)
{
	FUIAction Action_LoadReportXML
	(
		FExecuteAction::CreateSP(this, &SMemTagTreeView::LoadReportXML),
		FCanExecuteAction::CreateSP(this, &SMemTagTreeView::CanLoadReportXML)
	);
	MenuBuilder.AddMenuEntry
	(
		LOCTEXT("ContextMenu_LoadReportXML", "Load Report XML..."),
		LOCTEXT("ContextMenu_LoadReportXML_Desc", "Loads a report specification file (LLMReportTypes.xml)"),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FolderOpen"),
		Action_LoadReportXML,
		NAME_None,
		EUserInterfaceActionType::Button
	);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::HasCustomNodeFilter() const
{
	return TrackersFilter != uint64(-1) ||
		   TagSetFilter != FMemoryTagSet::InvalidTagSetId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::FilterNodeCustom(const FTableTreeNode& InNode) const
{
	if (InNode.Is<FMemTagNode>())
	{
		const FMemTagNode& MemTagNode = InNode.As<FMemTagNode>();
		if (FMemoryTracker::IsValidTrackerId(MemTagNode.GetMemTrackerId()) &&
			((FMemoryTracker::AsFlag(MemTagNode.GetMemTrackerId()) & TrackersFilter) == 0))
		{
			return false;
		}
		if (MemTagNode.GetMemTagSetId() != TagSetFilter)
		{
			return false;
		}
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::SelectMemTagNode(FMemoryTagId MemTagId)
{
	TSharedPtr<FMemTagNode> NodePtr = GetMemTagNode(MemTagId);
	if (NodePtr)
	{
		TreeView->SetSelection(NodePtr);
		TreeView->RequestScrollIntoView(NodePtr);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Load Report XML button action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::CanLoadReportXML() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::LoadReportXML()
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (!ProfilerWindow.IsValid())
	{
		return;
	}

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (!DesktopPlatform)
	{
		return;
	}

	FSlateApplication::Get().CloseToolTip();

	const FString DefaultPath(FPaths::RootDir() / TEXT("Engine/Binaries/DotNET/CsvTools"));
	const FString DefaultFile(TEXT("LLMReportTypes.xml"));

	TArray<FString> Files;
	if (!DesktopPlatform->OpenFileDialog(
		FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
		LOCTEXT("LoadReportXML_FileDesc", "Open the LLMReportTypes.xml file...").ToString(),
		DefaultPath,
		DefaultFile, // Not actually used. See FDesktopPlatformWindows::FileDialogShared implementation. :(
		LOCTEXT("LoadReportXML_FileFilter", "XML files (*.xml)|*.xml|All files (*.*)|*.*").ToString(),
		EFileDialogFlags::None,
		Files))
	{
		return;
	}

	if (Files.Num() != 1)
	{
		return;
	}

	FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
	SharedState.RemoveAllMemTagGraphTracks();
	SharedState.CreateTracksFromReport(Files[0]);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Button actions re graph tracks
////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SMemTagTreeView::ShowSelectedTracks_OnClicked()
{
	CreateGraphTracksForSelectedMemTags();
	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SMemTagTreeView::HideAllTracks_OnClicked()
{
	RemoveAllGraphTracks();
	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::TryCreateGraphTrackForNode(FMemorySharedState& SharedState, const FBaseTreeNode& Node) const
{
	if (Node.Is<FMemTagNode>())
	{
		const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
		if (MemTagNode.IsValidMemTag())
		{
			const FMemoryTrackerId MemTrackerId = MemTagNode.GetMemTrackerId();
			const FMemoryTagId MemTagId = MemTagNode.GetMemTagId();
			SharedState.CreateMemTagGraphTrack(MemTrackerId, MemTagId);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::TryRemoveGraphTrackForNode(FMemorySharedState& SharedState, const FBaseTreeNode& Node) const
{
	if (Node.Is<FMemTagNode>())
	{
		const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
		if (MemTagNode.IsValidMemTag())
		{
			const FMemoryTrackerId MemTrackerId = MemTagNode.GetMemTrackerId();
			const FMemoryTagId MemTagId = MemTagNode.GetMemTagId();
			SharedState.RemoveMemTagGraphTrack(MemTrackerId, MemTagId);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Create memory graph tracks (LLM graph series) for the selected memory tags
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::CanCreateGraphTracksForSelectedMemTags() const
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	return ProfilerWindow.IsValid() && TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::CreateGraphTracksForSelectedMemTags()
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();

		const TArray<FTableTreeNodePtr> SelectedNodes = TreeView->GetSelectedItems();
		for (const FTableTreeNodePtr& SelectedNode : SelectedNodes)
		{
			TryCreateGraphTrackForNode(SharedState, *SelectedNode);

			if (SelectedNode->IsGroup())
			{
				const TArray<FBaseTreeNodePtr>& Children = SelectedNode->GetFilteredChildren();
				for (const FBaseTreeNodePtr& Child : Children)
				{
					TryCreateGraphTrackForNode(SharedState, *Child);
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Create memory graph tracks (LLM graph series) for the visible memory tags
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::CanCreateGraphTracksForVisibleMemTags() const
{
	return FilteredNodesPtr->Num() > 0 && FilteredGroupNodes.Num() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::CreateGraphTracksForVisibleMemTags()
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();

		for (const FTableTreeNodePtr& GroupNode : FilteredGroupNodes)
		{
			CreateGraphTracksRec(SharedState, *GroupNode);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::CreateGraphTracksRec(FMemorySharedState& SharedState, const FBaseTreeNode& Node)
{
	if (Node.Is<FMemTagNode>())
	{
		const FMemTagNode& MemTagNode = Node.As<FMemTagNode>();
		TryCreateGraphTrackForNode(SharedState, MemTagNode);
	}

	if (Node.IsGroup())
	{
		const TArray<FBaseTreeNodePtr>& Children = Node.GetFilteredChildren();
		for (const FBaseTreeNodePtr& Child : Children)
		{
			CreateGraphTracksRec(SharedState, *Child);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Create memory graph tracks (LLM graph series) for all memory tags
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::CanCreateAllGraphTracks() const
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	return ProfilerWindow.IsValid() && TagSetFilter == SystemsTagSet;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::CreateAllGraphTracks()
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();

		for (const FTableTreeNodePtr& NodePtr : TableRowNodes)
		{
			// Only create graph tracks for tags of the selected tag set.
			if (NodePtr->Is<FMemTagNode>())
			{
				const FMemTagNode& MemTagNode = NodePtr->As<FMemTagNode>();
				if (MemTagNode.GetMemTagSetId() != TagSetFilter)
				{
					continue;
				}
			}

			TryCreateGraphTrackForNode(SharedState, *NodePtr);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Remove memory graph tracks (LLM graph series) for the selected memory tags
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::CanRemoveGraphTracksForSelectedMemTags() const
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	return ProfilerWindow.IsValid() && TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::RemoveGraphTracksForSelectedMemTags()
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();

		const TArray<FTableTreeNodePtr> SelectedNodes = TreeView->GetSelectedItems();
		for (const FTableTreeNodePtr& SelectedNode : SelectedNodes)
		{
			TryRemoveGraphTrackForNode(SharedState, *SelectedNode);

			if (SelectedNode->IsGroup())
			{
				const TArray<FBaseTreeNodePtr>& Children = SelectedNode->GetFilteredChildren();
				for (const FBaseTreeNodePtr& Child : Children)
				{
					TryRemoveGraphTrackForNode(SharedState, *Child);
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Remove all memory graph tracks (LLM graph series)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::CanRemoveAllGraphTracks() const
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	return ProfilerWindow.IsValid();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::RemoveAllGraphTracks()
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
		SharedState.RemoveAllMemTagGraphTracks();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Generate new color for selected memory tags
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::CanGenerateColorForSelectedMemTags() const
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	return ProfilerWindow.IsValid() && TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::GenerateColorForSelectedMemTags() const
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		const TArray<FTableTreeNodePtr> SelectedNodes = TreeView->GetSelectedItems();
		for (const FTableTreeNodePtr& SelectedNode : SelectedNodes)
		{
			if (SelectedNode->Is<FMemTagNode>())
			{
				FMemTagNode& MemTagNode = SelectedNode->As<FMemTagNode>();
				SetRandomColorToNode(MemTagNode);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::SetColorToNode(FMemTagNode& MemTagNode, FLinearColor Color) const
{
	if (MemTagNode.IsGroup())
	{
		const TArray<FBaseTreeNodePtr>& Children = MemTagNode.GetFilteredChildren();
		for (const FBaseTreeNodePtr& Child : Children)
		{
			if (Child->Is<FMemTagNode>())
			{
				FMemTagNode& ChildMemTagNode = Child->As<FMemTagNode>();
				SetColorToNode(ChildMemTagNode, Color);
			}
		}
		return;
	}

	FMemoryTag* MemTag = MemTagNode.GetMemTag();
	if (!MemTag)
	{
		return;
	}

	MemTag->SetColor(Color);
	const FLinearColor BorderColor(FMath::Min(Color.R + 0.4f, 1.0f), FMath::Min(Color.G + 0.4f, 1.0f), FMath::Min(Color.B + 0.4f, 1.0f), 1.0f);

	const FMemoryTrackerId MemTrackerId = MemTagNode.GetMemTrackerId();
	const FMemoryTagSetId MemTagSetId = MemTagNode.GetMemTagSetId();
	const FMemoryTagId MemTagId = MemTagNode.GetMemTagId();

	TSharedPtr<FMemoryGraphTrack> MainGraphTrack;
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
		MainGraphTrack = SharedState.GetMainGraphTrack();
	}

	for (const TSharedPtr<FMemoryGraphTrack>& GraphTrack : MemTag->GetGraphTracks())
	{
		TSharedPtr<FMemTagGraphSeries> MemTagSeries = GraphTrack->GetMemTagSeries(MemTrackerId, MemTagSetId, MemTagId);
		if (MemTagSeries)
		{
			if (GraphTrack == MainGraphTrack)
			{
				MemTagSeries->SetColor(Color, BorderColor, Color.CopyWithNewOpacity(0.1f));
			}
			else
			{
				MemTagSeries->SetColor(Color, BorderColor);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::SetRandomColorToNode(FMemTagNode& MemTagNode) const
{
	if (MemTagNode.IsGroup())
	{
		const TArray<FBaseTreeNodePtr>& Children = MemTagNode.GetFilteredChildren();
		for (const FBaseTreeNodePtr& Child : Children)
		{
			if (Child->Is<FMemTagNode>())
			{
				FMemTagNode& ChildMemTagNode = Child->As<FMemTagNode>();
				SetRandomColorToNode(ChildMemTagNode);
			}
		}
		return;
	}

	FMemoryTag* MemTag = MemTagNode.GetMemTag();
	if (!MemTag)
	{
		return;
	}

	MemTag->SetRandomColor();
	FLinearColor Color = MemTag->GetColor();

	const FLinearColor BorderColor(FMath::Min(Color.R + 0.4f, 1.0f), FMath::Min(Color.G + 0.4f, 1.0f), FMath::Min(Color.B + 0.4f, 1.0f), 1.0f);

	const FMemoryTrackerId MemTrackerId = MemTagNode.GetMemTrackerId();
	const FMemoryTagSetId MemTagSetId = MemTagNode.GetMemTagSetId();
	const FMemoryTagId MemTagId = MemTagNode.GetMemTagId();

	TSharedPtr<FMemoryGraphTrack> MainGraphTrack;
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();
		MainGraphTrack = SharedState.GetMainGraphTrack();
	}

	for (const TSharedPtr<FMemoryGraphTrack>& GraphTrack : MemTag->GetGraphTracks())
	{
		TSharedPtr<FMemTagGraphSeries> MemTagSeries = GraphTrack->GetMemTagSeries(MemTrackerId, MemTagSetId, MemTagId);
		if (MemTagSeries)
		{
			if (GraphTrack == MainGraphTrack)
			{
				MemTagSeries->SetColor(Color, BorderColor, Color.CopyWithNewOpacity(0.1f));
			}
			else
			{
				MemTagSeries->SetColor(Color, BorderColor);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Edit color for selected memory tags
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::CanEditColorForSelectedMemTags() const
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	return ProfilerWindow.IsValid() && TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::EditColorForSelectedMemTags()
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		EditableColorValue = FLinearColor(0.5f, 0.5f, 0.5f, 1.0f);
		const TArray<FTableTreeNodePtr> SelectedNodes = TreeView->GetSelectedItems();
		for (const FTableTreeNodePtr& SelectedNode : SelectedNodes)
		{
			if (SelectedNode->Is<FMemTagNode>())
			{
				FMemTagNode& MemTagNode = SelectedNode->As<FMemTagNode>();
				EditableColorValue = MemTagNode.GetMemTag()->GetColor();
				break;
			}
		}

		FColorPickerArgs PickerArgs;
		{
			PickerArgs.bUseAlpha = true;
			PickerArgs.bOnlyRefreshOnMouseUp = false;
			PickerArgs.bOnlyRefreshOnOk = false;
			PickerArgs.bExpandAdvancedSection = false;
			//PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
			PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &SMemTagTreeView::SetEditableColor);
			PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &SMemTagTreeView::ColorPickerCancelled);
			//PickerArgs.OnInteractivePickBegin = FSimpleDelegate::CreateSP(this, &SMemTagTreeView::InteractivePickBegin);
			//PickerArgs.OnInteractivePickEnd = FSimpleDelegate::CreateSP(this, &SMemTagTreeView::InteractivePickEnd);
			//PickerArgs.OnColorPickerWindowClosed = FOnWindowClosed::CreateSP(this, &SMemTagTreeView::ColorPickerClosed);
			PickerArgs.InitialColor = EditableColorValue;
			PickerArgs.ParentWidget = SharedThis(this);
		}

		OpenColorPicker(PickerArgs);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor SMemTagTreeView::GetEditableColor() const
{
	return EditableColorValue;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::SetEditableColor(FLinearColor NewColor)
{
	EditableColorValue = NewColor;

	const TArray<FTableTreeNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	for (const FTableTreeNodePtr& SelectedNode : SelectedNodes)
	{
		if (SelectedNode->Is<FMemTagNode>())
		{
			FMemTagNode& MemTagNode = SelectedNode->As<FMemTagNode>();
			SetColorToNode(MemTagNode, EditableColorValue);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::ColorPickerCancelled(FLinearColor OriginalColor)
{
	SetEditableColor(OriginalColor);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Budgets
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SMemTagTreeView::MakeBudgetSettingsMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);
	MenuBuilder.SetSearchable(false);

	MenuBuilder.BeginSection("File", LOCTEXT("BudgetSettingsMenu_Section_File", "File"));
	for (const auto& BudgetFile : AvailableBudgetFiles)
	{
		FText Label = FText::FromString(FString::Printf(TEXT("%s (%s)"), *BudgetFile->Name, *BudgetFile->FileName));
		MenuBuilder.AddMenuEntry(
			Label,
			TAttribute<FText>(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMemTagTreeView::SelectBudgetFile, BudgetFile),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SMemTagTreeView::IsBudgetFileSelected, BudgetFile)),
			NAME_None,
			EUserInterfaceActionType::Check);
	}
	MenuBuilder.AddMenuEntry(
		LOCTEXT("OpenBudgetFile_Text", "Open File..."),
		LOCTEXT("OpenBudgetFile_ToolTip", "Choose budget file..."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SMemTagTreeView::OpenAndSelectBudgetFile),
			FCanExecuteAction()),
		NAME_None,
		EUserInterfaceActionType::Button);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("BudgetMode", LOCTEXT("BudgetSettingsMenu_Section_Budget", "Budget"));
	for (const auto& BudgetMode : AvailableBudgetModes)
	{
		MenuBuilder.AddMenuEntry(
			FText::FromString(BudgetMode->Name),
			TAttribute<FText>(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMemTagTreeView::SelectBudgetMode, BudgetMode),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SMemTagTreeView::IsBudgetModeSelected, BudgetMode)),
			NAME_None,
			EUserInterfaceActionType::Check);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("BudgetPlatform", LOCTEXT("BudgetSettingsMenu_Section_Platform", "Platform"));
	for (const auto& BudgetPlatform : AvailableBudgetPlatforms)
	{
		MenuBuilder.AddMenuEntry(
			FText::FromString(BudgetPlatform->Name),
			TAttribute<FText>(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMemTagTreeView::SelectBudgetPlatform, BudgetPlatform),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SMemTagTreeView::IsBudgetPlatformSelected, BudgetPlatform)),
			NAME_None,
			EUserInterfaceActionType::Check);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::OpenAndSelectBudgetFile()
{
	TSharedPtr<FMemTagBudgetFileDesc> BudgetFile = OpenBudgetFile();
	if (BudgetFile)
	{
		SelectBudgetFile(BudgetFile);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FMemTagBudgetFileDesc> SMemTagTreeView::OpenBudgetFile()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (!DesktopPlatform)
	{
		return nullptr;
	}

	FSlateApplication::Get().CloseToolTip();

	const FString DefaultPath(FPaths::RootDir() / TEXT("Engine/Programs/UnrealInsights/Config"));

	TArray<FString> Files;
	if (!DesktopPlatform->OpenFileDialog(
		FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
		LOCTEXT("LoadBudgetFile_FileDesc", "Open the Memory Budget .xml file...").ToString(),
		DefaultPath,
		FString(),
		LOCTEXT("LoadBudgetFile_FileFilter", "XML files (*.xml)|*.xml|All files (*.*)|*.*").ToString(),
		EFileDialogFlags::None,
		Files))
	{
		return nullptr;
	}

	if (Files.Num() != 1)
	{
		return nullptr;
	}

	const FString& FileName = Files[0];
	FString Name = FPaths::GetCleanFilename(FileName);

	for (TSharedPtr<FMemTagBudgetFileDesc>& BudgetFile : AvailableBudgetFiles)
	{
		if (Name.Equals(BudgetFile->Name, ESearchCase::IgnoreCase) &&
			FileName.Equals(BudgetFile->FileName, ESearchCase::IgnoreCase))
		{
			return BudgetFile;
		}
	}

	TSharedPtr<FMemTagBudgetFileDesc> NewBudgetFile = MakeShared<FMemTagBudgetFileDesc>(Name, FileName);
	AvailableBudgetFiles.Add(NewBudgetFile);
	return NewBudgetFile;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::SelectBudgetFile(TSharedPtr<FMemTagBudgetFileDesc> InBudgetFile)
{
	if (InBudgetFile.IsValid())
	{
		if (!SelectedBudgetFile.IsValid() || *SelectedBudgetFile != *InBudgetFile)
		{
			SelectedBudgetFile = InBudgetFile;
			CurrentBudget = nullptr;
			if (Session.IsValid())
			{
				TraceServices::IStringStore* StringStore = &const_cast<TraceServices::IAnalysisSession*>(Session.Get())->GetStringStore();
				CurrentBudget = MakeShared<FMemTagBudget>(StringStore);
				CurrentBudget->SetName(InBudgetFile->Name);
				if (!CurrentBudget->LoadFromFile(InBudgetFile->FileName))
				{
					// error
				}
			}
			OnBudgetChanged();
		}
	}
	else if (SelectedBudgetFile.IsValid())
	{
		SelectedBudgetFile = nullptr;
		CurrentBudget = nullptr;
		OnBudgetChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::IsBudgetFileSelected(TSharedPtr<FMemTagBudgetFileDesc> InBudgetFile) const
{
	return SelectedBudgetFile.IsValid () && *InBudgetFile == *SelectedBudgetFile;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::SelectBudgetMode(TSharedPtr<FMemTagBudgetModeDesc> InBudgetMode)
{
	if (InBudgetMode.IsValid())
	{
		if (!SelectedBudgetMode.IsValid() ||
			!SelectedBudgetMode->Name.Equals(InBudgetMode->Name, ESearchCase::IgnoreCase))
		{
			SelectedBudgetMode = InBudgetMode;
			bShouldUpdateBudgets = true;
		}
	}
	else
	{
		if (SelectedBudgetMode.IsValid())
		{
			SelectedBudgetMode = nullptr;
			bShouldUpdateBudgets = true;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::IsBudgetModeSelected(TSharedPtr<FMemTagBudgetModeDesc> InBudgetMode) const
{
	return SelectedBudgetMode.IsValid() && *InBudgetMode == *SelectedBudgetMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::SelectBudgetPlatform(TSharedPtr<FMemTagBudgetPlatformDesc> InBudgetPlatform)
{
	if (InBudgetPlatform.IsValid())
	{
		if (!SelectedBudgetPlatform.IsValid() || !SelectedBudgetPlatform->Name.Equals(InBudgetPlatform->Name, ESearchCase::IgnoreCase))
		{
			SelectedBudgetPlatform = InBudgetPlatform;
			bShouldUpdateBudgets = true;
		}
	}
	else
	{
		if (SelectedBudgetPlatform.IsValid())
		{
			SelectedBudgetPlatform = nullptr;
			bShouldUpdateBudgets = true;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMemTagTreeView::IsBudgetPlatformSelected(TSharedPtr<FMemTagBudgetPlatformDesc> InBudgetPlatform) const
{
	return SelectedBudgetPlatform.IsValid() && *InBudgetPlatform == *SelectedBudgetPlatform;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* SMemTagTreeView::GetTagSetFilterBudgetCachedName() const
{
	if (!CurrentBudget.IsValid())
	{
		return nullptr;
	}

	if (TagSetFilter == SystemsTagSet)
	{
		return CurrentBudget->FindString(TEXTVIEW("Systems"));
	}
	if (TagSetFilter == AssetsTagSet)
	{
		return CurrentBudget->FindString(TEXTVIEW("Assets"));
	}
	if (TagSetFilter == AssetClassesTagSet)
	{
		return CurrentBudget->FindString(TEXTVIEW("AssetClasses"));
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* SMemTagTreeView::GetSelectedBudgetPlatformCachedName() const
{
	return CurrentBudget.IsValid() && SelectedBudgetPlatform.IsValid() ?
		CurrentBudget->FindString(SelectedBudgetPlatform->Name) : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FMemTagBudgetMode* SMemTagTreeView::GetSelectedBudgetMode() const
{
	return CurrentBudget.IsValid() && SelectedBudgetMode.IsValid() ?
		CurrentBudget->FindMode(SelectedBudgetMode->Name) : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::GetBudgetGrouping(const FMemTagBudgetGrouping*& OutGrouping, const FMemTagBudgetGrouping*& OutGroupingOverride) const
{
	OutGrouping = nullptr;
	OutGroupingOverride = nullptr;

	const FMemTagBudgetMode* BudgetMode = GetSelectedBudgetMode();
	if (!BudgetMode)
	{
		return;
	}

	const TCHAR* CachedTagSetName = GetTagSetFilterBudgetCachedName();
	if (!CachedTagSetName)
	{
		return;
	}

	const FMemTagBudgetPlatform& DefaultPlatform = BudgetMode->GetDefaultPlatform();
	const FMemTagBudgetTagSet* TagSet = DefaultPlatform.FindTagSet(CachedTagSetName);
	if (!TagSet)
	{
		return;
	}

	OutGrouping = TagSet->GetGrouping();

	const TCHAR* CachedPlatformName = GetSelectedBudgetPlatformCachedName();
	if (CachedPlatformName)
	{
		const FMemTagBudgetPlatform* PlatformOverride = BudgetMode->FindPlatformOverride(CachedPlatformName);
		if (PlatformOverride)
		{
			const FMemTagBudgetTagSet* TagSetOverride = PlatformOverride->FindTagSet(CachedTagSetName);
			if (TagSetOverride)
			{
				OutGroupingOverride = TagSetOverride->GetGrouping();
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::InitBudgetOptions()
{
	CurrentBudget.Reset();

	AvailableBudgetFiles.Reset();

	FConfigFile* Settings = GConfig->FindConfigFileWithBaseName(TEXT("UnrealInsightsSettings"));
	if (Settings != nullptr)
	{
		const FConfigSection* Section = Settings->FindSection(TEXT("Insights.MemoryProfiler"));
		if (Section != nullptr)
		{
			FName KeyName(TEXT("BudgetFilePath"));
			for (FConfigSection::TConstKeyIterator It(*Section, KeyName); It; ++It)
			{
				const FString& ValueString = It.Value().GetValue();

				FString Label;
				FParse::Value(*ValueString, TEXT("Label="), Label);

				FString Path;
				FParse::Value(*ValueString, TEXT("Path="), Path);

				FString FullPath = FPaths::Combine(*FPaths::RootDir(), Path);
				if (FPaths::FileExists(FullPath))
				{
					AvailableBudgetFiles.Insert(MakeShared<FMemTagBudgetFileDesc>(Label, FullPath), 0);
					UE_LOGF(LogMemoryProfiler, Log, "[MemTags] Budget file found (Label=\"%ls\", Path=\"%ls\")", *Label, *FullPath);
				}
				else
				{
					UE_LOGF(LogMemoryProfiler, Warning, "[MemTags] Budget file not found (Label=\"%ls\", Path=\"%ls\")", *Label, *FullPath);
				}
			}
		}
		else
		{
			UE_LOGF(LogMemoryProfiler, Log, "[MemTags] Couldn't find Insights.MemoryProfiler config section");
		}
	}
	else
	{
		UE_LOGF(LogMemoryProfiler, Log, "[MemTags] Couldn't find UnrealInsightsSettings config");
	}

	AvailableBudgetModes.Reset();
	AvailableBudgetPlatforms.Reset();
	AvailablePlatforms.Reset();

	if (AvailableBudgetFiles.Num() > 0)
	{
		SelectBudgetFile(AvailableBudgetFiles[0]);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::OnBudgetChanged()
{
	bIsLoadingBudget = true;

	// Save the selected Budget Mode.
	TSharedPtr<FMemTagBudgetModeDesc> PrevSelectedBudgetMode = SelectedBudgetMode;

	// Save the selected Budget Platform.
	TSharedPtr<FMemTagBudgetPlatformDesc> PrevSelectedBudgetPlatform = SelectedBudgetPlatform;

	AvailableBudgetModes.Reset();
	AvailableBudgetPlatforms.Reset();
	AvailablePlatforms.Reset();

	if (CurrentBudget.IsValid())
	{
		// The "default" budget platform is always the first one.
		AvailableBudgetPlatforms.Add(MakeShared<FMemTagBudgetPlatformDesc>(TEXT("Default")));
		AvailablePlatforms.Add(AvailableBudgetPlatforms[0]->Name);

		// Add modes and platforms available in the current budget.
		CurrentBudget->EnumerateModes([this](const TCHAR* CachedModeName, const FMemTagBudgetMode& BudgetMode)
			{
				FString ModeName(CachedModeName);
				AvailableBudgetModes.Add(MakeShared<FMemTagBudgetModeDesc>(ModeName));

				BudgetMode.EnumeratePlatforms([this](const TCHAR* CachedPlatformName, const FMemTagBudgetPlatform& Platform)
					{
						FString PlatformName(CachedPlatformName);
						bool bIsAlreadyInSet = false;
						AvailablePlatforms.Add(PlatformName, &bIsAlreadyInSet);
						if (!bIsAlreadyInSet)
						{
							AvailableBudgetPlatforms.Add(MakeShared<FMemTagBudgetPlatformDesc>(PlatformName));
						}
					});
			});
	}

	// Restore selection for Budget Mode.
	if (PrevSelectedBudgetMode.IsValid())
	{
		bool bFound = false;
		for (const auto& BudgetMode : AvailableBudgetModes)
		{
			if (PrevSelectedBudgetMode->Name.Equals(BudgetMode->Name, ESearchCase::CaseSensitive))
			{
				SelectBudgetMode(BudgetMode);
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			if (AvailableBudgetModes.Num() > 0)
			{
				// Select first available mode.
				SelectBudgetMode(AvailableBudgetModes[0]);
			}
			else
			{
				SelectBudgetMode(nullptr);
			}
		}
	}
	if (!SelectedBudgetMode.IsValid() && AvailableBudgetModes.Num() > 0)
	{
		// Select the first available budget mode.
		SelectBudgetMode(AvailableBudgetModes[0]);
	}

	// Restore selection for Budget Platform.
	if (PrevSelectedBudgetPlatform.IsValid())
	{
		bool bFound = false;
		for (const auto& BudgetPlatform : AvailableBudgetPlatforms)
		{
			if (PrevSelectedBudgetPlatform->Name.Equals(BudgetPlatform->Name, ESearchCase::CaseSensitive))
			{
				SelectBudgetPlatform(BudgetPlatform);
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			if (AvailableBudgetPlatforms.Num() > 0)
			{
				// Select the first available platform.
				SelectBudgetPlatform(AvailableBudgetPlatforms[0]);
			}
			else
			{
				SelectBudgetPlatform(nullptr);
			}
		}
	}
	if (!SelectedBudgetPlatform.IsValid() && AvailableBudgetPlatforms.Num() > 0)
	{
		// Select the first available platform.
		SelectBudgetPlatform(AvailableBudgetPlatforms[0]);
	}

	bIsLoadingBudget = false;
	bShouldUpdateBudgets = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::ApplyBudgetToNodes()
{
	if (bIsLoadingBudget)
	{
		return;
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (!ProfilerWindow.IsValid())
	{
		return;
	}
	FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();

	if (!CurrentBudget.IsValid())
	{
		ResetBudgetForAllNodes();
		return;
	}

	const FMemTagBudgetMode* BudgetMode = GetSelectedBudgetMode();
	if (!BudgetMode)
	{
		ResetBudgetForAllNodes();
		return;
	}

	const TCHAR* CachedTagSetName = GetTagSetFilterBudgetCachedName();
	const FMemTagBudgetPlatform& DefaultPlatform = BudgetMode->GetDefaultPlatform();
	const FMemTagBudgetTagSet* TagSet = DefaultPlatform.FindTagSet(CachedTagSetName);
	if (!TagSet)
	{
		ResetBudgetForAllNodes();
		return;
	}

	const FMemTagBudgetGrouping* BudgetGrouping = TagSet->GetGrouping();
	const FMemTagBudgetGrouping* BudgetGroupingOverride = nullptr;

	const TCHAR* CachedDefaultTrackerName = CurrentBudget->FindString(TEXTVIEW("Default"));
	const TCHAR* CachedPlatformTrackerName = CurrentBudget->FindString(TEXTVIEW("Platform"));

	const FMemTagBudgetTracker* DefaultTracker = TagSet->FindTracker(CachedDefaultTrackerName);
	const FMemTagBudgetTracker* PlatformTracker = TagSet->FindTracker(CachedPlatformTrackerName);

	const FMemTagBudgetTracker* DefaultTrackerOverride = nullptr;
	const FMemTagBudgetTracker* PlatformTrackerOverride = nullptr;
	const TCHAR* CachedPlatformName = GetSelectedBudgetPlatformCachedName();
	if (CachedPlatformName)
	{
		const FMemTagBudgetPlatform* PlatformOverride = BudgetMode->FindPlatformOverride(CachedPlatformName);
		if (PlatformOverride)
		{
			const FMemTagBudgetTagSet* TagSetOverride = PlatformOverride->FindTagSet(CachedTagSetName);
			if (TagSetOverride)
			{
				BudgetGroupingOverride = TagSetOverride->GetGrouping();
				DefaultTrackerOverride = TagSetOverride->FindTracker(CachedDefaultTrackerName);
				PlatformTrackerOverride = TagSetOverride->FindTracker(CachedPlatformTrackerName);
			}
		}
	}

	if (BudgetGrouping)
	{
		UpdateBudgetGroupsRec(BudgetGrouping, BudgetGroupingOverride, Root);
	}

	for (const FTableTreeNodePtr& NodePtr : TableRowNodes)
	{
		if (!NodePtr->Is<FMemTagNode>())
		{
			continue;
		}

		FMemTagNode& MemTagNode = NodePtr->As<FMemTagNode>();
		FMemoryTag* MemoryTag = MemTagNode.GetMemTag();
		if (!MemoryTag)
		{
			continue;
		}

		if (MemTagNode.GetMemTagSetId() != TagSetFilter)
		{
			continue;
		}

		int32 TagSetIndex = (int32)MemoryTag->GetTagSetId();
		check(TagSetIndex >= 0 && TagSetIndex < 3);

		const TCHAR* CachedTagName = CurrentBudget->FindString(MemoryTag->GetStatFullName());

		bool bIsDefaultTracker = MemoryTag->GetTrackerId() == SharedState.GetDefaultTrackerId();

		const FMemTagBudgetTracker* BudgetTrackerOverride = bIsDefaultTracker ? DefaultTrackerOverride : PlatformTrackerOverride;
		if (BudgetTrackerOverride)
		{
			const int64* ValuePtr = BudgetTrackerOverride->FindValue(CachedTagName);
			if (ValuePtr)
			{
				MemTagNode.SetSizeBudget(*ValuePtr);
				UpdateHighThreshold(MemTagNode, SharedState);
				continue;
			}
		}

		const FMemTagBudgetTracker* BudgetTracker = bIsDefaultTracker ? DefaultTracker : PlatformTracker;
		if (BudgetTracker)
		{
			const int64* ValuePtr = BudgetTracker->FindValue(CachedTagName);
			if (ValuePtr)
			{
				MemTagNode.SetSizeBudget(*ValuePtr);
				UpdateHighThreshold(MemTagNode, SharedState);
				continue;
			}
		}

		MemTagNode.ResetSizeBudget();
		UpdateHighThreshold(MemTagNode, SharedState);
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.01)
	{
		UE_LOGF(LogMemoryProfiler, Log, "[MemTags] Budgets updated in %.4fs", TotalTime);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::UpdateBudgetGroupsRec(const FMemTagBudgetGrouping* BudgetGrouping, const FMemTagBudgetGrouping* BudgetGroupingOverride, const FBaseTreeNodePtr& GroupPtr)
{
	for (const FBaseTreeNodePtr& NodePtr : GroupPtr->GetChildren())
	{
		if (NodePtr->Is<FMemTagBudgetGroupNode>())
		{
			FMemTagBudgetGroupNode& GroupNode = NodePtr->As<FMemTagBudgetGroupNode>();
			const FMemTagBudgetGroup* BudgetGroup = nullptr;
			if (BudgetGroupingOverride)
			{
				BudgetGroup = BudgetGroupingOverride->FindGroup(GroupNode.GetBudgetGroupName());
			}
			if (!BudgetGroup)
			{
				BudgetGroup = BudgetGrouping->FindGroup(GroupNode.GetBudgetGroupName());
			}
			if (BudgetGroup)
			{
				GroupNode.SetSizeBudget(BudgetGroup->GetMemMax());
			}
		}
		else if (NodePtr->IsGroup())
		{
			UpdateBudgetGroupsRec(BudgetGrouping, BudgetGroupingOverride, NodePtr);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::ResetBudgetForAllNodes()
{
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (!ProfilerWindow.IsValid())
	{
		return;
	}
	FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();

	ResetBudgetGroupsRec(Root);

	for (const FTableTreeNodePtr& NodePtr : TableRowNodes)
	{
		if (NodePtr->Is<FMemTagNode>())
		{
			FMemTagNode& MemTagNode = NodePtr->As<FMemTagNode>();
			MemTagNode.ResetSizeBudget();
			UpdateHighThreshold(MemTagNode, SharedState);
		}
		else if (NodePtr->Is<FMemTagBudgetGroupNode>())
		{
			FMemTagBudgetGroupNode& GroupNode = NodePtr->As<FMemTagBudgetGroupNode>();
			GroupNode.ResetSizeBudget();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::ResetBudgetGroupsRec(const FBaseTreeNodePtr& GroupPtr)
{
	for (const FBaseTreeNodePtr& NodePtr : GroupPtr->GetChildren())
	{
		if (NodePtr->Is<FMemTagBudgetGroupNode>())
		{
			FMemTagBudgetGroupNode& GroupNode = NodePtr->As<FMemTagBudgetGroupNode>();
			GroupNode.ResetSizeBudget();
		}
		else if (NodePtr->IsGroup())
		{
			ResetBudgetGroupsRec(NodePtr);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::UpdateHighThreshold(const FMemTagNode& MemTagNode, FMemorySharedState& SharedState)
{
	TSharedPtr<FMemoryGraphTrack> GraphTrack = SharedState.GetMemTagGraphTrack(MemTagNode.GetMemTrackerId(), MemTagNode.GetMemTagId());
	if (GraphTrack.IsValid())
	{
		UpdateHighThreshold(MemTagNode, *GraphTrack);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemTagTreeView::UpdateHighThreshold(const FMemTagNode& MemTagNode, FMemoryGraphTrack& GraphTrack)
{
	for (TSharedPtr<FGraphSeries>& Series : GraphTrack.GetSeries())
	{
		if (Series->Is<FMemTagGraphSeries>())
		{
			FMemTagGraphSeries& MemTagGraphSeries = Series->As<FMemTagGraphSeries>();
			if (MemTagGraphSeries.GetTrackerId() == MemTagNode.GetMemTrackerId() &&
				MemTagGraphSeries.GetTagId() == MemTagNode.GetMemTagId())
			{
				if (MemTagNode.HasSizeBudget())
				{
					MemTagGraphSeries.SetHighThresholdValue((double)MemTagNode.GetSizeBudget());
				}
				else
				{
					MemTagGraphSeries.ResetHighThresholdValue();
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
