// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Widgets/Input/SSegmentedControl.h"

// TraceInsights
#include "Insights/Table/Widgets/SSessionTableTreeView.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryTag.h"
#include "Insights/MemoryProfiler/ViewModels/MemTagTable.h"

namespace UE::Insights::MemoryProfiler
{

class FMemTagBudget;
class FMemTagBudgetGrouping;
class FMemTagBudgetMode;
class FMemTagNode;
class FMemoryGraphTrack;
class FMemorySharedState;
class SMemoryProfilerWindow;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FMemTagBudgetFileDesc
{
	FMemTagBudgetFileDesc(const FString& InName, const FString& InFileName)
		: Name(InName)
		, FileName(InFileName)
	{
	}

	bool operator==(const FMemTagBudgetFileDesc& Other) const
	{
		return Name.Equals(Other.Name, ESearchCase::IgnoreCase)
			&& FileName.Equals(Other.FileName, ESearchCase::IgnoreCase);
	}

	bool operator!=(const FMemTagBudgetFileDesc& Other) const
	{
		return !(*this == Other);
	}

	FString Name;
	FString FileName;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FMemTagBudgetModeDesc
{
	FMemTagBudgetModeDesc() {}
	FMemTagBudgetModeDesc(const FString& InName) : Name(InName) {}

	bool operator==(const FMemTagBudgetModeDesc& Other) const
	{
		return Name.Equals(Other.Name, ESearchCase::IgnoreCase);
	}

	bool operator!=(const FMemTagBudgetModeDesc& Other) const
	{
		return !(*this == Other);
	}

	FString Name;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FMemTagBudgetPlatformDesc
{
	FMemTagBudgetPlatformDesc() {}
	FMemTagBudgetPlatformDesc(const FString& InName) : Name(InName) {}

	bool operator==(const FMemTagBudgetPlatformDesc& Other) const
	{
		return Name.Equals(Other.Name, ESearchCase::IgnoreCase);
	}

	bool operator!=(const FMemTagBudgetPlatformDesc& Other) const
	{
		return !(*this == Other);
	}

	FString Name;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * A custom widget used to display the list of LLM tags and their aggregated stats.
 */
class SMemTagTreeView : public SSessionTableTreeView
{
public:
	enum class EInvestigationMode
	{
		Default,
		Diff,
		MinMax,

		Count
	};

public:
	/** Default constructor. */
	SMemTagTreeView();

	/** Virtual destructor. */
	virtual ~SMemTagTreeView();

	SLATE_BEGIN_ARGS(SMemTagTreeView) {}
	SLATE_END_ARGS()

	/**
	 * Converts profiler window weak pointer to a shared pointer and returns it.
	 * Make sure the returned pointer is valid before trying to dereference it.
	 */
	TSharedPtr<SMemoryProfilerWindow> GetProfilerWindow() const
	{
		return ProfilerWindowWeakPtr.Pin();
	}

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<SMemoryProfilerWindow> InProfilerWindow);

	virtual void ConstructHeaderArea(TSharedRef<SVerticalBox> InHostBox) override;
	virtual void ConstructFooterArea(TSharedRef<SVerticalBox> InHostBox) override;
	virtual TSharedPtr<SWidget> ConstructFilterToolbar() override;
	virtual TSharedPtr<SWidget> ConstructToolbar() override;
	virtual TSharedPtr<SWidget> ConstructFooter() override;

	TSharedPtr<FMemTagTable> GetMemTagTable()
	{
		return StaticCastSharedPtr<FMemTagTable>(GetTable());
	}

	TSharedPtr<const FMemTagTable> GetMemTagTable() const
	{
		return StaticCastSharedPtr<const FMemTagTable>(GetTable());
	}

	virtual void Reset();

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/**
	 * Rebuilds the tree (if necessary).
	 * @param bResync - If true, it forces a resync with list of LLM tags from Analysis, even if the list did not changed since last sync.
	 */
	virtual void RebuildTree(bool bResync);

	bool CheckIfShouldUpdateStats() const;
	void UpdateStats();

	////////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedPtr<FMemTagNode> GetMemTagNode(FMemoryTagId MemTagId) const
	{
		return MemTagNodesIdMap.FindRef(MemTagId);
	}
	void SelectMemTagNode(FMemoryTagId MemTagId);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedPtr<FMemTagBudget> GetBudget() const { return CurrentBudget; }
	void GetBudgetGrouping(const FMemTagBudgetGrouping*& OutGrouping, const FMemTagBudgetGrouping*& OutGroupingOverride) const;

	FMemoryTagSetId GetTagSet() const { return TagSetFilter; }
	bool IsSystemsTagSet() const { return TagSetFilter == SystemsTagSet; }
	bool IsAssetsTagSet() const { return TagSetFilter == AssetsTagSet; }
	bool IsAssetClassesTagSet() const { return TagSetFilter == AssetClassesTagSet; }

protected:
	const TCHAR* GetTagSetFilterBudgetCachedName() const;
	const TCHAR* GetSelectedBudgetPlatformCachedName() const;
	const FMemTagBudgetMode* GetSelectedBudgetMode() const;

	virtual void InternalCreateGroupings() override;

	virtual void ExtendMenu(TSharedRef<FExtender> Extender) override;

	virtual bool HasCustomNodeFilter() const override;
	virtual bool FilterNodeCustom(const FTableTreeNode& InNode) const override;

	virtual void TreeView_OnMouseButtonDoubleClick(FTableTreeNodePtr NodePtr) override;
	virtual void TreeView_OnSelectionChanged(FTableTreeNodePtr SelectedItem, ESelectInfo::Type SelectInfo) override;

private:
	TSharedPtr<SWidget> ConstructTopSettings();
	TSharedPtr<SWidget> ConstructTagSetAndViewPreset();
	TSharedRef<SButton> ConstructHideAllButton();
	TSharedRef<SButton> ConstructShowSelectedButton();
	TSharedRef<SWidget> ConstructTrackHeightControls();
	TSharedRef<SWidget> ConstructBudgetSettings();
	TSharedPtr<SWidget> ConstructTimeMarkers();
	TSharedPtr<SWidget> ConstructTimeMarkerA();
	TSharedPtr<SWidget> ConstructTimeMarkerB();

	void UpdateSelectionStatsText();
	FText GetNumSelectedTagsText() const { return NumSelectedTagsText; }
	FText GetSelectedTagsText() const { return SelectedTagsText; }
	FText GetSelectionSizeAText() const { return SelectionSizeAText; }
	FText GetSelectionSizeBText() const { return SelectionSizeBText; }
	FText GetSelectionDiffText() const { return SelectionDiffText; }

	void InitAvailableViewPresets();

	void UpdateStatsInternal();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Context Menu

	void ExtendMenuBeforeMisc(FMenuBuilder& MenuBuilder);
	void ExtendMenuCreateGraphTracks(FMenuBuilder& MenuBuilder);
	void ExtendMenuRemoveGraphTracks(FMenuBuilder& MenuBuilder);
	void ExtendMenuAfterMisc(FMenuBuilder& MenuBuilder);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Trackers' Filter

	TSharedRef<SWidget> MakeTrackersMenu();
	void CreateTrackersMenuSection(FMenuBuilder& MenuBuilder);

	void ToggleTracker(FMemoryTrackerId InTrackerId);
	bool IsTrackerChecked(FMemoryTrackerId InTrackerId) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tag Set Selection

	void UpdateAvailableTagSets();
	void TagSet_OnSelectionChanged(TSharedPtr<FMemoryTagSetId> NewTagSet, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> TagSet_OnGenerateWidget(TSharedPtr<FMemoryTagSetId> InTagSet) const;
	FText TagSet_GetSelectedText() const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Misc

	bool CanLoadReportXML() const;
	void LoadReportXML();

	FReply ShowSelectedTracks_OnClicked();
	FReply HideAllTracks_OnClicked();

	void TryCreateGraphTrackForNode(FMemorySharedState& SharedState, const FBaseTreeNode& Node) const;
	void TryRemoveGraphTrackForNode(FMemorySharedState& SharedState, const FBaseTreeNode& Node) const;

	// Create memory graph tracks for the selected LLM tags
	bool CanCreateGraphTracksForSelectedMemTags() const;
	void CreateGraphTracksForSelectedMemTags();

	// Create memory graph tracks for the visible LLM tags
	bool CanCreateGraphTracksForVisibleMemTags() const;
	void CreateGraphTracksForVisibleMemTags();
	void CreateGraphTracksRec(FMemorySharedState& SharedState, const FBaseTreeNode& Node);

	// Create all memory graph tracks
	bool CanCreateAllGraphTracks() const;
	void CreateAllGraphTracks();

	// Remove memory graph tracks for the selected LLM tags
	bool CanRemoveGraphTracksForSelectedMemTags() const;
	void RemoveGraphTracksForSelectedMemTags();

	// Remove all memory graph tracks
	bool CanRemoveAllGraphTracks() const;
	void RemoveAllGraphTracks();

	// Generate new color for the selected LLM tags
	bool CanGenerateColorForSelectedMemTags() const;
	void GenerateColorForSelectedMemTags() const;

	// Edit color for the selected LLM tags
	bool CanEditColorForSelectedMemTags() const;
	void EditColorForSelectedMemTags();

	void SetColorToNode(FMemTagNode& MemTagNode, FLinearColor CustomColor) const;
	void SetRandomColorToNode(FMemTagNode& MemTagNode) const;
	FLinearColor GetEditableColor() const;
	void SetEditableColor(FLinearColor NewColor);
	void ColorPickerCancelled(FLinearColor OriginalColor);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Budgets

	TSharedRef<SWidget> MakeBudgetSettingsMenu();

	TSharedPtr<FMemTagBudgetFileDesc> OpenBudgetFile();

	void OpenAndSelectBudgetFile();
	void SelectBudgetFile(TSharedPtr<FMemTagBudgetFileDesc> InBudgetFile);
	bool IsBudgetFileSelected(TSharedPtr<FMemTagBudgetFileDesc> InBudgetFile) const;
	void SelectBudgetMode(TSharedPtr<FMemTagBudgetModeDesc> InBudgetMode);
	bool IsBudgetModeSelected(TSharedPtr<FMemTagBudgetModeDesc> InBudgetMode) const;
	void SelectBudgetPlatform(TSharedPtr<FMemTagBudgetPlatformDesc> InBudgetPlatform);
	bool IsBudgetPlatformSelected(TSharedPtr<FMemTagBudgetPlatformDesc> InBudgetPlatform) const;

	void InitBudgetOptions();
	void OnBudgetChanged();
	void ApplyBudgetToNodes();
	void UpdateBudgetGroupsRec(const FMemTagBudgetGrouping* BudgetGrouping, const FMemTagBudgetGrouping* BudgetGroupingOverride, const FBaseTreeNodePtr& GroupPtr);
	void ResetBudgetForAllNodes();
	void ResetBudgetGroupsRec(const FBaseTreeNodePtr& GroupPtr);
	void UpdateHighThreshold(const FMemTagNode& MemTagNode, FMemorySharedState& SharedState);
	void UpdateHighThreshold(const FMemTagNode& MemTagNode, FMemoryGraphTrack& GraphTrack);

private:
	/** A weak pointer to the Memory Insights window. */
	TWeakPtr<SMemoryProfilerWindow> ProfilerWindowWeakPtr;

	//////////////////////////////////////////////////
	// Tree Nodes

	/** The serial number of the memory tag list maintained by the MemorySharedState object (updated last time we have synced MemTagNodes with it). */
	uint32 LastMemoryTagListSerialNumber = 0;

	/** All LLM tag nodes, stored as FMemoryTagId -> TSharedPtr<FMemTagNode>. */
	TMap<FMemoryTagId, TSharedPtr<FMemTagNode>> MemTagNodesIdMap;

	//////////////////////////////////////////////////
	// Filters

	/** Filter the LLM tags by tracker. */
	uint64 TrackersFilter = uint64(-1);

	/** Filter the LLM tags by tag set. */
	FMemoryTagSetId TagSetFilter = FMemoryTagSet::DefaultTagSetId;
	FMemoryTagSetId SystemsTagSet = 0;
	FMemoryTagSetId AssetsTagSet = 1;
	FMemoryTagSetId AssetClassesTagSet = 2;
	TArray<TSharedPtr<FMemoryTagSetId>> AvailableTagSets;
	TSharedPtr<SSegmentedControl<FMemoryTagSetId>> TagSetsSegmentedControl;

	//////////////////////////////////////////////////

	FText NumSelectedTagsText;
	FText SelectedTagsText;
	FText SelectionSizeAText;
	FText SelectionSizeBText;
	FText SelectionDiffText;

	FLinearColor EditableColorValue;

	//////////////////////////////////////////////////

	TSharedPtr<FMemTagBudget> CurrentBudget;

	TArray<TSharedPtr<FMemTagBudgetFileDesc>> AvailableBudgetFiles;
	TArray<TSharedPtr<FMemTagBudgetModeDesc>> AvailableBudgetModes;
	TArray<TSharedPtr<FMemTagBudgetPlatformDesc>> AvailableBudgetPlatforms;
	TSet<FString> AvailablePlatforms;

	TSharedPtr<FMemTagBudgetFileDesc> SelectedBudgetFile;
	TSharedPtr<FMemTagBudgetModeDesc> SelectedBudgetMode;
	TSharedPtr<FMemTagBudgetPlatformDesc> SelectedBudgetPlatform;

	bool bIsLoadingBudget = false;

	//////////////////////////////////////////////////

	EInvestigationMode Mode = EInvestigationMode::Default;

	// A, B, B-A
	double StatsTimeA = 0.0;
	double StatsTimeB = 0.0;

	// Sample Count, Min, Max, Average
	double StatsStartTime = 0.0;
	double StatsEndTime = 0.0;

	bool bShouldUpdateStats = false;
	bool bShouldUpdateBudgets = false;
	bool bAreTimeMarkerSettingsVisible = false;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
