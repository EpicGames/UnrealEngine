// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FExtender;
struct FInsightsMajorTabConfig;

namespace UE::Insights::MemoryProfiler
{

/** Ribbon based toolbar used as a main menu in the Profiler window. */
class SMemoryProfilerToolbar : public SCompoundWidget
{
public:
	/** Default constructor. */
	SMemoryProfilerToolbar();

	/** Virtual destructor. */
	virtual ~SMemoryProfilerToolbar();

	SLATE_BEGIN_ARGS(SMemoryProfilerToolbar) {}
	SLATE_ARGUMENT(TSharedPtr<FExtender>, ToolbarExtender);
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs, const FInsightsMajorTabConfig& Config);
};

} // namespace UE::Insights::MemoryProfiler
