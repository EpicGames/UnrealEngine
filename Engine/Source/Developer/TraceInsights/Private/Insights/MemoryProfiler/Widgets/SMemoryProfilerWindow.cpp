// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMemoryProfilerWindow.h"

#include "Features/IModularFeatures.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Docking/SDockTab.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/MemoryProfiler/MemoryProfilerManager.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocTable.h"
#include "Insights/MemoryProfiler/ViewModels/MemorySharedState.h"
#include "Insights/MemoryProfiler/Widgets/SMemAllocTableTreeView.h"
#include "Insights/MemoryProfiler/Widgets/SMemInvestigationView.h"
#include "Insights/MemoryProfiler/Widgets/SMemoryProfilerToolbar.h"
#include "Insights/MemoryProfiler/Widgets/SMemTagTreeView.h"
#include "Insights/TimingProfiler/Tracks/TimeRulerTrack.h"
#include "Insights/TimingProfiler/ViewModels/TimeMarker.h"
#include "Insights/TimingProfiler/Widgets/SUserAnnotationsPanel.h"
#include "Insights/Widgets/SModulesView.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::MemoryProfiler"

const FName FMemoryProfilerTabs::TimingViewID(TEXT("TimingView"));
const FName FMemoryProfilerTabs::MemInvestigationViewID(TEXT("MemInvestigation"));
const FName FMemoryProfilerTabs::MemTagTreeViewID(TEXT("LowLevelMemTags"));
const FName FMemoryProfilerTabs::MemAllocTableTreeViewID(TEXT("MemAllocTableTreeView"));
const FName FMemoryProfilerTabs::ModulesViewID(TEXT("Modules"));
const FName FMemoryProfilerTabs::UserAnnotationsID(TEXT("UserAnnotations"));

namespace UE::Insights::MemoryProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

SMemoryProfilerWindow::SMemoryProfilerWindow()
	: SMajorTabWindow(FInsightsManagerTabs::MemoryProfilerTabId)
	, SharedState(MakeShared<FMemorySharedState>())
{
	CreateTimingViewMarkers();

	IModularFeatures::Get().RegisterModularFeature(Timing::TimingViewExtenderFeatureName, &SharedState.Get());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SMemoryProfilerWindow::~SMemoryProfilerWindow()
{
	CloseAllOpenTabs();

	IModularFeatures::Get().UnregisterModularFeature(Timing::TimingViewExtenderFeatureName, &SharedState.Get());

	check(ModulesView == nullptr);
	check(MemTagTreeView == nullptr);
	check(MemInvestigationView == nullptr);
	check(TimingView == nullptr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* SMemoryProfilerWindow::GetAnalyticsEventName() const
{
	return TEXT("Insights.Usage.MemoryProfiler");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::Reset()
{
	SharedState->ResetMemoryTags();

	if (TimingView)
	{
		TimingView->Reset();
		ResetTimingViewMarkers();
	}

	if (MemInvestigationView)
	{
		MemInvestigationView->Reset();
	}

	if (MemTagTreeView)
	{
		MemTagTreeView->Reset();
	}

	if (ModulesView)
	{
		ModulesView->Reset();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SMemoryProfilerWindow::SpawnTab_TimingView(const FSpawnTabArgs& Args)
{
	FMemoryProfilerManager::Get()->SetTimingViewVisible(true);

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(TimingView, TimingProfiler::STimingView, FInsightsManagerTabs::MemoryProfilerTabId)
		];

	SharedState->SetTimingView(TimingView);
	SharedState->BindCommands();

	TimingView->Reset(true);
	TimingView->OnSelectionChanged().AddSP(this, &SMemoryProfilerWindow::OnTimeSelectionChanged);
	TimingView->OnCustomTimeMarkerChanged().AddSP(this, &SMemoryProfilerWindow::OnTimeMarkerChanged);
	ResetTimingViewMarkers();
	TimingView->HideAllDefaultTracks();

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SMemoryProfilerWindow::OnTimingViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::OnTimingViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FMemoryProfilerManager::Get()->SetTimingViewVisible(false);

	if (TimingView)
	{
		TimingView->OnSelectionChanged().RemoveAll(this);
		TimingView->OnCustomTimeMarkerChanged().RemoveAll(this);
		TimingView = nullptr;
	}

	SharedState->SetTimingView(nullptr);

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SMemoryProfilerWindow::SpawnTab_MemInvestigationView(const FSpawnTabArgs& Args)
{
	FMemoryProfilerManager::Get()->SetMemInvestigationViewVisible(true);

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(MemInvestigationView, SMemInvestigationView, SharedThis(this))
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SMemoryProfilerWindow::OnMemInvestigationViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::OnMemInvestigationViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FMemoryProfilerManager::Get()->SetMemInvestigationViewVisible(false);
	MemInvestigationView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SMemoryProfilerWindow::SpawnTab_MemTagTreeView(const FSpawnTabArgs& Args)
{
	FMemoryProfilerManager::Get()->SetMemTagTreeViewVisible(true);

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(MemTagTreeView, SMemTagTreeView, SharedThis(this))
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SMemoryProfilerWindow::OnMemTagTreeViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::OnMemTagTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FMemoryProfilerManager::Get()->SetMemTagTreeViewVisible(false);
	MemTagTreeView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SMemoryProfilerWindow::SpawnTab_MemAllocTableTreeView(const FSpawnTabArgs& Args, int32 TabIndex)
{
	//FMemoryProfilerManager::Get()->SetMemAllocTableTreeViewVisible(TabIndex, true);

	TSharedRef<FMemAllocTable> MemAllocTable = MakeShared<FMemAllocTable>();
	MemAllocTable->Reset();
	MemAllocTable->SetDisplayName(FText::FromString(TEXT("MemAllocs")));

	TSharedPtr<SMemAllocTableTreeView> MemAllocTableTreeView;

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(MemAllocTableTreeView, SMemAllocTableTreeView, MemAllocTable)
		];

	MemAllocTableTreeView->SetLogListingName(FMemoryProfilerManager::Get()->GetLogListingName());
	MemAllocTableTreeView->SetTabIndex(TabIndex);
	MemAllocTableTreeViews.Add(MemAllocTableTreeView);

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SMemoryProfilerWindow::OnMemAllocTableTreeViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::OnMemAllocTableTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	TSharedRef<SMemAllocTableTreeView> MemAllocTableTreeView = StaticCastSharedRef<SMemAllocTableTreeView>(TabBeingClosed->GetContent());

	FName ClosingTabId = FMemoryProfilerTabs::MemAllocTableTreeViewID;
	ClosingTabId.SetNumber(MemAllocTableTreeView->GetTabIndex());

	TSharedPtr<FQueryTargetWindowSpec> TargetToDelete;
	const TArray<TSharedPtr<FQueryTargetWindowSpec>>& Targets = SharedState->GetQueryTargets();
	for (int32 Index = 0; Index < Targets.Num(); ++Index)
	{
		if (Targets[Index]->GetName() == ClosingTabId)
		{
			TargetToDelete = Targets[Index];
			break;
		}
	}

	if (TargetToDelete.IsValid())
	{
		SharedState->RemoveQueryTarget(TargetToDelete);
	}

	if (Targets.Num() > 0)
	{
		TSharedPtr<FQueryTargetWindowSpec> NewSelection = Targets[0];
		SharedState->SetCurrentQueryTarget(NewSelection);
		if (MemInvestigationView.IsValid())
		{
			MemInvestigationView->QueryTarget_OnSelectionChanged(NewSelection, ESelectInfo::Type::Direct);
		}
	}

	GetTabManager()->UnregisterTabSpawner(ClosingTabId);

	MemAllocTableTreeView->OnClose();
	MemAllocTableTreeViews.Remove(MemAllocTableTreeView);

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::CloseMemAllocTableTreeTabs()
{
	const TArray<TSharedPtr<FQueryTargetWindowSpec>>& Targets = SharedState->GetQueryTargets();
	while (Targets.Num() > 0)
	{
		FName Name = Targets[0]->GetName();
		SharedState->RemoveQueryTarget(Targets[0]);

		if (Name != FQueryTargetWindowSpec::NewWindow)
		{
			HideTab(Name);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SMemAllocTableTreeView> SMemoryProfilerWindow::ShowMemAllocTableTreeViewTab()
{
	if (!GetTabManager().IsValid())
	{
		return nullptr;
	}
	FTabManager* TabManagerPtr = GetTabManager().Get();

	if (SharedState->GetCurrentQueryTarget()->GetName() == FQueryTargetWindowSpec::NewWindow)
	{
		++LastMemAllocTableTreeViewIndex;
		FName TabId = FMemoryProfilerTabs::MemAllocTableTreeViewID;
		TabId.SetNumber(LastMemAllocTableTreeViewIndex);

		check(GetWorkspaceMenuGroup().IsValid());
		const TSharedRef<FWorkspaceItem> Group = GetWorkspaceMenuGroup().ToSharedRef();

		FText MemAllocTableTreeViewTabDisplayName = FText::Format(LOCTEXT("MemoryProfiler.MemAllocTableTreeViewTabTitle", "Allocs Table {0}"), FText::AsNumber(LastMemAllocTableTreeViewIndex));
		TabManagerPtr->RegisterTabSpawner(TabId, FOnSpawnTab::CreateRaw(this, &SMemoryProfilerWindow::SpawnTab_MemAllocTableTreeView, LastMemAllocTableTreeViewIndex))
			.SetDisplayName(MemAllocTableTreeViewTabDisplayName)
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.MemAllocTableTreeView"))
			.SetGroup(Group);

		TSharedPtr<FQueryTargetWindowSpec> NewTarget = MakeShared<FQueryTargetWindowSpec>(TabId, MemAllocTableTreeViewTabDisplayName);
		SharedState->AddQueryTarget(NewTarget);
		SharedState->SetCurrentQueryTarget(NewTarget);
		MemInvestigationView->QueryTarget_OnSelectionChanged(NewTarget, ESelectInfo::Type::Direct);
	}

	FName TabId = SharedState->GetCurrentQueryTarget()->GetName();
	if (TabManagerPtr->HasTabSpawner(TabId))
	{
		TSharedPtr<SDockTab> Tab = TabManagerPtr->TryInvokeTab(TabId);
		if (Tab)
		{
			TSharedRef<SMemAllocTableTreeView> MemAllocTableTreeView = StaticCastSharedRef<SMemAllocTableTreeView>(Tab->GetContent());

			if (SharedState->GetCurrentQueryTarget()->GetName() == FQueryTargetWindowSpec::NewWindow)
			{
				Tab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SMemoryProfilerWindow::OnMemAllocTableTreeViewTabClosed));
			}
			return MemAllocTableTreeView;
		}
	}

	return nullptr;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SMemAllocTableTreeView> SMemoryProfilerWindow::OpenNewMemAllocTableTreeViewTab(const FText& InTabDisplayName)
{
	if (!GetTabManager().IsValid())
	{
		return nullptr;
	}
	FTabManager* TabManagerPtr = GetTabManager().Get();

	++LastMemAllocTableTreeViewIndex;
	FName TabId = FMemoryProfilerTabs::MemAllocTableTreeViewID;
	TabId.SetNumber(LastMemAllocTableTreeViewIndex);

	check(GetWorkspaceMenuGroup().IsValid());
	const TSharedRef<FWorkspaceItem> Group = GetWorkspaceMenuGroup().ToSharedRef();

	TabManagerPtr->RegisterTabSpawner(TabId, FOnSpawnTab::CreateRaw(this, &SMemoryProfilerWindow::SpawnTab_MemAllocTableTreeView, LastMemAllocTableTreeViewIndex))
		.SetDisplayName(InTabDisplayName)
		.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.MemAllocTableTreeView"))
		.SetGroup(Group);

	TSharedPtr<SDockTab> Tab = TabManagerPtr->TryInvokeTab(TabId);
	if (!Tab.IsValid())
	{
		return nullptr;
	}

	return StaticCastSharedRef<SMemAllocTableTreeView>(Tab->GetContent());
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SMemoryProfilerWindow::SpawnTab_ModulesView(const FSpawnTabArgs& Args)
{
	FMemoryProfilerManager::Get()->SetModulesViewVisible(true);

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(ModulesView, SModulesView)
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SMemoryProfilerWindow::OnModulesViewClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::OnModulesViewClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FMemoryProfilerManager::Get()->SetModulesViewVisible(false);
	ModulesView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SMemoryProfilerWindow::SpawnTab_UserAnnotations(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(UserAnnotationsPanel, TimingProfiler::SUserAnnotationsPanel)
			.HostTimingView(TimingView)
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SMemoryProfilerWindow::OnUserAnnotationsTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::OnUserAnnotationsTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	UserAnnotationsPanel = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	TSharedPtr<FMemoryProfilerManager> MemoryProfilerManager = FMemoryProfilerManager::Get();
	ensure(MemoryProfilerManager.IsValid());

	SetCommandList(MemoryProfilerManager->GetCommandList());

	SMajorTabWindow::FArguments Args;
	SMajorTabWindow::Construct(Args, ConstructUnderMajorTab, ConstructUnderWindow);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FWorkspaceItem> SMemoryProfilerWindow::CreateWorkspaceMenuGroup()
{
	return GetTabManager()->AddLocalWorkspaceMenuCategory(LOCTEXT("MemoryProfilerMenuGroupName", "Memory Insights"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::RegisterTabSpawners()
{
	check(GetTabManager().IsValid());
	FTabManager* TabManagerPtr = GetTabManager().Get();
	check(GetWorkspaceMenuGroup().IsValid());
	const TSharedRef<FWorkspaceItem> Group = GetWorkspaceMenuGroup().ToSharedRef();

	IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(FInsightsManagerTabs::MemoryProfilerTabId);

	if (Config.ShouldRegisterMinorTab(FMemoryProfilerTabs::TimingViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FMemoryProfilerTabs::TimingViewID, FOnSpawnTab::CreateRaw(this, &SMemoryProfilerWindow::SpawnTab_TimingView))
			.SetDisplayName(LOCTEXT("TimingViewTabTitle", "Timing View"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TimingView"))
			.SetTooltipText(LOCTEXT("TimingViewTabToolTip", "Opens the Timing View."))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FMemoryProfilerTabs::MemInvestigationViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FMemoryProfilerTabs::MemInvestigationViewID, FOnSpawnTab::CreateRaw(this, &SMemoryProfilerWindow::SpawnTab_MemInvestigationView))
			.SetDisplayName(LOCTEXT("MemInvestigationViewTabTitle", "Investigation"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.MemInvestigationView"))
			.SetTooltipText(LOCTEXT("MemInvestigationViewTabToolTip", "Opens the Memory Investigation panel.\n\nIt analyses the detailed trace data about memory allocations.\nSee documentation about \"-trace=memalloc\" or \"-trace=memory\"."))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FMemoryProfilerTabs::MemTagTreeViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FMemoryProfilerTabs::MemTagTreeViewID, FOnSpawnTab::CreateRaw(this, &SMemoryProfilerWindow::SpawnTab_MemTagTreeView))
			.SetDisplayName(LOCTEXT("MemTagTreeViewTabTitle", "Memory Tags"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.MemTagTreeView"))
			.SetTooltipText(LOCTEXT("MemTagTreeViewTabToolTip", "Opens the Memory Tags (LLM) panel.\n\nIt analyses the per-frame trace data emitted by the Low-Level Memory Tracker system (LLM).\nSee documentation \nabout \"-llm -llmtagsets=assets,assetclasses\" command line parameters."))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FMemoryProfilerTabs::ModulesViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FMemoryProfilerTabs::ModulesViewID, FOnSpawnTab::CreateRaw(this, &SMemoryProfilerWindow::SpawnTab_ModulesView))
			.SetDisplayName(LOCTEXT("ModulesViewTabTitle", "Modules"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ModulesView"))
			.SetTooltipText(LOCTEXT("ModulesViewTabToolTip", "Opens the Modules panel."))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FMemoryProfilerTabs::UserAnnotationsID))
	{
		TabManagerPtr->RegisterTabSpawner(FMemoryProfilerTabs::UserAnnotationsID, FOnSpawnTab::CreateRaw(this, &SMemoryProfilerWindow::SpawnTab_UserAnnotations))
			.SetDisplayName(LOCTEXT("UserAnnotationsTabTitle", "Annotations"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.Annotation"))
			.SetGroup(Group);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<FTabManager::FLayout> SMemoryProfilerWindow::CreateDefaultTabLayout() const
{
	return FTabManager::NewLayout("InsightsMemoryProfilerLayout_v1.4")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.65f)
				->SetHideTabWell(true)
				->AddTab(FMemoryProfilerTabs::TimingViewID, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.35f)
				->AddTab(FMemoryProfilerTabs::MemInvestigationViewID, ETabState::OpenedTab)
				->AddTab(FMemoryProfilerTabs::MemTagTreeViewID, ETabState::OpenedTab)
				->AddTab(FMemoryProfilerTabs::ModulesViewID, ETabState::OpenedTab)
				->AddTab(FMemoryProfilerTabs::UserAnnotationsID, ETabState::OpenedTab)
				->SetForegroundTab(FMemoryProfilerTabs::MemInvestigationViewID)
			)
		);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SMemoryProfilerWindow::CreateToolbar(TSharedPtr<FExtender> Extender)
{
	IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(FInsightsManagerTabs::MemoryProfilerTabId);

	return SNew(SMemoryProfilerToolbar, Config).ToolbarExtender(Extender);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::OnTimeSelectionChanged(Timing::ETimeChangedFlags InFlags, double InStartTime, double InEndTime)
{
	if (InFlags != Timing::ETimeChangedFlags::Interactive)
	{
		if (InStartTime < InEndTime)
		{
			const uint32 NumTimeMarkers = CustomTimeMarkers.Num();
			if (NumTimeMarkers >= 1)
			{
				CustomTimeMarkers[0]->SetTime(InStartTime);
			}
			if (NumTimeMarkers >= 2)
			{
				CustomTimeMarkers[1]->SetTime(InEndTime);
			}

			double Time = InEndTime;
			for (uint32 Index = 2; Index < NumTimeMarkers; ++Index)
			{
				const double TimeMarkerTime = CustomTimeMarkers[Index]->GetTime();
				if (TimeMarkerTime < Time)
				{
					CustomTimeMarkers[Index]->SetTime(Time);
				}
				else
				{
					Time = TimeMarkerTime;
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::CreateTimingViewMarkers()
{
	check(CustomTimeMarkers.Num() == 0);

	constexpr uint32 MaxNumTimeMarkers = 5;

	TCHAR TimeMarkerName[2];
	TimeMarkerName[1] = 0;

	for (uint32 Index = 0; Index < MaxNumTimeMarkers; ++Index)
	{
		TSharedRef<TimingProfiler::FTimeMarker> TimeMarker = MakeShared<TimingProfiler::FTimeMarker>();

		TimeMarkerName[0] = static_cast<TCHAR>(TEXT('A') + Index); // "A", "B", "C", etc.
		TimeMarker->SetName(TimeMarkerName);

		const uint32 HueStep = 256 / MaxNumTimeMarkers;
		const uint8 H = uint8(HueStep * Index);
		const uint8 S = 192;
		const uint8 V = 255;
		const FLinearColor Color = FLinearColor::MakeFromHSV8(H, S, V);
		TimeMarker->SetColor(Color);

		TimeMarker->SetTime((Index + 1) * 10.0); // 10s, 20s, 30s, etc.
		TimeMarker->SetVisibility(false);

		CustomTimeMarkers.Add(TimeMarker);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::ResetTimingViewMarkers()
{
	TSharedRef<TimingProfiler::FTimeRulerTrack> TimeRulerTrack = TimingView->GetTimeRulerTrack();

	TimeRulerTrack->RemoveAllTimeMarkers();

	// Hide the "Default Time Marker".
	TSharedRef<TimingProfiler::FTimeMarker> DefaultTimeMarker = TimingView->GetDefaultTimeMarker();
	DefaultTimeMarker->SetVisibility(false);

	const uint32 NumTimeMarkers = CustomTimeMarkers.Num();
	for (uint32 Index = 0; Index < NumTimeMarkers; ++Index)
	{
		TSharedRef<TimingProfiler::FTimeMarker>& TimeMarker = CustomTimeMarkers[Index];
		TimeMarker->SetTime((Index + 1) * 10.0); // 10s, 20s, 30s, etc.
		TimeRulerTrack->AddTimeMarker(TimeMarker);
	}

	UpdateTimingViewMarkers();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::OnMemoryRuleChanged()
{
	UpdateTimingViewMarkers();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::UpdateTimingViewMarkers()
{
	TSharedPtr<FMemoryRuleSpec> Rule = SharedState->GetCurrentMemoryRule();
	const uint32 NumVisibleTimeMarkers = Rule ? Rule->GetNumTimeMarkers() : 0;

	const uint32 NumTimeMarkers = CustomTimeMarkers.Num();
	ensure(NumVisibleTimeMarkers <= NumTimeMarkers);

	for (uint32 Index = 0; Index < NumTimeMarkers; ++Index)
	{
		TSharedRef<TimingProfiler::FTimeMarker>& TimeMarker = CustomTimeMarkers[Index];
		if (Index < NumVisibleTimeMarkers)
		{
			TimeMarker->SetVisibility(true);
		}
		else
		{
			TimeMarker->SetVisibility(false);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMemoryProfilerWindow::OnTimeMarkerChanged(Timing::ETimeChangedFlags InFlags, TSharedRef<Timing::ITimeMarker> InTimeMarker)
{
	const int32 NumTimeMarkers = CustomTimeMarkers.Num();

	// Find index of the changing time marker.
	int32 ChangedTimeMarkerIndex = -1;
	for (int32 Index = 0; Index < NumTimeMarkers; ++Index)
	{
		TSharedRef<TimingProfiler::FTimeMarker>& TimeMarker = CustomTimeMarkers[Index];
		if (TimeMarker == InTimeMarker)
		{
			ChangedTimeMarkerIndex = Index;
			break;
		}
	}

	// Change Time Marker A when changing the default time marker (i.e. when using Ctrl + click/drag).
	if (ChangedTimeMarkerIndex < 0 &&
		NumTimeMarkers > 0 &&
		TimingView.IsValid() &&
		InTimeMarker == TimingView->GetDefaultTimeMarker())
	{
		TSharedRef<TimingProfiler::FTimeMarker>& TimeMarkerA = CustomTimeMarkers[0];
		TimeMarkerA->SetTime(InTimeMarker->GetTime());
		ChangedTimeMarkerIndex = 0;
	}

	// Ensure the rest of time markers are ordered by time.
	if (ChangedTimeMarkerIndex >= 0)
	{
		for (int32 Index = 0; Index < ChangedTimeMarkerIndex; ++Index)
		{
			TSharedRef<TimingProfiler::FTimeMarker>& TimeMarker = CustomTimeMarkers[Index];
			if (TimeMarker->GetTime() > InTimeMarker->GetTime())
			{
				TimeMarker->SetTime(InTimeMarker->GetTime());
			}
		}
		for (int32 Index = ChangedTimeMarkerIndex + 1; Index < NumTimeMarkers; ++Index)
		{
			TSharedRef<TimingProfiler::FTimeMarker>& TimeMarker = CustomTimeMarkers[Index];
			if (TimeMarker->GetTime() < InTimeMarker->GetTime())
			{
				TimeMarker->SetTime(InTimeMarker->GetTime());
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler

#undef LOCTEXT_NAMESPACE
