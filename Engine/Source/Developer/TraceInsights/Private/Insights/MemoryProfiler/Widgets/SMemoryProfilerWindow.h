// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/NameTypes.h"

// TraceInsights
#include "Insights/ITimingViewSession.h" // for ETimeChangedFlags and ITimeMarker
#include "Insights/Widgets/SMajorTabWindow.h"

namespace UE::Insights
{
	class STableTreeView;
	class SModulesView;
}

namespace UE::Insights::TimingProfiler
{
	class FTimeMarker;
	class STimingView;
	class SUserAnnotationsPanel;
}

namespace UE::Insights::MemoryProfiler
{

class FMemorySharedState;
class SMemInvestigationView;
class SMemTagTreeView;
class SMemAllocTableTreeView;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Implements the Memory Insights window. */
class SMemoryProfilerWindow : public ::Insights::SMajorTabWindow
{
public:
	/** Default constructor. */
	SMemoryProfilerWindow();

	/** Virtual destructor. */
	virtual ~SMemoryProfilerWindow();

	SLATE_BEGIN_ARGS(SMemoryProfilerWindow) {}
	SLATE_END_ARGS()

	virtual void Reset() override;

	/** Constructs this widget. */
	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);

	TSharedPtr<TimingProfiler::STimingView> GetTimingView() const { return TimingView; }
	TSharedPtr<SMemInvestigationView> GetMemInvestigationView() const { return MemInvestigationView; }
	TSharedPtr<SMemTagTreeView> GetMemTagTreeView() const { return MemTagTreeView; }

	void CloseMemAllocTableTreeTabs();
	TSharedPtr<SMemAllocTableTreeView> ShowMemAllocTableTreeViewTab();
	TSharedPtr<SMemAllocTableTreeView> OpenNewMemAllocTableTreeViewTab(const FText& InTabDisplayName);

	uint32 GetNumCustomTimeMarkers() const { return (uint32)CustomTimeMarkers.Num(); }
	const TSharedRef<TimingProfiler::FTimeMarker>& GetCustomTimeMarker(uint32 Index) const { return CustomTimeMarkers[Index]; }
	const TArray<TSharedRef<TimingProfiler::FTimeMarker>>& GetCustomTimeMarkers() const { return CustomTimeMarkers; }

	FMemorySharedState& GetSharedState() { return *SharedState; }
	const FMemorySharedState& GetSharedState() const { return *SharedState; }

	void OnMemoryRuleChanged();
	void OnTimeMarkerChanged(Timing::ETimeChangedFlags InFlags, TSharedRef<Timing::ITimeMarker> InTimeMarker);

protected:
	virtual const TCHAR* GetAnalyticsEventName() const override;
	virtual TSharedRef<FWorkspaceItem> CreateWorkspaceMenuGroup() override;
	virtual void RegisterTabSpawners() override;
	virtual TSharedRef<FTabManager::FLayout> CreateDefaultTabLayout() const override;
	virtual TSharedRef<SWidget> CreateToolbar(TSharedPtr<FExtender> Extender);

private:
	TSharedRef<SDockTab> SpawnTab_TimingView(const FSpawnTabArgs& Args);
	void OnTimingViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_MemInvestigationView(const FSpawnTabArgs& Args);
	void OnMemInvestigationViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_MemTagTreeView(const FSpawnTabArgs& Args);
	void OnMemTagTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_MemAllocTableTreeView(const FSpawnTabArgs& Args, int32 TabIndex);
	void OnMemAllocTableTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_ModulesView(const FSpawnTabArgs& Args);
	void OnModulesViewClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_UserAnnotations(const FSpawnTabArgs& Args);
	void OnUserAnnotationsTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	void OnTimeSelectionChanged(Timing::ETimeChangedFlags InFlags, double InStartTime, double InEndTime);

	void CreateTimingViewMarkers();
	void ResetTimingViewMarkers();
	void UpdateTimingViewMarkers();

private:
	TSharedRef<FMemorySharedState> SharedState;

	/** The Timing view (multi-track) widget */
	TSharedPtr<TimingProfiler::STimingView> TimingView;

	TArray<TSharedRef<TimingProfiler::FTimeMarker>> CustomTimeMarkers;

	/** The Memory Investigation (Allocation Queries) view widget */
	TSharedPtr<SMemInvestigationView> MemInvestigationView;

	/** The Memory Tags tree view widget */
	TSharedPtr<SMemTagTreeView> MemTagTreeView;

	/** The list of Allocations table tree view widgets */
	TArray<TSharedPtr<SMemAllocTableTreeView>> MemAllocTableTreeViews;

	/** The Modules view widget. */
	TSharedPtr<SModulesView> ModulesView;

	/** The User Annotations panel widget. */
	TSharedPtr<TimingProfiler::SUserAnnotationsPanel> UserAnnotationsPanel;

	const int32 MaxMemAllocTableTreeViews = 4;
	int32 LastMemAllocTableTreeViewIndex = -1;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::MemoryProfiler
