// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkingProfilerCommands.h"

#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/NetworkingProfilerManager.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FNetworkingProfilerCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

FNetworkingProfilerCommands::FNetworkingProfilerCommands()
	: TCommands<FNetworkingProfilerCommands>(
		TEXT("NetworkingProfilerCommands"),
		NSLOCTEXT("Contexts", "NetworkingProfilerCommands", "Insights - Networking Insights"),
		NAME_None,
		FInsightsStyle::GetStyleSetName())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// UI_COMMAND takes long for the compiler to optimize
UE_DISABLE_OPTIMIZATION_SHIP
void FNetworkingProfilerCommands::RegisterCommands()
{
	UI_COMMAND(TogglePacketViewVisibility,
		"Packets",
		"Toggles the visibility of the Packets view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(TogglePacketContentViewVisibility,
		"Packet Content",
		"Toggles the visibility of the Packet Content view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleNetStatsViewVisibility,
		"Net Stats",
		"Toggles the visibility of the Net Stats view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleNetStatsCountersViewVisibility,
		"Net Stats Counters",
		"Toggles the visibility of the Net Stats view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////
// Toggle Commands
////////////////////////////////////////////////////////////////////////////////////////////////////

//INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FNetworkingProfilerActionManager, This, ToggleAAAViewVisibility, IsAAAViewVisible, ShowHideAAAView)

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
