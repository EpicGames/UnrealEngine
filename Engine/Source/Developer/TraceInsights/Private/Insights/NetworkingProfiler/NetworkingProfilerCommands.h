// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/Commands.h"

// TraceInsights
#include "Insights/Common/CommandUtils.h"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class that holds all profiler commands.
 */
class FNetworkingProfilerCommands : public TCommands<FNetworkingProfilerCommands>
{
public:
	/** Default constructor. */
	FNetworkingProfilerCommands();

	/** Initialize commands. */
	virtual void RegisterCommands() override;

public:
	/** Toggles visibility for the Packet view. Per profiler window, custom command. */
	TSharedPtr<FUICommandInfo> TogglePacketViewVisibility;

	/** Toggles visibility for the Packet Content view. Per profiler window, custom command. */
	TSharedPtr<FUICommandInfo> TogglePacketContentViewVisibility;

	/** Toggles visibility for the Net Stats view. Per profiler window, custom command. */
	TSharedPtr<FUICommandInfo> ToggleNetStatsViewVisibility;

	/** Toggles visibility for the Net Stats Counters view. Per profiler window, custom command. */
	TSharedPtr<FUICommandInfo> ToggleNetStatsCountersViewVisibility;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class that provides helper functions for the commands to avoid cluttering profiler manager with many small functions.
 * Can't contain any variables. Directly operates on the profiler manager instance.
 */
class FNetworkingProfilerActionManager
{
	friend class FNetworkingProfilerManager;

private:
	/** Private constructor. */
	FNetworkingProfilerActionManager(class FNetworkingProfilerManager* Instance)
		: This(Instance)
	{}

	//////////////////////////////////////////////////
	// Toggle Commands

	//INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleAAAViewVisibility)

	//////////////////////////////////////////////////

protected:
	/** Reference to the global instance of the profiler manager. */
	class FNetworkingProfilerManager* This;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
