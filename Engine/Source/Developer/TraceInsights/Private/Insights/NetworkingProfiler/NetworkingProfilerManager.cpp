// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkingProfilerManager.h"

#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

// TraceInsights
#include "Insights/Common/InsightsMenuBuilder.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/Widgets/SNetworkingProfilerWindow.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler"

namespace UE::Insights::NetworkingProfiler
{

DEFINE_LOG_CATEGORY(LogNetworkingProfiler);

TSharedPtr<FNetworkingProfilerManager> FNetworkingProfilerManager::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FNetworkingProfilerManager> FNetworkingProfilerManager::Get()
{
	return FNetworkingProfilerManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FNetworkingProfilerManager> FNetworkingProfilerManager::CreateInstance()
{
	ensure(!FNetworkingProfilerManager::Instance.IsValid());
	if (FNetworkingProfilerManager::Instance.IsValid())
	{
		FNetworkingProfilerManager::Instance.Reset();
	}

	FNetworkingProfilerManager::Instance = MakeShared<FNetworkingProfilerManager>(FInsightsManager::Get()->GetCommandList());

	return FNetworkingProfilerManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FNetworkingProfilerManager::FNetworkingProfilerManager(TSharedRef<FUICommandList> InCommandList)
	: CommandList(InCommandList)
	, ActionManager(this)
	, MajorTabId(FInsightsManagerTabs::NetworkingProfilerTabId)
	, LogListingName("NetworkingInsights")
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetworkingProfilerManager::Initialize(IUnrealInsightsModule& InsightsModule)
{
	ensure(!bIsInitialized);
	if (bIsInitialized)
	{
		return;
	}
	bIsInitialized = true;

	UE_LOGF(LogNetworkingProfiler, Log, "Initialize");

	// Register tick functions.
	OnTick = FTickerDelegate::CreateSP(this, &FNetworkingProfilerManager::Tick);
	OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(OnTick, 0.0f);

	FNetworkingProfilerCommands::Register();
	BindCommands();

	FInsightsManager::Get()->GetSessionChangedEvent().AddSP(this, &FNetworkingProfilerManager::OnSessionChanged);
	OnSessionChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetworkingProfilerManager::Shutdown()
{
	if (!bIsInitialized)
	{
		return;
	}
	bIsInitialized = false;

	// If the MessageLog module was already unloaded as part of the global Shutdown process, do not load it again.
	if (FModuleManager::Get().IsModuleLoaded("MessageLog"))
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		if (MessageLogModule.IsRegisteredLogListing(GetLogListingName()))
		{
			MessageLogModule.UnregisterLogListing(GetLogListingName());
		}
	}

	FInsightsManager::Get()->GetSessionChangedEvent().RemoveAll(this);

	FNetworkingProfilerCommands::Unregister();

	// Unregister tick function.
	FTSTicker::RemoveTicker(OnTickHandle);

	FNetworkingProfilerManager::Instance.Reset();

	UE_LOGF(LogNetworkingProfiler, Log, "Shutdown");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FNetworkingProfilerManager::~FNetworkingProfilerManager()
{
	ensure(!bIsInitialized);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetworkingProfilerManager::BindCommands()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static TSharedPtr<SDockTab> NeverReuse(const FTabId&)
{
	return nullptr;
}

void FNetworkingProfilerManager::RegisterMajorTabs(IUnrealInsightsModule& InsightsModule)
{
	if (!bIsTabRegistered)
	{
		bIsTabRegistered = true;
		const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(MajorTabId);
		if (Config.bIsAvailable)
		{
			// Register tab spawner for the Networking Insights.
			FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(MajorTabId,
					FOnSpawnTab::CreateRaw(this, &FNetworkingProfilerManager::SpawnTab),
					FCanSpawnTab::CreateRaw(this, &FNetworkingProfilerManager::CanSpawnTab))
				.SetReuseTabMethod(FOnFindTabToReuse::CreateStatic(&NeverReuse))
				.SetDisplayName(Config.TabLabel.IsSet() ? Config.TabLabel.GetValue() : LOCTEXT("NetworkingProfilerTabTitle", "Networking Insights"))
				.SetTooltipText(Config.TabTooltip.IsSet() ? Config.TabTooltip.GetValue() : LOCTEXT("NetworkingProfilerTooltipText", "Open the Networking Insights tab."))
				.SetIcon(Config.TabIcon.IsSet() ? Config.TabIcon.GetValue() : FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.NetworkingProfiler"));

			TSharedRef<FWorkspaceItem> Group = Config.WorkspaceGroup.IsValid() ? Config.WorkspaceGroup.ToSharedRef() : FInsightsManager::Get()->GetInsightsMenuBuilder()->GetInsightsToolsGroup();
			TabSpawnerEntry.SetGroup(Group);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetworkingProfilerManager::UnregisterMajorTabs()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(MajorTabId);
	bIsTabRegistered = false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> FNetworkingProfilerManager::SpawnTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	// Register OnTabClosed to handle Networking profiler manager shutdown.
	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FNetworkingProfilerManager::OnTabClosed));

	// Create the SNetworkingProfilerWindow widget.
	TSharedRef<SNetworkingProfilerWindow> Window = SNew(SNetworkingProfilerWindow, DockTab, Args.GetOwnerWindow());
	DockTab->SetContent(Window);

	AddProfilerWindow(Window);

	return DockTab;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FNetworkingProfilerManager::CanSpawnTab(const FSpawnTabArgs& Args) const
{
	return bIsAvailable;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetworkingProfilerManager::OnTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	TSharedRef<SNetworkingProfilerWindow> Window = StaticCastSharedRef<SNetworkingProfilerWindow>(TabBeingClosed->GetContent());

	RemoveProfilerWindow(Window);

	// Disable TabClosed delegate.
	TabBeingClosed->SetOnTabClosed(SDockTab::FOnTabClosedCallback());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedRef<FUICommandList> FNetworkingProfilerManager::GetCommandList() const
{
	return CommandList;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FNetworkingProfilerCommands& FNetworkingProfilerManager::GetCommands()
{
	return FNetworkingProfilerCommands::Get();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FNetworkingProfilerActionManager& FNetworkingProfilerManager::GetActionManager()
{
	return FNetworkingProfilerManager::Instance->ActionManager;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FNetworkingProfilerManager::Tick(float DeltaTime)
{
	// Check if session has Networking events (to spawn the tab), but not too often.
	if (!bIsAvailable && AvailabilityCheck.Tick())
	{
		uint32 NetTraceVersion = 0;

		TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();
		check(InsightsManager.IsValid());

		TSharedPtr<const TraceServices::IAnalysisSession> Session = InsightsManager->GetSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			if (Session->IsAnalysisComplete())
			{
				// Never check again during this session.
				AvailabilityCheck.Disable();
			}

			const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
			if (NetProfilerProvider)
			{
				NetTraceVersion = NetProfilerProvider->GetNetTraceVersion();
			}
		}
		else
		{
			// Do not check again until the next session changed event (see OnSessionChanged).
			AvailabilityCheck.Disable();
		}

		if (NetTraceVersion > 0)
		{
			bIsAvailable = true;

			FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
			MessageLogModule.RegisterLogListing(GetLogListingName(), LOCTEXT("NetworkingInsights", "Networking Insights"));
			MessageLogModule.EnableMessageLogDisplay(true);

			if (FGlobalTabmanager::Get()->HasTabSpawner(MajorTabId))
			{
				// Spawn 2 tabs.
				UE_LOGF(LogNetworkingProfiler, Log, "Opening the \"Networking Insights\" tabs...");
				FGlobalTabmanager::Get()->TryInvokeTab(MajorTabId);
				FGlobalTabmanager::Get()->TryInvokeTab(MajorTabId);
			}
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetworkingProfilerManager::OnSessionChanged()
{
	UE_LOGF(LogNetworkingProfiler, Log, "OnSessionChanged");

	bIsAvailable = false;
	if (FInsightsManager::Get()->GetSession().IsValid())
	{
		AvailabilityCheck.Enable(1.0);
	}
	else
	{
		AvailabilityCheck.Disable();
	}

	for (TWeakPtr<SNetworkingProfilerWindow> WndWeakPtr : ProfilerWindows)
	{
		TSharedPtr<SNetworkingProfilerWindow> Wnd = WndWeakPtr.Pin();
		if (Wnd)
		{
			Wnd->Reset();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
