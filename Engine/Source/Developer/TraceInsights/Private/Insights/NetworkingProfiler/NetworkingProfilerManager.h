// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Ticker.h"
#include "Framework/Commands/UICommandList.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Common/AvailabilityCheck.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/NetworkingProfiler/NetworkingProfilerCommands.h"

namespace UE::Insights::NetworkingProfiler
{

class SNetworkingProfilerWindow;

DECLARE_LOG_CATEGORY_EXTERN(LogNetworkingProfiler, Log, All);

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * This class manages the Networking Profiler (Networking Insights) state and settings.
 */
class FNetworkingProfilerManager : public TSharedFromThis<FNetworkingProfilerManager>, public IInsightsComponent
{
	friend class FNetworkingProfilerActionManager;

public:
	/** Creates the Networking Profiler manager, only one instance can exist. */
	FNetworkingProfilerManager(TSharedRef<FUICommandList> InCommandList);

	/** Virtual destructor. */
	virtual ~FNetworkingProfilerManager();

	/** Creates an instance of the Networking Profiler manager. */
	static TSharedPtr<FNetworkingProfilerManager> CreateInstance();

	/**
	 * @return the global instance of the Networking Profiler manager.
	 * This is an internal singleton and cannot be used outside TraceInsights.
	 * For external use:
	 *     IUnrealInsightsModule& Module = FModuleManager::Get().LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	 *     Module.GetNetworkingProfilerManager();
	 */
	static TSharedPtr<FNetworkingProfilerManager> Get();

	bool IsAvailable() const { return bIsAvailable; }

	//////////////////////////////////////////////////
	// IInsightsComponent

	virtual void Initialize(IUnrealInsightsModule& InsightsModule) override;
	virtual void Shutdown() override;
	virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) override;
	virtual void UnregisterMajorTabs() override;

	//////////////////////////////////////////////////

	/** @return UI command list for the Networking Profiler manager. */
	const TSharedRef<FUICommandList> GetCommandList() const;

	/** @return an instance of the Networking Profiler commands. */
	static const FNetworkingProfilerCommands& GetCommands();

	/** @return an instance of the Networking Profiler action manager. */
	static FNetworkingProfilerActionManager& GetActionManager();

	/** @return the number of the "Networking Insights" windows currently available. */
	int32 GetNumProfilerWindows() const
	{
		return ProfilerWindows.Num();
	}

	/**
	 * Converts profiler window weak pointer to a shared pointer and returns it.
	 * Make sure the returned pointer is valid before trying to dereference it.
	 */
	TSharedPtr<SNetworkingProfilerWindow> GetProfilerWindow(int32 Index) const
	{
		return (Index >= 0 && Index < ProfilerWindows.Num()) ? ProfilerWindows[Index].Pin() : nullptr;
	}
	TSharedPtr<SNetworkingProfilerWindow> GetProfilerWindowChecked(int32 Index) const
	{
		return ProfilerWindows[Index].Pin();
	}

	const FName& GetMajorTabId() const { return MajorTabId; }
	const FName& GetLogListingName() const { return LogListingName; }

	void OnSessionChanged();

private:
	/** Binds our UI commands to delegates. */
	void BindCommands();

	/** Called to spawn the Networking Profiler major tab. */
	TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args);

	bool CanSpawnTab(const FSpawnTabArgs& Args) const;

	/** Callback called when the Networking Profiler major tab is closed. */
	void OnTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	/** Updates this manager, done through FCoreTicker. */
	bool Tick(float DeltaTime);

	void AddProfilerWindow(const TSharedRef<SNetworkingProfilerWindow>& InProfilerWindow)
	{
		ProfilerWindows.Add(InProfilerWindow);
	}

	void RemoveProfilerWindow(const TSharedRef<SNetworkingProfilerWindow>& InProfilerWindow)
	{
		ProfilerWindows.Remove(InProfilerWindow);
	}

private:
	bool bIsInitialized = false;
	bool bIsAvailable = false;

	/** If the tab is registered in the tab manager. */
	bool bIsTabRegistered = false;

	FAvailabilityCheck AvailabilityCheck;

	/** The delegate to be invoked when this manager ticks. */
	FTickerDelegate OnTick;

	/** Handle to the registered OnTick. */
	FTSTicker::FDelegateHandle OnTickHandle;

	/** List of UI commands for this manager. This will be filled by this and corresponding classes. */
	TSharedRef<FUICommandList> CommandList;

	/** An instance of the Networking Profiler action manager. */
	FNetworkingProfilerActionManager ActionManager;

	/** A list of weak pointers to the Networking Profiler windows. */
	TArray<TWeakPtr<class SNetworkingProfilerWindow>> ProfilerWindows;

	/** The id of the Networking Insights major tab. */
	FName MajorTabId;

	/** The name of the Networking Profiler log listing. */
	FName LogListingName;

	/** A shared pointer to the global instance of the Networking Profiler manager. */
	static TSharedPtr<FNetworkingProfilerManager> Instance;
};

} // namespace UE::Insights::NetworkingProfiler
