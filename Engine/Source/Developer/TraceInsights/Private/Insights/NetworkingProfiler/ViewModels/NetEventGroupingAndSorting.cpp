// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetEventGroupingAndSorting.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/NetworkingProfiler/ViewModels/NetEventNodeHelper.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::FNetEventNode"

#define INSIGHTS_ENSURE ensure
//#define INSIGHTS_ENSURE(...)
 
// Sort by name (ascending).
#define INSIGHTS_DEFAULT_SORTING_NODES(A, B) return A->GetName().LexicalLess(B->GetName());
//#define INSIGHTS_DEFAULT_SORTING_NODES(A, B) return A->GetDefaultSortOrder() < B->GetDefaultSortOrder();

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorting by Event Type
////////////////////////////////////////////////////////////////////////////////////////////////////

FNetEventNodeSortingByEventType::FNetEventNodeSortingByEventType(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByEventType")),
		LOCTEXT("Sorting_ByEventType_Name", "By Type"),
		LOCTEXT("Sorting_ByEventType_Title", "Sort By Type"),
		LOCTEXT("Sorting_ByEventType_Desc", "Sort by event type."),
		InColumnRef)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetEventNodeSortingByEventType::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeA = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(A);

			INSIGHTS_ENSURE(B.IsValid() && B->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeB = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(B);

			if (NetEventNodeA->GetType() == NetEventNodeB->GetType())
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by net event type (ascending).
				return NetEventNodeA->GetType() < NetEventNodeB->GetType();
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeA = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(A);

			INSIGHTS_ENSURE(B.IsValid() && B->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeB = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(B);

			if (NetEventNodeA->GetType() == NetEventNodeB->GetType())
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by net event type (descending).
				return NetEventNodeB->GetType() < NetEventNodeA->GetType();
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sort by Instance Count
////////////////////////////////////////////////////////////////////////////////////////////////////

FNetEventNodeSortingByInstanceCount::FNetEventNodeSortingByInstanceCount(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByInstanceCount")),
		LOCTEXT("Sorting_ByInstanceCount_Name", "By Instance Count"),
		LOCTEXT("Sorting_ByInstanceCount_Title", "Sort By Instance Count"),
		LOCTEXT("Sorting_ByInstanceCount_Desc", "Sort by aggregated instance count."),
		InColumnRef)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetEventNodeSortingByInstanceCount::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeA = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(A);
			const uint64 ValueA = NetEventNodeA->GetAggregatedStats().InstanceCount;

			INSIGHTS_ENSURE(B.IsValid() && B->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeB = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(B);
			const uint64 ValueB = NetEventNodeB->GetAggregatedStats().InstanceCount;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by instance count (ascending).
				return ValueA < ValueB;
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeA = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(A);
			const uint64 ValueA = NetEventNodeA->GetAggregatedStats().InstanceCount;

			INSIGHTS_ENSURE(B.IsValid() && B->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeB = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(B);
			const uint64 ValueB = NetEventNodeB->GetAggregatedStats().InstanceCount;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by instance count (descending).
				return ValueB < ValueA;
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sort by Total Inclusive Size
////////////////////////////////////////////////////////////////////////////////////////////////////

FNetEventNodeSortingByTotalInclusiveSize::FNetEventNodeSortingByTotalInclusiveSize(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByTotalInclusiveSize")),
		LOCTEXT("Sorting_ByTotalInclusiveSize_Name", "By Total Inclusive Size"),
		LOCTEXT("Sorting_ByTotalInclusiveSize_Title", "Sort By Total Inclusive Size"),
		LOCTEXT("Sorting_ByTotalInclusiveSize_Desc", "Sort by aggregated total inclusive size."),
		InColumnRef)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetEventNodeSortingByTotalInclusiveSize::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeA = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(A);
			const uint32 ValueA = NetEventNodeA->GetAggregatedStats().TotalInclusive;

			INSIGHTS_ENSURE(B.IsValid() && B->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeB = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(B);
			const uint32 ValueB = NetEventNodeB->GetAggregatedStats().TotalInclusive;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by total inclusive size (ascending).
				return ValueA < ValueB;
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeA = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(A);
			const uint32 ValueA = NetEventNodeA->GetAggregatedStats().TotalInclusive;

			INSIGHTS_ENSURE(B.IsValid() && B->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeB = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(B);
			const uint32 ValueB = NetEventNodeB->GetAggregatedStats().TotalInclusive;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by total inclusive size (descending).
				return ValueB < ValueA;
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sort by Total Exclusive Size
////////////////////////////////////////////////////////////////////////////////////////////////////

FNetEventNodeSortingByTotalExclusiveSize::FNetEventNodeSortingByTotalExclusiveSize(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByTotalExclusiveSize")),
		LOCTEXT("Sorting_ByTotalExclusiveSize_Name", "By Total Exclusive Size"),
		LOCTEXT("Sorting_ByTotalExclusiveSize_Title", "Sort By Total Exclusive Size"),
		LOCTEXT("Sorting_ByTotalExclusiveSize_Desc", "Sort by aggregated total exclusive size."),
		InColumnRef)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetEventNodeSortingByTotalExclusiveSize::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeA = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(A);
			const uint32 ValueA = NetEventNodeA->GetAggregatedStats().TotalExclusive;

			INSIGHTS_ENSURE(B.IsValid() && B->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeB = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(B);
			const uint32 ValueB = NetEventNodeB->GetAggregatedStats().TotalExclusive;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by total exclusive size (ascending).
				return ValueA < ValueB;
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeA = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(A);
			const uint32 ValueA = NetEventNodeA->GetAggregatedStats().TotalExclusive;

			INSIGHTS_ENSURE(B.IsValid() && B->Is<FNetEventNode>());
			const FNetEventNodePtr NetEventNodeB = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(B);
			const uint32 ValueB = NetEventNodeB->GetAggregatedStats().TotalExclusive;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by total exclusive size (descending).
				return ValueB < ValueA;
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef INSIGHTS_DEFAULT_SORTING_NODES
#undef INSIGHTS_ENSURE
#undef LOCTEXT_NAMESPACE
