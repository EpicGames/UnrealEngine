// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

// TraceInsights
#include "Insights/NetworkingProfiler/ViewModels/NetEventNode.h"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorters
////////////////////////////////////////////////////////////////////////////////////////////////////

class FNetEventNodeSortingByEventType: public FTableCellValueSorter
{
public:
	FNetEventNodeSortingByEventType(TSharedRef<FTableColumn> InColumnRef);

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FNetEventNodeSortingByInstanceCount : public FTableCellValueSorter
{
public:
	FNetEventNodeSortingByInstanceCount(TSharedRef<FTableColumn> InColumnRef);

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FNetEventNodeSortingByTotalInclusiveSize : public FTableCellValueSorter
{
public:
	FNetEventNodeSortingByTotalInclusiveSize(TSharedRef<FTableColumn> InColumnRef);

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FNetEventNodeSortingByTotalExclusiveSize : public FTableCellValueSorter
{
public:
	FNetEventNodeSortingByTotalExclusiveSize(TSharedRef<FTableColumn> InColumnRef);

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// Organizers
////////////////////////////////////////////////////////////////////////////////////////////////////

/** Enumerates types of grouping or sorting for the net event nodes. */
enum class ENetEventGroupingMode
{
	/** Creates a single group for all net events. */
	Flat,

	/** Creates one group for one letter. */
	ByName,

	/** Creates one group for each net event type. */
	ByType,

	/** Creates one group for each net event level. */
	ByLevel,

	/** Invalid enum type, may be used as a number of enumerations. */
	InvalidOrMax,
};

/** Type definition for shared pointers to instances of ENetEventGroupingMode. */
typedef TSharedPtr<ENetEventGroupingMode> ENetEventGroupingModePtr;

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
