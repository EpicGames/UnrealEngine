// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetEventNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::FNetEventNode"

namespace UE::Insights::NetworkingProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FNetEventNode)

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetEventNode::ResetAggregatedStats()
{
	AggregatedStats = TraceServices::FNetProfilerAggregatedStats();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetEventNode::SetAggregatedStats(const TraceServices::FNetProfilerAggregatedStats& InAggregatedStats)
{
	AggregatedStats = InAggregatedStats;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
