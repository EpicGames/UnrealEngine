// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// TraceServices
#include "TraceServices/Model/NetProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class ENetEventNodeType
{
	/** The NetEventNode is a Net Event. */
	NetEvent,

	/** The NetEventNode is a Net Object. */
	//NetObject,

	/** The NetEventNode is a group node. */
	Group,

	/** Invalid enum type, may be used as a number of enumerations. */
	InvalidOrMax,
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FNetEventNode;

/** Type definition for shared pointers to instances of FNetEventNode. */
typedef TSharedPtr<class FNetEventNode> FNetEventNodePtr;

/** Type definition for shared references to instances of FNetEventNode. */
typedef TSharedRef<class FNetEventNode> FNetEventNodeRef;

/** Type definition for shared references to const instances of FNetEventNode. */
typedef TSharedRef<const class FNetEventNode> FNetEventNodeRefConst;

/** Type definition for weak references to instances of FNetEventNode. */
typedef TWeakPtr<class FNetEventNode> FNetEventNodeWeak;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class used to store information about a timer node (used in the SNetStatsView).
 */
class FNetEventNode : public FBaseTreeNode
{
	INSIGHTS_DECLARE_RTTI(FNetEventNode, FBaseTreeNode)

public:
	static constexpr uint32 InvalidEventTypeIndex = uint32(-1);

public:
	/** Initialization constructor for the NetEvent node. */
	FNetEventNode(uint32 InEventTypeIndex, const FName InName, ENetEventNodeType InType, uint32 InLevel)
		: FBaseTreeNode(InName, InType == ENetEventNodeType::Group)
		, EventTypeIndex(InEventTypeIndex)
		, Type(InType)
		, Level(InLevel)
	{
		ResetAggregatedStats();
	}

	/** Initialization constructor for the group node. */
	explicit FNetEventNode(const FName InGroupName)
		: FBaseTreeNode(InGroupName, true)
		, EventTypeIndex(InvalidEventTypeIndex)
		, Type(ENetEventNodeType::Group)
		, Level(0)
	{
		ResetAggregatedStats();
	}

	uint32 GetEventTypeIndex() const { return EventTypeIndex; }

	/**
	 * @return a type of this NetEvent node or ENetEventNodeType::Group for group nodes.
	 */
	ENetEventNodeType GetType() const { return Type; }

	void SetLevel(uint32 InLevel) const { return ; }
	uint32 GetLevel() const { return Level; }

	/**
	 * @return the aggregated stats for this NetEvent node.
	 */
	const TraceServices::FNetProfilerAggregatedStats& GetAggregatedStats() const { return AggregatedStats; }

	void ResetAggregatedStats();
	void SetAggregatedStats(const TraceServices::FNetProfilerAggregatedStats& AggregatedStats);

private:
	/** The NetEvent type index. */
	const uint32 EventTypeIndex;

	/** Holds the type of this NetEvent node. */
	const ENetEventNodeType Type;

	uint32 Level;

	/** Aggregated stats. */
	TraceServices::FNetProfilerAggregatedStats AggregatedStats;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
