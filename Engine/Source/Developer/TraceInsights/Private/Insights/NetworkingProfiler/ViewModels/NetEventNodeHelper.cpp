// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetEventNodeHelper.h"

// TraceInsights
#include "Insights/InsightsStyle.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::FNetEventNode"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// NetEventNode Type Helper
////////////////////////////////////////////////////////////////////////////////////////////////////

FText NetEventNodeTypeHelper::ToText(const ENetEventNodeType NodeType)
{
	static_assert(static_cast<int>(ENetEventNodeType::InvalidOrMax) == 2, "Not all cases are handled in switch below!?");
	switch (NodeType)
	{
		case ENetEventNodeType::NetEvent:	return LOCTEXT("Type_Name_NetEvent", "Net Event");
		case ENetEventNodeType::Group:		return LOCTEXT("Type_Name_Group", "Group");
		default:							return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText NetEventNodeTypeHelper::ToDescription(const ENetEventNodeType NodeType)
{
	static_assert(static_cast<int>(ENetEventNodeType::InvalidOrMax) == 2, "Not all cases are handled in switch below!?");
	switch (NodeType)
	{
		case ENetEventNodeType::NetEvent:	return LOCTEXT("Type_Desc_NetEvent", "Net event");
		case ENetEventNodeType::Group:		return LOCTEXT("Type_Desc_Group", "Group node");
		default:							return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* NetEventNodeTypeHelper::GetIconForGroup()
{
	return FInsightsStyle::GetBrush("Icons.Group.TreeItem");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* NetEventNodeTypeHelper::GetIconForNetEventNodeType(const ENetEventNodeType NodeType)
{
	static_assert(static_cast<int>(ENetEventNodeType::InvalidOrMax) == 2, "Not all cases are handled in switch below!?");
	switch (NodeType)
	{
		case ENetEventNodeType::NetEvent:	return FInsightsStyle::GetBrush("Icons.NetEvent.TreeItem");
		case ENetEventNodeType::Group:		return FInsightsStyle::GetBrush("Icons.Group.TreeItem");
		default:							return nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// NetEventNode Grouping Helper
////////////////////////////////////////////////////////////////////////////////////////////////////

FText NetEventNodeGroupingHelper::ToText(const ENetEventGroupingMode GroupingMode)
{
	static_assert(static_cast<int>(ENetEventGroupingMode::InvalidOrMax) == 4, "Not all cases are handled in switch below!?");
	switch (GroupingMode)
	{
		case ENetEventGroupingMode::Flat:		return LOCTEXT("Grouping_Name_Flat",	"Flat");
		case ENetEventGroupingMode::ByName:		return LOCTEXT("Grouping_Name_ByName",	"Name");
		case ENetEventGroupingMode::ByType:		return LOCTEXT("Grouping_Name_ByType",	"Event Type");
		case ENetEventGroupingMode::ByLevel:	return LOCTEXT("Grouping_Name_ByLevel", "Level");
		default:								return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText NetEventNodeGroupingHelper::ToDescription(const ENetEventGroupingMode GroupingMode)
{
	static_assert(static_cast<int>(ENetEventGroupingMode::InvalidOrMax) == 4, "Not all cases are handled in switch below!?");
	switch (GroupingMode)
	{
		case ENetEventGroupingMode::Flat:		return LOCTEXT("Grouping_Desc_Flat",	"Creates a single group. Includes all net events.");
		case ENetEventGroupingMode::ByName:		return LOCTEXT("Grouping_Desc_ByName",	"Creates one group for one letter.");
		case ENetEventGroupingMode::ByType:		return LOCTEXT("Grouping_Desc_ByType",	"Creates one group for each net event type.");
		case ENetEventGroupingMode::ByLevel:	return LOCTEXT("Grouping_Desc_ByLevel", "Creates one group for each level.");
		default:								return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
