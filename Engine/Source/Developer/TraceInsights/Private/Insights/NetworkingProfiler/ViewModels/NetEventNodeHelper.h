// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Styling/SlateBrush.h"

// TraceInsights
#include "Insights/NetworkingProfiler/ViewModels/NetEventNode.h"
#include "Insights/NetworkingProfiler/ViewModels/NetEventGroupingAndSorting.h"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Helper struct that contains helper functions and classes for ENetEventNodeType enum. */
struct NetEventNodeTypeHelper
{
	/**
	 * @param Type - The value to get the text for.
	 *
	 * @return text representation of the specified ENetEventNodeType value.
	 */
	static FText ToText(const ENetEventNodeType Type);

	/**
	 * @param Type - The value to get the text for.
	 *
	 * @return text representation with more detailed explanation of the specified ENetEventNodeType value.
	 */
	static FText ToDescription(const ENetEventNodeType Type);

	static const FSlateBrush* GetIconForGroup();
	static const FSlateBrush* GetIconForNetEventNodeType(const ENetEventNodeType Type);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Helper struct that contains grouping static functions and classes. */
struct NetEventNodeGroupingHelper
{
	/**
	 * @param NetEventGroupingMode - The value to get the text for.
	 *
	 * @return text representation of the specified ENetEventGroupingMode value.
	 */
	static FText ToText(const ENetEventGroupingMode NetEventGroupingMode);

	/**
	 * @param NetEventGroupingMode - The value to get the text for.
	 *
	 * @return text representation with more detailed explanation of the specified ENetEventGroupingMode value.
	 */
	static FText ToDescription(const ENetEventGroupingMode NetEventGroupingMode);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
