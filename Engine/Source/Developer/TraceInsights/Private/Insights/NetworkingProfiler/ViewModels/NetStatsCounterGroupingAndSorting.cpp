// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetStatsCounterGroupingAndSorting.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/NetworkingProfiler/ViewModels/NetStatsCounterNodeHelper.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::FNetStatsCounterNode"

// Sort by name (ascending).
#define INSIGHTS_DEFAULT_SORTING_NODES(A, B) return A->GetName().LexicalLess(B->GetName());

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorting by Event Type
////////////////////////////////////////////////////////////////////////////////////////////////////

FNetStatsCounterNodeSortingByEventType::FNetStatsCounterNodeSortingByEventType(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByEventType")),
		LOCTEXT("Sorting_ByEventType_Name", "By Type"),
		LOCTEXT("Sorting_ByEventType_Title", "Sort By Type"),
		LOCTEXT("Sorting_ByEventType_Desc", "Sort by event type."),
		InColumnRef)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetStatsCounterNodeSortingByEventType::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			ensure(A.IsValid() && A->Is<FNetStatsCounterNode>());
			const FNetStatsCounterNodePtr NetStatsCounterNodeA = StaticCastSharedPtr<FNetStatsCounterNode, FBaseTreeNode>(A);

			ensure(B.IsValid() && B->Is<FNetStatsCounterNode>());
			const FNetStatsCounterNodePtr NetStatsCounterNodeB = StaticCastSharedPtr<FNetStatsCounterNode, FBaseTreeNode>(B);

			if (NetStatsCounterNodeA->GetType() == NetStatsCounterNodeB->GetType())
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by net event type (ascending).
				return NetStatsCounterNodeA->GetType() < NetStatsCounterNodeB->GetType();
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			ensure(A.IsValid() && A->Is<FNetStatsCounterNode>());
			const FNetStatsCounterNodePtr NetStatsCounterNodeA = StaticCastSharedPtr<FNetStatsCounterNode, FBaseTreeNode>(A);

			ensure(B.IsValid() && B->Is<FNetStatsCounterNode>());
			const FNetStatsCounterNodePtr NetStatsCounterNodeB = StaticCastSharedPtr<FNetStatsCounterNode, FBaseTreeNode>(B);

			if (NetStatsCounterNodeA->GetType() == NetStatsCounterNodeB->GetType())
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by net event type (descending).
				return NetStatsCounterNodeB->GetType() < NetStatsCounterNodeA->GetType();
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sort by Sum
////////////////////////////////////////////////////////////////////////////////////////////////////

FNetStatsCounterNodeSortingBySum::FNetStatsCounterNodeSortingBySum(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("BySum")),
		LOCTEXT("Sorting_BySum_Name", "By Sum"),
		LOCTEXT("Sorting_BySum_Title", "Sort By Sum"),
		LOCTEXT("Sorting_BySum_Desc", "Sort by aggregated Sum."),
		InColumnRef)
{
}

//////////////////////////////////////////////////////////////////////////////////////////////////

void FNetStatsCounterNodeSortingBySum::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			ensure(A.IsValid() && A->Is<FNetStatsCounterNode>());
			const FNetStatsCounterNodePtr NetStatsCounterNodeA = StaticCastSharedPtr<FNetStatsCounterNode, FBaseTreeNode>(A);
			const uint32 ValueA = NetStatsCounterNodeA->GetAggregatedStats().Sum;

			ensure(B.IsValid() && B->Is<FNetStatsCounterNode>());
			const FNetStatsCounterNodePtr NetStatsCounterNodeB = StaticCastSharedPtr<FNetStatsCounterNode, FBaseTreeNode>(B);
			const uint32 ValueB = NetStatsCounterNodeB->GetAggregatedStats().Sum;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by Sum (ascending).
				return ValueA < ValueB;
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			ensure(A.IsValid() && A->Is<FNetStatsCounterNode>());
			const FNetStatsCounterNodePtr NetStatsCounterNodeA = StaticCastSharedPtr<FNetStatsCounterNode, FBaseTreeNode>(A);
			const uint32 ValueA = NetStatsCounterNodeA->GetAggregatedStats().Sum;

			ensure(B.IsValid() && B->Is<FNetStatsCounterNode>());
			const FNetStatsCounterNodePtr NetStatsCounterNodeB = StaticCastSharedPtr<FNetStatsCounterNode, FBaseTreeNode>(B);
			const uint32 ValueB = NetStatsCounterNodeB->GetAggregatedStats().Sum;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by Sum (descending).
				return ValueB < ValueA;
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef INSIGHTS_DEFAULT_SORTING_NODES
#undef LOCTEXT_NAMESPACE
