// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

// TraceInsights
#include "Insights/NetworkingProfiler/ViewModels/NetStatsCounterNode.h"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorters
////////////////////////////////////////////////////////////////////////////////////////////////////

class FNetStatsCounterNodeSortingByEventType: public FTableCellValueSorter
{
public:
	FNetStatsCounterNodeSortingByEventType(TSharedRef<FTableColumn> InColumnRef);

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FNetStatsCounterNodeSortingBySum : public FTableCellValueSorter
{
public:
	FNetStatsCounterNodeSortingBySum(TSharedRef<FTableColumn> InColumnRef);

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// Organizers
////////////////////////////////////////////////////////////////////////////////////////////////////

/** Enumerates types of grouping or sorting for the NetStatsCounter-nodes. */
enum class ENetStatsCounterGroupingMode
{
	/** Creates a single group for all */
	Flat,

	/** Creates one group for one letter. */
	ByName,

	/** Creates one group for each type. */
	ByType,

	/** Invalid enum type, may be used as a number of enumerations. */
	InvalidOrMax,
};

/** Type definition for shared pointers to instances of ENetStatsCounterGroupingMode. */
typedef TSharedPtr<ENetStatsCounterGroupingMode> ENetStatsCounterGroupingModePtr;

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
