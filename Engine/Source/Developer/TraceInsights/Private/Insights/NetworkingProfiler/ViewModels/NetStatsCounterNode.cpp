// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetStatsCounterNode.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::FNetStatsCounterNode"

namespace UE::Insights::NetworkingProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FNetStatsCounterNode)

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetStatsCounterNode::ResetAggregatedStats()
{
	AggregatedStats.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FNetStatsCounterNode::FormatAggregatedStatsValue(uint32 Value) const
{
	if (AggregatedStats.Count > 0)
	{
		return FText::AsNumber(Value);
	}

	return LOCTEXT("AggregatedStatsNA", "N/A");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FNetStatsCounterNode::GetTextForAggregatedStatsSum(bool bForTooltip) const
{
	return FormatAggregatedStatsValue(AggregatedStats.Sum);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FNetStatsCounterNode::GetTextForAggregatedStatsMin(bool bForTooltip) const
{
	return FormatAggregatedStatsValue(AggregatedStats.Min);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FNetStatsCounterNode::GetTextForAggregatedStatsMax(bool bForTooltip) const
{
	return FormatAggregatedStatsValue(AggregatedStats.Max);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FNetStatsCounterNode::GetTextForAggregatedStatsAverage(bool bForTooltip) const
{
	return FormatAggregatedStatsValue(AggregatedStats.Average);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetStatsCounterNode::SetAggregatedStats(const FNetProfilerAggregatedStatsCounterStats& InAggregatedStats)
{
	ensure(InAggregatedStats.StatsCounterTypeIndex == AggregatedStats.StatsCounterTypeIndex);
	AggregatedStats = InAggregatedStats;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
