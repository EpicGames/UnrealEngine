// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// TraceServices
#include "TraceServices/Model/NetProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class ENetStatsCounterNodeType
{
	/** A FrameStats node. */
	FrameStats,

	/** A PacketStats node. */
	PacketStats,

	/** A group node. */
	Group,

	/** Invalid enum type, may be used as a number of enumerations. */
	InvalidOrMax,
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FNetStatsCounterNode;

/** Type definition for shared pointers to instances of FNetStatsCounterNode. */
typedef TSharedPtr<class FNetStatsCounterNode> FNetStatsCounterNodePtr;

/** Type definition for shared references to instances of FNetStatsCounterNode. */
typedef TSharedRef<class FNetStatsCounterNode> FNetStatsCounterNodeRef;

/** Type definition for shared references to const instances of FNetStatsCounterNode. */
typedef TSharedRef<const class FNetStatsCounterNode> FNetStatsCounterNodeRefConst;

/** Type definition for weak references to instances of FNetStatsCounterNode. */
typedef TWeakPtr<class FNetStatsCounterNode> FNetStatsCounterNodeWeak;

////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Class used to store information about a net stats counter node (used in SNetStatsCountersView).
 */
class FNetStatsCounterNode : public FBaseTreeNode
{
	INSIGHTS_DECLARE_RTTI(FNetStatsCounterNode, FBaseTreeNode)

public:
	using FNetProfilerAggregatedStatsCounterStats = TraceServices::FNetProfilerAggregatedStatsCounterStats;
	using ENetProfilerStatsCounterType = TraceServices::ENetProfilerStatsCounterType;

	static constexpr uint32 InvalidCounterIndex = uint32(-1);

public:
	/** Initialization constructor for the stats node. */
	FNetStatsCounterNode(uint32 InStatsCounterTypeIndex, const FName InName, ENetProfilerStatsCounterType CounterType)
	: FBaseTreeNode(InName, false)
	, Type(CounterType == ENetProfilerStatsCounterType::Packet ? ENetStatsCounterNodeType::PacketStats : ENetStatsCounterNodeType::FrameStats)
	{
		AggregatedStats.StatsCounterTypeIndex = InStatsCounterTypeIndex;
		ResetAggregatedStats();
	}

	/** Initialization constructor for the group node. */
	explicit FNetStatsCounterNode(const FName InGroupName)
	: FBaseTreeNode(InGroupName, true)
	, Type(ENetStatsCounterNodeType::Group)
	{
		AggregatedStats.StatsCounterTypeIndex = InvalidCounterIndex;
		ResetAggregatedStats();
	}

	/**
	 * @return the counter id as provided by analyzer.
	 */
	uint32 GetCounterTypeIndex() const { return AggregatedStats.StatsCounterTypeIndex; }

	/**
	 * @return the type of this node or ENetStatsCounterNodeType::Group for group nodes.
	 */
	ENetStatsCounterNodeType GetType() const { return Type; }

	/**
	 * @return the aggregated stats of this stats counter.
	 */
	const FNetProfilerAggregatedStatsCounterStats& GetAggregatedStats() const { return AggregatedStats; }
	FNetProfilerAggregatedStatsCounterStats& GetAggregatedStats() { return AggregatedStats; }

	void ResetAggregatedStats();

	void SetAggregatedStats(const FNetProfilerAggregatedStatsCounterStats& AggregatedStats);

	const FText GetTextForAggregatedStatsSum(bool bForTooltip = false) const;
	const FText GetTextForAggregatedStatsMin(bool bForTooltip = false) const;
	const FText GetTextForAggregatedStatsMax(bool bForTooltip = false) const;
	const FText GetTextForAggregatedStatsAverage(bool bForTooltip = false) const;

private:
	const FText FormatAggregatedStatsValue(uint32 Value) const;

	/** The type of this node. */
	const ENetStatsCounterNodeType Type;

	/** Aggregated stats. */
	FNetProfilerAggregatedStatsCounterStats AggregatedStats;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
