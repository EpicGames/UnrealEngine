// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetStatsCounterNodeHelper.h"

// TraceInsights
#include "Insights/InsightsStyle.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::FNetStatsCounterNode"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// NetStatsCounterNode Type Helper
////////////////////////////////////////////////////////////////////////////////////////////////////

FText NetStatsCounterNodeTypeHelper::ToText(const ENetStatsCounterNodeType NodeType)
{
	static_assert(static_cast<int>(ENetStatsCounterNodeType::InvalidOrMax) == 3, "Not all cases are handled in switch below!?");
	switch (NodeType)
	{
		case ENetStatsCounterNodeType::FrameStats:	return LOCTEXT("Stats_Name_FrameStats", "FrameStats");
		case ENetStatsCounterNodeType::PacketStats:	return LOCTEXT("Stats_Name_PacketStats", "PacketStats");
		case ENetStatsCounterNodeType::Group:		return LOCTEXT("Stats_Name_Group", "Group");
		default:									return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText NetStatsCounterNodeTypeHelper::ToDescription(const ENetStatsCounterNodeType NodeType)
{
	static_assert(static_cast<int>(ENetStatsCounterNodeType::InvalidOrMax) == 3, "Not all cases are handled in switch below!?");
	switch (NodeType)
	{
		case ENetStatsCounterNodeType::FrameStats:	return LOCTEXT("Stats_Desc_FrameStats", "FrameStats node");
		case ENetStatsCounterNodeType::PacketStats:	return LOCTEXT("Stats_Desc_PacketStats", "PacketStats node");
		case ENetStatsCounterNodeType::Group:		return LOCTEXT("Stats_Desc_Group", "Group node");
		default:									return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FName NetStatsCounterNodeTypeHelper::ToBrushName(const ENetStatsCounterNodeType NodeType)
{
	static_assert(static_cast<int>(ENetStatsCounterNodeType::InvalidOrMax) == 3, "Not all cases are handled in switch below!?");
	switch (NodeType)
	{
		case ENetStatsCounterNodeType::FrameStats:
		case ENetStatsCounterNodeType::PacketStats:
			return TEXT("Profiler.FiltersAndPresets.StatTypeIcon");
		case ENetStatsCounterNodeType::Group:
			return TEXT("Profiler.Misc.GenericGroup");
		default:
			return NAME_None;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* NetStatsCounterNodeTypeHelper::GetIconForGroup()
{
	return FInsightsStyle::GetBrush(TEXT("Icons.Group.TreeItem"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* NetStatsCounterNodeTypeHelper::GetIcon(const ENetStatsCounterNodeType NodeType)
{
	static_assert(static_cast<int>(ENetStatsCounterNodeType::InvalidOrMax) == 3, "Not all cases are handled in switch below!?");
	switch (NodeType)
	{
		case ENetStatsCounterNodeType::FrameStats:
		case ENetStatsCounterNodeType::PacketStats:
			return FInsightsStyle::GetBrush(TEXT("Icons.NetEvent.TreeItem"));
		case ENetStatsCounterNodeType::Group:
			return FInsightsStyle::GetBrush(TEXT("Icons.Group.TreeItem"));
		default:
			return nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// NetStatsCounterNode Grouping Helper
////////////////////////////////////////////////////////////////////////////////////////////////////

FText NetStatsCounterNodeGroupingHelper::ToText(const ENetStatsCounterGroupingMode GroupingMode)
{
	static_assert(static_cast<int>(ENetStatsCounterGroupingMode::InvalidOrMax) == 3, "Not all cases are handled in switch below!?");
	switch (GroupingMode)
	{
		case ENetStatsCounterGroupingMode::Flat:	return LOCTEXT("Grouping_Name_Flat", "Flat");
		case ENetStatsCounterGroupingMode::ByName:	return LOCTEXT("Grouping_Name_ByName", "Name");
		case ENetStatsCounterGroupingMode::ByType:	return LOCTEXT("Grouping_Name_ByType", "Event Type");
		default:									return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText NetStatsCounterNodeGroupingHelper::ToDescription(const ENetStatsCounterGroupingMode GroupingMode)
{
	static_assert(static_cast<int>(ENetStatsCounterGroupingMode::InvalidOrMax) == 3, "Not all cases are handled in switch below!?");
	switch (GroupingMode)
	{
		case ENetStatsCounterGroupingMode::Flat:	return LOCTEXT("Grouping_Desc_Flat", "Creates a single group. Includes all counters.");
		case ENetStatsCounterGroupingMode::ByName:	return LOCTEXT("Grouping_Desc_ByName", "Creates one group for one letter.");
		case ENetStatsCounterGroupingMode::ByType:	return LOCTEXT("Grouping_Desc_ByType", "Creates one group for each net event type.");
		default:									return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FName NetStatsCounterNodeGroupingHelper::ToBrushName(const ENetStatsCounterGroupingMode GroupingMode)
{
	static_assert(static_cast<int>(ENetStatsCounterGroupingMode::InvalidOrMax) == 3, "Not all cases are handled in switch below!?");
	switch (GroupingMode)
	{
		case ENetStatsCounterGroupingMode::Flat:	return TEXT("Profiler.FiltersAndPresets.GroupNameIcon"); //TODO: "Icons.Grouping.Flat"
		case ENetStatsCounterGroupingMode::ByName:	return TEXT("Profiler.FiltersAndPresets.GroupNameIcon"); //TODO: "Icons.Grouping.ByName"
		case ENetStatsCounterGroupingMode::ByType:	return TEXT("Profiler.FiltersAndPresets.StatTypeIcon"); //TODO
		default:									return NAME_None;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
