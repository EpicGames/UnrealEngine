// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Styling/SlateBrush.h"

// TraceInsights
#include "Insights/NetworkingProfiler/ViewModels/NetStatsCounterNode.h"
#include "Insights/NetworkingProfiler/ViewModels/NetStatsCounterGroupingAndSorting.h"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Helper struct that contains helper functions and classes for EStatsNodeType enum. */
struct NetStatsCounterNodeTypeHelper
{
	/**
	 * @param Type - The value to get the text for.
	 *
	 * @return text representation of the specified EStatsNodeType value.
	 */
	static FText ToText(const ENetStatsCounterNodeType Type);

	/**
	 * @param Type - The value to get the text for.
	 *
	 * @return text representation with more detailed explanation of the specified EStatsNodeType value.
	 */
	static FText ToDescription(const ENetStatsCounterNodeType Type);

	/**
	 * @param Type - The value to get the brush name for.
	 *
	 * @return brush name of the specified EStatsNodeType value.
	 */
	static FName ToBrushName(const ENetStatsCounterNodeType Type);

	static const FSlateBrush* GetIconForGroup();
	static const FSlateBrush* GetIcon(const ENetStatsCounterNodeType Type);
};

/** Helper struct that contains grouping static functions and classes. */
struct NetStatsCounterNodeGroupingHelper
{
	/**
	 * @param NetEventGroupingMode - The value to get the text for.
	 *
	 * @return text representation of the specified ENetStatsCounterGroupingMode value.
	 */
	static FText ToText(const ENetStatsCounterGroupingMode NetEventGroupingMode);

	/**
	 * @param NetEventGroupingMode - The value to get the text for.
	 *
	 * @return text representation with more detailed explanation of the specified ENetStatsCounterGroupingMode value.
	 */
	static FText ToDescription(const ENetStatsCounterGroupingMode NetEventGroupingMode);

	/**
	 * @param NetEventGroupingMode - The value to get the brush name for.
	 *
	 * @return brush name of the specified ENetStatsCounterGroupingMode value.
	 */
	static FName ToBrushName(const ENetStatsCounterGroupingMode NetEventGroupingMode);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
