// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetStatsCountersViewColumnFactory.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"

// TraceInsights
#include "Insights/NetworkingProfiler/ViewModels/NetStatsCounterGroupingAndSorting.h"
#include "Insights/NetworkingProfiler/ViewModels/NetStatsCounterNodeHelper.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::SNetStatsCountersView"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Column identifiers

const FName FNetStatsCountersViewColumns::NameColumnID(TEXT("Name")); // TEXT("_Hierarchy")
const FName FNetStatsCountersViewColumns::TypeColumnID(TEXT("Type"));
const FName FNetStatsCountersViewColumns::InstanceCountColumnID(TEXT("Count"));

const FName FNetStatsCountersViewColumns::SumColumnID(TEXT("Sum"));
const FName FNetStatsCountersViewColumns::MaxCountColumnID(TEXT("Max"));
const FName FNetStatsCountersViewColumns::AverageCountColumnID(TEXT("Avg"));

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetStatsCountersViewColumnFactory::CreateNetStatsCountersViewColumns(TArray<TSharedRef<FTableColumn>>& Columns)
{
	Columns.Reset();

	Columns.Add(CreateNameColumn());
	Columns.Add(CreateTypeColumn());
	Columns.Add(CreateInstanceCountColumn());

	Columns.Add(CreateSumColumn());
	Columns.Add(CreateMaxCountColumn());
	Columns.Add(CreateAverageCountColumn());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsCountersViewColumnFactory::CreateNameColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsCountersViewColumns::NameColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Name_ColumnName", "Name"));
	Column.SetTitleName(LOCTEXT("Name_ColumnTitle", "NetStatsCounter or Group Name"));
	Column.SetDescription(LOCTEXT("Name_ColumnDesc", "Name of the timer or group"));

	Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered |
					ETableColumnFlags::IsHierarchy);

	Column.SetHorizontalAlignment(HAlign_Left);
	Column.SetInitialWidth(206.0f);
	Column.SetMinWidth(42.0f);

	Column.SetDataType(ETableCellDataType::Text);

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FDisplayNameValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByName>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsCountersViewColumnFactory::CreateTypeColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsCountersViewColumns::TypeColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Type_ColumnName", "Type"));
	Column.SetTitleName(LOCTEXT("Type_ColumnTitle", "Type"));
	Column.SetDescription(LOCTEXT("Type_ColumnDesc", "Type of counter or group"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Left);
	Column.SetInitialWidth(60.0f);

	Column.SetDataType(ETableCellDataType::Text);

	class FNetStatsCounterTypeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FNetStatsCountersViewColumns::TypeColumnID);
			const FNetStatsCounterNode& NetStatsCounterNode = static_cast<const FNetStatsCounterNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(NetStatsCounterNodeTypeHelper::ToText(NetStatsCounterNode.GetType())));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FNetStatsCounterTypeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FNetStatsCounterNodeSortingByEventType>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsCountersViewColumnFactory::CreateInstanceCountColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsCountersViewColumns::InstanceCountColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("InstanceCount_ColumnName", "Count"));
	Column.SetTitleName(LOCTEXT("InstanceCount_ColumnTitle", "Count"));
	Column.SetDescription(LOCTEXT("InstanceCount_ColumnDesc", "Number of counters in the selected range"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
//					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(60.0f);

	Column.SetDataType(ETableCellDataType::Int64);

	class FInstanceCountValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FNetStatsCountersViewColumns::InstanceCountColumnID);
			const FNetStatsCounterNode& NetStatsCounterNode = static_cast<const FNetStatsCounterNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(NetStatsCounterNode.GetAggregatedStats().Count)));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FInstanceCountValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
	Column.SetValueFormatter(Formatter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Inclusive  Columns
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsCountersViewColumnFactory::CreateSumColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsCountersViewColumns::SumColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Sum_ColumnName", "Sum"));
	Column.SetTitleName(LOCTEXT("Sum_ColumnTitle", "Sum"));
	Column.SetDescription(LOCTEXT("Sum_ColumnDesc", "Total sum of selected NetStatsCounters"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(SumColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Int64);
	Column.SetAggregation(ETableColumnAggregation::Sum);

	class FSumValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FNetStatsCountersViewColumns::SumColumnID);
			const FNetStatsCounterNode& NetStatsCounterNode = static_cast<const FNetStatsCounterNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(NetStatsCounterNode.GetAggregatedStats().Sum)));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FSumValueGetter>();
	Column.SetValueGetter(Getter);

	class FSumFormatter : public FTableCellValueFormatter
	{
	public:
		virtual FText FormatValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			ensure(Column.GetId() == FNetStatsCountersViewColumns::SumColumnID);
			const FNetStatsCounterNode& NetStatsCounterNode = static_cast<const FNetStatsCounterNode&>(Node);
			return NetStatsCounterNode.GetTextForAggregatedStatsSum();
		}
	};
	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FSumFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FNetStatsCounterNodeSortingBySum>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsCountersViewColumnFactory::CreateMaxCountColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsCountersViewColumns::MaxCountColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("MaxCount_ColumnName", "Max"));
	Column.SetTitleName(LOCTEXT("MaxCount_ColumnTitle", "Max Count"));
	Column.SetDescription(LOCTEXT("MaxCount_ColumnDesc", "Maximum count of NetStatsCounters in the selected range"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(CountColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Int64);

	class FMaxCountValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FNetStatsCountersViewColumns::MaxCountColumnID);
			const FNetStatsCounterNode& NetStatsCounterNode = static_cast<const FNetStatsCounterNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(NetStatsCounterNode.GetAggregatedStats().Max)));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMaxCountValueGetter>();
	Column.SetValueGetter(Getter);

	class FMaxCountFormatter : public FTableCellValueFormatter
	{
	public:
		virtual FText FormatValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			ensure(Column.GetId() == FNetStatsCountersViewColumns::MaxCountColumnID);
			const FNetStatsCounterNode& NetStatsCounterNode = static_cast<const FNetStatsCounterNode&>(Node);
			return NetStatsCounterNode.GetTextForAggregatedStatsMax();
		}
	};
	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FMaxCountFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsCountersViewColumnFactory::CreateAverageCountColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsCountersViewColumns::AverageCountColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("AvgInclusive_ColumnName", "Avg"));
	Column.SetTitleName(LOCTEXT("AvgInclusive_ColumnTitle", "Average count"));
	Column.SetDescription(LOCTEXT("AvgInclusive_ColumnDesc", "Average count in selected range"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(CountColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Int64);

	class FAverageCountValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FNetStatsCountersViewColumns::AverageCountColumnID);
			const FNetStatsCounterNode& NetStatsCounterNode = static_cast<const FNetStatsCounterNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(NetStatsCounterNode.GetAggregatedStats().Average)));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAverageCountValueGetter>();
	Column.SetValueGetter(Getter);

	class FAverageCountFormatter : public FTableCellValueFormatter
	{
	public:
		virtual FText FormatValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			ensure(Column.GetId() == FNetStatsCountersViewColumns::AverageCountColumnID);
			const FNetStatsCounterNode& NetStatsCounterNode = static_cast<const FNetStatsCounterNode&>(Node);
			return NetStatsCounterNode.GetTextForAggregatedStatsAverage();
		}
	};
	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FAverageCountFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
