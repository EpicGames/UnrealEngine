// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

namespace UE::Insights { class FTableColumn; }

namespace UE::Insights::NetworkingProfiler
{

// Column identifiers
struct FNetStatsCountersViewColumns
{
	static const FName NameColumnID;
	static const FName TypeColumnID;
	static const FName InstanceCountColumnID;
	static const FName SumColumnID;
	static const FName MaxCountColumnID;
	static const FName AverageCountColumnID;
};

struct FNetStatsCountersViewColumnFactory
{
public:
	static void CreateNetStatsCountersViewColumns(TArray<TSharedRef<FTableColumn>>& Columns);

	static TSharedRef<FTableColumn> CreateNameColumn();
	static TSharedRef<FTableColumn> CreateTypeColumn();
	static TSharedRef<FTableColumn> CreateInstanceCountColumn();
	static TSharedRef<FTableColumn> CreateSumColumn();
	static TSharedRef<FTableColumn> CreateMaxCountColumn();
	static TSharedRef<FTableColumn> CreateAverageCountColumn();

private:
	static constexpr float SumColumnInitialWidth = 60.0f;
	static constexpr float CountColumnInitialWidth = 50.0f;
};

} // namespace UE::Insights::NetworkingProfiler
