// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetStatsViewColumnFactory.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"

// TraceInsights
#include "Insights/NetworkingProfiler/ViewModels/NetEventGroupingAndSorting.h"
#include "Insights/NetworkingProfiler/ViewModels/NetEventNodeHelper.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::SNetStatsView"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Column identifiers

const FName FNetStatsViewColumns::NameColumnID(TEXT("Name")); // TEXT("_Hierarchy")
const FName FNetStatsViewColumns::TypeColumnID(TEXT("Type"));
const FName FNetStatsViewColumns::LevelColumnID(TEXT("Level"));
const FName FNetStatsViewColumns::InstanceCountColumnID(TEXT("Count"));

// Inclusive  columns
const FName FNetStatsViewColumns::TotalInclusiveSizeColumnID(TEXT("TotalIncl"));
const FName FNetStatsViewColumns::MaxInclusiveSizeColumnID(TEXT("MaxIncl"));
const FName FNetStatsViewColumns::AverageInclusiveSizeColumnID(TEXT("AverageIncl"));

// Exclusive  columns
const FName FNetStatsViewColumns::TotalExclusiveSizeColumnID(TEXT("TotalExcl"));
const FName FNetStatsViewColumns::MaxExclusiveSizeColumnID(TEXT("MaxExcl"));
const FName FNetStatsViewColumns::AverageExclusiveSizeColumnID(TEXT("AverageExcl"));

////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetStatsViewColumnFactory::CreateNetStatsViewColumns(TArray<TSharedRef<FTableColumn>>& Columns)
{
	Columns.Reset();

	Columns.Add(CreateNameColumn());
	Columns.Add(CreateTypeColumn());
	Columns.Add(CreateLevelColumn());
	Columns.Add(CreateInstanceCountColumn());

	Columns.Add(CreateTotalInclusiveSizeColumn());
	Columns.Add(CreateMaxInclusiveSizeColumn());
	Columns.Add(CreateAverageInclusiveSizeColumn());

	Columns.Add(CreateTotalExclusiveSizeColumn());
	Columns.Add(CreateMaxExclusiveSizeColumn());
	//Columns.Add(CreateAverageExclusiveSizeColumn());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsViewColumnFactory::CreateNameColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsViewColumns::NameColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Name_ColumnName", "Name"));
	Column.SetTitleName(LOCTEXT("Name_ColumnTitle", "NetEvent or Group Name"));
	Column.SetDescription(LOCTEXT("Name_ColumnDesc", "Name of the timer or group"));

	Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered |
					ETableColumnFlags::IsHierarchy);

	Column.SetHorizontalAlignment(HAlign_Left);
	Column.SetInitialWidth(206.0f);
	Column.SetMinWidth(42.0f);

	Column.SetDataType(ETableCellDataType::Text);

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FDisplayNameValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByName>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsViewColumnFactory::CreateTypeColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsViewColumns::TypeColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Type_ColumnName", "Type"));
	Column.SetTitleName(LOCTEXT("Type_ColumnTitle", "Type"));
	Column.SetDescription(LOCTEXT("Type_ColumnDesc", "Type of timer or group"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Left);
	Column.SetInitialWidth(60.0f);

	Column.SetDataType(ETableCellDataType::Text);

	class FNetEventTypeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FNetStatsViewColumns::TypeColumnID);
			const FNetEventNode& NetEventNode = static_cast<const FNetEventNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(NetEventNodeTypeHelper::ToText(NetEventNode.GetType())));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FNetEventTypeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByTextValue>(ColumnRef);
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FNetEventNodeSortingByEventType>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsViewColumnFactory::CreateLevelColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsViewColumns::LevelColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Level_ColumnName", "Level"));
	Column.SetTitleName(LOCTEXT("Level_ColumnTitle", "Level"));
	Column.SetDescription(LOCTEXT("Level_ColumnDesc", "Level of net event instances"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(40.0f);

	Column.SetDataType(ETableCellDataType::Int64);

	class FLevelValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FNetStatsViewColumns::LevelColumnID);
			const FNetEventNode& NetEventNode = static_cast<const FNetEventNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(NetEventNode.GetLevel())));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FLevelValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsViewColumnFactory::CreateInstanceCountColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsViewColumns::InstanceCountColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("InstanceCount_ColumnName", "Count"));
	Column.SetTitleName(LOCTEXT("InstanceCount_ColumnTitle", "Instance Count"));
	Column.SetDescription(LOCTEXT("InstanceCount_ColumnDesc", "Number of net event instances"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(60.0f);

	Column.SetDataType(ETableCellDataType::Int64);

	class FInstanceCountValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FNetStatsViewColumns::InstanceCountColumnID);
			const FNetEventNode& NetEventNode = static_cast<const FNetEventNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(NetEventNode.GetAggregatedStats().InstanceCount)));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FInstanceCountValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FNetEventNodeSortingByInstanceCount>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Inclusive  Columns
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsViewColumnFactory::CreateTotalInclusiveSizeColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsViewColumns::TotalInclusiveSizeColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("TotalInclusive_ColumnName", "Incl"));
	Column.SetTitleName(LOCTEXT("TotalInclusive_ColumnTitle", "Total Inclusive Size"));
	Column.SetDescription(LOCTEXT("TotalInclusive_ColumnDesc", "Total inclusive size (bits) of selected net event instances"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(TotalSizeColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Int64);
	Column.SetAggregation(ETableColumnAggregation::Sum);

	class FTotalInclusiveValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FNetStatsViewColumns::TotalInclusiveSizeColumnID);
			const FNetEventNode& NetEventNode = static_cast<const FNetEventNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(NetEventNode.GetAggregatedStats().TotalInclusive)));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTotalInclusiveValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FNetEventNodeSortingByTotalInclusiveSize>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsViewColumnFactory::CreateMaxInclusiveSizeColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsViewColumns::MaxInclusiveSizeColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("MaxInclusive_ColumnName", "I.Max"));
	Column.SetTitleName(LOCTEXT("MaxInclusive_ColumnTitle", "Max Inclusive Size"));
	Column.SetDescription(LOCTEXT("MaxInclusive_ColumnDesc", "Maximum inclusive size (bits) of selected net event instances"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(SizeColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Int64);

	class FMaxInclusiveValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FNetStatsViewColumns::MaxInclusiveSizeColumnID);
			const FNetEventNode& NetEventNode = static_cast<const FNetEventNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(NetEventNode.GetAggregatedStats().MaxInclusive)));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMaxInclusiveValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsViewColumnFactory::CreateAverageInclusiveSizeColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsViewColumns::AverageInclusiveSizeColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("AvgInclusive_ColumnName", "I.Avg"));
	Column.SetTitleName(LOCTEXT("AvgInclusive_ColumnTitle", "Average Inclusive Size"));
	Column.SetDescription(LOCTEXT("AvgInclusive_ColumnDesc", "Average inclusive size (bits) of selected net event instances"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(SizeColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Int64);

	class FAverageInclusiveValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FNetStatsViewColumns::AverageInclusiveSizeColumnID);
			const FNetEventNode& NetEventNode = static_cast<const FNetEventNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(NetEventNode.GetAggregatedStats().AverageInclusive)));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAverageInclusiveValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Exclusive  Columns
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsViewColumnFactory::CreateTotalExclusiveSizeColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsViewColumns::TotalExclusiveSizeColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("TotalExclusive_ColumnName", "Excl"));
	Column.SetTitleName(LOCTEXT("TotalExclusive_ColumnTitle", "Total Exclusive Size"));
	Column.SetDescription(LOCTEXT("TotalExclusive_ColumnDesc", "Total exclusive size (bits) of selected net event instances"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(TotalSizeColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Int64);
	Column.SetAggregation(ETableColumnAggregation::Sum);

	class FTotalExclusiveValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FNetStatsViewColumns::TotalExclusiveSizeColumnID);
			const FNetEventNode& NetEventNode = static_cast<const FNetEventNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(NetEventNode.GetAggregatedStats().TotalExclusive)));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTotalExclusiveValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value(ColumnRef));
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FNetEventNodeSortingByTotalExclusiveSize>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsViewColumnFactory::CreateMaxExclusiveSizeColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsViewColumns::MaxExclusiveSizeColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("MaxExclusive_ColumnName", "E.Max"));
	Column.SetTitleName(LOCTEXT("MaxExclusive_ColumnTitle", "Max Exclusive Size"));
	Column.SetDescription(LOCTEXT("MaxExclusive_ColumnDesc", "Maximum exclusive size (bits) of selected net event instances"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(SizeColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Int64);

	class FMaxExclusiveValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FNetStatsViewColumns::MaxExclusiveSizeColumnID);
			const FNetEventNode& NetEventNode = static_cast<const FNetEventNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(NetEventNode.GetAggregatedStats().MaxExclusive)));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMaxExclusiveValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FNetStatsViewColumnFactory::CreateAverageExclusiveSizeColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FNetStatsViewColumns::AverageExclusiveSizeColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("AvgExclusive_ColumnName", "E.Avg"));
	Column.SetTitleName(LOCTEXT("AvgExclusive_ColumnTitle", "Average Exclusive Size)"));
	Column.SetDescription(LOCTEXT("AvgExclusive_ColumnDesc", "Average exclusive size (bits) of selected net event instances"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(SizeColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Int64);

	class FAverageExclusiveValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FNetStatsViewColumns::AverageExclusiveSizeColumnID);
			const FNetEventNode& NetEventNode = static_cast<const FNetEventNode&>(Node);
			//return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(NetEventNode.GetAggregatedStats().AverageExclusive)));
			return TOptional<FTableCellValue>(FTableCellValue(int64(0)));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAverageExclusiveValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
