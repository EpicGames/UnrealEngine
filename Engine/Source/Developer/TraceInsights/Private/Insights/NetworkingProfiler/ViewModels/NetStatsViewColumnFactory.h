// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

namespace UE::Insights { class FTableColumn; }

namespace UE::Insights::NetworkingProfiler
{

// Column identifiers
struct FNetStatsViewColumns
{
	static const FName NameColumnID;
	static const FName TypeColumnID;
	static const FName LevelColumnID;
	static const FName InstanceCountColumnID;

	// Inclusive  columns
	static const FName TotalInclusiveSizeColumnID;
	static const FName MaxInclusiveSizeColumnID;
	static const FName AverageInclusiveSizeColumnID;

	// Exclusive  columns
	static const FName TotalExclusiveSizeColumnID;
	static const FName MaxExclusiveSizeColumnID;
	static const FName AverageExclusiveSizeColumnID;
};

struct FNetStatsViewColumnFactory
{
public:
	static void CreateNetStatsViewColumns(TArray<TSharedRef<FTableColumn>>& Columns);

	static TSharedRef<FTableColumn> CreateNameColumn();
	static TSharedRef<FTableColumn> CreateTypeColumn();
	static TSharedRef<FTableColumn> CreateLevelColumn();
	static TSharedRef<FTableColumn> CreateInstanceCountColumn();

	static TSharedRef<FTableColumn> CreateTotalInclusiveSizeColumn();
	static TSharedRef<FTableColumn> CreateMaxInclusiveSizeColumn();
	static TSharedRef<FTableColumn> CreateAverageInclusiveSizeColumn();

	static TSharedRef<FTableColumn> CreateTotalExclusiveSizeColumn();
	static TSharedRef<FTableColumn> CreateMaxExclusiveSizeColumn();
	static TSharedRef<FTableColumn> CreateAverageExclusiveSizeColumn();

private:
	static constexpr float TotalSizeColumnInitialWidth = 60.0f;
	static constexpr float SizeColumnInitialWidth = 50.0f;
};

} // namespace UE::Insights::NetworkingProfiler
