// Copyright Epic Games, Inc. All Rights Reserved.

#include "PacketContentViewDrawHelper.h"

#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/StringBuilder.h"
#include "Rendering/DrawElements.h"
#include "Styling/AppStyle.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/ViewModels/PacketContentViewport.h"
#include "Insights/ViewModels/DrawHelpers.h"

#include <limits>

#define INSIGHTS_USE_LEGACY_BORDER 0

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FPacketContentViewDrawStateBuilder
////////////////////////////////////////////////////////////////////////////////////////////////////

FPacketContentViewDrawStateBuilder::FPacketContentViewDrawStateBuilder(FPacketContentViewDrawState& InDrawState, const FPacketContentViewport& InViewport, float InFontScale)
	: DrawState(InDrawState)
	, Viewport(InViewport)
	, MaxDepth(-1)
	, LastEventX2()
	, LastBox()
	, EventFont(FAppStyle::Get().GetFontStyle("SmallFont"))
	, FontScale(InFontScale)
{
	DrawState.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FPacketContentViewDrawStateBuilder::AddEvent(const TraceServices::FNetProfilerContentEvent& Event, const TCHAR* EventName, uint64 NetId)
{
	DrawState.Events.AddUninitialized();
	FNetworkPacketEvent& PacketEvent = DrawState.Events.Last();
	PacketEvent.EventTypeIndex = Event.EventTypeIndex;
	PacketEvent.ObjectInstanceIndex = Event.ObjectInstanceIndex;
	PacketEvent.NetId = NetId;
	PacketEvent.BitOffset = Event.StartPos;
	PacketEvent.BitSize = static_cast<uint32>(Event.EndPos - Event.StartPos);
	PacketEvent.Level = Event.Level;

	const FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();

	float EventX1 = ViewportX.GetRoundedOffsetForValue(static_cast<double>(Event.StartPos));
	if (EventX1 > Viewport.GetWidth())
	{
		return;
	}

	//float EventX2 = EventSize == 0 ?
	//	EventX1 + 1.0f :
	//	ViewportX.GetScale() < 1.0f ?
	//	ViewportX.GetRoundedOffsetForValue(static_cast<double>(Event.EndPos - 1)) + 1.0f : // x2 = pos of last inclusive bit + 1px
	//	ViewportX.GetRoundedOffsetForValue(static_cast<double>(Event.EndPos));             // x2 = pos of first exclusive bit
	float EventX2 = ViewportX.GetRoundedOffsetForValue(static_cast<double>(Event.EndPos));
	if (EventX2 < 0)
	{
		return;
	}

	// Events are displayed with minimum 1px (including empty ones).
	if (EventX1 == EventX2)
	{
		EventX2 = EventX1 + 1.0f;
	}

	const int32 Depth = static_cast<int32>(Event.Level);
	if (Depth > MaxDepth)
	{
		MaxDepth = Depth;
	}

	// Ensure we have enough slots in array. See LastBox[Depth] usage.
	if (LastBox.Num() <= Depth)
	{
		LastBox.AddDefaulted(Depth + 1 - LastBox.Num());
	}

	// Ensure we have enough slots in array. See LastEventX2[Depth] usage.
	while (LastEventX2.Num() <= Depth)
	{
		LastEventX2.Add(-1.0f);
	}

	// Limit event width on the viewport's left side.
	// This also makes the text to be displayed in viewport, for very long events.
	const float MinX = -2.0f; // -2 allows event border to remain outside screen
	if (EventX1 < MinX)
	{
		EventX1 = MinX;
	}

	// Limit event width on the viewport's right side.
	const float MaxX = Viewport.GetWidth() + 2.0f; // +2 allows event border to remain outside screen
	if (EventX2 > MaxX)
	{
		EventX2 = MaxX;
	}

	const float EventW = EventX2 - EventX1;

	// Optimization...
	if (EventW == 1.0f && EventX1 == LastEventX2[Depth] - 1.0f)
	{
		// Do no draw 1 pixel event if the last event was ended on that pixel.
		return;
	}

	//////////////////////////////////////////////////
	// Coloring

	const uint32 Color = (Event.NameIndex + 0x2c2c57ed) * 0x2c2c57ed;
	const FLinearColor EventColorFill(((Color >> 16) & 0xFF) / 255.0f, ((Color >> 8) & 0xFF) / 255.0f, (Color & 0xFF) / 255.0f, 1.0f);
	constexpr float BorderColorFactor = 1.25f; // brighter border

	//////////////////////////////////////////////////

	// Save X2, for current depth.
	LastEventX2[Depth] = EventX2;

	if (EventW > 2.0f)
	{
		FBoxData& Box = LastBox[Depth];
		if (Box.X1 < Box.X2)
		{
			FlushBox(Box, Depth);
			Box.Reset();
		}

		// Fill inside of the event box.
		DrawState.InsideBoxes.AddUninitialized();
		FPacketContentViewDrawState::FBoxPrimitive& InsideBox = DrawState.InsideBoxes.Last();
		InsideBox.Depth = Depth;
		InsideBox.X = EventX1 + 1.0f;
		InsideBox.W = EventW - 2.0f;
		InsideBox.Color = EventColorFill;

		// Add border around the event box.
		DrawState.Borders.AddUninitialized();
		FPacketContentViewDrawState::FBoxPrimitive& BorderBox = DrawState.Borders.Last();
		BorderBox.Depth = Depth;
		BorderBox.X = EventX1;
		BorderBox.W = EventW;
		BorderBox.Color = FLinearColor(EventColorFill.R * BorderColorFactor, EventColorFill.G * BorderColorFactor, EventColorFill.B * BorderColorFactor, EventColorFill.A);
	}
	else // 1px or 2px boxes
	{
		FBoxData& Box = LastBox[Depth];

		// Check if we can merge this box with previous one, if any.
		// Note: We are assuming events are processed in sorted order by X1.
		if (Color == Box.Color && // same color
			EventX1 <= Box.X2) // overlapping or adjacent
		{
			// Merge it with previous box.
			Box.X2 = EventX2;
			DrawState.NumMergedBoxes++;
		}
		else
		{
			// Flush previous box, if any.
			if (Box.X1 < Box.X2)
			{
				FlushBox(Box, Depth);
			}

			// Start new "merge box".
			Box.X1 = EventX1;
			Box.X2 = EventX2;
			Box.Color = Color;
			Box.LinearColor = FLinearColor(EventColorFill.R * BorderColorFactor, EventColorFill.G * BorderColorFactor, EventColorFill.B * BorderColorFactor, EventColorFill.A);
		}
	}

	// Draw the name of the event.
	if (EventW > 8.0f)
	{
		TStringBuilder<512> Builder;

		Builder.Appendf(TEXT("%s"), EventName ? EventName : TEXT("?"));
		const float MinWidth = static_cast<float>(Builder.Len()) * 2.0f + 48.0f;
		if (EventW > MinWidth)
		{
			const TraceServices::FNetProfilerBunchInfo& Info = Event.BunchInfo;
			if (Info.bIsValid)
			{
				Builder.Appendf(TEXT(" ChannelId:%u"), Info.ChannelIndex);

				if (Info.bOpen && Info.bClose)
				{
					Builder.Append(TEXT(" | OpenTemp"));
				}
				else if (Event.BunchInfo.bOpen)
				{
					Builder.Append(TEXT(" | Open"));
				}
				else if (Event.BunchInfo.bClose)
				{
					Builder.Appendf(TEXT(" | Close: %s"), LexToString(TraceServices::ENetProfilerChannelCloseReason(Info.ChannelCloseReason)));
				}
				if (Event.BunchInfo.bReliable)
				{
					Builder.Appendf(TEXT(" | Reliable: ChSeq: %u"), Info.Seq);
				}
				if (Event.BunchInfo.bPartial)
				{
					Builder.Appendf(TEXT(" | Partial%s%s"), Info.bPartialInitial ? TEXT("Initial") : TEXT(""), Info.bPartialFinal ? TEXT("Final") : TEXT(""));
				}
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (Event.BunchInfo.bIsReplicationPaused)
				{
					Builder.Append(TEXT(" | ReplicationPaused"));
				}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				if (Event.BunchInfo.bHasMustBeMappedGUIDs)
				{
					Builder.Append(TEXT(" | HasMustBeMappedGUIDs"));
				}
				if (Event.BunchInfo.bHasPackageMapExports)
				{
					Builder.Append(TEXT(" | HasPackageMapExports"));
				}
				if (Event.BunchInfo.bPartialCustomExportsFinal)
				{
					Builder.Append(TEXT(" | HasCustomExports"));
				}

				Builder.Append(TEXT(", "));
			}

			if (Event.ObjectInstanceIndex != 0)
			{
				Builder.Appendf(TEXT(" (NetId:%" UINT64_FMT ", "), NetId);
			}
			else
			{
				Builder.Append(TEXT(" ("));
			}
			const uint32 EventSize = static_cast<uint32>(Event.EndPos - Event.StartPos);
			if (EventSize == 1)
			{
				Builder.Append(TEXT("1 bit)"));
			}
			else
			{
				Builder.Appendf(TEXT("%u bits)"), EventSize);
			}
		}

		const FString Name(Builder);

		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const int32 HorizontalOffset = FMath::RoundToInt((EventW - 2.0f) * FontScale);
		const int32 LastWholeCharacterIndex = FontMeasureService->FindLastWholeCharacterIndexBeforeOffset(Name, EventFont, HorizontalOffset, FontScale);

		if (LastWholeCharacterIndex >= 0)
		{
			// Grey threshold is shifted toward black (0.4 instead of 0.5 in test below) due to "area rule":
			// a large gray surface (background of an event in this case) is perceived lighter than a smaller area (text pixels).
			// Ref: https://books.google.ro/books?id=0pVr7dhmdWYC
			const bool bIsDarkColor = (EventColorFill.GetLuminance() < 0.4f);

			DrawState.Texts.AddDefaulted();
			FPacketContentViewDrawState::FTextPrimitive& DrawText = DrawState.Texts.Last();
			DrawText.Depth = Depth;
			DrawText.X = EventX1 + 2.0f;
			DrawText.Text = Name.Left(LastWholeCharacterIndex + 1);
			DrawText.bWhite = bIsDarkColor;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FPacketContentViewDrawStateBuilder::FlushBox(const FBoxData& Box, const int32 Depth)
{
	DrawState.Boxes.AddUninitialized();
	FPacketContentViewDrawState::FBoxPrimitive& DrawBox = DrawState.Boxes.Last();
	DrawBox.Depth = Depth;
	DrawBox.X = Box.X1;
	DrawBox.W = Box.X2 - Box.X1;
	DrawBox.Color = Box.LinearColor;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FPacketContentViewDrawStateBuilder::Flush()
{
	// Flush merged boxes.
	for (int32 Depth = 0; Depth <= MaxDepth; ++Depth)
	{
		const FBoxData& Box = LastBox[Depth];
		if (Box.X1 < Box.X2)
		{
			FlushBox(Box, Depth);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FPacketContentViewDrawHelper
////////////////////////////////////////////////////////////////////////////////////////////////////

FPacketContentViewDrawHelper::FPacketContentViewDrawHelper(const FDrawContext& InDrawContext, const FPacketContentViewport& InViewport)
	: DrawContext(InDrawContext)
	, Viewport(InViewport)
	, WhiteBrush(FAppStyle::Get().GetBrush("WhiteBrush"))
	, EventBorderBrush(FInsightsStyle::Get().GetBrush("EventBorder"))
	, HoveredEventBorderBrush(FInsightsStyle::Get().GetBrush("HoveredEventBorder"))
	, SelectedEventBorderBrush(FInsightsStyle::Get().GetBrush("SelectedEventBorder"))
	, EventFont(FAppStyle::Get().GetFontStyle("SmallFont"))
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FPacketContentViewDrawHelper::DrawBackground() const
{
	const FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();

	const float X0 = 0.0f;
	const float X1 = ViewportX.GetMinPos() - ViewportX.GetPos();
	const float X2 = ViewportX.GetMaxPos() - ViewportX.GetPos();
	const float X3 = FMath::CeilToFloat(Viewport.GetWidth());

	const float Y = 0.0f;
	const float H = FMath::CeilToFloat(Viewport.GetHeight());

	FDrawHelpers::DrawBackground(DrawContext, WhiteBrush, X0, X1, X2, X3, Y, H);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FPacketContentViewDrawHelper::GetColorByType(int32 Type)
{
	constexpr float Alpha = 1.0f;
	switch (Type)
	{
	case 0:
		return FLinearColor(0.75f, 0.25f, 0.25f, Alpha);

	case 1:
		return FLinearColor(0.25f, 0.75f, 0.25f, Alpha);

	case 2:
		return FLinearColor(0.25f, 0.25f, 0.75f, Alpha);

	case 3:
		return FLinearColor(0.75f, 0.75f, 0.25f, Alpha);

	case 4:
		return FLinearColor(0.25f, 0.75f, 0.75f, Alpha);

	case 5:
		return FLinearColor(0.75f, 0.25f, 0.75f, Alpha);

	//default:
	//	return FLinearColor(0.75f, 0.75f, 0.75f, Alpha);
	}

	uint32 Hash = (static_cast<uint32>(Type) + 0x2c2c57ed) * 0x2c2c57ed;
	return FLinearColor(((Hash >> 16) & 0xFF) / 255.0f,
						((Hash >>  8) & 0xFF) / 255.0f,
						((Hash      ) & 0xFF) / 255.0f,
						1.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FPacketContentViewDrawHelper::Draw(const FPacketContentViewDrawState& DrawState, const float Opacity) const
{
	// Draw filled boxes (merged borders).
	//if (Viewport.GetEventHeight() > 0.0f)
	{
		const float BoxY0 = Viewport.GetTopEventPosY();
		const float BoxDY = Viewport.GetEventHeight() + Viewport.GetEventDY();
		const float EventFillH = Viewport.GetEventHeight();
		for (const FPacketContentViewDrawState::FBoxPrimitive& Box : DrawState.Boxes)
		{
			DrawContext.DrawBox(Box.X, BoxY0 + BoxDY * (float)Box.Depth, Box.W, EventFillH, WhiteBrush, Box.Color.CopyWithNewOpacity(Opacity));
		}
		DrawContext.LayerId++;
	}

	// Draw filled boxes (event inside area).
	if (Viewport.GetEventHeight() > 2.0f)
	{
		const float BoxY0 = Viewport.GetTopEventPosY() + 1.0f;
		const float BoxDY = Viewport.GetEventHeight() + Viewport.GetEventDY();
		const float EventFillH = Viewport.GetEventHeight() - 2.0f;
		for (const FPacketContentViewDrawState::FBoxPrimitive& Box : DrawState.InsideBoxes)
		{
			DrawContext.DrawBox(Box.X, BoxY0 + BoxDY * (float)Box.Depth, Box.W, EventFillH, WhiteBrush, Box.Color.CopyWithNewOpacity(Opacity));
		}
		DrawContext.LayerId++;
	}

	// Draw borders.
	//if (Viewport.GetEventHeight() > 0.0f)
	{
#if INSIGHTS_USE_LEGACY_BORDER
		const float BoxY0 = Viewport.GetTopEventPosY();
		const float BoxDY = Viewport.GetEventHeight() + Viewport.GetEventDY();
		const float EventBorderH = Viewport.GetEventHeight();
		for (const FPacketContentViewDrawState::FBoxPrimitive& Box : DrawState.Borders)
		{
			DrawContext.DrawBox(Box.X, BoxY0 + BoxDY * (float)Box.Depth, Box.W, EventBorderH, EventBorderBrush, Box.Color.CopyWithNewOpacity(Opacity));
		}
		DrawContext.LayerId++;
#else
		if (Opacity == 1.0f)
		{
			const int32 EventBorderLayerId = DrawContext.LayerId - 2; // draw under the filled boxes
			const float BoxY0 = Viewport.GetTopEventPosY();
			const float BoxDY = Viewport.GetEventHeight() + Viewport.GetEventDY();
			const float EventBorderH = Viewport.GetEventHeight();
			for (const FPacketContentViewDrawState::FBoxPrimitive& Box : DrawState.Borders)
			{
				DrawContext.DrawBox(EventBorderLayerId, Box.X, BoxY0 + BoxDY * (float)Box.Depth, Box.W, EventBorderH, WhiteBrush, Box.Color);
			}
		}
		else
		{
			const float BoxY0 = Viewport.GetTopEventPosY();
			const float BoxDY = Viewport.GetEventHeight() + Viewport.GetEventDY();
			const float EventBorderH = Viewport.GetEventHeight();
			for (const FPacketContentViewDrawState::FBoxPrimitive& Box : DrawState.Borders)
			{
				FSlateRoundedBoxBrush Brush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), 0.0f, Box.Color.CopyWithNewOpacity(Opacity), 1.0f);
				DrawContext.DrawBox(Box.X, BoxY0 + BoxDY * (float)Box.Depth, Box.W, EventBorderH, &Brush, FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
			}
			DrawContext.LayerId++;
		}
#endif
	}

	// Draw texts.
	if (Viewport.GetEventHeight() > 10.0f)
	{
		const float TextY0 = Viewport.GetTopEventPosY() + 1.0f;
		const float TextDY = Viewport.GetEventHeight() + Viewport.GetEventDY();
		const FLinearColor WhiteColor(1.0f, 1.0f, 1.0f, Opacity);
		const FLinearColor BlackColor(0.0f, 0.0f, 0.0f, Opacity);
		for (const FPacketContentViewDrawState::FTextPrimitive& Text : DrawState.Texts)
		{
			DrawContext.DrawText(Text.X, TextY0 + TextDY * (float)Text.Depth, Text.Text, EventFont, Text.bWhite ? WhiteColor : BlackColor);
		}
		DrawContext.LayerId++;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FPacketContentViewDrawHelper::DrawEventHighlight(const FNetworkPacketEvent& Event, EHighlightMode Mode) const
{
	const FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();

	float EventX1 = ViewportX.GetRoundedOffsetForValue(static_cast<double>(Event.BitOffset));
	if (EventX1 > Viewport.GetWidth())
	{
		return;
	}

	float EventX2 = ViewportX.GetRoundedOffsetForValue(static_cast<double>(Event.BitOffset + Event.BitSize));
	if (EventX2 < 0)
	{
		return;
	}

	// Events are displayed with minimum 1px (including empty ones).
	if (EventX1 == EventX2)
	{
		EventX2 = EventX1 + 1.0f;
	}

	// Limit event width on the viewport's left side.
	const float MinX = -2.0f; // -2 allows event border to remain outside screen
	if (EventX1 < MinX)
	{
		EventX1 = MinX;
	}

	// Limit event width on the viewport's right side.
	const float MaxX = Viewport.GetWidth() + 2.0f; // +2 allows event border to remain outside screen
	if (EventX2 > MaxX)
	{
		EventX2 = MaxX;
	}

	const float EventW = EventX2 - EventX1;
	const float EventY = Viewport.GetTopEventPosY() + (Viewport.GetEventHeight() + Viewport.GetEventDY()) * static_cast<float>(Event.Level);

#if INSIGHTS_USE_LEGACY_BORDER
	constexpr float BorderOffset = 1.0f;
#else
	constexpr float BorderOffset = 2.0f;
#endif

	if (Mode == EHighlightMode::Hovered)
	{
		const FLinearColor Color(1.0f, 1.0f, 0.0f, 1.0f); // yellow

		// Draw border around the timing event box.
#if INSIGHTS_USE_LEGACY_BORDER
		DrawContext.DrawBox(EventX1 - BorderOffset, EventY - BorderOffset, EventW + 2 * BorderOffset, Viewport.GetEventHeight() + 2 * BorderOffset, HoveredEventBorderBrush, Color);
#else
		FSlateRoundedBoxBrush Brush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), 2.0f, Color, 2.0f);
		DrawContext.DrawBox(EventX1 - BorderOffset, EventY - BorderOffset, EventW + 2 * BorderOffset, Viewport.GetEventHeight() + 2 * BorderOffset, &Brush, FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
#endif
	}
	else // EHighlightMode::Selected or EHighlightMode::SelectedAndHovered
	{
		// Animate color from white (if selected and hovered) or yellow (if only selected) to black, using a squared sine function.
		const double Time = static_cast<double>(FPlatformTime::Cycles64()) * FPlatformTime::GetSecondsPerCycle64();
		float Hue = static_cast<float>(FMath::Sin(2.0 * Time));
		Hue = Hue * Hue; // squared, to ensure only positive [0 - 1] values
		const float Blue = (Mode == EHighlightMode::SelectedAndHovered) ? 0.0f : Hue;
		const FLinearColor Color(Hue, Hue, Blue, 1.0f);

		// Draw border around the timing event box.
#if INSIGHTS_USE_LEGACY_BORDER
		DrawContext.DrawBox(EventX1 - BorderOffset, EventY - BorderOffset, EventW + 2 * BorderOffset, Viewport.GetEventHeight() + 2 * BorderOffset, SelectedEventBorderBrush, Color);
#else
		FSlateRoundedBoxBrush Brush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), 2.0f, Color, 2.0f);
		DrawContext.DrawBox(EventX1 - BorderOffset, EventY - BorderOffset, EventW + 2 * BorderOffset, Viewport.GetEventHeight() + 2 * BorderOffset, &Brush, FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
#endif
	}
	DrawContext.LayerId++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef INSIGHTS_USE_LEGACY_BORDER
