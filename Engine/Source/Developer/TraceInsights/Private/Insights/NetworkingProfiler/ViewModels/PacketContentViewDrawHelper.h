// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Fonts/SlateFontInfo.h"
#include "Styling/WidgetStyle.h"

// TraceServices
#include "TraceServices/Model/NetProfiler.h"

struct FSlateBrush;

namespace UE::Insights { class FDrawContext; }

namespace UE::Insights::NetworkingProfiler
{

class FPacketContentViewport;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FNetworkPacketEvent
{
	uint32 EventTypeIndex;
	uint32 ObjectInstanceIndex;
	uint64 NetId;
	uint32 BitOffset;
	uint32 BitSize;
	uint32 Level;

	FNetworkPacketEvent()
		: EventTypeIndex(0)
		, ObjectInstanceIndex(0)
		, NetId(0)
		, BitOffset(0)
		, BitSize(0)
		, Level(0)
	{}

	FNetworkPacketEvent(uint32 InEventTypeIndex, uint32 InObjectInstanceIndex, uint64 InNetId, uint32 InBitOffset, uint32 InBitSize, uint32 InLevel)
		: EventTypeIndex(InEventTypeIndex)
		, ObjectInstanceIndex(InObjectInstanceIndex)
		, NetId(InNetId)
		, BitOffset(InBitOffset)
		, BitSize(InBitSize)
		, Level(InLevel)
	{}

	FNetworkPacketEvent(const FNetworkPacketEvent& Other)
		: EventTypeIndex(Other.EventTypeIndex)
		, ObjectInstanceIndex(Other.ObjectInstanceIndex)
		, NetId(Other.NetId)
		, BitOffset(Other.BitOffset)
		, BitSize(Other.BitSize)
		, Level(Other.Level)
	{
	}

	FNetworkPacketEvent& operator=(const FNetworkPacketEvent& Other)
	{
		EventTypeIndex = Other.EventTypeIndex;
		ObjectInstanceIndex = Other.ObjectInstanceIndex;
		NetId = Other.NetId;
		BitOffset = Other.BitOffset;
		BitSize = Other.BitSize;
		Level = Other.Level;
		return *this;
	}

	bool Equals(const FNetworkPacketEvent& Other) const
	{
		return EventTypeIndex == Other.EventTypeIndex
			&& ObjectInstanceIndex == Other.ObjectInstanceIndex
			&& NetId == Other.NetId
			&& BitOffset == Other.BitOffset
			&& BitSize == Other.BitSize
			&& Level == Other.Level;
	}

	static bool AreEquals(const FNetworkPacketEvent& A, const FNetworkPacketEvent& B)
	{
		return A.Equals(B);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FPacketContentViewDrawState
{
	struct FBoxPrimitive
	{
		int32 Depth;
		float X;
		float W;
		FLinearColor Color;
	};

	struct FTextPrimitive
	{
		int32 Depth;
		float X;
		FString Text;
		bool bWhite;
	};

	FPacketContentViewDrawState()
		: Events()
		, Boxes()
		, InsideBoxes()
		, Borders()
		, Texts()
		, NumLanes(0)
		, NumMergedBoxes(0)
	{
	}

	void Reset()
	{
		Events.Reset();
		Boxes.Reset();
		InsideBoxes.Reset();
		Borders.Reset();
		Texts.Reset();
		NumLanes = 0;
		NumMergedBoxes = 0;
	}

	int32 GetNumLanes() const { return NumLanes; }

	int32 GetNumEvents() const { return Events.Num(); }
	int32 GetNumMergedBoxes() const { return NumMergedBoxes; }
	int32 GetTotalNumBoxes() const { return Boxes.Num() + InsideBoxes.Num(); }

	TArray<FNetworkPacketEvent> Events;
	TArray<FBoxPrimitive> Boxes;
	TArray<FBoxPrimitive> InsideBoxes;
	TArray<FBoxPrimitive> Borders;
	TArray<FTextPrimitive> Texts;

	int32 NumLanes;

	// Debug stats.
	int32 NumMergedBoxes;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FPacketContentViewDrawStateBuilder
{
private:
	struct FBoxData
	{
		float X1;
		float X2;
		uint32 Color;
		FLinearColor LinearColor;

		FBoxData() : X1(0.0f), X2(0.0f), Color(0) {}
		void Reset() { X1 = 0.0f; X2 = 0.0f; Color = 0; }
	};

public:
	explicit FPacketContentViewDrawStateBuilder(FPacketContentViewDrawState& InState, const FPacketContentViewport& InViewport, float InFontScale);

	/**
	 * Non-copyable
	 */
	FPacketContentViewDrawStateBuilder(const FPacketContentViewDrawStateBuilder&) = delete;
	FPacketContentViewDrawStateBuilder& operator=(const FPacketContentViewDrawStateBuilder&) = delete;

	void AddEvent(const TraceServices::FNetProfilerContentEvent& Event, const TCHAR* Name, uint64 NetId);
	void Flush();

	int32 GetMaxDepth() const { return MaxDepth; }

private:
	void FlushBox(const FBoxData& Box, const int32 Depth);

private:
	FPacketContentViewDrawState& DrawState; // cached draw state to build
	const FPacketContentViewport& Viewport;

	int32 MaxDepth;

	TArray<float> LastEventX2; // X2 value for last event on each depth
	TArray<FBoxData> LastBox;

	const FSlateFontInfo EventFont;
	float FontScale;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FPacketContentViewDrawHelper
{
public:
	enum class EHighlightMode : uint32
	{
		Hovered = 1,
		Selected = 2,
		SelectedAndHovered = 3
	};

public:
	explicit FPacketContentViewDrawHelper(const FDrawContext& InDrawContext, const FPacketContentViewport& InViewport);

	/**
	 * Non-copyable
	 */
	FPacketContentViewDrawHelper(const FPacketContentViewDrawHelper&) = delete;
	FPacketContentViewDrawHelper& operator=(const FPacketContentViewDrawHelper&) = delete;

	const FSlateBrush* GetWhiteBrush() const { return WhiteBrush; }
	const FSlateFontInfo& GetEventFont() const { return EventFont; }

	void DrawBackground() const;
	void Draw(const FPacketContentViewDrawState& DrawState, const float Opacity = 1.0f) const;
	void DrawEventHighlight(const FNetworkPacketEvent& Event, EHighlightMode Mode) const;

	static FLinearColor GetColorByType(int32 Type);

private:
	const FDrawContext& DrawContext;
	const FPacketContentViewport& Viewport;

	const FSlateBrush* WhiteBrush;
	const FSlateBrush* EventBorderBrush;
	const FSlateBrush* HoveredEventBorderBrush;
	const FSlateBrush* SelectedEventBorderBrush;
	const FSlateFontInfo EventFont;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
