// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// TraceInsights
#include "Insights/ViewModels/AxisViewportDouble.h"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FPacketContentViewport
{
public:
	FPacketContentViewport()
	{
		Reset();
	}

	void Reset()
	{
		HorizontalAxisViewport.Reset();
		Height = 0.0f;
		TopEventPosY = 32.0f + 4.0f; // toolbar height + 4px
		EventHeight = 14.0f;
		EventDY = 2.0f;
	}

	const FAxisViewportDouble& GetHorizontalAxisViewport() const { return HorizontalAxisViewport; }
	FAxisViewportDouble& GetHorizontalAxisViewport() { return HorizontalAxisViewport; }

	float GetWidth() const { return HorizontalAxisViewport.GetSize(); }
	float GetHeight() const { return Height; }

	bool SetSize(const float InWidth, const float InHeight)
	{
		const bool bWidthChanged = HorizontalAxisViewport.SetSize(InWidth);
		if (bWidthChanged || Height != InHeight)
		{
			Height = InHeight;
			OnSizeChanged();
			return true;
		}
		return false;
	}

	float GetTopEventPosY() const { return TopEventPosY; }
	void SetTopEventPosY(float InTopEventPosY) { TopEventPosY = InTopEventPosY; }

	float GetEventHeight() const { return EventHeight; }
	void SetEventHeight(float InEventHeight) { EventHeight = InEventHeight; }

	float GetEventDY() const { return EventDY; }
	void SetEventDY(float InEventDY) { EventDY = InEventDY; }

private:
	void OnSizeChanged()
	{
	}

private:
	FAxisViewportDouble HorizontalAxisViewport;
	float Height;
	float TopEventPosY; // Y postion of events on level 0 (top lane)
	float EventHeight; // height of an event
	float EventDY; // vertical space between events in two adjacent lanes
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
