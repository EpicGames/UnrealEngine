// Copyright Epic Games, Inc. All Rights Reserved.

#include "PacketViewDrawHelper.h"

#include "Framework/Application/SlateApplication.h"
#include "Rendering/DrawElements.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/ViewModels/PacketViewport.h"
#include "Insights/NetworkingProfiler/ViewModels/PacketContentViewDrawHelper.h"
#include "Insights/ViewModels/DrawHelpers.h"

#include <limits>

#define INSIGHTS_USE_LEGACY_BORDER 0

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FNetworkPacketAggregatedSample
////////////////////////////////////////////////////////////////////////////////////////////////////

void FNetworkPacketAggregatedSample::AddPacket(const int32 PacketIndex, const TraceServices::FNetProfilerPacket& Packet)
{
	NumPackets++;

	const double TimeStamp = static_cast<double>(Packet.TimeStamp);
	if (TimeStamp < StartTime)
	{
		StartTime = TimeStamp;
	}
	if (TimeStamp > EndTime)
	{
		EndTime = TimeStamp;
	}

	if (Packet.TotalPacketSizeInBytes > LargestPacket.TotalSizeInBytes)
	{
		LargestPacket.Index = PacketIndex;
		LargestPacket.SequenceNumber = Packet.SequenceNumber;
		LargestPacket.ContentSizeInBits = Packet.ContentSizeInBits;
		LargestPacket.TotalSizeInBytes = Packet.TotalPacketSizeInBytes;
		LargestPacket.TimeStamp = TimeStamp;
		LargestPacket.Status = Packet.DeliveryStatus;
		LargestPacket.ConnectionState = Packet.ConnectionState;
	}

	switch (AggregatedStatus)
	{
	case TraceServices::ENetProfilerDeliveryStatus::Unknown:
		AggregatedStatus = Packet.DeliveryStatus;
		break;

	case TraceServices::ENetProfilerDeliveryStatus::Dropped:
		break;

	case TraceServices::ENetProfilerDeliveryStatus::Delivered:
		if (Packet.DeliveryStatus == TraceServices::ENetProfilerDeliveryStatus::Dropped)
		{
			AggregatedStatus = TraceServices::ENetProfilerDeliveryStatus::Dropped;
		}
		break;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FNetworkPacketSeriesBuilder
////////////////////////////////////////////////////////////////////////////////////////////////////

FNetworkPacketSeriesBuilder::FNetworkPacketSeriesBuilder(FNetworkPacketSeries& InSeries, const FPacketViewport& InViewport)
	: Series(InSeries)
	, Viewport(InViewport)
	, NumAddedPackets(0)
{
	SampleW = Viewport.GetSampleWidth();
	PacketsPerSample = Viewport.GetNumPacketsPerSample();
	NumSamples = FMath::Max(0, FMath::CeilToInt(Viewport.GetWidth() / SampleW));
	FirstPacketIndex = Viewport.GetFirstPacketIndex();

	Series.NumAggregatedPackets = 0;
	Series.Samples.Reset();
	Series.Samples.AddDefaulted(NumSamples);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FNetworkPacketAggregatedSample* FNetworkPacketSeriesBuilder::AddPacket(const int32 PacketIndex, const TraceServices::FNetProfilerPacket& Packet)
{
	NumAddedPackets++;

	int32 SampleIndex = (PacketIndex - FirstPacketIndex) / PacketsPerSample;
	if (SampleIndex >= 0 && SampleIndex < NumSamples)
	{
		FNetworkPacketAggregatedSample& Sample = Series.Samples[SampleIndex];
		Sample.AddPacket(PacketIndex, Packet);
		Series.NumAggregatedPackets++;
		return &Sample;
	}

	return nullptr;
}

void FNetworkPacketSeriesBuilder::SetHighlightEventTypeIndex(int32 EventTypeIndex)
{
	Series.HighlightEventTypeIndex = EventTypeIndex;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FPacketViewDrawHelper
////////////////////////////////////////////////////////////////////////////////////////////////////

FPacketViewDrawHelper::FPacketViewDrawHelper(const FDrawContext& InDrawContext, const FPacketViewport& InViewport)
	: DrawContext(InDrawContext)
	, Viewport(InViewport)
	, WhiteBrush(FAppStyle::Get().GetBrush("WhiteBrush"))
	//, EventBorderBrush(FInsightsStyle::Get().GetBrush("EventBorder"))
	, HoveredEventBorderBrush(FInsightsStyle::Get().GetBrush("HoveredEventBorder"))
	, SelectedEventBorderBrush(FInsightsStyle::Get().GetBrush("SelectedEventBorder"))
	, SelectionFont(FAppStyle::Get().GetFontStyle("SmallFont"))
	, NumPackets(0)
	, NumDrawSamples(0)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FPacketViewDrawHelper::DrawBackground() const
{
	const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

	const float X0 = 0.0f;
	const float X1 = ViewportX.GetMinPos() - ViewportX.GetPos();
	const float X2 = ViewportX.GetMaxPos() - ViewportX.GetPos();
	const float X3 = FMath::CeilToFloat(Viewport.GetWidth());

	const float Y = 0.0f;
	const float H = FMath::CeilToFloat(Viewport.GetHeight());

	FDrawHelpers::DrawBackground(DrawContext, WhiteBrush, X0, X1, X2, X3, Y, H);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FPacketViewDrawHelper::GetColorByStatus(TraceServices::ENetProfilerDeliveryStatus Status)
{
	constexpr float Alpha = 1.0f;
	switch (Status)
	{
	case TraceServices::ENetProfilerDeliveryStatus::Unknown:
		return FLinearColor(0.25f, 0.25f, 0.25f, Alpha);

	case TraceServices::ENetProfilerDeliveryStatus::Delivered:
		return FLinearColor(0.5f, 1.0f, 0.5f, Alpha);

	case TraceServices::ENetProfilerDeliveryStatus::Dropped:
		return FLinearColor(1.0f, 0.5f, 0.5f, Alpha);

	default:
		return FLinearColor(1.0f, 0.0f, 0.0f, Alpha);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FPacketViewDrawHelper::DrawCached(const FNetworkPacketSeries& Series) const
{
	if (Series.NumAggregatedPackets == 0)
	{
		return;
	}

	NumPackets += Series.NumAggregatedPackets;

	const float SampleW = Viewport.GetSampleWidth();
	const int32 NumSamples = Series.Samples.Num();

	const FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();

	const float ViewHeight = FMath::RoundToFloat(Viewport.GetHeight());
	const float BaselineY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(0.0));

	const FLinearColor ColorFilterMatch = FPacketContentViewDrawHelper::GetColorByType(Series.HighlightEventTypeIndex);

	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
	{
		const FNetworkPacketAggregatedSample& Sample = Series.Samples[SampleIndex];
		if (Sample.NumPackets == 0)
		{
			continue;
		}

		NumDrawSamples++;

		const float X = static_cast<float>(SampleIndex) * SampleW;

		//const float ValueY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(static_cast<double>(Sample.LargestPacket.ContentSizeInBits)));
		const float ValueY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(static_cast<double>(Sample.LargestPacket.TotalSizeInBytes * 8)));

		const float FilterMatchContentValueY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(static_cast<double>(Sample.FilterMatchHighlightSizeInBits)));
		const float FilterMatchContentH = FilterMatchContentValueY - BaselineY;
		const float FilterMatchContentY = ViewHeight - FilterMatchContentH;

		const float H = ValueY - BaselineY;
		const float Y = ViewHeight - H;

		FLinearColor ColorFill = GetColorByStatus(Sample.AggregatedStatus);
		if (!Sample.bAtLeastOnePacketMatchesFilter)
		{
			ColorFill.A = 0.1f;
		}
		const FLinearColor ColorBorder(ColorFill.R * 0.75f, ColorFill.G * 0.75f, ColorFill.B * 0.75f, ColorFill.A);

		if (SampleW > 2.0f)
		{
			DrawContext.DrawBox(X + 1.0f, Y + 1.0f, SampleW - 2.0f, H - 2.0f, WhiteBrush, ColorFill);

			if (Sample.FilterMatchHighlightSizeInBits > 0U)
			{
				DrawContext.DrawBox(X + 1.0f, FilterMatchContentY + 1.0f, SampleW - 2.0f, FilterMatchContentH - 2.0f, WhiteBrush, ColorFilterMatch);
			}

			// Draw border.
			DrawContext.DrawBox(X, Y, 1.0, H, WhiteBrush, ColorBorder);
			DrawContext.DrawBox(X + SampleW - 1.0f, Y, 1.0, H, WhiteBrush, ColorBorder);
			DrawContext.DrawBox(X + 1.0f, Y, SampleW - 2.0f, 1.0f, WhiteBrush, ColorBorder);
			DrawContext.DrawBox(X + 1.0f, Y + H - 1.0f, SampleW - 2.0f, 1.0f, WhiteBrush, ColorBorder);
		}
		else
		{
			DrawContext.DrawBox(X, Y, SampleW, H, WhiteBrush, ColorBorder);

			if (Sample.FilterMatchHighlightSizeInBits > 0U)
			{
				DrawContext.DrawBox(X, FilterMatchContentY, SampleW, FilterMatchContentH, WhiteBrush, ColorFilterMatch);
			}
		}
		if (Sample.AggregatedStatus == TraceServices::ENetProfilerDeliveryStatus::Dropped)
		{
			DrawContext.DrawBox(X + (SampleW/2.f), 0.f, 1.f, Y + H, WhiteBrush, FLinearColor::Red);
		}
	}

	DrawContext.LayerId++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FPacketViewDrawHelper::DrawSampleHighlight(const FNetworkPacketAggregatedSample& Sample, EHighlightMode Mode) const
{
	const float SampleW = Viewport.GetSampleWidth();
	const int32 PacketsPerSample = Viewport.GetNumPacketsPerSample();
	const int32 FirstPacketIndex = Viewport.GetFirstPacketIndex();
	const int32 SampleIndex = (Sample.LargestPacket.Index - FirstPacketIndex) / PacketsPerSample;
	const float X = static_cast<float>(SampleIndex) * SampleW;

	const FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();

	const float ViewHeight = FMath::RoundToFloat(Viewport.GetHeight());
	const float BaselineY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(0.0));

	//const double Value = static_cast<double>(Sample.LargestPacket.ContentSizeInBits);
	const double Value = static_cast<double>(Sample.LargestPacket.TotalSizeInBytes * 8);
	const float ValueY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(Value));

	const float H = ValueY - BaselineY;
	const float Y = ViewHeight - H;

#if INSIGHTS_USE_LEGACY_BORDER
	constexpr float BorderOffset = 1.0f;
#else
	constexpr float BorderOffset = 2.0f;
#endif

	if (Mode == EHighlightMode::Hovered)
	{
		const FLinearColor Color(1.0f, 1.0f, 0.0f, 1.0f); // yellow

		// Draw border around the hovered box.
#if INSIGHTS_USE_LEGACY_BORDER
		DrawContext.DrawBox(X - BorderOffset, Y - BorderOffset, SampleW + 2 * BorderOffset, H + 2 * BorderOffset, HoveredEventBorderBrush, Color);
#else
		FSlateRoundedBoxBrush Brush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), 2.0f, Color, 2.0f);
		DrawContext.DrawBox(X - BorderOffset, Y - BorderOffset, SampleW + 2 * BorderOffset, H + 2 * BorderOffset, &Brush, FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
#endif
	}
	else // EHighlightMode::Selected or EHighlightMode::SelectedAndHovered
	{
		// Animate color from white (if selected and hovered) or yellow (if only selected) to black, using a squared sine function.
		const double Time = static_cast<double>(FPlatformTime::Cycles64()) * FPlatformTime::GetSecondsPerCycle64();
		float Hue = static_cast<float>(FMath::Sin(2.0 * Time));
		Hue = Hue * Hue; // squared, to ensure only positive [0 - 1] values
		const float Blue = (Mode == EHighlightMode::SelectedAndHovered) ? 0.0f : Hue;
		const FLinearColor Color(Hue, Hue, Blue, 1.0f);

		// Draw border around the selected box.
#if INSIGHTS_USE_LEGACY_BORDER
		DrawContext.DrawBox(X - BorderOffset, Y - BorderOffset, SampleW + 2 * BorderOffset, H + 2 * BorderOffset, SelectedEventBorderBrush, Color);
#else
		FSlateRoundedBoxBrush Brush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), 2.0f, Color, 2.0f);
		DrawContext.DrawBox(X - BorderOffset, Y - BorderOffset, SampleW + 2 * BorderOffset, H + 2 * BorderOffset, &Brush, FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
#endif
	}
	DrawContext.LayerId++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FPacketViewDrawHelper::DrawSelection(int32 StartPacketIndex, int32 EndPacketIndex, double SelectionTimeSpan) const
{
	const float SampleW = Viewport.GetSampleWidth();
	const int32 PacketsPerSample = Viewport.GetNumPacketsPerSample();
	const int32 FirstPacketIndex = Viewport.GetFirstPacketIndex();

	const int32 StartSampleIndex = (StartPacketIndex - FirstPacketIndex) / PacketsPerSample;
	const float X1 = FMath::Max(-2.0f, static_cast<float>(StartSampleIndex) * SampleW);

	const int32 EndSampleIndex = (EndPacketIndex - FirstPacketIndex) / PacketsPerSample;
	const float X2 = FMath::Min(Viewport.GetWidth() + 2.0f, static_cast<float>(EndSampleIndex) * SampleW);

	const FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();

	const float ViewHeight = FMath::RoundToFloat(Viewport.GetHeight());
	const float Y = 0.0f;
	const float H = ViewHeight - Y;

	//// Animate color from white (if selected and hovered) or yellow (if only selected) to black, using a squared sine function.
	//const double Time = static_cast<double>(FPlatformTime::Cycles64()) * FPlatformTime::GetSecondsPerCycle64();
	//float Hue = static_cast<float>(FMath::Sin(2.0 * Time));
	//Hue = Hue * Hue; // squared, to ensure only positive [0 - 1] values
	//const FLinearColor Color(Hue, Hue, Hue, 1.0f);
	//
	//// Draw border around the selected box.
	//DrawContext.DrawBox(X1 - 1.0f, Y - 1.0f, X2 - X1 + 2.0f, H + 2.0f, SelectedEventBorderBrush, Color);

	// Fill the selected area.
	const int32 PacketCount = EndPacketIndex - StartPacketIndex;
	if (PacketCount > 1)
	{
		const float MinX = 0.0f;
		const float MaxX = Viewport.GetWidth();

		if (X1 <= MaxX && X2 >= MinX)
		{
			const FString Text = FString::Printf(TEXT("%d packets (%.3f s)"), PacketCount, (float)SelectionTimeSpan);
			FDrawHelpers::DrawSelection(DrawContext, MinX, MaxX, X1, X2, Y, H, 30.0f, Text, WhiteBrush, SelectionFont);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef INSIGHTS_USE_LEGACY_BORDER
