// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Fonts/SlateFontInfo.h"
#include "Styling/WidgetStyle.h"

// TraceServices
#include "TraceServices/Model/NetProfiler.h"

#include <limits>

struct FSlateBrush;

namespace UE::Insights { class FDrawContext; }

namespace UE::Insights::NetworkingProfiler
{

class FPacketViewport;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FNetworkPacket
{
	int32 Index;
	uint32 SequenceNumber;
	uint32 ContentSizeInBits;
	uint32 TotalSizeInBytes;
	double TimeStamp;
	TraceServices::ENetProfilerConnectionState ConnectionState;
	TraceServices::ENetProfilerDeliveryStatus Status;

	FNetworkPacket()
		: Index(-1)
		, SequenceNumber(0)
		, ContentSizeInBits(0)
		, TotalSizeInBytes(0)
		, TimeStamp(std::numeric_limits<double>::infinity())
		, ConnectionState(TraceServices::ENetProfilerConnectionState::USOCK_Invalid)
		, Status(TraceServices::ENetProfilerDeliveryStatus::Unknown)
	{}

	FNetworkPacket(const FNetworkPacket&) = default;
	FNetworkPacket& operator=(const FNetworkPacket&) = default;

	FNetworkPacket(FNetworkPacket&&) = default;
	FNetworkPacket& operator=(FNetworkPacket&&) = default;

	bool Equals(const FNetworkPacket& Other) const
	{
		return Index == Other.Index
			&& SequenceNumber == Other.SequenceNumber
			//&& ContentSizeInBits == Other.ContentSizeInBits
			&& TotalSizeInBytes == Other.TotalSizeInBytes
			/*&& TimeStamp == Other.TimeStamp
			&& Status == Other.Status*/;
	}

	static bool AreEquals(const FNetworkPacket& A, const FNetworkPacket& B)
	{
		return A.Equals(B);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FNetworkPacketAggregatedSample
{
	int32 NumPackets;

	double StartTime; // min TimeSent of all packets in this aggregated sample
	double EndTime; // max TimeSent of all packets in this aggregated sample

	/** Aggregated status for packets in this aggregated sample.
	 *    Unknown  --> all aggregated packets are unknown
	 *    Sent     --> all aggregated packets are sent; none is received or confirmed lost
	 *    Received --> at least one packet in the sample set is confirmed received and none are confirmed lost
	 *    Lost     --> at least one packet in the sample set is confirmed lost
	**/
	TraceServices::ENetProfilerDeliveryStatus AggregatedStatus;

	FNetworkPacket LargestPacket;

	bool bAtLeastOnePacketMatchesFilter;

	uint32 FilterMatchHighlightSizeInBits;

	FNetworkPacketAggregatedSample()
		: NumPackets(0)
		, StartTime(DBL_MAX)
		, EndTime(-DBL_MAX)
		, AggregatedStatus(TraceServices::ENetProfilerDeliveryStatus::Unknown)
		, LargestPacket()
		, bAtLeastOnePacketMatchesFilter(true)
		, FilterMatchHighlightSizeInBits(0)
	{}

	FNetworkPacketAggregatedSample(const FNetworkPacketAggregatedSample&) = default;
	FNetworkPacketAggregatedSample& operator=(const FNetworkPacketAggregatedSample&) = default;

	FNetworkPacketAggregatedSample(FNetworkPacketAggregatedSample&&) = default;
	FNetworkPacketAggregatedSample& operator=(FNetworkPacketAggregatedSample&&) = default;

	void AddPacket(const int32 PacketIndex, const TraceServices::FNetProfilerPacket& Packet);

	bool Equals(const FNetworkPacketAggregatedSample& Other) const
	{
		return NumPackets == Other.NumPackets
			&& StartTime == Other.StartTime
			&& EndTime == Other.EndTime
			&& AggregatedStatus == Other.AggregatedStatus
			&& LargestPacket.Equals(Other.LargestPacket);
	}

	static bool AreEquals(const FNetworkPacketAggregatedSample& A, const FNetworkPacketAggregatedSample& B)
	{
		return A.Equals(B);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FNetworkPacketSeries
{
	int32 NumAggregatedPackets; // total number of packets aggregated in samples; i.e. sum of all Sample.NumPackets

	TArray<FNetworkPacketAggregatedSample> Samples;

	int32 HighlightEventTypeIndex;

	FNetworkPacketSeries()
		: NumAggregatedPackets(0)
		, Samples()
		, HighlightEventTypeIndex(-1)
	{
	}

	void Reset()
	{
		NumAggregatedPackets = 0;
		Samples.Reset();
		HighlightEventTypeIndex = -1;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FNetworkPacketSeriesBuilder
{
public:
	explicit FNetworkPacketSeriesBuilder(FNetworkPacketSeries& InSeries, const FPacketViewport& InViewport);

	/**
	 * Non-copyable
	 */
	FNetworkPacketSeriesBuilder(const FNetworkPacketSeriesBuilder&) = delete;
	FNetworkPacketSeriesBuilder& operator=(const FNetworkPacketSeriesBuilder&) = delete;

	FNetworkPacketAggregatedSample* AddPacket(int32 PacketIndex, const TraceServices::FNetProfilerPacket& Packet);

	int32 GetNumAddedPackets() const { return NumAddedPackets; }

	void SetHighlightEventTypeIndex(int32 EventTypeIndex);

private:
	FNetworkPacketSeries& Series; // series to update
	const FPacketViewport& Viewport;

	float SampleW; // width of a sample, in Slate units
	int32 PacketsPerSample; // number of packets in a sample
	int32 FirstPacketIndex; // index of the first packet in the first sample; can be negative
	int32 NumSamples; // total number of samples

	// Debug stats.
	int32 NumAddedPackets; // counts total number of added packets; i.e. number of AddPacket() calls
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FPacketViewDrawHelper
{
public:
	enum class EHighlightMode : uint32
	{
		Hovered = 1,
		Selected = 2,
		SelectedAndHovered = 3
	};

public:
	explicit FPacketViewDrawHelper(const FDrawContext& InDrawContext, const FPacketViewport& InViewport);

	/**
	 * Non-copyable
	 */
	FPacketViewDrawHelper(const FPacketViewDrawHelper&) = delete;
	FPacketViewDrawHelper& operator=(const FPacketViewDrawHelper&) = delete;

	void DrawBackground() const;
	void DrawCached(const FNetworkPacketSeries& Series) const;
	void DrawSampleHighlight(const FNetworkPacketAggregatedSample& Sample, EHighlightMode Mode) const;
	void DrawSelection(int32 StartPacketIndex, int32 EndPacketIndex, double SelectionTimeSpan) const;

	static FLinearColor GetColorByStatus(TraceServices::ENetProfilerDeliveryStatus Status);

	int32 GetNumPackets() const { return NumPackets; }
	int32 GetNumDrawSamples() const { return NumDrawSamples; }

private:
	const FDrawContext& DrawContext;
	const FPacketViewport& Viewport;

	const FSlateBrush* WhiteBrush;
	//const FSlateBrush* EventBorderBrush;
	const FSlateBrush* HoveredEventBorderBrush;
	const FSlateBrush* SelectedEventBorderBrush;
	const FSlateFontInfo SelectionFont;

	// Debug stats.
	mutable int32 NumPackets;
	mutable int32 NumDrawSamples;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
