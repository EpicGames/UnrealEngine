// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// TraceInsights
#include "Insights/ViewModels/AxisViewportInt32.h"
#include "Insights/ViewModels/AxisViewportDouble.h"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FPacketViewport
{
private:
	static constexpr float SLATE_UNITS_TOLERANCE = 0.1f;

public:
	FPacketViewport()
	{
		Reset();
	}

	void Reset()
	{
		HorizontalAxisViewport.Reset();
		VerticalAxisViewport.Reset();
	}

	const FAxisViewportInt32& GetHorizontalAxisViewport() const { return HorizontalAxisViewport; }
	FAxisViewportInt32& GetHorizontalAxisViewport() { return HorizontalAxisViewport; }

	const FAxisViewportDouble& GetVerticalAxisViewport() const { return VerticalAxisViewport; }
	FAxisViewportDouble& GetVerticalAxisViewport() { return VerticalAxisViewport; }

	float GetWidth() const { return HorizontalAxisViewport.GetSize(); }
	float GetHeight() const { return VerticalAxisViewport.GetSize(); }

	bool SetSize(const float InWidth, const float InHeight)
	{
		const bool bWidthChanged = HorizontalAxisViewport.SetSize(InWidth);
		const bool bHeightChanged = VerticalAxisViewport.SetSize(InHeight);
		if (bWidthChanged || bHeightChanged)
		{
			OnSizeChanged();
			return true;
		}
		return false;
	}

	float GetSampleWidth() const { return HorizontalAxisViewport.GetSampleSize(); }
	int32 GetNumPacketsPerSample() const { return HorizontalAxisViewport.GetNumSamplesPerPixel(); }
	int32 GetFirstPacketIndex() const { return HorizontalAxisViewport.GetValueAtOffset(0.0f); }

private:
	void OnSizeChanged()
	{
	}

private:
	FAxisViewportInt32 HorizontalAxisViewport;
	FAxisViewportDouble VerticalAxisViewport;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
