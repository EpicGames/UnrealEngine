// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNetStatsCountersTableCell.h"

#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/Widgets/SNetStatsCountersTableRow.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::SNetStatsCountersView"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SNetStatsCountersTableCell::Construct(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	TablePtr = InArgs._TablePtr;
	ColumnPtr = InArgs._ColumnPtr;
	NetStatsCounterNodePtr = InArgs._NetStatsCounterNodePtr;

	ensure(TablePtr.IsValid());
	ensure(ColumnPtr.IsValid());
	ensure(NetStatsCounterNodePtr.IsValid());

	SetHoveredCellDelegate = InArgs._OnSetHoveredCell;

	ChildSlot
	[
		GenerateWidgetForColumn(InArgs, TableRow)
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetStatsCountersTableCell::GenerateWidgetForColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	if (InArgs._IsNameColumn)
	{
		return GenerateWidgetForNameColumn(InArgs, TableRow);
	}
	else
	{
		return GenerateWidgetForStatsColumn(InArgs, TableRow);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetStatsCountersTableCell::GenerateWidgetForNameColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	return
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SExpanderArrow, TableRow)
		]

		// Info icon + tooltip
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Visibility(this, &SNetStatsCountersTableCell::GetHintIconVisibility)
			.Image(FInsightsStyle::GetBrush("Icons.Hint.TreeItem"))
			.ToolTip(GetRowToolTip(TableRow))
		]

		// Name
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(this, &SNetStatsCountersTableCell::GetDisplayName)
			.HighlightText(InArgs._HighlightText)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			.ColorAndOpacity(this, &SNetStatsCountersTableCell::GetColorAndOpacity)
			.ShadowColorAndOpacity(this, &SNetStatsCountersTableCell::GetShadowColorAndOpacity)
		]

		// Name Suffix
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Visibility(this, &SNetStatsCountersTableCell::HasExtraDisplayName)
			.Text(this, &SNetStatsCountersTableCell::GetExtraDisplayName)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			.ColorAndOpacity(this, &SNetStatsCountersTableCell::GetExtraColorAndOpacity)
			.ShadowColorAndOpacity(this, &SNetStatsCountersTableCell::GetShadowColorAndOpacity)
		]
	;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<IToolTip> SNetStatsCountersTableCell::GetRowToolTip(const TSharedRef<ITableRow>& TableRow) const
{
	TSharedRef<SNetStatsCountersTableRow> Row = StaticCastSharedRef<SNetStatsCountersTableRow, ITableRow>(TableRow);
	return Row->GetRowToolTip();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetStatsCountersTableCell::GetValueAsText() const
{
	return NetStatsCounterNodePtr->IsGroup() ? FText::GetEmpty() : ColumnPtr->GetValueAsText(*NetStatsCounterNodePtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetStatsCountersTableCell::GenerateWidgetForStatsColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	return
		SNew(SHorizontalBox)

		// Value
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		.HAlign(ColumnPtr->GetHorizontalAlignment())
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(this, &SNetStatsCountersTableCell::GetValueAsText)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			.ColorAndOpacity(this, &SNetStatsCountersTableCell::GetStatsColorAndOpacity)
			.ShadowColorAndOpacity(this, &SNetStatsCountersTableCell::GetShadowColorAndOpacity)
		]
	;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
