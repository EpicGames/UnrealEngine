// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

// TraceInsights
#include "Insights/NetworkingProfiler/ViewModels/NetStatsCounterNode.h"

class ITableRow;

namespace UE::Insights
{
	class FTable;
	class FTableColumn;
}

namespace UE::Insights::NetworkingProfiler
{

DECLARE_DELEGATE_ThreeParams(FSetHoveredNetStatsCountersTableCell, TSharedPtr<FTable> /*TablePtr*/, TSharedPtr<FTableColumn> /*ColumnPtr*/, FNetStatsCounterNodePtr /*NetStatsCounterNodePtr*/);

class SNetStatsCountersTableCell : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNetStatsCountersTableCell) {}
		SLATE_EVENT(FSetHoveredNetStatsCountersTableCell, OnSetHoveredCell)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(TSharedPtr<FTable>, TablePtr)
		SLATE_ARGUMENT(TSharedPtr<FTableColumn>, ColumnPtr)
		SLATE_ARGUMENT(FNetStatsCounterNodePtr, NetStatsCounterNodePtr)
		SLATE_ARGUMENT(bool, IsNameColumn)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow);

protected:
	TSharedRef<SWidget> GenerateWidgetForColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow);
	TSharedRef<SWidget> GenerateWidgetForNameColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow);
	TSharedRef<SWidget> GenerateWidgetForStatsColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow);

	/**
	 * The system will use this event to notify a widget that the cursor has entered it. This event is NOT bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 */
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
		SetHoveredCellDelegate.ExecuteIfBound(TablePtr, ColumnPtr, NetStatsCounterNodePtr);
	}

	/**
	 * The system will use this event to notify a widget that the cursor has left it. This event is NOT bubbled.
	 *
	 * @param MouseEvent Information about the input event
	 */
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		SCompoundWidget::OnMouseLeave(MouseEvent);
		SetHoveredCellDelegate.ExecuteIfBound(nullptr, nullptr, nullptr);
	}

	/**
	 * Called during drag and drop when the drag enters a widget.
	 *
	 * Enter/Leave events in slate are meant as lightweight notifications.
	 * So we do not want to capture mouse or set focus in response to these.
	 * However, OnDragEnter must also support external APIs (e.g. OLE Drag/Drop)
	 * Those require that we let them know whether we can handle the content
	 * being dragged OnDragEnter.
	 *
	 * The concession is to return a can_handled/cannot_handle
	 * boolean rather than a full FReply.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether the contents of the DragDropEvent can potentially be processed by this widget.
	 */
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		SCompoundWidget::OnDragEnter(MyGeometry, DragDropEvent);
		SetHoveredCellDelegate.ExecuteIfBound(TablePtr, ColumnPtr, NetStatsCounterNodePtr);
	}

	/**
	 * Called during drag and drop when the drag leaves a widget.
	 *
	 * @param DragDropEvent   The drag and drop event.
	 */
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent)  override
	{
		SCompoundWidget::OnDragLeave(DragDropEvent);
		SetHoveredCellDelegate.ExecuteIfBound(nullptr, nullptr, nullptr);
	}

	EVisibility GetHintIconVisibility() const
	{
		return IsHovered() ? EVisibility::Visible : EVisibility::Hidden;
	}

	TSharedPtr<class IToolTip> GetRowToolTip(const TSharedRef<ITableRow>& TableRow) const;

	FText GetDisplayName() const
	{
		return NetStatsCounterNodePtr->GetDisplayName();
	}

	FText GetExtraDisplayName() const
	{
		return NetStatsCounterNodePtr->GetExtraDisplayName();
	}

	EVisibility HasExtraDisplayName() const
	{
		return NetStatsCounterNodePtr->HasExtraDisplayName() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	FText GetValueAsText() const;

	FSlateColor GetColorAndOpacity() const
	{
		const FLinearColor TextColor =
			NetStatsCounterNodePtr->IsFiltered() ?
				FLinearColor(1.0f, 1.0f, 1.0f, 0.5f) :
				FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
		return TextColor;
	}

	FSlateColor GetExtraColorAndOpacity() const
	{
		const FLinearColor TextColor =
			NetStatsCounterNodePtr->IsFiltered() ?
				FLinearColor(0.5f, 0.5f, 0.5f, 0.5f) :
				FLinearColor(0.5f, 0.5f, 0.5f, 1.0f);
		return TextColor;
	}

	FSlateColor GetStatsColorAndOpacity() const
	{
		const FLinearColor TextColor =
			NetStatsCounterNodePtr->IsFiltered() ?
				FLinearColor(1.0f, 1.0f, 1.0f, 0.5f) :
			NetStatsCounterNodePtr->GetAggregatedStats().Count == 0 ?
				FLinearColor(1.0f, 1.0f, 1.0f, 0.6f) :
				FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
		return TextColor;
	}

	FLinearColor GetShadowColorAndOpacity() const
	{
		const FLinearColor ShadowColor =
			NetStatsCounterNodePtr->IsFiltered() ?
				FLinearColor(0.f, 0.f, 0.f, 0.25f) :
				FLinearColor(0.0f, 0.0f, 0.0f, 0.5f);
		return ShadowColor;
	}

protected:
	/** A shared pointer to the table view model. */
	TSharedPtr<FTable> TablePtr; // TODO: TSharedRef

	/** A shared pointer to the table column view model. */
	TSharedPtr<FTableColumn> ColumnPtr; // TODO: TSharedRef

	/** A shared pointer to the timer node. */
	FNetStatsCounterNodePtr NetStatsCounterNodePtr; // TODO: TSharedRef

	FSetHoveredNetStatsCountersTableCell SetHoveredCellDelegate;
};

} // namespace UE::Insights::NetworkingProfiler
