// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNetStatsCountersTableRow.h"

#include "SlateOptMacros.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/Widgets/SNetStatsCountersTableCell.h"
#include "Insights/NetworkingProfiler/Widgets/SNetStatsCountersViewTooltip.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::SNetStatsView"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SNetStatsCountersTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	OnShouldBeEnabled = InArgs._OnShouldBeEnabled;
	IsColumnVisibleDelegate = InArgs._OnIsColumnVisible;
	GetColumnOutlineHAlignmentDelegate = InArgs._OnGetColumnOutlineHAlignmentDelegate;
	SetHoveredCellDelegate = InArgs._OnSetHoveredCell;

	HighlightText = InArgs._HighlightText;

	TablePtr = InArgs._TablePtr;
	NetStatsCounterNodePtr = InArgs._NetStatsCounterNodePtr;

	RowToolTip = MakeShared<SNetStatsCounterTableRowToolTip>(NetStatsCounterNodePtr);

	SetEnabled(TAttribute<bool>(this, &SNetStatsCountersTableRow::HandleShouldBeEnabled));

	SMultiColumnTableRow<FNetStatsCounterNodePtr>::Construct(SMultiColumnTableRow<FNetStatsCounterNodePtr>::FArguments(), InOwnerTableView);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetStatsCountersTableRow::GenerateWidgetForColumn(const FName& ColumnId)
{
	TSharedPtr<FTableColumn> ColumnPtr = TablePtr->FindColumnChecked(ColumnId);

	return
		SNew(SOverlay)
		.Visibility(EVisibility::SelfHitTestInvisible)

		+SOverlay::Slot()
		.Padding(0.0f)
		[
			SNew(SImage)
			.Image(FInsightsStyle::GetBrush("TreeTable.RowBackground"))
			.ColorAndOpacity(this, &SNetStatsCountersTableRow::GetBackgroundColorAndOpacity)
		]

		+SOverlay::Slot()
		.Padding(0.0f)
		[
			SNew(SImage)
			.Image(this, &SNetStatsCountersTableRow::GetOutlineBrush, ColumnId)
			.ColorAndOpacity(this, &SNetStatsCountersTableRow::GetOutlineColorAndOpacity)
		]

		+SOverlay::Slot()
		[
			SNew(SNetStatsCountersTableCell, SharedThis(this))
			.Visibility(this, &SNetStatsCountersTableRow::IsColumnVisible, ColumnId)
			.TablePtr(TablePtr)
			.ColumnPtr(ColumnPtr)
			.NetStatsCounterNodePtr(NetStatsCounterNodePtr)
			.HighlightText(HighlightText)
			.IsNameColumn(ColumnPtr->IsHierarchy())
			.OnSetHoveredCell(this, &SNetStatsCountersTableRow::OnSetHoveredCell)
		];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SNetStatsCountersTableRow::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return SMultiColumnTableRow<FNetStatsCounterNodePtr>::OnDragDetected(MyGeometry, MouseEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<IToolTip> SNetStatsCountersTableRow::GetRowToolTip() const
{
	return RowToolTip.ToSharedRef();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersTableRow::InvalidateContent()
{
	RowToolTip->InvalidateWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SNetStatsCountersTableRow::GetBackgroundColorAndOpacity() const
{
	return GetBackgroundColorAndOpacity(NetStatsCounterNodePtr->GetAggregatedStats().Count);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SNetStatsCountersTableRow::GetBackgroundColorAndOpacity(uint32 Size) const
{
	const FLinearColor Color =	Size > 1000 ? FLinearColor(0.3f, 0.0f, 0.0f, 1.0f) :
								Size > 100  ? FLinearColor(0.3f, 0.1f, 0.0f, 1.0f) :
								Size > 10   ? FLinearColor(0.0f, 0.1f, 0.0f, 1.0f) :
								              FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
	return Color;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SNetStatsCountersTableRow::GetOutlineColorAndOpacity() const
{
	return FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* SNetStatsCountersTableRow::GetOutlineBrush(const FName ColumnId) const
{
	EHorizontalAlignment HAlign = HAlign_Center;
	if (GetColumnOutlineHAlignmentDelegate.IsBound())
	{
		HAlign = GetColumnOutlineHAlignmentDelegate.Execute(ColumnId);
	}
	return FInsightsStyle::GetOutlineBrush(HAlign);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersTableRow::HandleShouldBeEnabled() const
{
	if (NetStatsCounterNodePtr->IsGroup())
	{
		return true;
	}
	if (OnShouldBeEnabled.IsBound())
	{
		return OnShouldBeEnabled.Execute(NetStatsCounterNodePtr);
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EVisibility SNetStatsCountersTableRow::IsColumnVisible(const FName ColumnId) const
{
	EVisibility Result = EVisibility::Collapsed;
	if (IsColumnVisibleDelegate.IsBound())
	{
		Result = IsColumnVisibleDelegate.Execute(ColumnId) ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return Result;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersTableRow::OnSetHoveredCell(TSharedPtr<FTable> InTablePtr, TSharedPtr<FTableColumn> InColumnPtr, FNetStatsCounterNodePtr InNetStatsCounterNodePtr)
{
	SetHoveredCellDelegate.ExecuteIfBound(InTablePtr, InColumnPtr, InNetStatsCounterNodePtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
