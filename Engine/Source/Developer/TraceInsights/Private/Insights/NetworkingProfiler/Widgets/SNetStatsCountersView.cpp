// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNetStatsCountersView.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "Templates/UniquePtr.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Views/STableViewBase.h"

// TraceServices
#include "TraceServices/Model/NetProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/NetworkingProfilerManager.h"
#include "Insights/NetworkingProfiler/ViewModels/NetStatsCounterNodeHelper.h"
#include "Insights/NetworkingProfiler/ViewModels/NetStatsCountersViewColumnFactory.h"
#include "Insights/NetworkingProfiler/Widgets/SNetStatsCountersTableRow.h"
#include "Insights/NetworkingProfiler/Widgets/SNetStatsCountersViewTooltip.h"
#include "Insights/NetworkingProfiler/Widgets/SNetworkingProfilerWindow.h"
#include "Insights/NetworkingProfiler/Widgets/SPacketContentView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::SNetStatsCountersView"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

SNetStatsCountersView::SNetStatsCountersView()
	: ProfilerWindow()
	, Table(MakeShared<FTable>())
	, bExpansionSaved(false)
	, bFilterOutZeroCountStatsCounters(true)
	, GroupingMode(ENetStatsCounterGroupingMode::ByType)
	, AvailableSorters()
	, CurrentSorter(nullptr)
	, ColumnBeingSorted(GetDefaultColumnBeingSorted())
	, ColumnSortMode(GetDefaultColumnSortMode())
	, NextTimestamp(0)
	, ObjectsChangeCount(0)
	, GameInstanceIndex(0)
	, ConnectionIndex(0)
	, ConnectionMode(TraceServices::ENetProfilerConnectionMode::Outgoing)
	, StatsPacketStartIndex(0)
	, StatsPacketEndIndex(0)
{
	FMemory::Memset(bStatsCounterTypeIsVisible, 1);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SNetStatsCountersView::~SNetStatsCountersView()
{
	// Remove ourselves from the Insights manager.
	if (FInsightsManager::Get().IsValid())
	{
		FInsightsManager::Get()->GetSessionChangedEvent().RemoveAll(this);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SNetStatsCountersView::Construct(const FArguments& InArgs, TSharedPtr<SNetworkingProfilerWindow> InProfilerWindow)
{
	ProfilerWindow = InProfilerWindow;

	SAssignNew(ExternalScrollbar, SScrollBar)
	.AlwaysShowScrollbar(true);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoHeight()
		.Padding(2.0f, 2.0f, 2.0f, 2.0f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				// Search box
				+ SHorizontalBox::Slot()				
				.Padding(2.0f)
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SAssignNew(SearchBox, SSearchBox)
					.HintText(LOCTEXT("SearchBoxHint", "Search NetStatsCounters or groups"))
					.OnTextChanged(this, &SNetStatsCountersView::SearchBox_OnTextChanged)
					.IsEnabled(this, &SNetStatsCountersView::SearchBox_IsEnabled)
					.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search NetStatsCounters or groups"))
				]

				// Filter out net stats counter types with zero count
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(2.0f)
				.AutoWidth()
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
					.HAlign(HAlign_Center)
					.Padding(3.0f)
					.OnCheckStateChanged(this, &SNetStatsCountersView::FilterOutZeroCountStatsCounters_OnCheckStateChanged)
					.IsChecked(this, &SNetStatsCountersView::FilterOutZeroCountStatsCounters_IsChecked)
					.ToolTipText(LOCTEXT("FilterOutZeroCountStatsCounters_Tooltip", "Filter out the stats counters having zero total instance counts (aggregated stats)."))
					[
						SNew(SImage)
						.Image(FInsightsStyle::Get().GetBrush("Icons.ZeroCountFilter"))
					]
				]
			]

			// Group by
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("GroupByText", "Group by"))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.MinDesiredWidth(128.0f)
					[
						SAssignNew(GroupByComboBox, SComboBox<TSharedPtr<ENetStatsCounterGroupingMode>>)
						.ToolTipText(this, &SNetStatsCountersView::GroupBy_GetSelectedTooltipText)
						.OptionsSource(&GroupByOptionsSource)
						.OnSelectionChanged(this, &SNetStatsCountersView::GroupBy_OnSelectionChanged)
						.OnGenerateWidget(this, &SNetStatsCountersView::GroupBy_OnGenerateWidget)
						[
							SNew(STextBlock)
							.Text(this, &SNetStatsCountersView::GroupBy_GetSelectedText)
						]
					]
				]
			]
		]

		// Tree view
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.0f, 2.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(0.0f)
			[
				SAssignNew(TreeView, STreeView<FNetStatsCounterNodePtr>)
				.ExternalScrollbar(ExternalScrollbar)
				.SelectionMode(ESelectionMode::Multi)
				.TreeItemsSource(&FilteredGroupNodes)
				.OnGetChildren(this, &SNetStatsCountersView::TreeView_OnGetChildren)
				.OnGenerateRow(this, &SNetStatsCountersView::TreeView_OnGenerateRow)
				.OnSelectionChanged(this, &SNetStatsCountersView::TreeView_OnSelectionChanged)
				.OnMouseButtonDoubleClick(this, &SNetStatsCountersView::TreeView_OnMouseButtonDoubleClick)
				.OnContextMenuOpening(FOnContextMenuOpening::CreateSP(this, &SNetStatsCountersView::TreeView_GetMenuContent))
				.HeaderRow
				(
					SAssignNew(TreeViewHeaderRow, SHeaderRow)
					.Visibility(EVisibility::Visible)
				)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f)
			[
				SNew(SBox)
				.WidthOverride(FOptionalSize(13.0f))
				[
					ExternalScrollbar.ToSharedRef()
				]
			]
		]
	];

	InitializeAndShowHeaderColumns();
	//BindCommands();

	// Create the search filters: text based, type based etc.
	TextFilter = MakeShared<FNetStatsCounterNodeTextFilter>(FNetStatsCounterNodeTextFilter::FItemToStringArray::CreateSP(this, &SNetStatsCountersView::HandleItemToStringArray));
	Filters = MakeShared<FNetStatsCounterNodeFilterCollection>();
	Filters->Add(TextFilter);

	CreateGroupByOptionsSources();
	CreateSortings();

	// Register ourselves with the Insights manager.
	FInsightsManager::Get()->GetSessionChangedEvent().AddSP(this, &SNetStatsCountersView::InsightsManager_OnSessionChanged);

	// Update the Session (i.e. when analysis session was already started).
	InsightsManager_OnSessionChanged();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> SNetStatsCountersView::TreeView_GetMenuContent()
{
	const TArray<FNetStatsCounterNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	const int32 NumSelectedNodes = SelectedNodes.Num();
	FNetStatsCounterNodePtr SelectedNode = NumSelectedNodes ? SelectedNodes[0] : nullptr;

	FText SelectionStr;

	if (NumSelectedNodes == 0)
	{
		SelectionStr = LOCTEXT("NothingSelected", "Nothing selected");
	}
	else if (NumSelectedNodes == 1)
	{
		FString ItemName = SelectedNode->GetName().ToString();
		const int32 MaxStringLen = 64;
		if (ItemName.Len() > MaxStringLen)
		{
			ItemName = ItemName.Left(MaxStringLen) + TEXT("...");
		}
		SelectionStr = FText::FromString(ItemName);
	}
	else
	{
		SelectionStr = FText::Format(LOCTEXT("MultipleSelection_Fmt", "{0} selected items"), FText::AsNumber(NumSelectedNodes));
	}

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, NULL);

	// Selection menu
	MenuBuilder.BeginSection("Selection", LOCTEXT("ContextMenu_Header_Selection", "Selection"));
	{
		struct FLocal
		{
			static bool ReturnFalse()
			{
				return false;
			}
		};

		FUIAction DummyUIAction;
		DummyUIAction.CanExecuteAction = FCanExecuteAction::CreateStatic(&FLocal::ReturnFalse);
		MenuBuilder.AddMenuEntry
		(
			SelectionStr,
			LOCTEXT("ContextMenu_Selection", "Currently selected items"),
			FSlateIcon(),
			DummyUIAction,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Misc", LOCTEXT("ContextMenu_Header_Misc", "Miscellaneous"));
	{
		/*TODO
		MenuBuilder.AddMenuEntry
		(
			FTimersViewCommands::Get().Command_CopyToClipboard,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);
		*/

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_Header_Misc_Sort", "Sort By"),
			LOCTEXT("ContextMenu_Header_Misc_Sort_Desc", "Sort by column"),
			FNewMenuDelegate::CreateSP(this, &SNetStatsCountersView::TreeView_BuildSortByMenu),
			false,
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.SortBy")
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Columns", LOCTEXT("ContextMenu_Header_Columns", "Columns"));
	{
		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_Header_Columns_View", "View Column"),
			LOCTEXT("ContextMenu_Header_Columns_View_Desc", "Hides or shows columns"),
			FNewMenuDelegate::CreateSP(this, &SNetStatsCountersView::TreeView_BuildViewColumnMenu),
			false,
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ViewColumn")
		);

		FUIAction Action_ShowAllColumns
		(
			FExecuteAction::CreateSP(this, &SNetStatsCountersView::ContextMenu_ShowAllColumns_Execute),
			FCanExecuteAction::CreateSP(this, &SNetStatsCountersView::ContextMenu_ShowAllColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_Header_Columns_ShowAllColumns", "Show All Columns"),
			LOCTEXT("ContextMenu_Header_Columns_ShowAllColumns_Desc", "Resets tree view to show all columns."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_ShowAllColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);

		FUIAction Action_ResetColumns
		(
			FExecuteAction::CreateSP(this, &SNetStatsCountersView::ContextMenu_ResetColumns_Execute),
			FCanExecuteAction::CreateSP(this, &SNetStatsCountersView::ContextMenu_ResetColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_Header_Columns_ResetColumns", "Reset Columns to Default"),
			LOCTEXT("ContextMenu_Header_Columns_ResetColumns_Desc", "Resets columns to default."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_ResetColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::TreeView_BuildSortByMenu(FMenuBuilder& MenuBuilder)
{
	// TODO: Refactor later @see TSharedPtr<SWidget> SCascadePreviewViewportToolBar::GenerateViewMenu() const

	MenuBuilder.BeginSection("ColumnName", LOCTEXT("ContextMenu_Header_Misc_ColumnName", "Column Name"));

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (Column.IsVisible() && Column.CanBeSorted())
		{
			FUIAction Action_SortByColumn
			(
				FExecuteAction::CreateSP(this, &SNetStatsCountersView::ContextMenu_SortByColumn_Execute, Column.GetId()),
				FCanExecuteAction::CreateSP(this, &SNetStatsCountersView::ContextMenu_SortByColumn_CanExecute, Column.GetId()),
				FIsActionChecked::CreateSP(this, &SNetStatsCountersView::ContextMenu_SortByColumn_IsChecked, Column.GetId())
			);
			MenuBuilder.AddMenuEntry
			(
				Column.GetTitleName(),
				Column.GetDescription(),
				FSlateIcon(),
				Action_SortByColumn,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}
	}

	MenuBuilder.EndSection();

	//-----------------------------------------------------------------------------

	MenuBuilder.BeginSection("SortMode", LOCTEXT("ContextMenu_Header_Misc_Sort_SortMode", "Sort Mode"));
	{
		FUIAction Action_SortAscending
		(
			FExecuteAction::CreateSP(this, &SNetStatsCountersView::ContextMenu_SortMode_Execute, EColumnSortMode::Ascending),
			FCanExecuteAction::CreateSP(this, &SNetStatsCountersView::ContextMenu_SortMode_CanExecute, EColumnSortMode::Ascending),
			FIsActionChecked::CreateSP(this, &SNetStatsCountersView::ContextMenu_SortMode_IsChecked, EColumnSortMode::Ascending)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_Header_Misc_Sort_SortAscending", "Sort Ascending"),
			LOCTEXT("ContextMenu_Header_Misc_Sort_SortAscending_Desc", "Sorts ascending"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortUp"),
			Action_SortAscending,
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		FUIAction Action_SortDescending
		(
			FExecuteAction::CreateSP(this, &SNetStatsCountersView::ContextMenu_SortMode_Execute, EColumnSortMode::Descending),
			FCanExecuteAction::CreateSP(this, &SNetStatsCountersView::ContextMenu_SortMode_CanExecute, EColumnSortMode::Descending),
			FIsActionChecked::CreateSP(this, &SNetStatsCountersView::ContextMenu_SortMode_IsChecked, EColumnSortMode::Descending)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_Header_Misc_Sort_SortDescending", "Sort Descending"),
			LOCTEXT("ContextMenu_Header_Misc_Sort_SortDescending_Desc", "Sorts descending"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortDown"),
			Action_SortDescending,
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::TreeView_BuildViewColumnMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("ViewColumn", LOCTEXT("ContextMenu_Header_Columns_View", "View Column"));

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		FUIAction Action_ToggleColumn
		(
			FExecuteAction::CreateSP(this, &SNetStatsCountersView::ToggleColumnVisibility, Column.GetId()),
			FCanExecuteAction::CreateSP(this, &SNetStatsCountersView::CanToggleColumnVisibility, Column.GetId()),
			FIsActionChecked::CreateSP(this, &SNetStatsCountersView::IsColumnVisible, Column.GetId())
		);
		MenuBuilder.AddMenuEntry
		(
			Column.GetTitleName(),
			Column.GetDescription(),
			FSlateIcon(),
			Action_ToggleColumn,
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::InitializeAndShowHeaderColumns()
{
	// Create columns.
	TArray<TSharedRef<FTableColumn>> Columns;
	FNetStatsCountersViewColumnFactory::CreateNetStatsCountersViewColumns(Columns);
	Table->SetColumns(Columns);

	// Show columns.
	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		if (ColumnRef->ShouldBeVisible())
		{
			ShowColumn(ColumnRef->GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetStatsCountersView::GetColumnHeaderText(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.GetShortName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetStatsCountersView::TreeViewHeaderRow_GenerateColumnMenu(const FTableColumn& Column)
{
	bool bIsMenuVisible = false;

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, NULL);
	{
		if (Column.CanBeHidden())
		{
			MenuBuilder.BeginSection("Column", LOCTEXT("TreeViewHeaderRow_Header_Column", "Column"));

			FUIAction Action_HideColumn
			(
				FExecuteAction::CreateSP(this, &SNetStatsCountersView::HideColumn, Column.GetId()),
				FCanExecuteAction::CreateSP(this, &SNetStatsCountersView::CanHideColumn, Column.GetId())
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("TreeViewHeaderRow_HideColumn", "Hide"),
				LOCTEXT("TreeViewHeaderRow_HideColumn_Desc", "Hides the selected column"),
				FSlateIcon(),
				Action_HideColumn,
				NAME_None,
				EUserInterfaceActionType::Button
			);

			bIsMenuVisible = true;
			MenuBuilder.EndSection();
		}

		if (Column.CanBeSorted())
		{
			MenuBuilder.BeginSection("SortMode", LOCTEXT("ContextMenu_Header_Misc_Sort_SortMode", "Sort Mode"));

			FUIAction Action_SortAscending
			(
				FExecuteAction::CreateSP(this, &SNetStatsCountersView::HeaderMenu_SortMode_Execute, Column.GetId(), EColumnSortMode::Ascending),
				FCanExecuteAction::CreateSP(this, &SNetStatsCountersView::HeaderMenu_SortMode_CanExecute, Column.GetId(), EColumnSortMode::Ascending),
				FIsActionChecked::CreateSP(this, &SNetStatsCountersView::HeaderMenu_SortMode_IsChecked, Column.GetId(), EColumnSortMode::Ascending)
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_Header_Misc_Sort_SortAscending", "Sort Ascending"),
				LOCTEXT("ContextMenu_Header_Misc_Sort_SortAscending_Desc", "Sorts ascending"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortUp"),
				Action_SortAscending,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);

			FUIAction Action_SortDescending
			(
				FExecuteAction::CreateSP(this, &SNetStatsCountersView::HeaderMenu_SortMode_Execute, Column.GetId(), EColumnSortMode::Descending),
				FCanExecuteAction::CreateSP(this, &SNetStatsCountersView::HeaderMenu_SortMode_CanExecute, Column.GetId(), EColumnSortMode::Descending),
				FIsActionChecked::CreateSP(this, &SNetStatsCountersView::HeaderMenu_SortMode_IsChecked, Column.GetId(), EColumnSortMode::Descending)
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_Header_Misc_Sort_SortDescending", "Sort Descending"),
				LOCTEXT("ContextMenu_Header_Misc_Sort_SortDescending_Desc", "Sorts descending"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortDown"),
				Action_SortDescending,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);

			bIsMenuVisible = true;
			MenuBuilder.EndSection();
		}

		//if (Column.CanBeFiltered())
		//{
		//	MenuBuilder.BeginSection("FilterMode", LOCTEXT("ContextMenu_Header_Misc_Filter_FilterMode", "Filter Mode"));
		//	bIsMenuVisible = true;
		//	MenuBuilder.EndSection();
		//}
	}

	/*
	TODO:
	- Show top ten
	- Show top bottom
	- Filter by list (avg, median, 10%, 90%, etc.)
	- Text box for filtering for each column instead of one text box used for filtering
	- Grouping button for flat view modes (show at most X groups, show all groups for names)
	*/

	return bIsMenuVisible ? MenuBuilder.MakeWidget() : (TSharedRef<SWidget>)SNullWidget::NullWidget;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::InsightsManager_OnSessionChanged()
{
	TSharedPtr<const TraceServices::IAnalysisSession> NewSession = FInsightsManager::Get()->GetSession();

	if (NewSession != Session)
	{
		Session = NewSession;
		Reset();
	}
	else
	{
		UpdateTree();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::UpdateTree()
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	CreateGroups();

	Stopwatch.Update();
	const double Time1 = Stopwatch.GetAccumulatedTime();

	SortTreeNodes();

	Stopwatch.Update();
	const double Time2 = Stopwatch.GetAccumulatedTime();

	ApplyFiltering();

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.1)
	{
		UE_LOGF(LogNetworkingProfiler, Log, "[NetStats] Tree view updated in %.3fs (%d events) --> G:%.3fs + S:%.3fs + F:%.3fs",
			TotalTime, NetStatsCounterNodes.Num(), Time1, Time2 - Time1, TotalTime - Time2);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::ApplyFiltering()
{
	FilteredGroupNodes.Reset();

	// Apply filter to all groups and its children.
	const int32 NumGroups = GroupNodes.Num();
	for (int32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
	{
		FNetStatsCounterNodePtr& GroupPtr = GroupNodes[GroupIndex];
		GroupPtr->ClearFilteredChildren();
		const bool bIsGroupVisible = Filters->PassesAllFilters(GroupPtr);

		const TArray<FBaseTreeNodePtr>& GroupChildren = GroupPtr->GetChildren();
		int32 NumVisibleChildren = 0;
		for (const FBaseTreeNodePtr& ChildPtr : GroupChildren)
		{
			const FNetStatsCounterNodePtr& NodePtr = StaticCastSharedPtr<FNetStatsCounterNode, FBaseTreeNode>(ChildPtr);

			const bool bIsChildVisible = (!bFilterOutZeroCountStatsCounters || NodePtr->GetAggregatedStats().Count > 0)
									  && bStatsCounterTypeIsVisible[static_cast<int>(NodePtr->GetType())]
									  && Filters->PassesAllFilters(NodePtr);
			if (bIsChildVisible)
			{
				// Add a child
				GroupPtr->AddFilteredChild(NodePtr);
				NumVisibleChildren++;
			}
		}

		if (bIsGroupVisible || NumVisibleChildren > 0)
		{
			// Add a group.
			FilteredGroupNodes.Add(GroupPtr);
			GroupPtr->SetExpansion(true);
		}
		else
		{
			GroupPtr->SetExpansion(false);
		}
	}

	// Only expand net event nodes if we have a text filter.
	const bool bNonEmptyTextFilter = !TextFilter->GetRawFilterText().IsEmpty();
	if (bNonEmptyTextFilter)
	{
		if (!bExpansionSaved)
		{
			ExpandedNodes.Empty();
			TreeView->GetExpandedItems(ExpandedNodes);
			bExpansionSaved = true;
		}

		for (const FNetStatsCounterNodePtr& GroupPtr : FilteredGroupNodes)
		{
			TreeView->SetItemExpansion(GroupPtr, GroupPtr->IsExpanded());
		}
	}
	else
	{
		if (bExpansionSaved)
		{
			// Restore previously expanded nodes when the text filter is disabled.
			TreeView->ClearExpandedItems();
			for (auto It = ExpandedNodes.CreateConstIterator(); It; ++It)
			{
				TreeView->SetItemExpansion(*It, true);
			}
			bExpansionSaved = false;
		}
	}

	// Request tree refresh
	TreeView->RequestTreeRefresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::HandleItemToStringArray(const FNetStatsCounterNodePtr& FNetStatsCounterNodePtr, TArray<FString>& OutSearchStrings) const
{
	OutSearchStrings.Add(FNetStatsCounterNodePtr->GetName().GetPlainNameString());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetStatsCountersView::GetToggleButtonForNetStatsCounterType(const ENetStatsCounterNodeType NodeType)
{
	return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.Padding(FMargin(4.0f, 2.0f, 4.0f, 2.0f)).Padding(2.0f)
		.OnCheckStateChanged(this, &SNetStatsCountersView::FilterByNetStatsCounterType_OnCheckStateChanged, NodeType)
		.IsChecked(this, &SNetStatsCountersView::FilterByNetStatsCounterType_IsChecked, NodeType)
		.ToolTipText(NetStatsCounterNodeTypeHelper::ToDescription(NodeType))
		[
			SNew(SHorizontalBox)

			//+ SHorizontalBox::Slot()
			//.AutoWidth()
			//.VAlign(VAlign_Center)
			//[
			//	SNew(SImage)
			//	.Image(NetStatsCounterNodeTypeHelper::GetIcon(NodeType))
			//]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(NetStatsCounterNodeTypeHelper::ToText(NodeType))
			]
		];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::FilterOutZeroCountStatsCounters_OnCheckStateChanged(ECheckBoxState NewState)
{
	bFilterOutZeroCountStatsCounters = (NewState == ECheckBoxState::Checked);
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState SNetStatsCountersView::FilterOutZeroCountStatsCounters_IsChecked() const
{
	return bFilterOutZeroCountStatsCounters ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::FilterByNetStatsCounterType_OnCheckStateChanged(ECheckBoxState NewState, const ENetStatsCounterNodeType InStatType)
{
	bStatsCounterTypeIsVisible[static_cast<int>(InStatType)] = (NewState == ECheckBoxState::Checked);
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState SNetStatsCountersView::FilterByNetStatsCounterType_IsChecked(const ENetStatsCounterNodeType InStatType) const
{
	return bStatsCounterTypeIsVisible[static_cast<int>(InStatType)] ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// TreeView
////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::TreeView_Refresh()
{
	if (TreeView.IsValid())
	{
		TreeView->RequestTreeRefresh();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::TreeView_OnSelectionChanged(FNetStatsCounterNodePtr SelectedItem, ESelectInfo::Type SelectInfo)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::TreeView_OnGetChildren(FNetStatsCounterNodePtr InParent, TArray<FNetStatsCounterNodePtr>& OutChildren)
{
	const TArray<FBaseTreeNodePtr>& Children = InParent->GetFilteredChildren();
	OutChildren.Reset(Children.Num());
	for (const FBaseTreeNodePtr& Child : Children)
	{
		OutChildren.Add(StaticCastSharedPtr<FNetStatsCounterNode, FBaseTreeNode>(Child));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::TreeView_OnMouseButtonDoubleClick(FNetStatsCounterNodePtr NetEventNodePtr)
{
	if (NetEventNodePtr->IsGroup())
	{
		const bool bIsGroupExpanded = TreeView->IsItemExpanded(NetEventNodePtr);
		TreeView->SetItemExpansion(NetEventNodePtr, !bIsGroupExpanded);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Tree View's Table Row
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableRow> SNetStatsCountersView::TreeView_OnGenerateRow(FNetStatsCounterNodePtr NetStatsCounterNodePtr, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<ITableRow> TableRow =
		SNew(SNetStatsCountersTableRow, OwnerTable)
		.OnShouldBeEnabled(this, &SNetStatsCountersView::TableRow_ShouldBeEnabled)
		.OnIsColumnVisible(this, &SNetStatsCountersView::IsColumnVisible)
		.OnSetHoveredCell(this, &SNetStatsCountersView::TableRow_SetHoveredCell)
		.OnGetColumnOutlineHAlignmentDelegate(this, &SNetStatsCountersView::TableRow_GetColumnOutlineHAlignment)
		.HighlightText(this, &SNetStatsCountersView::TableRow_GetHighlightText)
		.TablePtr(Table)
		.NetStatsCounterNodePtr(NetStatsCounterNodePtr);

	return TableRow;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersView::TableRow_ShouldBeEnabled(FNetStatsCounterNodePtr NodePtr) const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::TableRow_SetHoveredCell(TSharedPtr<FTable> InTablePtr, TSharedPtr<FTableColumn> InColumnPtr, FNetStatsCounterNodePtr InNodePtr)
{
	HoveredColumnId = InColumnPtr ? InColumnPtr->GetId() : FName();

	const bool bIsAnyMenusVisible = FSlateApplication::Get().AnyMenusVisible();
	if (!HasMouseCapture() && !bIsAnyMenusVisible)
	{
		HoveredNodePtr = InNodePtr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EHorizontalAlignment SNetStatsCountersView::TableRow_GetColumnOutlineHAlignment(const FName ColumnId) const
{
	const TIndirectArray<SHeaderRow::FColumn>& Columns = TreeViewHeaderRow->GetColumns();

	// First column
	if (Columns.Num() == 0 ||
		Columns[0].ColumnId == ColumnId)
	{
		return HAlign_Left;
	}
	// Last column
	else if (Columns[Columns.Num() - 1].ColumnId == ColumnId)
	{
		return HAlign_Right;
	}
	// Middle columns
	{
		return HAlign_Center;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetStatsCountersView::TableRow_GetHighlightText() const
{
	return SearchBox->GetText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SearchBox
////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::SearchBox_OnTextChanged(const FText& InFilterText)
{
	TextFilter->SetRawFilterText(InFilterText);
	SearchBox->SetError(TextFilter->GetFilterErrorText());
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersView::SearchBox_IsEnabled() const
{
	return NetStatsCounterNodes.Num() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Grouping
////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::CreateGroups()
{
	if (GroupingMode == ENetStatsCounterGroupingMode::Flat)
	{
		GroupNodes.Reset();

		const FName GroupName(TEXT("All"));
		FNetStatsCounterNodePtr GroupPtr = MakeShared<FNetStatsCounterNode>(GroupName);
		GroupNodes.Add(GroupPtr);

		for (const FNetStatsCounterNodePtr& NodePtr : NetStatsCounterNodes)
		{
			GroupPtr->AddChildAndSetParent(NodePtr);
		}
		TreeView->SetItemExpansion(GroupPtr, true);
	}
	// Creates one group for each stat type.
	else if (GroupingMode == ENetStatsCounterGroupingMode::ByType)
	{
		TMap<ENetStatsCounterNodeType, FNetStatsCounterNodePtr> GroupNodeSet;
		for (const FNetStatsCounterNodePtr& NodePtr : NetStatsCounterNodes)
		{
			const ENetStatsCounterNodeType NodeType = NodePtr->GetType();
			FNetStatsCounterNodePtr GroupPtr = GroupNodeSet.FindRef(NodeType);
			if (!GroupPtr)
			{
				const FName GroupName = *NetStatsCounterNodeTypeHelper::ToText(NodeType).ToString();
				GroupPtr = GroupNodeSet.Add(NodeType, MakeShared<FNetStatsCounterNode>(GroupName));
			}
			GroupPtr->AddChildAndSetParent(NodePtr);
			TreeView->SetItemExpansion(GroupPtr, true);
		}
		GroupNodeSet.KeySort([](const ENetStatsCounterNodeType& A, const ENetStatsCounterNodeType& B) { return A < B; }); // sort groups by type
		GroupNodeSet.GenerateValueArray(GroupNodes);
	}
	// Creates one group for one letter.
	else if (GroupingMode == ENetStatsCounterGroupingMode::ByName)
	{
		TMap<TCHAR, FNetStatsCounterNodePtr> GroupNodeSet;
		for (const FNetStatsCounterNodePtr& NodePtr : NetStatsCounterNodes)
		{
			FString FirstLetterStr(NodePtr->GetName().GetPlainNameString().Left(1).ToUpper());
			const TCHAR FirstLetter = FirstLetterStr[0];
			FNetStatsCounterNodePtr GroupPtr = GroupNodeSet.FindRef(FirstLetter);
			if (!GroupPtr)
			{
				const FName GroupName(FirstLetterStr);
				GroupPtr = GroupNodeSet.Add(FirstLetter, MakeShared<FNetStatsCounterNode>(GroupName));
			}
			GroupPtr->AddChildAndSetParent(NodePtr);
		}
		GroupNodeSet.KeySort([](const TCHAR& A, const TCHAR& B) { return A < B; }); // sort groups alphabetically
		GroupNodeSet.GenerateValueArray(GroupNodes);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::CreateGroupByOptionsSources()
{
	GroupByOptionsSource.Reset(3);

	// Must be added in order of elements in the ENetStatsCounterGroupingMode.
	GroupByOptionsSource.Add(MakeShared<ENetStatsCounterGroupingMode>(ENetStatsCounterGroupingMode::ByType));
	GroupByOptionsSource.Add(MakeShared<ENetStatsCounterGroupingMode>(ENetStatsCounterGroupingMode::Flat));
	GroupByOptionsSource.Add(MakeShared<ENetStatsCounterGroupingMode>(ENetStatsCounterGroupingMode::ByName));
	
	ENetStatsCounterGroupingModePtr* GroupingModePtrPtr = GroupByOptionsSource.FindByPredicate([&](const ENetStatsCounterGroupingModePtr InGroupingModePtr) { return *InGroupingModePtr == GroupingMode; });
	if (GroupingModePtrPtr != nullptr)
	{
		GroupByComboBox->SetSelectedItem(*GroupingModePtrPtr);
	}

	GroupByComboBox->RefreshOptions();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::GroupBy_OnSelectionChanged(TSharedPtr<ENetStatsCounterGroupingMode> NewGroupingMode, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		GroupingMode = *NewGroupingMode;

		CreateGroups();
		SortTreeNodes();
		ApplyFiltering();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetStatsCountersView::GroupBy_OnGenerateWidget(TSharedPtr<ENetStatsCounterGroupingMode> InGroupingMode) const
{
	return SNew(STextBlock)
		.Text(NetStatsCounterNodeGroupingHelper::ToText(*InGroupingMode))
		.ToolTipText(NetStatsCounterNodeGroupingHelper::ToDescription(*InGroupingMode));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetStatsCountersView::GroupBy_GetSelectedText() const
{
	return NetStatsCounterNodeGroupingHelper::ToText(GroupingMode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetStatsCountersView::GroupBy_GetSelectedTooltipText() const
{
	return NetStatsCounterNodeGroupingHelper::ToDescription(GroupingMode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorting
////////////////////////////////////////////////////////////////////////////////////////////////////

const FName SNetStatsCountersView::GetDefaultColumnBeingSorted()
{
	return FNetStatsCountersViewColumns::SumColumnID;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const EColumnSortMode::Type SNetStatsCountersView::GetDefaultColumnSortMode()
{
	return EColumnSortMode::Descending;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::CreateSortings()
{
	AvailableSorters.Reset();
	CurrentSorter = nullptr;

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		if (ColumnRef->CanBeSorted())
		{
			TSharedPtr<ITableCellValueSorter> SorterPtr = ColumnRef->GetValueSorter();
			if (ensure(SorterPtr.IsValid()))
			{
				AvailableSorters.Add(SorterPtr);
			}
		}
	}

	UpdateCurrentSortingByColumn();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::UpdateCurrentSortingByColumn()
{
	TSharedPtr<FTableColumn> ColumnPtr = Table->FindColumn(ColumnBeingSorted);
	CurrentSorter = ColumnPtr.IsValid() ? ColumnPtr->GetValueSorter() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::SortTreeNodes()
{
	if (CurrentSorter.IsValid())
	{
		for (FNetStatsCounterNodePtr& Root : GroupNodes)
		{
			SortTreeNodesRec(*Root, *CurrentSorter);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::SortTreeNodesRec(FNetStatsCounterNode& Node, const ITableCellValueSorter& Sorter)
{
	ESortMode SortMode = (ColumnSortMode == EColumnSortMode::Type::Descending) ? ESortMode::Descending : ESortMode::Ascending;
	Node.SortChildren(Sorter, SortMode);

	for (FBaseTreeNodePtr ChildPtr : Node.GetChildren())
	{
		if (ChildPtr->GetChildrenCount() > 0)
		{
			SortTreeNodesRec(*StaticCastSharedPtr<FNetStatsCounterNode>(ChildPtr), Sorter);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EColumnSortMode::Type SNetStatsCountersView::GetSortModeForColumn(const FName ColumnId) const
{
	if (ColumnBeingSorted != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return ColumnSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::SetSortModeForColumn(const FName& ColumnId, const EColumnSortMode::Type SortMode)
{
	ColumnBeingSorted = ColumnId;
	ColumnSortMode = SortMode;
	UpdateCurrentSortingByColumn();

	SortTreeNodes();
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode)
{
	SetSortModeForColumn(ColumnId, SortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortMode action (HeaderMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersView::HeaderMenu_SortMode_IsChecked(const FName ColumnId, const EColumnSortMode::Type InSortMode)
{
	return ColumnBeingSorted == ColumnId && ColumnSortMode == InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersView::HeaderMenu_SortMode_CanExecute(const FName ColumnId, const EColumnSortMode::Type InSortMode) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.CanBeSorted();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::HeaderMenu_SortMode_Execute(const FName ColumnId, const EColumnSortMode::Type InSortMode)
{
	SetSortModeForColumn(ColumnId, InSortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortMode action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersView::ContextMenu_SortMode_IsChecked(const EColumnSortMode::Type InSortMode)
{
	return ColumnSortMode == InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersView::ContextMenu_SortMode_CanExecute(const EColumnSortMode::Type InSortMode) const
{
	return true; //ColumnSortMode != InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::ContextMenu_SortMode_Execute(const EColumnSortMode::Type InSortMode)
{
	SetSortModeForColumn(ColumnBeingSorted, InSortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortByColumn action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersView::ContextMenu_SortByColumn_IsChecked(const FName ColumnId)
{
	return ColumnId == ColumnBeingSorted;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersView::ContextMenu_SortByColumn_CanExecute(const FName ColumnId) const
{
	return true; //ColumnId != ColumnBeingSorted;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::ContextMenu_SortByColumn_Execute(const FName ColumnId)
{
	SetSortModeForColumn(ColumnId, EColumnSortMode::Descending);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ShowColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersView::CanShowColumn(const FName ColumnId) const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::ShowColumn(const FName ColumnId)
{
	FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	Column.Show();

	SHeaderRow::FColumn::FArguments ColumnArgs;
	ColumnArgs
		.ColumnId(Column.GetId())
		.DefaultLabel(Column.GetShortName())
		.ToolTip(SNetStatsCountersViewTooltip::GetColumnTooltip(Column))
		.HAlignHeader(Column.GetHorizontalAlignment())
		.VAlignHeader(VAlign_Center)
		.HeaderContentPadding(FMargin(2.0f))
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Fill)
		.SortMode(this, &SNetStatsCountersView::GetSortModeForColumn, Column.GetId())
		.OnSort(this, &SNetStatsCountersView::OnSortModeChanged)
		.FillWidth(Column.GetInitialWidth())
		//.FixedWidth(Column.IsFixedWidth() ? Column.GetInitialWidth() : TOptional<float>())
		.HeaderContent()
		[
			SNew(SBox)
			.HeightOverride(24.0f)
			.Padding(FMargin(0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SNetStatsCountersView::GetColumnHeaderText, Column.GetId())
			]
		]
		.MenuContent()
		[
			TreeViewHeaderRow_GenerateColumnMenu(Column)
		];

	int32 ColumnIndex = 0;
	const int32 NewColumnPosition = Table->GetColumnPositionIndex(ColumnId);
	const int32 NumColumns = TreeViewHeaderRow->GetColumns().Num();
	for (; ColumnIndex < NumColumns; ColumnIndex++)
	{
		const SHeaderRow::FColumn& CurrentColumn = TreeViewHeaderRow->GetColumns()[ColumnIndex];
		const int32 CurrentColumnPosition = Table->GetColumnPositionIndex(CurrentColumn.ColumnId);
		if (NewColumnPosition < CurrentColumnPosition)
		{
			break;
		}
	}

	TreeViewHeaderRow->InsertColumn(ColumnArgs, ColumnIndex);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// HideColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersView::CanHideColumn(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.CanBeHidden();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::HideColumn(const FName ColumnId)
{
	FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	Column.Hide();

	TreeViewHeaderRow->RemoveColumn(ColumnId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ToggleColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersView::IsColumnVisible(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersView::CanToggleColumnVisibility(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return !Column.IsVisible() || Column.CanBeHidden();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::ToggleColumnVisibility(const FName ColumnId)
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	if (Column.IsVisible())
	{
		HideColumn(ColumnId);
	}
	else
	{
		ShowColumn(ColumnId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// "Show All Columns" action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersView::ContextMenu_ShowAllColumns_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::ContextMenu_ShowAllColumns_Execute()
{
	ColumnBeingSorted = GetDefaultColumnBeingSorted();
	ColumnSortMode = GetDefaultColumnSortMode();
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (!Column.IsVisible())
		{
			ShowColumn(Column.GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ResetColumns action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsCountersView::ContextMenu_ResetColumns_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::ContextMenu_ResetColumns_Execute()
{
	ColumnBeingSorted = GetDefaultColumnBeingSorted();
	ColumnSortMode = GetDefaultColumnSortMode();
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (Column.ShouldBeVisible() && !Column.IsVisible())
		{
			ShowColumn(Column.GetId());
		}
		else if (!Column.ShouldBeVisible() && Column.IsVisible())
		{
			HideColumn(Column.GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::Reset()
{
	GameInstanceIndex = 0;
	ConnectionIndex = 0;
	ConnectionMode = TraceServices::ENetProfilerConnectionMode::Outgoing;

	StatsPacketStartIndex = 0;
	StatsPacketEndIndex = 0;

	RebuildTree(true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Check if we need to update the lists of NetStatsCounters, but not too often.
	const uint64 Time = FPlatformTime::Cycles64();
	if (Time > NextTimestamp)
	{
		RebuildTree(false);

		const uint64 WaitTime = static_cast<uint64>(0.5 / FPlatformTime::GetSecondsPerCycle64()); // 500ms
		NextTimestamp = Time + WaitTime;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::RebuildTree(bool bResync)
{
	FStopwatch SyncStopwatch;
	FStopwatch Stopwatch;
	Stopwatch.Start();

	if (bResync)
	{
		NetStatsCounterNodes.Empty();
	}

	const uint32 PreviousNodeCount = NetStatsCounterNodes.Num();

	SyncStopwatch.Start();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
		if (NetProfilerProvider)
		{
			NetProfilerProvider->ReadNetStatsCounterTypes([this, &bResync, NetProfilerProvider](const TraceServices::FNetProfilerStatsCounterType* NetStatsCounterTypes, uint64 InNetStatsCounterTypeCount)
			{
				const uint32 NetStatsCounterTypeCount = static_cast<uint32>(InNetStatsCounterTypeCount);
				if (NetStatsCounterTypeCount != NetStatsCounterNodes.Num())
				{
					NetStatsCounterNodes.Empty(NetStatsCounterTypeCount);
					for (uint32 Index = 0; Index < NetStatsCounterTypeCount; ++Index)
					{
						const TraceServices::FNetProfilerStatsCounterType& NetStatsCounterType = NetStatsCounterTypes[Index];
						ensure(NetStatsCounterType.StatsCounterTypeIndex == Index);
						const TCHAR* NamePtr = nullptr;
						NetProfilerProvider->ReadName(NetStatsCounterType.NameIndex, [&NamePtr](const TraceServices::FNetProfilerName& InName)
						{
							NamePtr = InName.Name;
						});
						const FName Name(NamePtr);
						FNetStatsCounterNodePtr NetEventNodePtr = MakeShared<FNetStatsCounterNode>(NetStatsCounterType.StatsCounterTypeIndex, Name, NetStatsCounterType.Type);
						NetStatsCounterNodes.Add(NetEventNodePtr);
					}
				}
			});
		}
	}
	SyncStopwatch.Stop();

	if (bResync || NetStatsCounterNodes.Num() != PreviousNodeCount)
	{
		UpdateTree();
		UpdateStatsInternal();

		// Save selection.
		TArray<FNetStatsCounterNodePtr> SelectedItems;
		TreeView->GetSelectedItems(SelectedItems);

		TreeView->RebuildList();

		// Restore selection.
		if (SelectedItems.Num() > 0)
		{
			TreeView->ClearSelection();
			for (FNetStatsCounterNodePtr& NodePtr : SelectedItems)
			{
				NodePtr = GetNetStatsCounterNode(NodePtr->GetCounterTypeIndex());
			}
			SelectedItems.RemoveAll([](const FNetStatsCounterNodePtr& NodePtr) { return !NodePtr.IsValid(); });
			if (SelectedItems.Num() > 0)
			{
				TreeView->SetItemSelection(SelectedItems, true);
				TreeView->RequestScrollIntoView(SelectedItems.Last());
			}
		}
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.01)
	{
		const double SyncTime = SyncStopwatch.GetAccumulatedTime();
		UE_LOGF(LogNetworkingProfiler, Log, "[NetStatsCounters] Tree view rebuilt in %.4fs (%.4fs + %.4fs) --> %d net stats counters (%d added)",
			TotalTime, SyncTime, TotalTime - SyncTime, NetStatsCounterNodes.Num(), NetStatsCounterNodes.Num() - PreviousNodeCount);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::ResetStats()
{
	GameInstanceIndex = 0;
	ConnectionIndex = 0;
	ConnectionMode = TraceServices::ENetProfilerConnectionMode::Outgoing;

	StatsPacketStartIndex = 0;
	StatsPacketEndIndex = 0;

	UpdateStatsInternal();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::UpdateStats(uint32 InGameInstanceIndex, uint32 InConnectionIndex, TraceServices::ENetProfilerConnectionMode InConnectionMode, uint32 InStatsPacketStartIndex, uint32 InStatsPacketEndIndex)
{
	GameInstanceIndex = InGameInstanceIndex;
	ConnectionIndex = InConnectionIndex;
	ConnectionMode = InConnectionMode;

	StatsPacketStartIndex = InStatsPacketStartIndex;
	StatsPacketEndIndex = InStatsPacketEndIndex;

	UpdateStatsInternal();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::UpdateStatsInternal()
{
	if (StatsPacketStartIndex >= StatsPacketEndIndex)
	{
		// keep previous aggregated stats
		return;
	}

	FStopwatch AggregationStopwatch;
	FStopwatch Stopwatch;
	Stopwatch.Start();

	for (const FNetStatsCounterNodePtr& NetEventNodePtr : NetStatsCounterNodes)
	{
		NetEventNodePtr->ResetAggregatedStats();
	}

	if (Session.IsValid())
	{
		TUniquePtr<TraceServices::ITable<TraceServices::FNetProfilerAggregatedStatsCounterStats>> AggregationResultTable;

		AggregationStopwatch.Start();
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
			if (NetProfilerProvider)
			{
				// CreateAggregation requires [PacketStartIndex, PacketEndIndex] as inclusive interval
				AggregationResultTable.Reset(NetProfilerProvider->CreateStatsCountersAggregation(ConnectionIndex, ConnectionMode, StatsPacketStartIndex, StatsPacketEndIndex - 1));
			}
		}
		AggregationStopwatch.Stop();

		if (AggregationResultTable.IsValid())
		{
			TUniquePtr<TraceServices::ITableReader<TraceServices::FNetProfilerAggregatedStatsCounterStats>> TableReader(AggregationResultTable->CreateReader());
			while (TableReader->IsValid())
			{
				const TraceServices::FNetProfilerAggregatedStatsCounterStats* Row = TableReader->GetCurrentRow();
				FNetStatsCounterNodePtr NetStatsCounterNodePtr = GetNetStatsCounterNode(Row->StatsCounterTypeIndex);
				if (NetStatsCounterNodePtr)
				{
					NetStatsCounterNodePtr->SetAggregatedStats(*Row);

					TSharedPtr<ITableRow> TableRowPtr = TreeView->WidgetFromItem(NetStatsCounterNodePtr);
					if (TableRowPtr.IsValid())
					{
						TSharedPtr<SNetStatsCountersTableRow> RowPtr = StaticCastSharedPtr<SNetStatsCountersTableRow, ITableRow>(TableRowPtr);
						RowPtr->InvalidateContent();
					}
				}
				TableReader->NextRow();
			}
		}
	}

	UpdateTree();

	const TArray<FNetStatsCounterNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	if (SelectedNodes.Num() > 0)
	{
		TreeView->RequestScrollIntoView(SelectedNodes[0]);
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	const double AggregationTime = AggregationStopwatch.GetAccumulatedTime();
	UE_LOGF(LogNetworkingProfiler, Log, "[NetStats] Aggregated stats updated in %.4fs (%.4fs + %.4fs)",
		TotalTime, AggregationTime, TotalTime - AggregationTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FNetStatsCounterNodePtr SNetStatsCountersView::GetNetStatsCounterNode(uint32 EventTypeIndex) const
{
	return (EventTypeIndex < (uint32)NetStatsCounterNodes.Num()) ? NetStatsCounterNodes[EventTypeIndex] : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsCountersView::SelectNetStatsCounterNode(uint32 EventTypeIndex)
{
	FNetStatsCounterNodePtr NodePtr = GetNetStatsCounterNode(EventTypeIndex);
	if (NodePtr)
	{
		TreeView->SetSelection(NodePtr);
		TreeView->RequestScrollIntoView(NodePtr);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
