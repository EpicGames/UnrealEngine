// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNetStatsTableCell.h"

#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/Widgets/SNetStatsTableRow.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::SNetStatsView"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SNetStatsTableCell::Construct(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	TablePtr = InArgs._TablePtr;
	ColumnPtr = InArgs._ColumnPtr;
	NetEventNodePtr = InArgs._NetEventNodePtr;

	ensure(TablePtr.IsValid());
	ensure(ColumnPtr.IsValid());
	ensure(NetEventNodePtr.IsValid());

	SetHoveredCellDelegate = InArgs._OnSetHoveredCell;

	ChildSlot
	[
		GenerateWidgetForColumn(InArgs, TableRow)
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetStatsTableCell::GenerateWidgetForColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	if (InArgs._IsNameColumn)
	{
		return GenerateWidgetForNameColumn(InArgs, TableRow);
	}
	else
	{
		return GenerateWidgetForStatsColumn(InArgs, TableRow);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetStatsTableCell::GenerateWidgetForNameColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	return
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SExpanderArrow, TableRow)
		]

		// Info icon + tooltip
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Visibility(this, &SNetStatsTableCell::GetHintIconVisibility)
			.Image(FInsightsStyle::GetBrush("Icons.Hint.TreeItem"))
			.ToolTip(GetRowToolTip(TableRow))
		]

		// Name
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(this, &SNetStatsTableCell::GetDisplayName)
			.HighlightText(InArgs._HighlightText)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			.ColorAndOpacity(this, &SNetStatsTableCell::GetColorAndOpacity)
			.ShadowColorAndOpacity(this, &SNetStatsTableCell::GetShadowColorAndOpacity)
		]

		// Name Suffix
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Visibility(this, &SNetStatsTableCell::HasExtraDisplayName)
			.Text(this, &SNetStatsTableCell::GetExtraDisplayName)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			.ColorAndOpacity(this, &SNetStatsTableCell::GetExtraColorAndOpacity)
			.ShadowColorAndOpacity(this, &SNetStatsTableCell::GetShadowColorAndOpacity)
		]
	;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<IToolTip> SNetStatsTableCell::GetRowToolTip(const TSharedRef<ITableRow>& TableRow) const
{
	TSharedRef<SNetStatsTableRow> Row = StaticCastSharedRef<SNetStatsTableRow, ITableRow>(TableRow);
	return Row->GetRowToolTip();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetStatsTableCell::GetValueAsText() const
{
	return NetEventNodePtr->IsGroup() ? FText::GetEmpty() : ColumnPtr->GetValueAsText(*NetEventNodePtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetStatsTableCell::GenerateWidgetForStatsColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	return
		SNew(SHorizontalBox)

		// Value
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		.HAlign(ColumnPtr->GetHorizontalAlignment())
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(this, &SNetStatsTableCell::GetValueAsText)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			.ColorAndOpacity(this, &SNetStatsTableCell::GetStatsColorAndOpacity)
			.ShadowColorAndOpacity(this, &SNetStatsTableCell::GetShadowColorAndOpacity)
		]
	;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
