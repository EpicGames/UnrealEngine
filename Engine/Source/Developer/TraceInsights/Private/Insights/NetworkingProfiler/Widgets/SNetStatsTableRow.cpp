// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNetStatsTableRow.h"

#include "SlateOptMacros.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/Widgets/SNetStatsTableCell.h"
#include "Insights/NetworkingProfiler/Widgets/SNetStatsViewTooltip.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::SNetStatsView"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SNetStatsTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	OnShouldBeEnabled = InArgs._OnShouldBeEnabled;
	IsColumnVisibleDelegate = InArgs._OnIsColumnVisible;
	GetColumnOutlineHAlignmentDelegate = InArgs._OnGetColumnOutlineHAlignmentDelegate;
	SetHoveredCellDelegate = InArgs._OnSetHoveredCell;

	HighlightText = InArgs._HighlightText;

	TablePtr = InArgs._TablePtr;
	NetEventNodePtr = InArgs._NetEventNodePtr;

	RowToolTip = MakeShared<SNetEventTableRowToolTip>(NetEventNodePtr);

	SetEnabled(TAttribute<bool>(this, &SNetStatsTableRow::HandleShouldBeEnabled));

	SMultiColumnTableRow<FNetEventNodePtr>::Construct(SMultiColumnTableRow<FNetEventNodePtr>::FArguments(), InOwnerTableView);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetStatsTableRow::GenerateWidgetForColumn(const FName& ColumnId)
{
	TSharedPtr<FTableColumn> ColumnPtr = TablePtr->FindColumnChecked(ColumnId);

	return
		SNew(SOverlay)
		.Visibility(EVisibility::SelfHitTestInvisible)

		+SOverlay::Slot()
		.Padding(0.0f)
		[
			SNew(SImage)
			.Image(FInsightsStyle::GetBrush("TreeTable.RowBackground"))
			.ColorAndOpacity(this, &SNetStatsTableRow::GetBackgroundColorAndOpacity)
		]

		+SOverlay::Slot()
		.Padding(0.0f)
		[
			SNew(SImage)
			.Image(this, &SNetStatsTableRow::GetOutlineBrush, ColumnId)
			.ColorAndOpacity(this, &SNetStatsTableRow::GetOutlineColorAndOpacity)
		]

		+SOverlay::Slot()
		[
			SNew(SNetStatsTableCell, SharedThis(this))
			.Visibility(this, &SNetStatsTableRow::IsColumnVisible, ColumnId)
			.TablePtr(TablePtr)
			.ColumnPtr(ColumnPtr)
			.NetEventNodePtr(NetEventNodePtr)
			.HighlightText(HighlightText)
			.IsNameColumn(ColumnPtr->IsHierarchy())
			.OnSetHoveredCell(this, &SNetStatsTableRow::OnSetHoveredCell)
		];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SNetStatsTableRow::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return SMultiColumnTableRow<FNetEventNodePtr>::OnDragDetected(MyGeometry, MouseEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<IToolTip> SNetStatsTableRow::GetRowToolTip() const
{
	return RowToolTip.ToSharedRef();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsTableRow::InvalidateContent()
{
	RowToolTip->InvalidateWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SNetStatsTableRow::GetBackgroundColorAndOpacity() const
{
	return GetBackgroundColorAndOpacity(NetEventNodePtr->GetAggregatedStats().TotalInclusive);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SNetStatsTableRow::GetBackgroundColorAndOpacity(uint32 Size) const
{
	const FLinearColor Color =	Size > 1000 ? FLinearColor(0.3f, 0.0f, 0.0f, 1.0f) :
								Size > 100  ? FLinearColor(0.3f, 0.1f, 0.0f, 1.0f) :
								Size > 10   ? FLinearColor(0.0f, 0.1f, 0.0f, 1.0f) :
								              FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
	return Color;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SNetStatsTableRow::GetOutlineColorAndOpacity() const
{
	return FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* SNetStatsTableRow::GetOutlineBrush(const FName ColumnId) const
{
	EHorizontalAlignment HAlign = HAlign_Center;
	if (GetColumnOutlineHAlignmentDelegate.IsBound())
	{
		HAlign = GetColumnOutlineHAlignmentDelegate.Execute(ColumnId);
	}
	return FInsightsStyle::GetOutlineBrush(HAlign);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsTableRow::HandleShouldBeEnabled() const
{
	if (NetEventNodePtr->IsGroup())
	{
		return true;
	}
	if (OnShouldBeEnabled.IsBound())
	{
		return OnShouldBeEnabled.Execute(NetEventNodePtr);
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EVisibility SNetStatsTableRow::IsColumnVisible(const FName ColumnId) const
{
	EVisibility Result = EVisibility::Collapsed;
	if (IsColumnVisibleDelegate.IsBound())
	{
		Result = IsColumnVisibleDelegate.Execute(ColumnId) ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return Result;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsTableRow::OnSetHoveredCell(TSharedPtr<FTable> InTablePtr, TSharedPtr<FTableColumn> InColumnPtr, FNetEventNodePtr InNetEventNodePtr)
{
	SetHoveredCellDelegate.ExecuteIfBound(InTablePtr, InColumnPtr, InNetEventNodePtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
