// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

// TraceInsights
#include "Insights/NetworkingProfiler/ViewModels/NetEventNode.h"
#include "Insights/NetworkingProfiler/ViewModels/NetEventNodeHelper.h"

class IToolTip;

namespace UE::Insights
{
	class FTable;
	class FTableColumn;
}

namespace UE::Insights::NetworkingProfiler
{

class SNetEventTableRowToolTip;

DECLARE_DELEGATE_RetVal_OneParam(bool, FNetEventNodeShouldBeEnabledDelegate, FNetEventNodePtr /*NodePtr*/);
DECLARE_DELEGATE_RetVal_OneParam(bool, FIsColumnVisibleDelegate, const FName /*ColumnId*/);
DECLARE_DELEGATE_RetVal_OneParam(EHorizontalAlignment, FGetColumnOutlineHAlignmentDelegate, const FName /*ColumnId*/);
DECLARE_DELEGATE_ThreeParams(FSetHoveredNetEventTableCell, TSharedPtr<FTable> /*TablePtr*/, TSharedPtr<FTableColumn> /*ColumnPtr*/, FNetEventNodePtr /*NetEventNodePtr*/);

/** Widget that represents a table row in the tree control. Generates widgets for each column on demand. */
class SNetStatsTableRow : public SMultiColumnTableRow<FNetEventNodePtr>
{
public:
	SLATE_BEGIN_ARGS(SNetStatsTableRow) {}
		SLATE_EVENT(FNetEventNodeShouldBeEnabledDelegate, OnShouldBeEnabled)
		SLATE_EVENT(FIsColumnVisibleDelegate, OnIsColumnVisible)
		SLATE_EVENT(FGetColumnOutlineHAlignmentDelegate, OnGetColumnOutlineHAlignmentDelegate)
		SLATE_EVENT(FSetHoveredNetEventTableCell, OnSetHoveredCell)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(TSharedPtr<FTable>, TablePtr)
		SLATE_ARGUMENT(FNetEventNodePtr, NetEventNodePtr)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override;

	/**
	 * Called when Slate detects that a widget started to be dragged.
	 * Usage:
	 * A widget can ask Slate to detect a drag.
	 * OnMouseDown() reply with FReply::Handled().DetectDrag(SharedThis(this)).
	 * Slate will either send an OnDragDetected() event or do nothing.
	 * If the user releases a mouse button or leaves the widget before
	 * a drag is triggered (maybe user started at the very edge) then no event will be
	 * sent.
	 *
	 * @param  InMyGeometry  Widget geometry
	 * @param  InMouseEvent  MouseMove that triggered the drag
	 *
	 */
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	TSharedRef<IToolTip> GetRowToolTip() const;
	void InvalidateContent();

protected:
	FSlateColor GetBackgroundColorAndOpacity() const;
	FSlateColor GetBackgroundColorAndOpacity(uint32 Size) const;
	FSlateColor GetOutlineColorAndOpacity() const;
	const FSlateBrush* GetOutlineBrush(const FName ColumnId) const;
	bool HandleShouldBeEnabled() const;
	EVisibility IsColumnVisible(const FName ColumnId) const;
	void OnSetHoveredCell(TSharedPtr<FTable> InTablePtr, TSharedPtr<FTableColumn> InColumnPtr, FNetEventNodePtr InNetEventNodePtr);

protected:
	/** A shared pointer to the table view model. */
	TSharedPtr<FTable> TablePtr;

	/** Data context for this table row. */
	FNetEventNodePtr NetEventNodePtr;

	FNetEventNodeShouldBeEnabledDelegate OnShouldBeEnabled;
	FIsColumnVisibleDelegate IsColumnVisibleDelegate;
	FSetHoveredNetEventTableCell SetHoveredCellDelegate;
	FGetColumnOutlineHAlignmentDelegate GetColumnOutlineHAlignmentDelegate;

	/** Text to be highlighted on timer name. */
	TAttribute<FText> HighlightText;

	TSharedPtr<SNetEventTableRowToolTip> RowToolTip;
};

} // namespace UE::Insights::NetworkingProfiler
