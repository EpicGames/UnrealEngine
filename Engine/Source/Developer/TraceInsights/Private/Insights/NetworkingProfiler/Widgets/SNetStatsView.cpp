// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNetStatsView.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "Templates/UniquePtr.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Views/STableViewBase.h"

// TraceServices
#include "TraceServices/Model/NetProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/NetworkingProfilerManager.h"
#include "Insights/NetworkingProfiler/ViewModels/NetStatsViewColumnFactory.h"
#include "Insights/NetworkingProfiler/Widgets/SNetStatsViewTooltip.h"
#include "Insights/NetworkingProfiler/Widgets/SNetStatsTableRow.h"
#include "Insights/NetworkingProfiler/Widgets/SNetworkingProfilerWindow.h"
#include "Insights/NetworkingProfiler/Widgets/SPacketContentView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::SNetStatsView"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

SNetStatsView::SNetStatsView()
	: ProfilerWindowWeakPtr()
	, Table(MakeShared<FTable>())
	, bExpansionSaved(false)
	, bFilterOutZeroCountEvents(false)
	, GroupingMode(ENetEventGroupingMode::Flat)
	, AvailableSorters()
	, CurrentSorter(nullptr)
	, ColumnBeingSorted(GetDefaultColumnBeingSorted())
	, ColumnSortMode(GetDefaultColumnSortMode())
	, NextTimestamp(0)
	, ObjectsChangeCount(0)
	, GameInstanceIndex(0)
	, ConnectionIndex(0)
	, ConnectionMode(TraceServices::ENetProfilerConnectionMode::Outgoing)
	, StatsPacketStartIndex(0)
	, StatsPacketEndIndex(0)
	, StatsStartPosition(0)
	, StatsEndPosition(0)
{
	FMemory::Memset(bNetEventTypeIsVisible, 1);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SNetStatsView::~SNetStatsView()
{
	// Remove ourselves from the Insights manager.
	if (FInsightsManager::Get().IsValid())
	{
		FInsightsManager::Get()->GetSessionChangedEvent().RemoveAll(this);
	}

	Session.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SNetStatsView::Construct(const FArguments& InArgs, TSharedRef<SNetworkingProfilerWindow> InProfilerWindow)
{
	ProfilerWindowWeakPtr = InProfilerWindow;

	SAssignNew(ExternalScrollbar, SScrollBar)
	.AlwaysShowScrollbar(true);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoHeight()
		.Padding(2.0f, 2.0f, 2.0f, 2.0f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				// Search box
				+ SHorizontalBox::Slot()
				.Padding(2.0f)
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SAssignNew(SearchBox, SSearchBox)
					.HintText(LOCTEXT("SearchBoxHint", "Search net events or groups"))
					.OnTextChanged(this, &SNetStatsView::SearchBox_OnTextChanged)
					.IsEnabled(this, &SNetStatsView::SearchBox_IsEnabled)
					.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search net events or groups"))
				]

				// Filter out net event types with zero instance count
				+ SHorizontalBox::Slot()
				.Padding(2.0f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
					.HAlign(HAlign_Center)
					.Padding(3.0f)
					.OnCheckStateChanged(this, &SNetStatsView::FilterOutZeroCountEvents_OnCheckStateChanged)
					.IsChecked(this, &SNetStatsView::FilterOutZeroCountEvents_IsChecked)
					.ToolTipText(LOCTEXT("FilterOutZeroCountEvents_Tooltip", "Filter out the net event types having zero total instance count (aggregated stats)."))
					[
						SNew(SImage)
						.Image(FInsightsStyle::Get().GetBrush("Icons.ZeroCountFilter"))
					]
				]
			]

			// Group by
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("GroupByText", "Group by"))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.MinDesiredWidth(128.0f)
					[
						SAssignNew(GroupByComboBox, SComboBox<TSharedPtr<ENetEventGroupingMode>>)
						.ToolTipText(this, &SNetStatsView::GroupBy_GetSelectedTooltipText)
						.OptionsSource(&GroupByOptionsSource)
						.OnSelectionChanged(this, &SNetStatsView::GroupBy_OnSelectionChanged)
						.OnGenerateWidget(this, &SNetStatsView::GroupBy_OnGenerateWidget)
						[
							SNew(STextBlock)
							.Text(this, &SNetStatsView::GroupBy_GetSelectedText)
						]
					]
				]
			]
		]

		// Tree view
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.0f, 2.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(0.0f)
			[
				SAssignNew(TreeView, STreeView<FNetEventNodePtr>)
				.ExternalScrollbar(ExternalScrollbar)
				.SelectionMode(ESelectionMode::Multi)
				.TreeItemsSource(&FilteredGroupNodes)
				.OnGetChildren(this, &SNetStatsView::TreeView_OnGetChildren)
				.OnGenerateRow(this, &SNetStatsView::TreeView_OnGenerateRow)
				.OnSelectionChanged(this, &SNetStatsView::TreeView_OnSelectionChanged)
				.OnMouseButtonDoubleClick(this, &SNetStatsView::TreeView_OnMouseButtonDoubleClick)
				.OnContextMenuOpening(FOnContextMenuOpening::CreateSP(this, &SNetStatsView::TreeView_GetMenuContent))
				.HeaderRow
				(
					SAssignNew(TreeViewHeaderRow, SHeaderRow)
					.Visibility(EVisibility::Visible)
				)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f)
			[
				SNew(SBox)
				.WidthOverride(FOptionalSize(13.0f))
				[
					ExternalScrollbar.ToSharedRef()
				]
			]
		]
	];

	InitializeAndShowHeaderColumns();

	// Create the search filters: text based, type based etc.
	TextFilter = MakeShared<FNetEventNodeTextFilter>(FNetEventNodeTextFilter::FItemToStringArray::CreateSP(this, &SNetStatsView::HandleItemToStringArray));
	Filters = MakeShared<FNetEventNodeFilterCollection>();
	Filters->Add(TextFilter);

	CreateGroupByOptionsSources();
	CreateSortings();

	// Register ourselves with the Insights manager.
	FInsightsManager::Get()->GetSessionChangedEvent().AddSP(this, &SNetStatsView::InsightsManager_OnSessionChanged);

	// Update the Session (i.e. when analysis session was already started).
	InsightsManager_OnSessionChanged();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> SNetStatsView::TreeView_GetMenuContent()
{
	const TArray<FNetEventNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	const int32 NumSelectedNodes = SelectedNodes.Num();
	FNetEventNodePtr SelectedNode = NumSelectedNodes ? SelectedNodes[0] : nullptr;

	FText SelectionStr;
	if (NumSelectedNodes == 0)
	{
		SelectionStr = LOCTEXT("NothingSelected", "Nothing selected");
	}
	else if (NumSelectedNodes == 1)
	{
		FString ItemName = SelectedNode->GetName().ToString();
		const int32 MaxStringLen = 64;
		if (ItemName.Len() > MaxStringLen)
		{
			ItemName = ItemName.Left(MaxStringLen) + TEXT("...");
		}
		SelectionStr = FText::FromString(ItemName);
	}
	else
	{
		SelectionStr = FText::Format(LOCTEXT("MultipleSelection_Fmt", "{0} selected items"), FText::AsNumber(NumSelectedNodes));
	}

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, NULL);

	// Selection menu
	MenuBuilder.BeginSection("Selection", LOCTEXT("ContextMenu_Section_Selection", "Selection"));
	{
		struct FLocal
		{
			static bool ReturnFalse()
			{
				return false;
			}
		};

		FUIAction DummyUIAction;
		DummyUIAction.CanExecuteAction = FCanExecuteAction::CreateStatic(&FLocal::ReturnFalse);
		MenuBuilder.AddMenuEntry
		(
			SelectionStr,
			LOCTEXT("ContextMenu_Selection", "Currently selected items"),
			FSlateIcon(),
			DummyUIAction,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::TreeView_BuildSortByMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("SortColumn", LOCTEXT("ContextMenu_Section_SortColumn", "Sort Column"));

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (Column.IsVisible() && Column.CanBeSorted())
		{
			FUIAction Action_SortByColumn
			(
				FExecuteAction::CreateSP(this, &SNetStatsView::ContextMenu_SortByColumn_Execute, Column.GetId()),
				FCanExecuteAction::CreateSP(this, &SNetStatsView::ContextMenu_SortByColumn_CanExecute, Column.GetId()),
				FIsActionChecked::CreateSP(this, &SNetStatsView::ContextMenu_SortByColumn_IsChecked, Column.GetId())
			);
			MenuBuilder.AddMenuEntry
			(
				Column.GetTitleName(),
				Column.GetDescription(),
				FSlateIcon(),
				Action_SortByColumn,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}
	}

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("SortMode", LOCTEXT("ContextMenu_Section_SortMode", "Sort Mode"));
	{
		FUIAction Action_SortAscending
		(
			FExecuteAction::CreateSP(this, &SNetStatsView::ContextMenu_SortMode_Execute, EColumnSortMode::Ascending),
			FCanExecuteAction::CreateSP(this, &SNetStatsView::ContextMenu_SortMode_CanExecute, EColumnSortMode::Ascending),
			FIsActionChecked::CreateSP(this, &SNetStatsView::ContextMenu_SortMode_IsChecked, EColumnSortMode::Ascending)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_SortAscending", "Sort Ascending"),
			LOCTEXT("ContextMenu_SortAscending_Desc", "Sorts ascending."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortUp"),
			Action_SortAscending,
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		FUIAction Action_SortDescending
		(
			FExecuteAction::CreateSP(this, &SNetStatsView::ContextMenu_SortMode_Execute, EColumnSortMode::Descending),
			FCanExecuteAction::CreateSP(this, &SNetStatsView::ContextMenu_SortMode_CanExecute, EColumnSortMode::Descending),
			FIsActionChecked::CreateSP(this, &SNetStatsView::ContextMenu_SortMode_IsChecked, EColumnSortMode::Descending)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_SortDescending", "Sort Descending"),
			LOCTEXT("ContextMenu_SortDescending_Desc", "Sorts descending."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortDown"),
			Action_SortDescending,
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::TreeView_BuildViewColumnMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Columns", LOCTEXT("ContextMenu_Section_Columns", "Columns"));

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		FUIAction Action_ToggleColumn
		(
			FExecuteAction::CreateSP(this, &SNetStatsView::ToggleColumnVisibility, Column.GetId()),
			FCanExecuteAction::CreateSP(this, &SNetStatsView::CanToggleColumnVisibility, Column.GetId()),
			FIsActionChecked::CreateSP(this, &SNetStatsView::IsColumnVisible, Column.GetId())
		);
		MenuBuilder.AddMenuEntry
		(
			Column.GetTitleName(),
			Column.GetDescription(),
			FSlateIcon(),
			Action_ToggleColumn,
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::InitializeAndShowHeaderColumns()
{
	// Create columns.
	TArray<TSharedRef<FTableColumn>> Columns;
	FNetStatsViewColumnFactory::CreateNetStatsViewColumns(Columns);
	Table->SetColumns(Columns);

	// Show columns.
	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		if (ColumnRef->ShouldBeVisible())
		{
			ShowColumn(ColumnRef->GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetStatsView::GetColumnHeaderText(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.GetShortName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetStatsView::TreeViewHeaderRow_GenerateColumnMenu(const FTableColumn& Column)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, NULL);

	MenuBuilder.BeginSection("Sorting", LOCTEXT("ContextMenu_Section_Sorting", "Sorting"));
	{
		if (Column.CanBeSorted())
		{
			FUIAction Action_SortAscending
			(
				FExecuteAction::CreateSP(this, &SNetStatsView::HeaderMenu_SortMode_Execute, Column.GetId(), EColumnSortMode::Ascending),
				FCanExecuteAction::CreateSP(this, &SNetStatsView::HeaderMenu_SortMode_CanExecute, Column.GetId(), EColumnSortMode::Ascending),
				FIsActionChecked::CreateSP(this, &SNetStatsView::HeaderMenu_SortMode_IsChecked, Column.GetId(), EColumnSortMode::Ascending)
			);
			MenuBuilder.AddMenuEntry
			(
				FText::Format(LOCTEXT("ContextMenu_SortAscending_Fmt", "Sort Ascending (by {0})"), Column.GetTitleName()),
				FText::Format(LOCTEXT("ContextMenu_SortAscending_Desc_Fmt", "Sorts ascending by {0}."), Column.GetTitleName()),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortUp"),
				Action_SortAscending,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);

			FUIAction Action_SortDescending
			(
				FExecuteAction::CreateSP(this, &SNetStatsView::HeaderMenu_SortMode_Execute, Column.GetId(), EColumnSortMode::Descending),
				FCanExecuteAction::CreateSP(this, &SNetStatsView::HeaderMenu_SortMode_CanExecute, Column.GetId(), EColumnSortMode::Descending),
				FIsActionChecked::CreateSP(this, &SNetStatsView::HeaderMenu_SortMode_IsChecked, Column.GetId(), EColumnSortMode::Descending)
			);
			MenuBuilder.AddMenuEntry
			(
				FText::Format(LOCTEXT("ContextMenu_SortDescending_Fmt", "Sort Descending (by {0})"), Column.GetTitleName()),
				FText::Format(LOCTEXT("ContextMenu_SortDescending_Desc_Fmt", "Sorts descending by {0}."), Column.GetTitleName()),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortDown"),
				Action_SortDescending,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_SortBy_SubMenu", "Sort By"),
			LOCTEXT("ContextMenu_SortBy_SubMenu_Desc", "Sorts by a column."),
			FNewMenuDelegate::CreateSP(this, &SNetStatsView::TreeView_BuildSortByMenu),
			false,
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.SortBy")
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("ColumnVisibility", LOCTEXT("ContextMenu_Section_ColumnVisibility", "Column Visibility"));
	{
		if (Column.CanBeHidden())
		{

			FUIAction Action_HideColumn
			(
				FExecuteAction::CreateSP(this, &SNetStatsView::HideColumn, Column.GetId()),
				FCanExecuteAction::CreateSP(this, &SNetStatsView::CanHideColumn, Column.GetId())
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_HideColumn", "Hide"),
				LOCTEXT("ContextMenu_HideColumn_Desc", "Hides the selected column."),
				FSlateIcon(),
				Action_HideColumn,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_ViewColumn_SubMenu", "View Column"),
			LOCTEXT("ContextMenu_ViewColumn_SubMenu_Desc", "Hides or shows columns."),
			FNewMenuDelegate::CreateSP(this, &SNetStatsView::TreeView_BuildViewColumnMenu),
			false,
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ViewColumn")
		);

		FUIAction Action_ShowAllColumns
		(
			FExecuteAction::CreateSP(this, &SNetStatsView::ContextMenu_ShowAllColumns_Execute),
			FCanExecuteAction::CreateSP(this, &SNetStatsView::ContextMenu_ShowAllColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_ShowAllColumns", "Show All Columns"),
			LOCTEXT("ContextMenu_ShowAllColumns_Desc", "Resets tree view to show all columns."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_ShowAllColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);

		FUIAction Action_ResetColumns
		(
			FExecuteAction::CreateSP(this, &SNetStatsView::ContextMenu_ResetColumns_Execute),
			FCanExecuteAction::CreateSP(this, &SNetStatsView::ContextMenu_ResetColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_ResetColumns", "Reset Columns to Default"),
			LOCTEXT("ContextMenu_ResetColumns_Desc", "Resets columns to default."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_ResetColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::InsightsManager_OnSessionChanged()
{
	TSharedPtr<const TraceServices::IAnalysisSession> NewSession = FInsightsManager::Get()->GetSession();
	if (NewSession != Session)
	{
		Session = NewSession;
		Reset();
	}
	else
	{
		UpdateTree();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::UpdateTree()
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	CreateGroups();

	Stopwatch.Update();
	const double Time1 = Stopwatch.GetAccumulatedTime();

	SortTreeNodes();

	Stopwatch.Update();
	const double Time2 = Stopwatch.GetAccumulatedTime();

	ApplyFiltering();

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.1)
	{
		UE_LOGF(LogNetworkingProfiler, Log, "[NetStats] Tree view updated in %.3fs (%d events) --> G:%.3fs + S:%.3fs + F:%.3fs",
			TotalTime, NetEventNodes.Num(), Time1, Time2 - Time1, TotalTime - Time2);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::ApplyFiltering()
{
	FilteredGroupNodes.Reset();

	// Apply filter to all groups and its children.
	const int32 NumGroups = GroupNodes.Num();
	for (int32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
	{
		FNetEventNodePtr& GroupPtr = GroupNodes[GroupIndex];
		GroupPtr->ClearFilteredChildren();
		const bool bIsGroupVisible = Filters->PassesAllFilters(GroupPtr);

		const TArray<FBaseTreeNodePtr>& GroupChildren = GroupPtr->GetChildren();
		int32 NumVisibleChildren = 0;
		for (const FBaseTreeNodePtr& ChildPtr : GroupChildren)
		{
			const FNetEventNodePtr& NodePtr = StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(ChildPtr);

			const bool bIsChildVisible = (!bFilterOutZeroCountEvents || NodePtr->GetAggregatedStats().InstanceCount > 0)
									  && bNetEventTypeIsVisible[static_cast<int>(NodePtr->GetType())]
									  && Filters->PassesAllFilters(NodePtr);
			if (bIsChildVisible)
			{
				// Add a child.
				GroupPtr->AddFilteredChild(NodePtr);
				NumVisibleChildren++;
			}
		}

		if (bIsGroupVisible || NumVisibleChildren > 0)
		{
			// Add a group.
			FilteredGroupNodes.Add(GroupPtr);
			GroupPtr->SetExpansion(true);
		}
		else
		{
			GroupPtr->SetExpansion(false);
		}
	}

	// Only expand net event nodes if we have a text filter.
	const bool bNonEmptyTextFilter = !TextFilter->GetRawFilterText().IsEmpty();
	if (bNonEmptyTextFilter)
	{
		if (!bExpansionSaved)
		{
			ExpandedNodes.Empty();
			TreeView->GetExpandedItems(ExpandedNodes);
			bExpansionSaved = true;
		}

		for (const FNetEventNodePtr& GroupPtr : FilteredGroupNodes)
		{
			TreeView->SetItemExpansion(GroupPtr, GroupPtr->IsExpanded());
		}
	}
	else
	{
		if (bExpansionSaved)
		{
			// Restore previously expanded nodes when the text filter is disabled.
			TreeView->ClearExpandedItems();
			for (auto It = ExpandedNodes.CreateConstIterator(); It; ++It)
			{
				TreeView->SetItemExpansion(*It, true);
			}
			bExpansionSaved = false;
		}
	}

	// Request tree refresh
	TreeView->RequestTreeRefresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::HandleItemToStringArray(const FNetEventNodePtr& FNetEventNodePtr, TArray<FString>& OutSearchStrings) const
{
	OutSearchStrings.Add(FNetEventNodePtr->GetName().GetPlainNameString());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetStatsView::GetToggleButtonForNetEventType(const ENetEventNodeType NodeType)
{
	return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.Padding(FMargin(4.0f, 2.0f, 4.0f, 2.0f))
		.HAlign(HAlign_Center)
		.OnCheckStateChanged(this, &SNetStatsView::FilterByNetEventType_OnCheckStateChanged, NodeType)
		.IsChecked(this, &SNetStatsView::FilterByNetEventType_IsChecked, NodeType)
		.ToolTipText(NetEventNodeTypeHelper::ToDescription(NodeType))
		[
			SNew(SHorizontalBox)

			//+ SHorizontalBox::Slot()
			//.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			//.AutoWidth()
			//.VAlign(VAlign_Center)
			//[
			//	SNew(SImage)
			//	.Image(NetEventNodeTypeHelper::GetIconForNetEventNodeType(NodeType))
			//]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(NetEventNodeTypeHelper::ToText(NodeType))
			]
		];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::FilterOutZeroCountEvents_OnCheckStateChanged(ECheckBoxState NewState)
{
	bFilterOutZeroCountEvents = (NewState == ECheckBoxState::Checked);
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState SNetStatsView::FilterOutZeroCountEvents_IsChecked() const
{
	return bFilterOutZeroCountEvents ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::FilterByNetEventType_OnCheckStateChanged(ECheckBoxState NewState, const ENetEventNodeType InStatType)
{
	bNetEventTypeIsVisible[static_cast<int>(InStatType)] = (NewState == ECheckBoxState::Checked);
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState SNetStatsView::FilterByNetEventType_IsChecked(const ENetEventNodeType InStatType) const
{
	return bNetEventTypeIsVisible[static_cast<int>(InStatType)] ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// TreeView
////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::TreeView_Refresh()
{
	if (TreeView.IsValid())
	{
		TreeView->RequestTreeRefresh();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::TreeView_OnSelectionChanged(FNetEventNodePtr SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		TArray<FNetEventNodePtr> SelectedItems = TreeView->GetSelectedItems();
		if (SelectedItems.Num() == 1 && !SelectedItems[0]->IsGroup())
		{
			//TODO: FNetworkingProfilerManager::Get()->SetSelectedNetEvent(SelectedItems[0]->GetEventTypeIndex());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::TreeView_OnGetChildren(FNetEventNodePtr InParent, TArray<FNetEventNodePtr>& OutChildren)
{
	const TArray<FBaseTreeNodePtr>& Children = InParent->GetFilteredChildren();
	OutChildren.Reset(Children.Num());
	for (const FBaseTreeNodePtr& Child : Children)
	{
		OutChildren.Add(StaticCastSharedPtr<FNetEventNode, FBaseTreeNode>(Child));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::TreeView_OnMouseButtonDoubleClick(FNetEventNodePtr NetEventNodePtr)
{
	if (NetEventNodePtr->IsGroup())
	{
		const bool bIsGroupExpanded = TreeView->IsItemExpanded(NetEventNodePtr);
		TreeView->SetItemExpansion(NetEventNodePtr, !bIsGroupExpanded);
	}
	else
	{
		TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = GetProfilerWindow();
		TSharedPtr<SPacketContentView> PacketContentView = ProfilerWindow.IsValid() ? ProfilerWindow->GetPacketContentView() : nullptr;
		if (PacketContentView.IsValid())
		{
			const uint32 EventTypeIndex = NetEventNodePtr->GetEventTypeIndex();
			const uint32 FilterEventTypeIndex = PacketContentView->GetFilterEventTypeIndex();

			if (EventTypeIndex == FilterEventTypeIndex && PacketContentView->IsFilterByEventTypeEnabled())
			{
				PacketContentView->DisableFilterEventType();
			}
			else
			{
				PacketContentView->EnableFilterEventType(EventTypeIndex);
				//PacketContentView->FindFirstEvent();
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Tree View's Table Row
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableRow> SNetStatsView::TreeView_OnGenerateRow(FNetEventNodePtr NetEventNodePtr, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<ITableRow> TableRow =
		SNew(SNetStatsTableRow, OwnerTable)
		.OnShouldBeEnabled(this, &SNetStatsView::TableRow_ShouldBeEnabled)
		.OnIsColumnVisible(this, &SNetStatsView::IsColumnVisible)
		.OnSetHoveredCell(this, &SNetStatsView::TableRow_SetHoveredCell)
		.OnGetColumnOutlineHAlignmentDelegate(this, &SNetStatsView::TableRow_GetColumnOutlineHAlignment)
		.HighlightText(this, &SNetStatsView::TableRow_GetHighlightText)
		.TablePtr(Table)
		.NetEventNodePtr(NetEventNodePtr);

	return TableRow;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsView::TableRow_ShouldBeEnabled(FNetEventNodePtr NodePtr) const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::TableRow_SetHoveredCell(TSharedPtr<FTable> InTablePtr, TSharedPtr<FTableColumn> InColumnPtr, FNetEventNodePtr InNodePtr)
{
	HoveredColumnId = InColumnPtr ? InColumnPtr->GetId() : FName();

	const bool bIsAnyMenusVisible = FSlateApplication::Get().AnyMenusVisible();
	if (!HasMouseCapture() && !bIsAnyMenusVisible)
	{
		HoveredNodePtr = InNodePtr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EHorizontalAlignment SNetStatsView::TableRow_GetColumnOutlineHAlignment(const FName ColumnId) const
{
	const TIndirectArray<SHeaderRow::FColumn>& Columns = TreeViewHeaderRow->GetColumns();

	// First column
	if (Columns.Num() == 0 ||
		Columns[0].ColumnId == ColumnId)
	{
		return HAlign_Left;
	}
	// Last column
	else if (Columns[Columns.Num() - 1].ColumnId == ColumnId)
	{
		return HAlign_Right;
	}
	// Middle columns
	{
		return HAlign_Center;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetStatsView::TableRow_GetHighlightText() const
{
	return SearchBox->GetText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SearchBox
////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::SearchBox_OnTextChanged(const FText& InFilterText)
{
	TextFilter->SetRawFilterText(InFilterText);
	SearchBox->SetError(TextFilter->GetFilterErrorText());
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsView::SearchBox_IsEnabled() const
{
	return NetEventNodes.Num() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Grouping
////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::CreateGroups()
{
	if (GroupingMode == ENetEventGroupingMode::Flat)
	{
		GroupNodes.Reset();

		const FName GroupName(TEXT("All"));
		FNetEventNodePtr GroupPtr = MakeShared<FNetEventNode>(GroupName);
		GroupNodes.Add(GroupPtr);

		for (const FNetEventNodePtr& NodePtr : NetEventNodes)
		{
			GroupPtr->AddChildAndSetParent(NodePtr);
		}
		TreeView->SetItemExpansion(GroupPtr, true);
	}
	// Creates one group for each stat type.
	else if (GroupingMode == ENetEventGroupingMode::ByType)
	{
		TMap<ENetEventNodeType, FNetEventNodePtr> GroupNodeSet;
		for (const FNetEventNodePtr& NodePtr : NetEventNodes)
		{
			const ENetEventNodeType NodeType = NodePtr->GetType();
			FNetEventNodePtr GroupPtr = GroupNodeSet.FindRef(NodeType);
			if (!GroupPtr)
			{
				const FName GroupName = *NetEventNodeTypeHelper::ToText(NodeType).ToString();
				GroupPtr = GroupNodeSet.Add(NodeType, MakeShared<FNetEventNode>(GroupName));
			}
			GroupPtr->AddChildAndSetParent(NodePtr);
			TreeView->SetItemExpansion(GroupPtr, true);
		}
		GroupNodeSet.KeySort([](const ENetEventNodeType& A, const ENetEventNodeType& B) { return A < B; }); // sort groups by type
		GroupNodeSet.GenerateValueArray(GroupNodes);
	}
	// Creates one group for one letter.
	else if (GroupingMode == ENetEventGroupingMode::ByName)
	{
		TMap<TCHAR, FNetEventNodePtr> GroupNodeSet;
		for (const FNetEventNodePtr& NodePtr : NetEventNodes)
		{
			FString FirstLetterStr(NodePtr->GetName().GetPlainNameString().Left(1).ToUpper());
			const TCHAR FirstLetter = FirstLetterStr[0];
			FNetEventNodePtr GroupPtr = GroupNodeSet.FindRef(FirstLetter);
			if (!GroupPtr)
			{
				const FName GroupName(FirstLetterStr);
				GroupPtr = GroupNodeSet.Add(FirstLetter, MakeShared<FNetEventNode>(GroupName));
			}
			GroupPtr->AddChildAndSetParent(NodePtr);
		}
		GroupNodeSet.KeySort([](const TCHAR& A, const TCHAR& B) { return A < B; }); // sort groups alphabetically
		GroupNodeSet.GenerateValueArray(GroupNodes);
	}
	// Creates one group for each level.
	else if (GroupingMode == ENetEventGroupingMode::ByLevel)
	{
		TMap<uint32, FNetEventNodePtr> GroupNodeSet;
		for (const FNetEventNodePtr& NodePtr : NetEventNodes)
		{
			const uint32 Level = NodePtr->GetLevel();
			FNetEventNodePtr GroupPtr = GroupNodeSet.FindRef(Level);
			if (!GroupPtr)
			{
				const FName GroupName(*FString::Printf(TEXT("Level %d"), Level));
				GroupPtr = GroupNodeSet.Add(Level, MakeShared<FNetEventNode>(GroupName));
			}
			GroupPtr->AddChildAndSetParent(NodePtr);
			TreeView->SetItemExpansion(GroupPtr, true);
		}
		GroupNodeSet.KeySort([](const uint32& A, const uint32& B) { return A < B; }); // sort groups by level
		GroupNodeSet.GenerateValueArray(GroupNodes);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::CreateGroupByOptionsSources()
{
	GroupByOptionsSource.Reset(3);

	// Must be added in order of elements in the ENetEventGroupingMode.
	GroupByOptionsSource.Add(MakeShared<ENetEventGroupingMode>(ENetEventGroupingMode::Flat));
	GroupByOptionsSource.Add(MakeShared<ENetEventGroupingMode>(ENetEventGroupingMode::ByName));
	//GroupByOptionsSource.Add(MakeShared<ENetEventGroupingMode>(ENetEventGroupingMode::ByType));
	GroupByOptionsSource.Add(MakeShared<ENetEventGroupingMode>(ENetEventGroupingMode::ByLevel));

	ENetEventGroupingModePtr* GroupingModePtrPtr = GroupByOptionsSource.FindByPredicate([&](const ENetEventGroupingModePtr InGroupingModePtr) { return *InGroupingModePtr == GroupingMode; });
	if (GroupingModePtrPtr != nullptr)
	{
		GroupByComboBox->SetSelectedItem(*GroupingModePtrPtr);
	}

	GroupByComboBox->RefreshOptions();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::GroupBy_OnSelectionChanged(TSharedPtr<ENetEventGroupingMode> NewGroupingMode, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		GroupingMode = *NewGroupingMode;

		CreateGroups();
		SortTreeNodes();
		ApplyFiltering();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetStatsView::GroupBy_OnGenerateWidget(TSharedPtr<ENetEventGroupingMode> InGroupingMode) const
{
	return SNew(STextBlock)
		.Text(NetEventNodeGroupingHelper::ToText(*InGroupingMode))
		.ToolTipText(NetEventNodeGroupingHelper::ToDescription(*InGroupingMode));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetStatsView::GroupBy_GetSelectedText() const
{
	return NetEventNodeGroupingHelper::ToText(GroupingMode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetStatsView::GroupBy_GetSelectedTooltipText() const
{
	return NetEventNodeGroupingHelper::ToDescription(GroupingMode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorting
////////////////////////////////////////////////////////////////////////////////////////////////////

const FName SNetStatsView::GetDefaultColumnBeingSorted()
{
	return FNetStatsViewColumns::TotalInclusiveSizeColumnID;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const EColumnSortMode::Type SNetStatsView::GetDefaultColumnSortMode()
{
	return EColumnSortMode::Descending;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::CreateSortings()
{
	AvailableSorters.Reset();
	CurrentSorter = nullptr;

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		if (ColumnRef->CanBeSorted())
		{
			TSharedPtr<ITableCellValueSorter> SorterPtr = ColumnRef->GetValueSorter();
			if (ensure(SorterPtr.IsValid()))
			{
				AvailableSorters.Add(SorterPtr);
			}
		}
	}

	UpdateCurrentSortingByColumn();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::UpdateCurrentSortingByColumn()
{
	TSharedPtr<FTableColumn> ColumnPtr = Table->FindColumn(ColumnBeingSorted);
	CurrentSorter = ColumnPtr.IsValid() ? ColumnPtr->GetValueSorter() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::SortTreeNodes()
{
	if (CurrentSorter.IsValid())
	{
		for (FNetEventNodePtr& Root : GroupNodes)
		{
			SortTreeNodesRec(*Root, *CurrentSorter);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::SortTreeNodesRec(FNetEventNode& Node, const ITableCellValueSorter& Sorter)
{
	ESortMode SortMode = (ColumnSortMode == EColumnSortMode::Type::Descending) ? ESortMode::Descending : ESortMode::Ascending;
	Node.SortChildren(Sorter, SortMode);

	for (FBaseTreeNodePtr ChildPtr : Node.GetChildren())
	{
		if (ChildPtr->GetChildrenCount() > 0)
		{
			SortTreeNodesRec(*StaticCastSharedPtr<FNetEventNode>(ChildPtr), Sorter);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EColumnSortMode::Type SNetStatsView::GetSortModeForColumn(const FName ColumnId) const
{
	if (ColumnBeingSorted != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return ColumnSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::SetSortModeForColumn(const FName& ColumnId, const EColumnSortMode::Type SortMode)
{
	ColumnBeingSorted = ColumnId;
	ColumnSortMode = SortMode;
	UpdateCurrentSortingByColumn();

	SortTreeNodes();
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode)
{
	SetSortModeForColumn(ColumnId, SortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortMode action (HeaderMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsView::HeaderMenu_SortMode_IsChecked(const FName ColumnId, const EColumnSortMode::Type InSortMode)
{
	return ColumnBeingSorted == ColumnId && ColumnSortMode == InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsView::HeaderMenu_SortMode_CanExecute(const FName ColumnId, const EColumnSortMode::Type InSortMode) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.CanBeSorted();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::HeaderMenu_SortMode_Execute(const FName ColumnId, const EColumnSortMode::Type InSortMode)
{
	SetSortModeForColumn(ColumnId, InSortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortMode action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsView::ContextMenu_SortMode_IsChecked(const EColumnSortMode::Type InSortMode)
{
	return ColumnSortMode == InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsView::ContextMenu_SortMode_CanExecute(const EColumnSortMode::Type InSortMode) const
{
	return true; //ColumnSortMode != InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::ContextMenu_SortMode_Execute(const EColumnSortMode::Type InSortMode)
{
	SetSortModeForColumn(ColumnBeingSorted, InSortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortByColumn action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsView::ContextMenu_SortByColumn_IsChecked(const FName ColumnId)
{
	return ColumnId == ColumnBeingSorted;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsView::ContextMenu_SortByColumn_CanExecute(const FName ColumnId) const
{
	return true; //ColumnId != ColumnBeingSorted;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::ContextMenu_SortByColumn_Execute(const FName ColumnId)
{
	SetSortModeForColumn(ColumnId, EColumnSortMode::Descending);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ShowColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsView::CanShowColumn(const FName ColumnId) const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::ShowColumn(const FName ColumnId)
{
	FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	Column.Show();

	SHeaderRow::FColumn::FArguments ColumnArgs;
	ColumnArgs
		.ColumnId(Column.GetId())
		.DefaultLabel(Column.GetShortName())
		.ToolTip(SNetStatsViewTooltip::GetColumnTooltip(Column))
		.HAlignHeader(Column.GetHorizontalAlignment())
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Fill)
		.InitialSortMode(Column.GetInitialSortMode())
		.SortMode(this, &SNetStatsView::GetSortModeForColumn, Column.GetId())
		.OnSort(this, &SNetStatsView::OnSortModeChanged)
		.FillWidth(Column.GetInitialWidth())
		//.FixedWidth(Column.IsFixedWidth() ? Column.GetInitialWidth() : TOptional<float>())
		.HeaderContent()
		[
			SNew(SBox)
			.HeightOverride(24.0f)
			.Padding(FMargin(0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SNetStatsView::GetColumnHeaderText, Column.GetId())
			]
		]
		.MenuContent()
		[
			TreeViewHeaderRow_GenerateColumnMenu(Column)
		];

	int32 ColumnIndex = 0;
	const int32 NewColumnPosition = Table->GetColumnPositionIndex(ColumnId);
	const int32 NumColumns = TreeViewHeaderRow->GetColumns().Num();
	for (; ColumnIndex < NumColumns; ColumnIndex++)
	{
		const SHeaderRow::FColumn& CurrentColumn = TreeViewHeaderRow->GetColumns()[ColumnIndex];
		const int32 CurrentColumnPosition = Table->GetColumnPositionIndex(CurrentColumn.ColumnId);
		if (NewColumnPosition < CurrentColumnPosition)
		{
			break;
		}
	}

	TreeViewHeaderRow->InsertColumn(ColumnArgs, ColumnIndex);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// HideColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsView::CanHideColumn(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.CanBeHidden();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::HideColumn(const FName ColumnId)
{
	FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	Column.Hide();

	TreeViewHeaderRow->RemoveColumn(ColumnId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ToggleColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsView::IsColumnVisible(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsView::CanToggleColumnVisibility(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return !Column.IsVisible() || Column.CanBeHidden();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::ToggleColumnVisibility(const FName ColumnId)
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	if (Column.IsVisible())
	{
		HideColumn(ColumnId);
	}
	else
	{
		ShowColumn(ColumnId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// "Show All Columns" action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsView::ContextMenu_ShowAllColumns_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::ContextMenu_ShowAllColumns_Execute()
{
	ColumnBeingSorted = GetDefaultColumnBeingSorted();
	ColumnSortMode = GetDefaultColumnSortMode();
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (!Column.IsVisible())
		{
			ShowColumn(Column.GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ResetColumns action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SNetStatsView::ContextMenu_ResetColumns_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::ContextMenu_ResetColumns_Execute()
{
	ColumnBeingSorted = GetDefaultColumnBeingSorted();
	ColumnSortMode = GetDefaultColumnSortMode();
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (Column.ShouldBeVisible() && !Column.IsVisible())
		{
			ShowColumn(Column.GetId());
		}
		else if (!Column.ShouldBeVisible() && Column.IsVisible())
		{
			HideColumn(Column.GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::Reset()
{
	GameInstanceIndex = 0;
	ConnectionIndex = 0;
	ConnectionMode = TraceServices::ENetProfilerConnectionMode::Outgoing;

	StatsPacketStartIndex = 0;
	StatsPacketEndIndex = 0;
	StatsStartPosition = 0;
	StatsEndPosition = 0;

	RebuildTree(true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Check if we need to update the lists of net events, but not too often.
	const uint64 Time = FPlatformTime::Cycles64();
	if (Time > NextTimestamp)
	{
		RebuildTree(false);

		const uint64 WaitTime = static_cast<uint64>(0.5 / FPlatformTime::GetSecondsPerCycle64()); // 500ms
		NextTimestamp = Time + WaitTime;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::RebuildTree(bool bResync)
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	FStopwatch SyncStopwatch;
	SyncStopwatch.Start();

	if (bResync)
	{
		NetEventNodes.Empty();
	}

	const uint32 PreviousNodeCount = NetEventNodes.Num();

	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
		if (NetProfilerProvider)
		{
			NetProfilerProvider->ReadEventTypes([this, &bResync, NetProfilerProvider](const TraceServices::FNetProfilerEventType* NetEvents, uint64 InNetEventCount)
			{
				const uint32 NetEventCount = static_cast<uint32>(InNetEventCount);
				if (NetEventCount != NetEventNodes.Num())
				{
					NetEventNodes.Empty(NetEventCount);
					for (uint32 Index = 0; Index < NetEventCount; ++Index)
					{
						const TraceServices::FNetProfilerEventType& NetEvent = NetEvents[Index];
						ensure(NetEvent.EventTypeIndex == Index);
						const TCHAR* NamePtr = nullptr;
						NetProfilerProvider->ReadName(NetEvent.NameIndex, [&NamePtr](const TraceServices::FNetProfilerName& InName)
						{
							NamePtr = InName.Name;
						});
						const FName Name(NamePtr);
						const ENetEventNodeType Type = ENetEventNodeType::NetEvent;
						FNetEventNodePtr NetEventNodePtr = MakeShared<FNetEventNode>(NetEvent.EventTypeIndex, Name, Type, NetEvent.Level);
						NetEventNodes.Add(NetEventNodePtr);
					}
					ensure(NetEventNodes.Num() == NetEventCount);
				}
			});
		}
	}

	SyncStopwatch.Stop();

	if (bResync || NetEventNodes.Num() != PreviousNodeCount)
	{
		UpdateTree();
		UpdateStatsInternal();

		// Save selection.
		TArray<FNetEventNodePtr> SelectedItems;
		TreeView->GetSelectedItems(SelectedItems);

		TreeView->RebuildList();

		// Restore selection.
		if (SelectedItems.Num() > 0)
		{
			TreeView->ClearSelection();
			for (FNetEventNodePtr& NodePtr : SelectedItems)
			{
				NodePtr = GetNetEventNode(NodePtr->GetEventTypeIndex());
			}
			SelectedItems.RemoveAll([](const FNetEventNodePtr& NodePtr) { return !NodePtr.IsValid(); });
			if (SelectedItems.Num() > 0)
			{
				TreeView->SetItemSelection(SelectedItems, true);
				TreeView->RequestScrollIntoView(SelectedItems.Last());
			}
		}
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.01)
	{
		const double SyncTime = SyncStopwatch.GetAccumulatedTime();
		UE_LOGF(LogNetworkingProfiler, Log, "[NetStats] Tree view rebuilt in %.4fs (sync: %.4fs + update: %.4fs) --> %d net events (%d added)",
			TotalTime, SyncTime, TotalTime - SyncTime, NetEventNodes.Num(), NetEventNodes.Num() - PreviousNodeCount);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::ResetStats()
{
	GameInstanceIndex = 0;
	ConnectionIndex = 0;
	ConnectionMode = TraceServices::ENetProfilerConnectionMode::Outgoing;

	StatsPacketStartIndex = 0;
	StatsPacketEndIndex = 0;
	StatsStartPosition = 0;
	StatsEndPosition = 0;

	UpdateStatsInternal();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::UpdateStats(uint32 InGameInstanceIndex, uint32 InConnectionIndex, TraceServices::ENetProfilerConnectionMode InConnectionMode, uint32 InStatsPacketStartIndex, uint32 InStatsPacketEndIndex, uint32 InStatsStartPosition, uint32 InStatsEndPosition)
{
	GameInstanceIndex = InGameInstanceIndex;
	ConnectionIndex = InConnectionIndex;
	ConnectionMode = InConnectionMode;

	StatsPacketStartIndex = InStatsPacketStartIndex;
	StatsPacketEndIndex = InStatsPacketEndIndex;
	StatsStartPosition = InStatsStartPosition;
	StatsEndPosition = InStatsEndPosition;

	UpdateStatsInternal();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::UpdateStatsInternal()
{
	if (StatsPacketStartIndex >= StatsPacketEndIndex || // no packet selected?
		(StatsPacketEndIndex - StatsPacketStartIndex == 1 && StatsStartPosition >= StatsEndPosition)) // single packet, but invalid bit range?
	{
		// keep previous aggregated stats
		return;
	}

	FStopwatch AggregationStopwatch;
	FStopwatch Stopwatch;
	Stopwatch.Start();

	for (const FNetEventNodePtr& NetEventNodePtr : NetEventNodes)
	{
		NetEventNodePtr->ResetAggregatedStats();
	}

	if (Session.IsValid())
	{
		TUniquePtr<TraceServices::ITable<TraceServices::FNetProfilerAggregatedStats>> AggregationResultTable;

		AggregationStopwatch.Start();
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
			if (NetProfilerProvider)
			{
				// CreateAggregation requires [PacketStartIndex, PacketEndIndex] as inclusive interval and [StartPos, EndPos) as exclusive interval.
				AggregationResultTable.Reset(NetProfilerProvider->CreateAggregation(ConnectionIndex, ConnectionMode, StatsPacketStartIndex, StatsPacketEndIndex - 1, StatsStartPosition, StatsEndPosition));
			}
		}
		AggregationStopwatch.Stop();

		if (AggregationResultTable.IsValid())
		{
			TUniquePtr<TraceServices::ITableReader<TraceServices::FNetProfilerAggregatedStats>> TableReader(AggregationResultTable->CreateReader());
			while (TableReader->IsValid())
			{
				const TraceServices::FNetProfilerAggregatedStats* Row = TableReader->GetCurrentRow();
				FNetEventNodePtr NetEventNodePtr = GetNetEventNode(Row->EventTypeIndex);
				if (NetEventNodePtr)
				{
					NetEventNodePtr->SetAggregatedStats(*Row);

					TSharedPtr<ITableRow> TableRowPtr = TreeView->WidgetFromItem(NetEventNodePtr);
					if (TableRowPtr.IsValid())
					{
						TSharedPtr<SNetStatsTableRow> RowPtr = StaticCastSharedPtr<SNetStatsTableRow, ITableRow>(TableRowPtr);
						RowPtr->InvalidateContent();
					}
				}
				TableReader->NextRow();
			}
		}
	}

	UpdateTree();

	const TArray<FNetEventNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	if (SelectedNodes.Num() > 0)
	{
		TreeView->RequestScrollIntoView(SelectedNodes[0]);
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	const double AggregationTime = AggregationStopwatch.GetAccumulatedTime();
	UE_LOGF(LogNetworkingProfiler, Log, "[NetStats] Aggregated stats updated in %.4fs (%.4fs + %.4fs)",
		TotalTime, AggregationTime, TotalTime - AggregationTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FNetEventNodePtr SNetStatsView::GetNetEventNode(uint32 EventTypeIndex) const
{
	return (EventTypeIndex < (uint32)NetEventNodes.Num()) ? NetEventNodes[EventTypeIndex] : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsView::SelectNetEventNode(uint32 EventTypeIndex)
{
	FNetEventNodePtr NodePtr = GetNetEventNode(EventTypeIndex);
	if (NodePtr)
	{
		TreeView->SetSelection(NodePtr);
		TreeView->RequestScrollIntoView(NodePtr);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
