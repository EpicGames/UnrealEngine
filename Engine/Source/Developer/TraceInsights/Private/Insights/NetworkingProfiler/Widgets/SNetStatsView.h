// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Misc/FilterCollection.h"
#include "Misc/TextFilter.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

// TraceInsights
#include "Insights/NetworkingProfiler/ViewModels/NetEventGroupingAndSorting.h"
#include "Insights/NetworkingProfiler/ViewModels/NetEventNode.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMenuBuilder;

namespace TraceServices
{
	class IAnalysisSession;
}

namespace UE::Insights
{
	class FTable;
	class FTableColumn;
	class ITableCellValueSorter;
}

namespace UE::Insights::NetworkingProfiler
{

class SNetworkingProfilerWindow;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** The filter collection - used for updating the list of tree nodes. */
typedef TFilterCollection<const FNetEventNodePtr&> FNetEventNodeFilterCollection;

/** The text based filter - used for updating the list of tree nodes. */
typedef TTextFilter<const FNetEventNodePtr&> FNetEventNodeTextFilter;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * A custom widget used to display the list of net events and their aggregated stats.
 */
class SNetStatsView : public SCompoundWidget
{
public:
	/** Default constructor. */
	SNetStatsView();

	/** Virtual destructor. */
	virtual ~SNetStatsView();

	SLATE_BEGIN_ARGS(SNetStatsView) {}
	SLATE_END_ARGS()

	/**
	 * Converts profiler window weak pointer to a shared pointer and returns it.
	 * Make sure the returned pointer is valid before trying to dereference it.
	 */
	TSharedPtr<SNetworkingProfilerWindow> GetProfilerWindow() const
	{
		return ProfilerWindowWeakPtr.Pin();
	}

	TSharedPtr<FTable> GetTable() const { return Table; }

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs, TSharedRef<SNetworkingProfilerWindow> InProfilerWindow);

	/**
	 * Ticks this widget. Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry - the space allotted for this widget
	 * @param  InCurrentTime    - current absolute real time
	 * @param  InDeltaTime      - real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void Reset();

	/**
	 * Rebuilds the tree (if necessary).
	 * @param bResync - If true, it forces a resync with list of net events from Analysis, even if the list did not changed since last sync.
	 */
	void RebuildTree(bool bResync);

	void ResetStats();
	void UpdateStats(uint32 InGameInstanceIndex, uint32 InConnectionIndex, TraceServices::ENetProfilerConnectionMode InConnectionMode, uint32 InStatsPacketStartIndex, uint32 InStatsPacketEndIndex, uint32 InStatsStartPosition, uint32 InStatsEndPosition);

	FNetEventNodePtr GetNetEventNode(uint32 EventTypeIndex) const;
	void SelectNetEventNode(uint32 EventTypeIndex);

protected:
	void UpdateTree();

	void UpdateStatsInternal();

	/** Called when the analysis session has changed. */
	void InsightsManager_OnSessionChanged();

	/**
	 * Populates OutSearchStrings with the strings that should be used in searching.
	 *
	 * @param GroupOrStatNodePtr - the group and stat node to get a text description from.
	 * @param OutSearchStrings   - an array of strings to use in searching.
	 *
	 */
	void HandleItemToStringArray(const FNetEventNodePtr& GroupOrStatNodePtr, TArray<FString>& OutSearchStrings) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Context Menu

	TSharedPtr<SWidget> TreeView_GetMenuContent();
	void TreeView_BuildSortByMenu(FMenuBuilder& MenuBuilder);
	void TreeView_BuildViewColumnMenu(FMenuBuilder& MenuBuilder);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Columns' Header

	void InitializeAndShowHeaderColumns();

	FText GetColumnHeaderText(const FName ColumnId) const;

	TSharedRef<SWidget> TreeViewHeaderRow_GenerateColumnMenu(const FTableColumn& Column);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Misc

	void TreeView_Refresh();

	/**
	 * Called by STreeView to retrieves the children for the specified parent item.
	 * @param InParent    - The parent node to retrieve the children from.
	 * @param OutChildren - List of children for the parent node.
	 */
	void TreeView_OnGetChildren(FNetEventNodePtr InParent, TArray<FNetEventNodePtr>& OutChildren);

	/** Called by STreeView when selection has changed. */
	void TreeView_OnSelectionChanged(FNetEventNodePtr SelectedItem, ESelectInfo::Type SelectInfo);

	/** Called by STreeView when a tree item is double clicked. */
	void TreeView_OnMouseButtonDoubleClick(FNetEventNodePtr TreeNode);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Table Row

	/** Called by STreeView to generate a table row for the specified item. */
	TSharedRef<ITableRow> TreeView_OnGenerateRow(FNetEventNodePtr TreeNode, const TSharedRef<STableViewBase>& OwnerTable);

	bool TableRow_ShouldBeEnabled(FNetEventNodePtr NodePtr) const;

	void TableRow_SetHoveredCell(TSharedPtr<FTable> TablePtr, TSharedPtr<FTableColumn> ColumnPtr, FNetEventNodePtr NodePtr);
	EHorizontalAlignment TableRow_GetColumnOutlineHAlignment(const FName ColumnId) const;

	FText TableRow_GetHighlightText() const;
	
	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Filtering

	/** Populates the group and stat tree with items based on the current data. */
	void ApplyFiltering();

	void FilterOutZeroCountEvents_OnCheckStateChanged(ECheckBoxState NewRadioState);
	ECheckBoxState FilterOutZeroCountEvents_IsChecked() const;

	TSharedRef<SWidget> GetToggleButtonForNetEventType(const ENetEventNodeType InNetEventType);
	void FilterByNetEventType_OnCheckStateChanged(ECheckBoxState NewRadioState, const ENetEventNodeType InNetEventType);
	ECheckBoxState FilterByNetEventType_IsChecked(const ENetEventNodeType InNetEventType) const;

	bool SearchBox_IsEnabled() const;
	void SearchBox_OnTextChanged(const FText& InFilterText);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Grouping

	void CreateGroups();
	void CreateGroupByOptionsSources();

	void GroupBy_OnSelectionChanged(TSharedPtr<ENetEventGroupingMode> NewGroupingMode, ESelectInfo::Type SelectInfo);

	TSharedRef<SWidget> GroupBy_OnGenerateWidget(TSharedPtr<ENetEventGroupingMode> InGroupingMode) const;

	FText GroupBy_GetSelectedText() const;

	FText GroupBy_GetSelectedTooltipText() const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Sorting

	static const FName GetDefaultColumnBeingSorted();
	static const EColumnSortMode::Type GetDefaultColumnSortMode();

	void CreateSortings();

	void UpdateCurrentSortingByColumn();
	void SortTreeNodes();
	void SortTreeNodesRec(FNetEventNode& Node, const ITableCellValueSorter& Sorter);

	EColumnSortMode::Type GetSortModeForColumn(const FName ColumnId) const;
	void SetSortModeForColumn(const FName& ColumnId, EColumnSortMode::Type SortMode);
	void OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Sorting actions

	// SortMode (HeaderMenu)
	bool HeaderMenu_SortMode_IsChecked(const FName ColumnId, const EColumnSortMode::Type InSortMode);
	bool HeaderMenu_SortMode_CanExecute(const FName ColumnId, const EColumnSortMode::Type InSortMode) const;
	void HeaderMenu_SortMode_Execute(const FName ColumnId, const EColumnSortMode::Type InSortMode);

	// SortMode (ContextMenu)
	bool ContextMenu_SortMode_IsChecked(const EColumnSortMode::Type InSortMode);
	bool ContextMenu_SortMode_CanExecute(const EColumnSortMode::Type InSortMode) const;
	void ContextMenu_SortMode_Execute(const EColumnSortMode::Type InSortMode);

	// SortByColumn (ContextMenu)
	bool ContextMenu_SortByColumn_IsChecked(const FName ColumnId);
	bool ContextMenu_SortByColumn_CanExecute(const FName ColumnId) const;
	void ContextMenu_SortByColumn_Execute(const FName ColumnId);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Column visibility actions

	// ShowColumn
	bool CanShowColumn(const FName ColumnId) const;
	void ShowColumn(const FName ColumnId);

	// HideColumn
	bool CanHideColumn(const FName ColumnId) const;
	void HideColumn(const FName ColumnId);

	// ToggleColumnVisibility
	bool IsColumnVisible(const FName ColumnId) const;
	bool CanToggleColumnVisibility(const FName ColumnId) const;
	void ToggleColumnVisibility(const FName ColumnId);

	// ShowAllColumns (ContextMenu)
	bool ContextMenu_ShowAllColumns_CanExecute() const;
	void ContextMenu_ShowAllColumns_Execute();

	// MinMaxMedColumns (ContextMenu)
	bool ContextMenu_ShowMinMaxMedColumns_CanExecute() const;
	void ContextMenu_ShowMinMaxMedColumns_Execute();

	// ResetColumns (ContextMenu)
	bool ContextMenu_ResetColumns_CanExecute() const;
	void ContextMenu_ResetColumns_Execute();

	////////////////////////////////////////////////////////////////////////////////////////////////////

protected:
	/** A weak pointer to the Networking Insights window. */
	TWeakPtr<SNetworkingProfilerWindow> ProfilerWindowWeakPtr;

	/** Table view model. */
	TSharedPtr<FTable> Table;

	/** The analysis session used to populate this widget. */
	TSharedPtr<const TraceServices::IAnalysisSession> Session;

	//////////////////////////////////////////////////
	// Tree View, Columns

	/** The tree widget which holds the net event tree nodes. */
	TSharedPtr<STreeView<FNetEventNodePtr>> TreeView;

	/** Holds the tree view header row widget which display all columns in the tree view. */
	TSharedPtr<SHeaderRow> TreeViewHeaderRow;

	/** External scrollbar used to synchronize tree view position. */
	TSharedPtr<SScrollBar> ExternalScrollbar;

	//////////////////////////////////////////////////
	// Hovered Column, Hovered Node

	/** Name of the column currently being hovered by the mouse. */
	FName HoveredColumnId;

	/** A shared pointer to the node currently being hovered by the mouse. */
	FNetEventNodePtr HoveredNodePtr;

	//////////////////////////////////////////////////
	// Net Event Nodes

	/** An array of group nodes. */
	TArray<FNetEventNodePtr> GroupNodes;

	/** A filtered array of group nodes to be displayed in the tree widget. */
	TArray<FNetEventNodePtr> FilteredGroupNodes;

	/** All net event nodes. Index in this array is EventTypeIndex. */
	TArray<FNetEventNodePtr> NetEventNodes;

	/** Currently expanded group nodes. */
	TSet<FNetEventNodePtr> ExpandedNodes;

	/** If true, the expanded nodes have been saved before applying a text filter. */
	bool bExpansionSaved;

	//////////////////////////////////////////////////
	// Search box and filters

	//bool bUseFiltering;

	/** The search box widget used to filter items displayed in the tree. */
	TSharedPtr<SSearchBox> SearchBox;

	/** The text based filter. */
	TSharedPtr<FNetEventNodeTextFilter> TextFilter;

	/** The filter collection. */
	TSharedPtr<FNetEventNodeFilterCollection> Filters;

	/** Holds the visibility of each net event type. */
	bool bNetEventTypeIsVisible[static_cast<int>(ENetEventNodeType::InvalidOrMax)];

	/** Filter out the net event types having zero total instance count (aggregated stats). */
	bool bFilterOutZeroCountEvents;

	//////////////////////////////////////////////////
	// Grouping

	//bool bUseGrouping;

	TArray<TSharedPtr<ENetEventGroupingMode>> GroupByOptionsSource;

	TSharedPtr<SComboBox<TSharedPtr<ENetEventGroupingMode>>> GroupByComboBox;

	/** How we group the net event nodes? */
	ENetEventGroupingMode GroupingMode;

	//////////////////////////////////////////////////
	// Sorting

	//bool bUseSorting;

	/** All available sorters. */
	TArray<TSharedPtr<ITableCellValueSorter>> AvailableSorters;

	/** Current sorter. It is nullptr if sorting is disabled. */
	TSharedPtr<ITableCellValueSorter> CurrentSorter;

	/** Name of the column currently being sorted. Can be NAME_None if sorting is disabled (CurrentSorting == nullptr) or if a complex sorting is used (CurrentSorting != nullptr). */
	FName ColumnBeingSorted;

	/** How we sort the nodes? Ascending or Descending. */
	EColumnSortMode::Type ColumnSortMode;

	//////////////////////////////////////////////////

	uint64 NextTimestamp;
	uint32 ObjectsChangeCount;

	uint32 GameInstanceIndex;
	uint32 ConnectionIndex;
	TraceServices::ENetProfilerConnectionMode ConnectionMode;
	uint32 StatsPacketStartIndex;
	uint32 StatsPacketEndIndex;
	uint32 StatsStartPosition;
	uint32 StatsEndPosition;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
