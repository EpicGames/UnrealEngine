// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNetStatsViewTooltip.h"

#include "SlateOptMacros.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

// TraceServices
#include "TraceServices/Model/NetProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/ViewModels/NetEventNode.h"
#include "Insights/NetworkingProfiler/ViewModels/NetEventNodeHelper.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::SNetStatsView"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedPtr<SToolTip> SNetStatsViewTooltip::GetTableTooltip(const FTable& Table)
{
	TSharedPtr<SToolTip> ColumnTooltip =
		SNew(SToolTip)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Table.GetDisplayName())
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Table.GetDescription())
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			]
		];

	return ColumnTooltip;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SToolTip> SNetStatsViewTooltip::GetColumnTooltip(const FTableColumn& Column)
{
	TSharedPtr<SToolTip> ColumnTooltip =
		SNew(SToolTip)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Column.GetTitleName())
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Column.GetDescription())
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			]
		];

	return ColumnTooltip;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SToolTip> SNetStatsViewTooltip::GetRowTooltip(const TSharedPtr<FNetEventNode> NetEventNodePtr)
{
	const TraceServices::FNetProfilerAggregatedStats& Stats = NetEventNodePtr->GetAggregatedStats();

	const FText InstanceCountText = FText::AsNumber(Stats.InstanceCount);

	const FText TotalInclusiveSizeText = FText::AsNumber(Stats.TotalInclusive);
	const FText MaxInclusiveSizeText = FText::AsNumber(Stats.MaxInclusive);
	const FText AvgInclusiveSizeText = FText::AsNumber(Stats.AverageInclusive);

	const FText TotalExclusiveSizeText = FText::AsNumber(Stats.TotalExclusive);
	const FText MaxExclusiveSizeText = FText::AsNumber(Stats.MaxExclusive);
	//const FText AvgExclusiveSizeText = FText::AsNumber(Stats.AverageExclusive);
	const FText AvgExclusiveSizeText;

	TSharedPtr<SGridPanel> GridPanel;
	TSharedPtr<SHorizontalBox> HBox;

	TSharedPtr<SToolTip> TableCellTooltip =
		SNew(SToolTip)
		[
			SAssignNew(HBox, SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SGridPanel)

					// Event Type Index: [Index]
					+ SGridPanel::Slot(0, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_Id", "Event Type Index:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(1, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(FText::AsNumber(NetEventNodePtr->GetEventTypeIndex()))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					]

					// Name: [Name]
					+ SGridPanel::Slot(0, 1)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_Name", "Name:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(1, 1)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.WrapTextAt(512.0f)
						.WrappingPolicy(ETextWrappingPolicy::AllowPerCharacterWrapping)
						.Text(FText::FromName(NetEventNodePtr->GetName()))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					]

					//// Group: [MetaGroupName]
					//+ SGridPanel::Slot(0, 2)
					//.Padding(2.0f)
					//[
					//	SNew(STextBlock)
					//	.Text(LOCTEXT("TT_Group", "Group:"))
					//	.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					//]
					//+ SGridPanel::Slot(1, 2)
					//.Padding(2.0f)
					//[
					//	SNew(STextBlock)
					//	.WrapTextAt(512.0f)
					//	.WrappingPolicy(ETextWrappingPolicy::AllowPerCharacterWrapping)
					//	.Text(FText::FromName(NetEventNodePtr->GetMetaGroupName()))
					//	.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					//]

					// Net Event Type: [Type]
					+ SGridPanel::Slot(0, 3)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_Type", "Type:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(1, 3)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(NetEventNodeTypeHelper::ToText(NetEventNodePtr->GetType()))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SGridPanel)

					+ SGridPanel::Slot(0, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_NumInstances", "Num Instances:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(1, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(InstanceCountText)
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SAssignNew(GridPanel, SGridPanel)

					+ SGridPanel::Slot(1, 0)
					.Padding(2.0f)
					.HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_InclusiveSize", "Inclusive"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(2, 0)
					.Padding(FMargin(8.0f, 2.0f, 2.0f, 2.0f))
					.HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_ExclusiveSize", "Exclusive"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]

					// Stats are added here.
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]
			]
		];

	int32 Row = 1;
	AddStatsRow(GridPanel, Row, LOCTEXT("TT_TotalSize",   "Total Size:"),   TotalInclusiveSizeText, TotalExclusiveSizeText);
	AddStatsRow(GridPanel, Row, LOCTEXT("TT_MaxSize",     "Max Size:"),     MaxInclusiveSizeText,   MaxExclusiveSizeText);
	AddStatsRow(GridPanel, Row, LOCTEXT("TT_AverageSize", "Average Size:"), AvgInclusiveSizeText,   AvgExclusiveSizeText);

	return TableCellTooltip;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetStatsViewTooltip::AddStatsRow(TSharedPtr<SGridPanel> Grid, int32& Row, const FText& Name, const FText& Value1, const FText& Value2)
{
	Grid->AddSlot(0, Row)
		.Padding(2.0f)
		[
			SNew(STextBlock)
			.Text(Name)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
		];

	Grid->AddSlot(1, Row)
		.Padding(2.0f)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(Value1)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
		];

	Grid->AddSlot(2, Row)
		.Padding(FMargin(8.0f, 2.0f, 2.0f, 2.0f))
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(Value2)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
		];

	Row++;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
