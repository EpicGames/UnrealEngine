// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/IToolTip.h"
#include "Widgets/SToolTip.h"

class SGridPanel;

namespace UE::Insights
{
	class FTable;
	class FTableColumn;
}

namespace UE::Insights::NetworkingProfiler
{

class FNetEventNode;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** NetStats View Tooltip */
class SNetStatsViewTooltip
{
public:
	SNetStatsViewTooltip() = delete;

	static TSharedPtr<SToolTip> GetTableTooltip(const FTable& Table);
	static TSharedPtr<SToolTip> GetColumnTooltip(const FTableColumn& Column);
	static TSharedPtr<SToolTip> GetRowTooltip(const TSharedPtr<FNetEventNode> TreeNodePtr);

private:
	static void AddStatsRow(TSharedPtr<SGridPanel> Grid, int32& Row, const FText& Name, const FText& Value1, const FText& Value2);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class SNetEventTableRowToolTip : public IToolTip
{
public:
	SNetEventTableRowToolTip(const TSharedPtr<FNetEventNode> InTreeNodePtr) : TreeNodePtr(InTreeNodePtr) {}
	virtual ~SNetEventTableRowToolTip() = default;

	virtual TSharedRef<class SWidget> AsWidget()
	{
		CreateToolTipWidget();
		return ToolTipWidget.ToSharedRef();
	}

	virtual TSharedRef<SWidget> GetContentWidget()
	{
		CreateToolTipWidget();
		return ToolTipWidget->GetContentWidget();
	}

	virtual void SetContentWidget(const TSharedRef<SWidget>& InContentWidget)
	{
		CreateToolTipWidget();
		ToolTipWidget->SetContentWidget(InContentWidget);
	}

	void InvalidateWidget()
	{
		ToolTipWidget.Reset();
	}

	virtual bool IsEmpty() const { return false; }
	virtual bool IsInteractive() const { return false; }
	virtual void OnOpening() {}
	virtual void OnClosed() {}

private:
	void CreateToolTipWidget()
	{
		if (!ToolTipWidget.IsValid())
		{
			ToolTipWidget = SNetStatsViewTooltip::GetRowTooltip(TreeNodePtr);
		}
	}

private:
	TSharedPtr<SToolTip> ToolTipWidget;
	const TSharedPtr<FNetEventNode> TreeNodePtr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
