// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNetworkingProfilerToolbar.h"

#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SBoxPanel.h"

// TraceInsights
#include "Insights/InsightsCommands.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/NetworkingProfilerCommands.h"
#include "Insights/NetworkingProfiler/Widgets/SNetworkingProfilerWindow.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

SNetworkingProfilerToolbar::SNetworkingProfilerToolbar()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SNetworkingProfilerToolbar::~SNetworkingProfilerToolbar()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerToolbar::Construct(const FArguments& InArgs, TSharedRef<SNetworkingProfilerWindow> InProfilerWindow, const FInsightsMajorTabConfig& Config)
{
	struct Local
	{
		static void FillViewToolbar(FToolBarBuilder& ToolbarBuilder, const FArguments &InArgs, TSharedRef<SNetworkingProfilerWindow> ProfilerWindow, const FInsightsMajorTabConfig& Config)
		{
			ToolbarBuilder.BeginSection("View");
			{
				if (Config.ShouldRegisterMinorTab(FNetworkingProfilerTabs::PacketViewID))
				{
					ToolbarBuilder.AddToolBarButton(FNetworkingProfilerCommands::Get().TogglePacketViewVisibility,
						NAME_None, TAttribute<FText>(), TAttribute<FText>(),
						FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.PacketView.ToolBar"));
				}

				if (Config.ShouldRegisterMinorTab(FNetworkingProfilerTabs::PacketContentViewID))
				{
					ToolbarBuilder.AddToolBarButton(FNetworkingProfilerCommands::Get().TogglePacketContentViewVisibility,
						NAME_None, TAttribute<FText>(), TAttribute<FText>(),
						FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.PacketContentView.ToolBar"));
				}

				if (Config.ShouldRegisterMinorTab(FNetworkingProfilerTabs::NetStatsViewID))
				{
					ToolbarBuilder.AddToolBarButton(FNetworkingProfilerCommands::Get().ToggleNetStatsViewVisibility,
						NAME_None, TAttribute<FText>(), TAttribute<FText>(),
						FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.NetStatsView.ToolBar"));
				}

				if (Config.ShouldRegisterMinorTab(FNetworkingProfilerTabs::NetStatsCountersViewID))
				{
					ToolbarBuilder.AddToolBarButton(FNetworkingProfilerCommands::Get().ToggleNetStatsCountersViewVisibility,
						NAME_None, TAttribute<FText>(), TAttribute<FText>(),
						FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.NetStatsView.ToolBar"));
				}
			}
			ToolbarBuilder.EndSection();
			ToolbarBuilder.BeginSection("Connection");
			{
				TSharedRef<SWidget> GameInstanceWidget = ProfilerWindow->CreateGameInstanceComboBox();
				ToolbarBuilder.AddWidget(GameInstanceWidget);

				TSharedRef<SWidget> ConnectionWidget = ProfilerWindow->CreateConnectionComboBox();
				ToolbarBuilder.AddWidget(ConnectionWidget);

				TSharedRef<SWidget> ConnectionModeWidget = ProfilerWindow->CreateConnectionModeComboBox();
				ToolbarBuilder.AddWidget(ConnectionModeWidget);
			}
			ToolbarBuilder.EndSection();
			//ToolbarBuilder.BeginSection("New");
			//{
			//	//TODO: ToolbarBuilder.AddToolBarButton(FNetworkingProfilerCommands::Get().OpenNewWindow,
			//		NAME_None, TAttribute<FText>(), TAttribute<FText>(),
			//		FSlateIcon(FInsightsStyle::GetStyleSetName(), "NewNetworkingInsights.Icon"));
			//}
			//ToolbarBuilder.EndSection();

			if (InArgs._ToolbarExtender.IsValid())
			{
				InArgs._ToolbarExtender->Apply("MainToolbar", EExtensionHook::First, ToolbarBuilder);
			}
		}

		static void FillRightSideToolbar(FToolBarBuilder& ToolbarBuilder, const FArguments &InArgs)
		{
			ToolbarBuilder.BeginSection("Debug");
			{
				ToolbarBuilder.AddToolBarButton(FInsightsCommands::Get().ToggleDebugInfo,
					NAME_None, FText(), TAttribute<FText>(),
					FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.Debug.ToolBar"));
			}
			ToolbarBuilder.EndSection();

			if (InArgs._ToolbarExtender.IsValid())
			{
				InArgs._ToolbarExtender->Apply("RightSideToolbar", EExtensionHook::First, ToolbarBuilder);
			}
		}
	};

	const TSharedPtr<FUICommandList> CommandList = InProfilerWindow->GetCommandList();

	FSlimHorizontalToolBarBuilder ToolbarBuilder(CommandList.ToSharedRef(), FMultiBoxCustomization::None);
	ToolbarBuilder.SetStyle(&FInsightsStyle::Get(), "PrimaryToolbar");
	Local::FillViewToolbar(ToolbarBuilder, InArgs, InProfilerWindow, Config);

	FSlimHorizontalToolBarBuilder RightSideToolbarBuilder(CommandList.ToSharedRef(), FMultiBoxCustomization::None);
	RightSideToolbarBuilder.SetStyle(&FInsightsStyle::Get(), "PrimaryToolbar");
	Local::FillRightSideToolbar(RightSideToolbarBuilder, InArgs);

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.FillWidth(1.0)
		.Padding(0.0f)
		[
			ToolbarBuilder.MakeWidget()
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0.0f)
		[
			RightSideToolbarBuilder.MakeWidget()
		]
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
