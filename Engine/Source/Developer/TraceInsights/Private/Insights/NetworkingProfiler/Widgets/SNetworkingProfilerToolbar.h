// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FExtender;
struct FInsightsMajorTabConfig;

namespace UE::Insights::NetworkingProfiler
{

class SNetworkingProfilerWindow;

class SNetworkingProfilerToolbar : public SCompoundWidget
{
public:
	/** Default constructor. */
	SNetworkingProfilerToolbar();

	/** Virtual destructor. */
	virtual ~SNetworkingProfilerToolbar();

	SLATE_BEGIN_ARGS(SNetworkingProfilerToolbar) {}
	SLATE_ARGUMENT(TSharedPtr<FExtender>, ToolbarExtender);
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs, TSharedRef<SNetworkingProfilerWindow> InProfilerWindow, const FInsightsMajorTabConfig& Config);
};

} // namespace UE::Insights::NetworkingProfiler
