// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNetworkingProfilerWindow.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Docking/SDockTab.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/NetworkingProfilerManager.h"
#include "Insights/NetworkingProfiler/Widgets/SNetStatsView.h"
#include "Insights/NetworkingProfiler/Widgets/SNetStatsCountersView.h"
#include "Insights/NetworkingProfiler/Widgets/SNetworkingProfilerToolbar.h"
#include "Insights/NetworkingProfiler/Widgets/SPacketContentView.h"
#include "Insights/NetworkingProfiler/Widgets/SPacketView.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler"

////////////////////////////////////////////////////////////////////////////////////////////////////

const FName FNetworkingProfilerTabs::PacketViewID(TEXT("PacketView"));
const FName FNetworkingProfilerTabs::PacketContentViewID(TEXT("PacketContent"));
const FName FNetworkingProfilerTabs::NetStatsViewID(TEXT("NetStats"));
const FName FNetworkingProfilerTabs::NetStatsCountersViewID(TEXT("NetStatsCounters"));

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

SNetworkingProfilerWindow::SNetworkingProfilerWindow()
	: SMajorTabWindow(FInsightsManagerTabs::NetworkingProfilerTabId)
	, SelectedPacketStartIndex(0)
	, SelectedPacketEndIndex(0)
	, SelectionStartPosition(0)
	, SelectionEndPosition(0)
	, SelectedEventTypeIndex(InvalidEventTypeIndex)
	, OldGameInstanceChangeCount(0U)
	, OldConnectionChangeCount(0U)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SNetworkingProfilerWindow::~SNetworkingProfilerWindow()
{
	CloseAllOpenTabs();

	check(NetStatsView == nullptr);
	check(NetStatsCountersView == nullptr);
	check(PacketContentView == nullptr);
	check(PacketView == nullptr);
}


////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* SNetworkingProfilerWindow::GetAnalyticsEventName() const
{
	return TEXT("Insights.Usage.NetworkingProfiler");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::Reset()
{
	if (PacketView)
	{
		PacketView->Reset();
	}

	if (PacketContentView)
	{
		PacketContentView->Reset();
	}

	if (NetStatsView)
	{
		NetStatsView->Reset();
	}

	if (NetStatsCountersView)
	{
		NetStatsCountersView->Reset();
	}

	AvailableGameInstances.Reset();
	SelectedGameInstance.Reset();
	AvailableConnections.Reset();
	SelectedConnection.Reset();
	AvailableConnectionModes.Reset();
	SelectedConnectionMode.Reset();

	SelectedPacketStartIndex = 0;
	SelectedPacketEndIndex = 0;
	SelectionStartPosition = 0;
	SelectionEndPosition = 0;

	SelectedEventTypeIndex = InvalidEventTypeIndex;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SNetworkingProfilerWindow::SpawnTab_PacketView(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(PacketView, SPacketView, SharedThis(this))
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SNetworkingProfilerWindow::OnPacketViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::OnPacketViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	PacketView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SNetworkingProfilerWindow::SpawnTab_PacketContentView(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(PacketContentView, SPacketContentView, SharedThis(this))
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SNetworkingProfilerWindow::OnPacketContentViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::OnPacketContentViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	PacketContentView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SNetworkingProfilerWindow::SpawnTab_NetStatsView(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(NetStatsView, SNetStatsView, SharedThis(this))
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SNetworkingProfilerWindow::OnNetStatsViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::OnNetStatsViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	NetStatsView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SNetworkingProfilerWindow::SpawnTab_NetStatsCountersView(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(NetStatsCountersView, SNetStatsCountersView, SharedThis(this))
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SNetworkingProfilerWindow::OnNetStatsCountersViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::OnNetStatsCountersViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	NetStatsCountersView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	TSharedPtr<FNetworkingProfilerManager> NetworkingProfilerManager = FNetworkingProfilerManager::Get();
	ensure(NetworkingProfilerManager.IsValid());

	SetCommandList(MakeShared<FUICommandList>());
	BindCommands();

	SMajorTabWindow::FArguments Args;
	SMajorTabWindow::Construct(Args, ConstructUnderMajorTab, ConstructUnderWindow);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FWorkspaceItem> SNetworkingProfilerWindow::CreateWorkspaceMenuGroup()
{
	return GetTabManager()->AddLocalWorkspaceMenuCategory(LOCTEXT("NetworkingProfilerMenuGroupName", "Networking Insights"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::RegisterTabSpawners()
{
	check(GetTabManager().IsValid());
	FTabManager* TabManagerPtr = GetTabManager().Get();
	check(GetWorkspaceMenuGroup().IsValid());
	const TSharedRef<FWorkspaceItem> Group = GetWorkspaceMenuGroup().ToSharedRef();

	IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(FInsightsManagerTabs::NetworkingProfilerTabId);

	if (Config.ShouldRegisterMinorTab(FNetworkingProfilerTabs::PacketViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FNetworkingProfilerTabs::PacketViewID, FOnSpawnTab::CreateRaw(this, &SNetworkingProfilerWindow::SpawnTab_PacketView))
			.SetDisplayName(LOCTEXT("PacketViewTabTitle", "Packet View"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.PacketView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FNetworkingProfilerTabs::PacketContentViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FNetworkingProfilerTabs::PacketContentViewID, FOnSpawnTab::CreateRaw(this, &SNetworkingProfilerWindow::SpawnTab_PacketContentView))
			.SetDisplayName(LOCTEXT("PacketContentViewTabTitle", "Packet Content"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.PacketContentView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FNetworkingProfilerTabs::NetStatsViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FNetworkingProfilerTabs::NetStatsViewID, FOnSpawnTab::CreateRaw(this, &SNetworkingProfilerWindow::SpawnTab_NetStatsView))
			.SetDisplayName(LOCTEXT("NetStatsViewTabTitle", "Net Stats"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.NetStatsView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FNetworkingProfilerTabs::NetStatsCountersViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FNetworkingProfilerTabs::NetStatsCountersViewID, FOnSpawnTab::CreateRaw(this, &SNetworkingProfilerWindow::SpawnTab_NetStatsCountersView))
			.SetDisplayName(LOCTEXT("NetworkingProfiler.NetStatsCountersViewTabTitle", "NetStatsCounters"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.NetStatsView"))
			.SetGroup(Group);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<FTabManager::FLayout> SNetworkingProfilerWindow::CreateDefaultTabLayout() const
{
	return FTabManager::NewLayout("InsightsNetworkingProfilerLayout_v1.4")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.65f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.35f)
					->AddTab(FNetworkingProfilerTabs::PacketViewID, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.65f)
					->AddTab(FNetworkingProfilerTabs::PacketContentViewID, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.35f)
				->AddTab(FNetworkingProfilerTabs::NetStatsViewID, ETabState::OpenedTab)
				->AddTab(FNetworkingProfilerTabs::NetStatsCountersViewID, ETabState::OpenedTab)
				->SetForegroundTab(FNetworkingProfilerTabs::NetStatsViewID)
			)
		);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SNetworkingProfilerWindow::CreateToolbar(TSharedPtr<FExtender> Extender)
{
	IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(FInsightsManagerTabs::NetworkingProfilerTabId);

	return SNew(SNetworkingProfilerToolbar, SharedThis(this), Config)
		.ToolbarExtender(Extender);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FNetworkingProfilerCommands& SNetworkingProfilerWindow::GetCommands() const
{
	return FNetworkingProfilerManager::GetCommands();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::BindCommands()
{
	check(GetCommandList().IsValid());

	Map_TogglePacketViewVisibility();
	Map_TogglePacketContentViewVisibility();
	Map_ToggleNetStatsViewVisibility();
	Map_ToggleNetStatsCountersViewVisibility();

	GetCommandList()->MapAction(FInsightsCommands::Get().ToggleDebugInfo, FInsightsManager::GetActionManager().GetAction_ToggleDebugInfo());
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Toggle Commands
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(SNetworkingProfilerWindow, this, TogglePacketViewVisibility, IsPacketViewVisible, ShowOrHidePacketView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(SNetworkingProfilerWindow, this, TogglePacketContentViewVisibility, IsPacketContentViewVisible, ShowOrHidePacketContentView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(SNetworkingProfilerWindow, this, ToggleNetStatsViewVisibility, IsNetStatsViewVisible, ShowOrHideNetStatsView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(SNetworkingProfilerWindow, this, ToggleNetStatsCountersViewVisibility, IsNetStatsCountersViewVisible, ShowOrHideNetStatsCountersView)

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();

	uint32 GameInstanceChangeCount = 0;
	uint32 ConnectionChangeCount = 0;
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
		if (NetProfilerProvider)
		{
			GameInstanceChangeCount = NetProfilerProvider->GetGameInstanceChangeCount();
			ConnectionChangeCount = NetProfilerProvider->GetConnectionChangeCount();
		}
	}

	if (GameInstanceChangeCount != OldGameInstanceChangeCount || ConnectionChangeCount != OldConnectionChangeCount)
	{
		OldGameInstanceChangeCount = GameInstanceChangeCount;
		OldConnectionChangeCount = ConnectionChangeCount;
		UpdateAvailableGameInstances();
	}

	uint32 ConnectionCount = 0;
	if (Session.IsValid() && SelectedGameInstance.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
		if (NetProfilerProvider)
		{
			ConnectionCount = NetProfilerProvider->GetConnectionCount(SelectedGameInstance->GetIndex());
		}
	}

	if (ConnectionCount != AvailableConnections.Num())
	{
		UpdateAvailableConnections();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::UpdateAvailableGameInstances()
{
	uint32 CurrentSelectedGameInstanceIndex = 0;
	bool bIsInstanceSelected = false;
	if (GameInstanceComboBox.IsValid())
	{
		TSharedPtr<FGameInstanceItem> CurrentSelectedGameInstance = GameInstanceComboBox->GetSelectedItem();
		if (CurrentSelectedGameInstance.IsValid())
		{
			CurrentSelectedGameInstanceIndex = CurrentSelectedGameInstance->GetIndex();
			bIsInstanceSelected = true;
		}
	}

	AvailableGameInstances.Reset();

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
		if (NetProfilerProvider)
		{
			NetProfilerProvider->ReadGameInstances([this, NetProfilerProvider](const TraceServices::FNetProfilerGameInstance& GameInstance)
			{
				if (NetProfilerProvider->GetConnectionCount(GameInstance.GameInstanceIndex) > 0)
				{
					AvailableGameInstances.Add(MakeShared<FGameInstanceItem>(GameInstance));
				}
			});
		}
	}

	if (GameInstanceComboBox.IsValid())
	{
		const uint32 AvailableGameInstanceCount = AvailableGameInstances.Num();
		if (bIsInstanceSelected && AvailableGameInstanceCount > CurrentSelectedGameInstanceIndex)
		{
			GameInstanceComboBox->SetSelectedItem(AvailableGameInstances[CurrentSelectedGameInstanceIndex]);
		}
		else
		{
			GameInstanceComboBox->SetSelectedItem(AvailableGameInstances.Num() > 0 ? AvailableGameInstances[0] : nullptr);
		}
	}

	UpdateAvailableConnections();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::UpdateAvailableConnections()
{
	uint32 CurrentSelectedConnectionIndex = 0;
	bool bIsConnectionSelected = false;
	if (ConnectionComboBox.IsValid())
	{
		TSharedPtr<FConnectionItem> CurrentSelectedConnection = ConnectionComboBox->GetSelectedItem();
		if (CurrentSelectedConnection.IsValid())
		{
			CurrentSelectedConnectionIndex = CurrentSelectedConnection->GetIndex();
			bIsConnectionSelected = true;
		}
	}

	AvailableConnections.Reset();

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid() && SelectedGameInstance.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
		if (NetProfilerProvider)
		{
			NetProfilerProvider->ReadConnections(SelectedGameInstance->GetIndex(), [this](const TraceServices::FNetProfilerConnection& Connection)
			{
				AvailableConnections.Add(MakeShared<FConnectionItem>(Connection));
			});
		}
	}

	if (ConnectionComboBox.IsValid())
	{
		const uint32 AvailableConnectionCount = AvailableConnections.Num();
		if (bIsConnectionSelected && AvailableConnectionCount > CurrentSelectedConnectionIndex)
		{
			ConnectionComboBox->SetSelectedItem(AvailableConnections[CurrentSelectedConnectionIndex]);
		}
		else
		{
			ConnectionComboBox->SetSelectedItem(AvailableConnections.Num() > 0 ? AvailableConnections[0] : nullptr);
		}
	}

	UpdateAvailableConnectionModes();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::UpdateAvailableConnectionModes()
{
	bool bIsConnectionModeSelected = false;
	TraceServices::ENetProfilerConnectionMode CurrentSelectedConnectionModeChoice = TraceServices::ENetProfilerConnectionMode::Incoming;
	if (ConnectionModeComboBox.IsValid())
	{
		TSharedPtr<FConnectionModeItem> CurrentSelectedConnectionMode = ConnectionModeComboBox->GetSelectedItem();
		if (CurrentSelectedConnectionMode.IsValid())
		{
			CurrentSelectedConnectionModeChoice = CurrentSelectedConnectionMode->Mode;
			bIsConnectionModeSelected = true;
		}
	}

	AvailableConnectionModes.Reset();

	if (SelectedConnection.IsValid())
	{
		if (SelectedConnection->Connection.bHasIncomingData)
		{
			AvailableConnectionModes.Add(MakeShared<FConnectionModeItem>(TraceServices::ENetProfilerConnectionMode::Incoming));
		}
		if (SelectedConnection->Connection.bHasOutgoingData)
		{
			AvailableConnectionModes.Add(MakeShared<FConnectionModeItem>(TraceServices::ENetProfilerConnectionMode::Outgoing));
		}
	}

	if (ConnectionModeComboBox.IsValid())
	{
		const uint32 AvailableConnectionModeCount = AvailableConnectionModes.Num();
		if (bIsConnectionModeSelected && AvailableConnectionModeCount == 2)
		{
			if (CurrentSelectedConnectionModeChoice == TraceServices::ENetProfilerConnectionMode::Outgoing)
			{
				ConnectionModeComboBox->SetSelectedItem(AvailableConnectionModes[1]);
			}
			else
			{
				ConnectionModeComboBox->SetSelectedItem(AvailableConnectionModes[0]);
			}
		}
		else
		{
			ConnectionModeComboBox->SetSelectedItem(AvailableConnectionModes.Num() > 0 ? AvailableConnectionModes[0] : nullptr);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetworkingProfilerWindow::FGameInstanceItem::GetText() const
{
	if (IsInstanceNameSet())
	{
		return FText::Format(LOCTEXT("GameInstanceItemFmt0", "Game Instance {0} [{1}]"), FText::AsNumber(GetIndex()),
			IsServer() ? FText::FromString("Server") : FText::FromString("Client"));
	}
	else
	{
		return FText::Format(LOCTEXT("GameInstanceItemFmt1", "Game Instance {0}"), FText::AsNumber(GetIndex()));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetworkingProfilerWindow::FGameInstanceItem::GetTooltipText() const
{
	if (IsInstanceNameSet())
	{
		return FText::Format(LOCTEXT("GameInstanceItemTooltipFmt0", "{0} Game Instance {1} [{2}]"), FText::FromString(GetInstanceName()), FText::AsNumber(GetIndex()),
			IsServer() ? FText::FromString("Server") : FText::FromString("Client"));
	}
	else
	{
		return FText::Format(LOCTEXT("GameInstanceItemTooltipFmt1", "Game Instance {0}"), FText::AsNumber(GetIndex()));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetworkingProfilerWindow::FConnectionItem::GetText() const
{
	if (IsConnectionAddressSet())
	{
		return FText::Format(LOCTEXT("ConnectionItemFmt1", "Connection {0} to {1}"), FText::AsNumber(GetIndex()),
			FText::FromString(GetConnectionAddress()));
	}
	else
	{
		return FText::Format(LOCTEXT("ConnectionItemFmt2", "Connection {0}"), FText::AsNumber(GetIndex()));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetworkingProfilerWindow::FConnectionItem::GetTooltipText() const
{
	if (IsConnectionNameSet())
	{
		if (IsConnectionAddressSet())
		{
			return FText::Format(LOCTEXT("ConnectionItemFmt3", "Connection {0} ({1}) to {2}"), FText::AsNumber(GetIndex()), FText::FromString(GetConnectionName()),
				FText::FromString(GetConnectionAddress()));
		}
		else
		{
			return FText::Format(LOCTEXT("ConnectionItemFmt4", "Connection {0} ({1})"), FText::AsNumber(GetIndex()), FText::FromString(GetConnectionName()));
		}
	}
	else
	{
		if (IsConnectionAddressSet())
		{
			return FText::Format(LOCTEXT("ConnectionItemFmt1", "Connection {0} to {1}"), FText::AsNumber(GetIndex()),
				FText::FromString(GetConnectionAddress()));
		}
		else
		{
			return FText::Format(LOCTEXT("ConnectionItemFmt2", "Connection {0}"), FText::AsNumber(GetIndex()));
		}
	}

}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetworkingProfilerWindow::FConnectionModeItem::GetText() const
{
	switch (Mode)
	{
	case TraceServices::ENetProfilerConnectionMode::Outgoing:
		return LOCTEXT("ConnectionMode_Outgoing", "Outgoing");

	case TraceServices::ENetProfilerConnectionMode::Incoming:
		return LOCTEXT("ConnectionMode_Incoming", "Incoming");

	default:
		return LOCTEXT("ConnectionMode_Unknown", "Unknown");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetworkingProfilerWindow::FConnectionModeItem::GetTooltipText() const
{
	return GetText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SNetworkingProfilerWindow::CreateGameInstanceComboBox()
{
	GameInstanceComboBox = SNew(SComboBox<TSharedPtr<FGameInstanceItem>>)
		.ToolTipText(this, &SNetworkingProfilerWindow::GameInstance_GetSelectedTooltipText)
		.OptionsSource(&AvailableGameInstances)
		.OnSelectionChanged(this, &SNetworkingProfilerWindow::GameInstance_OnSelectionChanged)
		.OnGenerateWidget(this, &SNetworkingProfilerWindow::GameInstance_OnGenerateWidget)
		.OnComboBoxOpening(this, &SNetworkingProfilerWindow::UpdateAvailableGameInstances)
		[
			SNew(STextBlock)
			.Text(this, &SNetworkingProfilerWindow::GameInstance_GetSelectedText)
		];
	return SNew(SBox)
		.Padding(FMargin(4.0f, 0.0f, 2.0f, 0.0f))
		[
			GameInstanceComboBox.ToSharedRef()
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SNetworkingProfilerWindow::GameInstance_OnGenerateWidget(TSharedPtr<FGameInstanceItem> InGameInstance) const
{
	return SNew(STextBlock)
		.Text(InGameInstance->GetText())
		.ToolTipText(InGameInstance->GetTooltipText());
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::GameInstance_OnSelectionChanged(TSharedPtr<FGameInstanceItem> NewGameInstance, ESelectInfo::Type SelectInfo)
{
	const bool bSameValue = (!SelectedGameInstance.IsValid() && !NewGameInstance.IsValid()) ||
							(SelectedGameInstance.IsValid() && NewGameInstance.IsValid() &&
								SelectedGameInstance->GetIndex() == NewGameInstance->GetIndex());

	SelectedGameInstance = NewGameInstance;

	if (!bSameValue)
	{
		UpdateAvailableConnections();
		if (PacketView.IsValid())
		{
			PacketView->SetConnection(GetSelectedGameInstanceIndex(), GetSelectedConnectionIndex(), GetSelectedConnectionMode());
		}
		UpdateAggregatedNetStats();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetworkingProfilerWindow::GameInstance_GetSelectedText() const
{
	return SelectedGameInstance.IsValid() ? SelectedGameInstance->GetText() : LOCTEXT("NoGameInstanceText", "Game Instance N/A");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetworkingProfilerWindow::GameInstance_GetSelectedTooltipText() const
{
	return SelectedGameInstance.IsValid() ? SelectedGameInstance->GetTooltipText() : FText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SNetworkingProfilerWindow::CreateConnectionComboBox()
{
	ConnectionComboBox = SNew(SComboBox<TSharedPtr<FConnectionItem>>)
		.ToolTipText(this, &SNetworkingProfilerWindow::Connection_GetSelectedTooltipText)
		.OptionsSource(&AvailableConnections)
		.OnSelectionChanged(this, &SNetworkingProfilerWindow::Connection_OnSelectionChanged)
		.OnGenerateWidget(this, &SNetworkingProfilerWindow::Connection_OnGenerateWidget)
		[
			SNew(STextBlock)
			.Text(this, &SNetworkingProfilerWindow::Connection_GetSelectedText)
		];
	return SNew(SBox)
		.Padding(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
		[
			ConnectionComboBox.ToSharedRef()
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SNetworkingProfilerWindow::Connection_OnGenerateWidget(TSharedPtr<FConnectionItem> InConnection) const
{
	return SNew(STextBlock)
		.Text(InConnection->GetText())
		.ToolTipText(InConnection->GetTooltipText());
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::Connection_OnSelectionChanged(TSharedPtr<FConnectionItem> NewConnection, ESelectInfo::Type SelectInfo)
{
	const bool bSameValue = (!SelectedConnection.IsValid() && !NewConnection.IsValid()) ||
							(SelectedConnection.IsValid() && NewConnection.IsValid() &&
								SelectedConnection->GetIndex() == NewConnection->GetIndex());

	SelectedConnection = NewConnection;

	if (!bSameValue)
	{
		UpdateAvailableConnectionModes();
		if (PacketView.IsValid())
		{
			PacketView->SetConnection(GetSelectedGameInstanceIndex(), GetSelectedConnectionIndex(), GetSelectedConnectionMode());
		}
		UpdateAggregatedNetStats();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetworkingProfilerWindow::Connection_GetSelectedText() const
{
	return SelectedConnection.IsValid() ? SelectedConnection->GetText() : LOCTEXT("NoConnectionText", "Connection N/A");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetworkingProfilerWindow::Connection_GetSelectedTooltipText() const
{
	return SelectedConnection.IsValid() ? SelectedConnection->GetTooltipText() : FText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SNetworkingProfilerWindow::CreateConnectionModeComboBox()
{
	ConnectionModeComboBox = SNew(SComboBox<TSharedPtr<FConnectionModeItem>>)
		.ToolTipText(this, &SNetworkingProfilerWindow::ConnectionMode_GetSelectedTooltipText)
		.OptionsSource(&AvailableConnectionModes)
		.OnSelectionChanged(this, &SNetworkingProfilerWindow::ConnectionMode_OnSelectionChanged)
		.OnGenerateWidget(this, &SNetworkingProfilerWindow::ConnectionMode_OnGenerateWidget)
		[
			SNew(STextBlock)
			.Text(this, &SNetworkingProfilerWindow::ConnectionMode_GetSelectedText)
		];

	return SNew(SBox)
		.Padding(FMargin(2.0f, 0.0f, 4.0f, 0.0f))
		[
			ConnectionModeComboBox.ToSharedRef()
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SNetworkingProfilerWindow::ConnectionMode_OnGenerateWidget(TSharedPtr<FConnectionModeItem> InConnectionMode) const
{
	return SNew(STextBlock)
		.Text(InConnectionMode->GetText())
		.ToolTipText(InConnectionMode->GetTooltipText());
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::ConnectionMode_OnSelectionChanged(TSharedPtr<FConnectionModeItem> NewConnectionMode, ESelectInfo::Type SelectInfo)
{
	const bool bSameValue = (!SelectedConnectionMode.IsValid() && !NewConnectionMode.IsValid()) ||
							(SelectedConnectionMode.IsValid() && NewConnectionMode.IsValid() &&
								SelectedConnectionMode->Mode == NewConnectionMode->Mode);

	SelectedConnectionMode = NewConnectionMode;

	if (!bSameValue)
	{
		if (PacketView.IsValid())
		{
			PacketView->SetConnection(GetSelectedGameInstanceIndex(), GetSelectedConnectionIndex(), GetSelectedConnectionMode());
		}
		UpdateAggregatedNetStats();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetworkingProfilerWindow::ConnectionMode_GetSelectedText() const
{
	return SelectedConnectionMode.IsValid() ? SelectedConnectionMode->GetText() : LOCTEXT("NoConnectionModeText", "N/A");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SNetworkingProfilerWindow::ConnectionMode_GetSelectedTooltipText() const
{
	return SelectedConnectionMode.IsValid() ? SelectedConnectionMode->GetTooltipText() : FText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::SetSelectedPacket(uint32 StartIndex, uint32 EndIndex, uint32 SinglePacketBitSize)
{
	if (StartIndex != SelectedPacketStartIndex || EndIndex != SelectedPacketEndIndex)
	{
		SelectedPacketStartIndex = StartIndex;
		SelectedPacketEndIndex = EndIndex;

		UpdateAggregatedNetStats();

		if (PacketContentView.IsValid())
		{
			if (SelectedPacketEndIndex == SelectedPacketStartIndex + 1 && // only one packet selected
				SelectedGameInstance.IsValid() &&
				SelectedConnection.IsValid() &&
				SelectedConnectionMode.IsValid())
			{
				PacketContentView->SetPacket(SelectedGameInstance->GetIndex(),
											 SelectedConnection->GetIndex(),
											 SelectedConnectionMode->Mode,
											 SelectedPacketStartIndex,
											 SinglePacketBitSize);
			}
			else
			{
				PacketContentView->ResetPacket();
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::SetSelectedBitRange(uint32 StartPos, uint32 EndPos)
{
	if (StartPos != SelectionStartPosition || EndPos != SelectionEndPosition)
	{
		SelectionStartPosition = StartPos;
		SelectionEndPosition = EndPos;
		UpdateAggregatedNetStats();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::SetSelectedEventTypeIndex(const uint32 InEventTypeIndex)
{
	if (InEventTypeIndex != SelectedEventTypeIndex)
	{
		SelectedEventTypeIndex = InEventTypeIndex;

		if (SelectedEventTypeIndex != InvalidEventTypeIndex)
		{
			if (NetStatsView)
			{
				NetStatsView->SelectNetEventNode(SelectedEventTypeIndex);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SNetworkingProfilerWindow::UpdateAggregatedNetStats()
{
	if (NetStatsView || NetStatsCountersView)
	{
		if (SelectedGameInstance.IsValid() &&
			SelectedConnection.IsValid() &&
			SelectedConnectionMode.IsValid() &&
			(SelectedPacketStartIndex < SelectedPacketEndIndex) &&
			(SelectedPacketStartIndex + 1 != SelectedPacketEndIndex || SelectionStartPosition < SelectionEndPosition))
		{
			if (NetStatsView)
			{
				NetStatsView->UpdateStats(SelectedGameInstance->GetIndex(),
									SelectedConnection->GetIndex(),
									SelectedConnectionMode->Mode,
									SelectedPacketStartIndex,
									SelectedPacketEndIndex,
									SelectionStartPosition,
									SelectionEndPosition);
			}
			if (NetStatsCountersView)
			{
				NetStatsCountersView->UpdateStats(SelectedGameInstance->GetIndex(),
									SelectedConnection->GetIndex(),
									SelectedConnectionMode->Mode,
									SelectedPacketStartIndex,
									SelectedPacketEndIndex);
			}
		}
		else
		{
			if (NetStatsView)
			{
				NetStatsView->ResetStats();
			}
			if (NetStatsCountersView)
			{
				NetStatsCountersView->ResetStats();
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
