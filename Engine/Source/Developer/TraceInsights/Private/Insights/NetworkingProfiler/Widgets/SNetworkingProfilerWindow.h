// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Input/SComboBox.h"

// TraceServices
#include "TraceServices/Model/NetProfiler.h"

// TraceInsights
#include "Insights/Common/CommandUtils.h"
#include "Insights/Widgets/SMajorTabWindow.h"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class SPacketView;
class SPacketContentView;
class SNetStatsView;
class SNetStatsCountersView;
class FNetworkingProfilerCommands;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Implements the Networking Insights window. */
class SNetworkingProfilerWindow : public ::Insights::SMajorTabWindow
{
private:
	struct FGameInstanceItem
	{
		/** Conversion constructor. */
		FGameInstanceItem(const TraceServices::FNetProfilerGameInstance& InGameInstance)
			: GameInstance(InGameInstance)
		{}

		uint32 GetIndex() const { return GameInstance.GameInstanceIndex; }
		bool IsInstanceNameSet() const { return GameInstance.InstanceName != nullptr; }
		const TCHAR* GetInstanceName() const { return GameInstance.InstanceName; }
		bool IsServer() const { return GameInstance.bIsServer; }
		FText GetText() const;
		FText GetTooltipText() const;

		const TraceServices::FNetProfilerGameInstance GameInstance;
	};

	struct FConnectionItem
	{
		/** Conversion constructor. */
		FConnectionItem(const TraceServices::FNetProfilerConnection& InConnection)
			: Connection(InConnection)
		{}

		uint32 GetIndex() const { return Connection.ConnectionIndex; }
		bool IsConnectionNameSet() const { return Connection.Name != nullptr; }
		bool IsConnectionAddressSet() const { return Connection.AddressString != nullptr; }
		const TCHAR* GetConnectionName() const { return Connection.Name; }
		const TCHAR* GetConnectionAddress() const { return Connection.AddressString; }
		FText GetText() const;
		FText GetTooltipText() const;

		const TraceServices::FNetProfilerConnection Connection;
	};

	struct FConnectionModeItem
	{
		/** Conversion constructor. */
		FConnectionModeItem(const TraceServices::ENetProfilerConnectionMode& InMode)
			: Mode(InMode)
		{}

		FText GetText() const;
		FText GetTooltipText() const;

		TraceServices::ENetProfilerConnectionMode Mode;
	};

public:
	/** Default constructor. */
	SNetworkingProfilerWindow();

	/** Virtual destructor. */
	virtual ~SNetworkingProfilerWindow();

	SLATE_BEGIN_ARGS(SNetworkingProfilerWindow) {}
	SLATE_END_ARGS()

	virtual void Reset() override;

	/** Constructs this widget. */
	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);

	TSharedPtr<SPacketView> GetPacketView() const { return PacketView; }
	const bool IsPacketViewVisible() const { return PacketView.IsValid(); }
	void ShowOrHidePacketView(const bool bVisibleState) { ShowHideTab(FNetworkingProfilerTabs::PacketViewID, bVisibleState); }

	TSharedPtr<SPacketContentView> GetPacketContentView() const { return PacketContentView; }
	const bool IsPacketContentViewVisible() const { return PacketContentView.IsValid(); }
	void ShowOrHidePacketContentView(const bool bVisibleState) { ShowHideTab(FNetworkingProfilerTabs::PacketContentViewID, bVisibleState); }

	TSharedPtr<SNetStatsView> GetNetStatsView() const { return NetStatsView; }
	const bool IsNetStatsViewVisible() const { return NetStatsView.IsValid(); }
	void ShowOrHideNetStatsView(const bool bVisibleState) { ShowHideTab(FNetworkingProfilerTabs::NetStatsViewID, bVisibleState); }

	TSharedPtr<SNetStatsCountersView> GetNetStatsCountersView() const { return NetStatsCountersView; }
	const bool IsNetStatsCountersViewVisible() const { return NetStatsCountersView.IsValid(); }
	void ShowOrHideNetStatsCountersView(const bool bVisibleState) { ShowHideTab(FNetworkingProfilerTabs::NetStatsCountersViewID, bVisibleState); }

	const TraceServices::FNetProfilerGameInstance* GetSelectedGameInstance() const { return SelectedGameInstance ? &SelectedGameInstance->GameInstance : nullptr; }
	uint32 GetSelectedGameInstanceIndex() const { return SelectedGameInstance ? SelectedGameInstance->GetIndex() : 0; }
	const TraceServices::FNetProfilerConnection* GetSelectedConnection() const { return SelectedConnection ? &SelectedConnection->Connection : nullptr; }
	uint32 GetSelectedConnectionIndex() const { return SelectedConnection ? SelectedConnection->GetIndex() : 0; }
	TraceServices::ENetProfilerConnectionMode GetSelectedConnectionMode() const { return SelectedConnectionMode ? SelectedConnectionMode->Mode : TraceServices::ENetProfilerConnectionMode::Outgoing; }

	TSharedRef<SWidget> CreateGameInstanceComboBox();
	TSharedRef<SWidget> CreateConnectionComboBox();
	TSharedRef<SWidget> CreateConnectionModeComboBox();

	void SetSelectedPacket(uint32 StartIndex, uint32 EndIndex, uint32 SinglePacketBitSize = 0);
	void SetSelectedBitRange(uint32 StartPos, uint32 EndPos);
	void SetSelectedEventTypeIndex(uint32 InEventTypeIndex);

protected:
	virtual const TCHAR* GetAnalyticsEventName() const override;
	virtual TSharedRef<FWorkspaceItem> CreateWorkspaceMenuGroup() override;
	virtual void RegisterTabSpawners() override;
	virtual TSharedRef<FTabManager::FLayout> CreateDefaultTabLayout() const override;
	virtual TSharedRef<SWidget> CreateToolbar(TSharedPtr<FExtender> Extender);

private:
	TSharedRef<SDockTab> SpawnTab_PacketView(const FSpawnTabArgs& Args);
	void OnPacketViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_PacketContentView(const FSpawnTabArgs& Args);
	void OnPacketContentViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_NetStatsView(const FSpawnTabArgs& Args);
	void OnNetStatsViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_NetStatsCountersView(const FSpawnTabArgs& Args);
	void OnNetStatsCountersViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	//////////////////////////////////////////////////

	const FNetworkingProfilerCommands& GetCommands() const;
	void BindCommands();

	//////////////////////////////////////////////////
	// Toggle Commands

	INSIGHTS_DECLARE_TOGGLE_COMMAND(TogglePacketViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(TogglePacketContentViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleNetStatsViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleNetStatsCountersViewVisibility)

	//////////////////////////////////////////////////

	void UpdateAggregatedNetStats();

	//////////////////////////////////////////////////

	/**
	 * Ticks this widget. Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	//////////////////////////////////////////////////

	void UpdateAvailableGameInstances();
	void UpdateAvailableConnections();
	void UpdateAvailableConnectionModes();

	TSharedRef<SWidget> GameInstance_OnGenerateWidget(TSharedPtr<FGameInstanceItem> InGameInstance) const;
	void GameInstance_OnSelectionChanged(TSharedPtr<FGameInstanceItem> NewGameInstance, ESelectInfo::Type SelectInfo);
	FText GameInstance_GetSelectedText() const;
	FText GameInstance_GetSelectedTooltipText() const;

	TSharedRef<SWidget> Connection_OnGenerateWidget(TSharedPtr<FConnectionItem> InConnection) const;
	void Connection_OnSelectionChanged(TSharedPtr<FConnectionItem> NewConnection, ESelectInfo::Type SelectInfo);
	FText Connection_GetSelectedText() const;
	FText Connection_GetSelectedTooltipText() const;

	TSharedRef<SWidget> ConnectionMode_OnGenerateWidget(TSharedPtr<FConnectionModeItem> InConnectionMode) const;
	void ConnectionMode_OnSelectionChanged(TSharedPtr<FConnectionModeItem> NewConnectionMode, ESelectInfo::Type SelectInfo);
	FText ConnectionMode_GetSelectedText() const;
	FText ConnectionMode_GetSelectedTooltipText() const;

	//////////////////////////////////////////////////

private:
	/** The Packet View widget */
	TSharedPtr<SPacketView> PacketView;

	/** The Packet Content widget */
	TSharedPtr<SPacketContentView> PacketContentView;

	/** The Net Stats widget */
	TSharedPtr<SNetStatsView> NetStatsView;

	/** The NetStatsCounters widget */
	TSharedPtr<SNetStatsCountersView> NetStatsCountersView;

	TSharedPtr<SComboBox<TSharedPtr<FGameInstanceItem>>> GameInstanceComboBox;
	TArray<TSharedPtr<FGameInstanceItem>> AvailableGameInstances;
	TSharedPtr<FGameInstanceItem> SelectedGameInstance;

	TSharedPtr<SComboBox<TSharedPtr<FConnectionItem>>> ConnectionComboBox;
	TArray<TSharedPtr<FConnectionItem>> AvailableConnections;
	TSharedPtr<FConnectionItem> SelectedConnection;

	TSharedPtr<SComboBox<TSharedPtr<FConnectionModeItem>>> ConnectionModeComboBox;
	TArray<TSharedPtr<FConnectionModeItem>> AvailableConnectionModes;
	TSharedPtr<FConnectionModeItem> SelectedConnectionMode;

	// [SelectedPacketStartIndex, SelectedPacketEndIndex) is the exclusive interval of selected packages.
	// NumSelectedPackets == SelectedPacketEndIndex - SelectedPacketStartIndex.
	uint32 SelectedPacketStartIndex;
	uint32 SelectedPacketEndIndex;

	// [SelectionStartPosition, SelectionEndPosition) is the exclusive selected bit range inside a single selected package.
	// Used only when NumSelectedPackets == SelectedPacketEndIndex - SelectedPacketStartIndex == 1.
	// SelectedBitSize == SelectionEndPosition - SelectionStartPosition.
	uint32 SelectionStartPosition;
	uint32 SelectionEndPosition;

	static const uint32 InvalidEventTypeIndex = uint32(-1);
	uint32 SelectedEventTypeIndex;

	uint32 OldGameInstanceChangeCount;
	uint32 OldConnectionChangeCount;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
