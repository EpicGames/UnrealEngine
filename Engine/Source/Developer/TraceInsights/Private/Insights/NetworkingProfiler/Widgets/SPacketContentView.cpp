// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPacketContentView.h"

#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Logging/MessageLog.h"
#include "MessageLogModule.h"
#include "Rendering/DrawElements.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SScrollBar.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/NetworkingProfilerManager.h"
#include "Insights/NetworkingProfiler/Widgets/SNetworkingProfilerWindow.h"
#include "Insights/NetworkingProfiler/Widgets/SPacketView.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::SPacketContentView"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

SPacketContentView::SPacketContentView()
	: ProfilerWindowWeakPtr()
	, DrawState(MakeShared<FPacketContentViewDrawState>())
	, FilteredDrawState(MakeShared<FPacketContentViewDrawState>())
	, AvailableAggregationModes()
	, SelectedAggregationMode(nullptr)
{
	Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SPacketContentView::~SPacketContentView()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::Reset()
{
	//ProfilerWindowWeakPtr

	Viewport.Reset();
	//FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();
	//ViewportX.SetScaleLimits(0.000001, 100000.0);
	//ViewportX.SetScale(1.0);
	bIsViewportDirty = true;

	GameInstanceIndex = 0;
	ConnectionIndex = 0;
	ConnectionMode = TraceServices::ENetProfilerConnectionMode::Outgoing;
	PacketIndex = 0;
	PacketSequence = 0;
	PacketBitSize = 0;

	bFilterByEventType = false;
	FilterEventTypeIndex = 0;
	FilterEventName = FText::GetEmpty();

	bFilterByNetId = false;
	FilterNetId = 0;

	bHighlightFilteredEvents = false;

	DrawState->Reset();
	FilteredDrawState->Reset();
	bIsStateDirty = true;

	MousePosition = FVector2D::ZeroVector;

	MousePositionOnButtonDown = FVector2D::ZeroVector;
	ViewportPosXOnButtonDown = 0.0f;

	MousePositionOnButtonUp = FVector2D::ZeroVector;

	bIsLMB_Pressed = false;
	bIsRMB_Pressed = false;

	bIsScrolling = false;

	HoveredEvent.Reset();
	SelectedEvent.Reset();
	Tooltip.Reset();

	//ThisGeometry

	CursorType = ECursorType::Default;

	UpdateDurationHistory.Reset();
	DrawDurationHistory.Reset();
	OnPaintDurationHistory.Reset();
	LastOnPaintTime = FPlatformTime::Cycles64();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SPacketContentView::AggregationMode_OnGenerateWidget(TSharedPtr<FAggregationModeItem> InAggregationMode) const
{
	return SNew(STextBlock)
		.Text(InAggregationMode->GetText())
		.ToolTipText(InAggregationMode->GetTooltipText());
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::AggregationMode_OnSelectionChanged(TSharedPtr<FAggregationModeItem> NewAggregationMode, ESelectInfo::Type SelectInfo)
{
	const bool bSameValue = (!SelectedAggregationMode.IsValid() && !NewAggregationMode.IsValid()) ||
		(SelectedAggregationMode.IsValid() && NewAggregationMode.IsValid() &&
			SelectedAggregationMode->Mode == NewAggregationMode->Mode);

	SelectedAggregationMode = NewAggregationMode;

	// Need to refresh selection
	if (!bSameValue)
	{
		TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = ProfilerWindowWeakPtr.Pin();
		if (ProfilerWindow.IsValid())
		{
			const TSharedPtr<SPacketView> PacketView = ProfilerWindow->GetPacketView();
			if (PacketView.IsValid())
			{
				PacketView->InvalidateState();			
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SPacketContentView::AggregationMode_GetSelectedText() const
{
	return SelectedAggregationMode.IsValid() ? SelectedAggregationMode->GetText() : LOCTEXT("NoAggregationModeText", "None");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SPacketContentView::AggregationMode_GetSelectedTooltipText() const
{
	return SelectedAggregationMode.IsValid() ? SelectedAggregationMode->GetTooltipText() : FText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SPacketContentView::FAggregationModeItem::GetText() const
{
	switch (Mode)
	{
	case TraceServices::ENetProfilerAggregationMode::Aggregate:
		return LOCTEXT("AggregationMode_Aggregate", "Aggregate");

	case TraceServices::ENetProfilerAggregationMode::InstanceMax:
		return LOCTEXT("AggregationMode_InstanceMax", "InstanceMax");

	default:
		return LOCTEXT("AggregationMode_None", "None");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SPacketContentView::FAggregationModeItem::GetTooltipText() const
{
	return GetText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SPacketContentView::CreateAggregationModeComboBox()
{
	AggregationModeComboBox = SNew(SComboBox<TSharedPtr<FAggregationModeItem>>)
		.ToolTipText(this, &SPacketContentView::AggregationMode_GetSelectedTooltipText)
		.OptionsSource(&AvailableAggregationModes)
		.OnSelectionChanged(this, &SPacketContentView::AggregationMode_OnSelectionChanged)
		.OnGenerateWidget(this, &SPacketContentView::AggregationMode_OnGenerateWidget)
		[
			SNew(STextBlock)
			.Text(this, &SPacketContentView::AggregationMode_GetSelectedText)
		];

	return AggregationModeComboBox.ToSharedRef();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SPacketContentView::Construct(const FArguments& InArgs, TSharedRef<SNetworkingProfilerWindow> InProfilerWindow)
{
	ProfilerWindowWeakPtr = InProfilerWindow;

	AvailableAggregationModes.Add(MakeShared<FAggregationModeItem>(TraceServices::ENetProfilerAggregationMode::None));
	AvailableAggregationModes.Add(MakeShared<FAggregationModeItem>(TraceServices::ENetProfilerAggregationMode::Aggregate));
	AvailableAggregationModes.Add(MakeShared<FAggregationModeItem>(TraceServices::ENetProfilerAggregationMode::InstanceMax));
	SelectedAggregationMode = AvailableAggregationModes[1];

	TSharedRef<SWidget> AggregationModeWidget = CreateAggregationModeComboBox();

	FSlimHorizontalToolBarBuilder ToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);
	ToolbarBuilder.SetStyle(&FInsightsStyle::Get(), "SecondaryToolbar2");

	ToolbarBuilder.BeginSection("FindPacket");
	{
		ToolbarBuilder.AddWidget(
			SNew(SBox)
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 0.0f, 2.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FindPacketText", "Find Packet:"))
			]
		);

		ToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &SPacketContentView::FindPreviousPacket)),
			NAME_None,
			TAttribute<FText>(),
			LOCTEXT("PreviousPacketToolTip", "Previous Packet"),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.FindPrevious.ToolBar"),
			EUserInterfaceActionType::Button
		);

		ToolbarBuilder.AddWidget(
			SNew(SBox)
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.RevertTextOnEscape(true)
				.SelectAllTextWhenFocused(true)
				.Text(this, &SPacketContentView::GetPacketText)
				.ToolTipText(LOCTEXT("SequenceNumber_Tooltip", "Sequence Number"))
				.OnTextCommitted(this, &SPacketContentView::Packet_OnTextCommitted)
				.MinDesiredWidth(40.0f)
			]
		);

		ToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &SPacketContentView::FindNextPacket)),
			NAME_None,
			TAttribute<FText>(),
			LOCTEXT("NextPacketToolTip", "Next Packet"),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.FindNext.ToolBar"),
			EUserInterfaceActionType::Button
		);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("FindEvent");
	{
		ToolbarBuilder.AddWidget(
			SNew(SBox)
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 0.0f, 2.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FindEventText", "Find Event:"))
			]
		);

		ToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &SPacketContentView::FindFirstEvent)),
			NAME_None,
			TAttribute<FText>(),
			LOCTEXT("FindFirstEventToolTip", "First Event"),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.FindFirst.ToolBar"),
			EUserInterfaceActionType::Button
		);

		ToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &SPacketContentView::FindPreviousEvent, EEventNavigationType::AnyLevel)),
			NAME_None,
			TAttribute<FText>(),
			LOCTEXT("FindPreviousEventToolTip", "Previous Event"),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.FindPrevious.ToolBar"),
			EUserInterfaceActionType::Button
		);

		ToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &SPacketContentView::FindNextEvent, EEventNavigationType::AnyLevel)),
			NAME_None,
			TAttribute<FText>(),
			LOCTEXT("FindNextEventToolTip", "Next Event"),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.FindNext.ToolBar"),
			EUserInterfaceActionType::Button
		);

		ToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &SPacketContentView::FindLastEvent)),
			NAME_None,
			TAttribute<FText>(),
			LOCTEXT("FindLastEventToolTip", "Last Event"),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.FindLast.ToolBar"),
			EUserInterfaceActionType::Button
		);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("FilterByNetId");
	{
		ToolbarBuilder.AddWidget(
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(4.0f, 0.0f, 2.0f, 0.0f))
			.AutoWidth()
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("FilterByNetId_Tooltip", "Filter events that have the specified NetId."))
				.IsChecked(this, &SPacketContentView::FilterByNetId_IsChecked)
				.OnCheckStateChanged(this, &SPacketContentView::FilterByNetId_OnCheckStateChanged)
				[
					SNew(SBox)
					.Padding(FMargin(2.0f, 0.0f, 0.0f, 0.0f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FilterByNetId_Text", "By NetId:"))
					]
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(2.0f, 0.0f, 4.0f, 0.0f))
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.RevertTextOnEscape(true)
				.SelectAllTextWhenFocused(true)
				.Text(this, &SPacketContentView::GetFilterNetIdText)
				.ToolTipText(LOCTEXT("NetId_Tooltip", "NetId"))
				.OnTextCommitted(this, &SPacketContentView::FilterNetId_OnTextCommitted)
				.MinDesiredWidth(40.0f)
			]
		);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("FilterByEventType");
	{
		ToolbarBuilder.AddWidget(
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(4.0f, 0.0f, 2.0f, 0.0f))
			.AutoWidth()
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("FilterByEventType_Tooltip", "Filter events that have the specified type.\n\nTo set the event type:\n\tdouble click either an event in the Packet Content view\n\tor an event type in the NetStats tree view."))
				.IsChecked(this, &SPacketContentView::FilterByEventType_IsChecked)
				.OnCheckStateChanged(this, &SPacketContentView::FilterByEventType_OnCheckStateChanged)
				[
					SNew(SBox)
					.Padding(FMargin(2.0f, 0.0f, 0.0f, 0.0f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FilterByEventType_Text", "By Type:"))
					]
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(2.0f, 0.0f, 4.0f, 0.0f))
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.Text(this, &SPacketContentView::GetFilterEventTypeText)
				.ToolTipText(LOCTEXT("EventType_Tooltip", "Event Type\n\nTo set the event type:\n\tdouble click either an event in the Packet Content view\n\tor an event type in the NetStats tree view."))
				.IsReadOnly(true)
				.MinDesiredWidth(120.0f)
			]
		);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("HighlightFilteredEvents");
	{
		ToolbarBuilder.AddWidget(
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 4.0f, 0.0f))
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("HighlightFilteredEvents_Tooltip", "Highlight filtered events."))
				.IsChecked(this, &SPacketContentView::HighlightFilteredEvents_IsChecked)
				.OnCheckStateChanged(this, &SPacketContentView::HighlightFilteredEvents_OnCheckStateChanged)
				[
					SNew(SBox)
					.Padding(FMargin(4.0f, 0.0f, 4.0f, 0.0f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("HighlightFilteredEvents_Text", "Highlight"))
					]
				]
			]
		);
	}
	ToolbarBuilder.EndSection();
	ToolbarBuilder.BeginSection("AggregationType");
	{
		ToolbarBuilder.AddWidget(
			SNew(SBox)
			.Padding(FMargin(12.0f, 0.0f, 0.0f, 0.0f))
			[
				AggregationModeWidget
			]
		);
	}
	ToolbarBuilder.EndSection();

	ChildSlot
	[
		SNew(SOverlay)
		.Visibility(EVisibility::SelfHitTestInvisible)

		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		.Padding(FMargin(0.0f))
		[
			ToolbarBuilder.MakeWidget()
		]

		+ SOverlay::Slot()
		.VAlign(VAlign_Bottom)
		.Padding(FMargin(0.0f))
		[
			SAssignNew(HorizontalScrollBar, SScrollBar)
			.Orientation(Orient_Horizontal)
			.AlwaysShowScrollbar(false)
			.Visibility(EVisibility::Visible)
			.OnUserScrolled(this, &SPacketContentView::HorizontalScrollBar_OnUserScrolled)
		]
	];

	UpdateHorizontalScrollBar();

	BindCommands();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::FindPreviousPacket()
{
	TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = ProfilerWindowWeakPtr.Pin();
	if (ProfilerWindow.IsValid())
	{
		TSharedPtr<SPacketView> PacketView = ProfilerWindow->GetPacketView();
		if (PacketView.IsValid())
		{
			PacketView->SelectPreviousPacket();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::FindNextPacket()
{
	TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = ProfilerWindowWeakPtr.Pin();
	if (ProfilerWindow.IsValid())
	{
		TSharedPtr<SPacketView> PacketView = ProfilerWindow->GetPacketView();
		if (PacketView.IsValid())
		{
			PacketView->SelectNextPacket();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SPacketContentView::GetPacketText() const
{
	return FText::AsNumber(PacketSequence);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::Packet_OnTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	if (InNewText.IsNumeric())
	{
		uint32 NewPacketSequence = 0;
		TTypeFromString<uint32>::FromString(NewPacketSequence, *InNewText.ToString());

		TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = ProfilerWindowWeakPtr.Pin();
		if (ProfilerWindow.IsValid())
		{
			TSharedPtr<SPacketView> PacketView = ProfilerWindow->GetPacketView();
			if (PacketView.IsValid())
			{
				PacketView->SelectPacketBySequenceNumber(NewPacketSequence);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::FindFirstEvent()
{
	if (FilteredDrawState->Events.Num() > 0)
	{
		SelectedEvent.Set(FilteredDrawState->Events[0]);
		OnSelectedEventChanged();
		BringEventIntoView(SelectedEvent);
	}
	else
	{
		FMessageLog ReportMessageLog(FNetworkingProfilerManager::Get()->GetLogListingName());
		ReportMessageLog.Error(LOCTEXT("NoEventFound", "No event found!"));
		ReportMessageLog.Notify();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::FindPreviousEvent(EEventNavigationType NavigationType)
{
	if (!SelectedEvent.IsValid())
	{
		FindFirstEvent();
		return;
	}

	FNetworkPacketEventRef PreviousSelectedEvent = SelectedEvent;

	const int32 EventCount = FilteredDrawState->Events.Num();
	for (int32 EventIndex = EventCount - 1; EventIndex >= 0; --EventIndex)
	{
		const FNetworkPacketEvent& Event = FilteredDrawState->Events[EventIndex];
		if (Event.Equals(SelectedEvent.Event))
		{
			if (EventIndex > 0)
			{
				switch (NavigationType)
				{
				case EEventNavigationType::AnyLevel:
					SelectedEvent.Set(FilteredDrawState->Events[EventIndex - 1]);
					OnSelectedEventChanged();
					break;
				case EEventNavigationType::SameLevel:
					for (int32 PrevEventIndex = EventIndex - 1; PrevEventIndex >= 0; --PrevEventIndex)
					{
						const FNetworkPacketEvent& PrevEvent = FilteredDrawState->Events[PrevEventIndex];
						if (Event.Level == PrevEvent.Level)
						{
							SelectedEvent.Set(FilteredDrawState->Events[PrevEventIndex]);
							OnSelectedEventChanged();
							break;
						}
					}
					break;
				default:
					break;
				}
				break;
			}
		}
		else if (Event.BitOffset <= SelectedEvent.Event.BitOffset)
		{
			if (Event.BitOffset < SelectedEvent.Event.BitOffset || Event.Level < SelectedEvent.Event.Level)
			{
				SelectedEvent.Set(Event);
				OnSelectedEventChanged();
				break;
			}
		}
	}

	BringEventIntoView(SelectedEvent);

	if (PreviousSelectedEvent.Equals(SelectedEvent))
	{
		FMessageLog ReportMessageLog(FNetworkingProfilerManager::Get()->GetLogListingName());
		ReportMessageLog.Error(LOCTEXT("NoEventFound", "No event found!"));
		ReportMessageLog.Notify();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::FindNextEvent(EEventNavigationType NavigationType)
{
	if (!SelectedEvent.IsValid())
	{
		FindLastEvent();
		return;
	}

	FNetworkPacketEventRef PreviousSelectedEvent = SelectedEvent;

	const int32 EventCount = FilteredDrawState->Events.Num();
	for (int32 EventIndex = 0; EventIndex < EventCount; ++EventIndex)
	{
		const FNetworkPacketEvent& Event = FilteredDrawState->Events[EventIndex];
		if (Event.Equals(SelectedEvent.Event))
		{
			if (EventIndex < EventCount - 1)
			{
				switch (NavigationType)
				{
				case EEventNavigationType::AnyLevel:
					SelectedEvent.Set(FilteredDrawState->Events[EventIndex + 1]);
					OnSelectedEventChanged();
					break;
				case EEventNavigationType::SameLevel:
					for (int32 NextEventIndex = EventIndex + 1; NextEventIndex <= EventCount - 1; ++NextEventIndex)
					{
						const FNetworkPacketEvent& NextEvent = FilteredDrawState->Events[NextEventIndex];
						if (Event.Level == NextEvent.Level)
						{
							SelectedEvent.Set(FilteredDrawState->Events[NextEventIndex]);
							OnSelectedEventChanged();
							break;
						}
					}
					break;
				default:
					break;
				}
				break;
			}
		}
		else if (Event.BitOffset >= SelectedEvent.Event.BitOffset)
		{
			if (Event.BitOffset > SelectedEvent.Event.BitOffset || Event.Level > SelectedEvent.Event.Level)
			{
				SelectedEvent.Set(Event);
				OnSelectedEventChanged();
				break;
			}
		}
	}

	BringEventIntoView(SelectedEvent);

	if (PreviousSelectedEvent.Equals(SelectedEvent))
	{
		FMessageLog ReportMessageLog(FNetworkingProfilerManager::Get()->GetLogListingName());
		ReportMessageLog.Error(LOCTEXT("NoEventFound", "No event found!"));
		ReportMessageLog.Notify();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::FindLastEvent()
{
	if (FilteredDrawState->Events.Num() > 0)
	{
		SelectedEvent.Set(FilteredDrawState->Events.Last());
		OnSelectedEventChanged();
		BringEventIntoView(SelectedEvent);
	}
	else
	{
		FMessageLog ReportMessageLog(FNetworkingProfilerManager::Get()->GetLogListingName());
		ReportMessageLog.Error(LOCTEXT("NoEventFound", "No event found!"));
		ReportMessageLog.Notify();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::FindPreviousLevel()
{
	if (!SelectedEvent.IsValid())
	{
		FindFirstEvent();
		return;
	}

	const int32 EventCount = FilteredDrawState->Events.Num();
	for (int32 EventIndex = 0; EventIndex < EventCount; ++EventIndex)
	{
		const FNetworkPacketEvent& Event = FilteredDrawState->Events[EventIndex];
		if (Event.Equals(SelectedEvent.Event))
		{
			for (int32 PrevEventIndex = EventIndex - 1; PrevEventIndex > 0; --PrevEventIndex)
			{
				const FNetworkPacketEvent& PrevEvent = FilteredDrawState->Events[PrevEventIndex];
				if (PrevEvent.Level < Event.Level &&
					PrevEvent.BitOffset <= Event.BitOffset &&
					PrevEvent.BitSize + PrevEvent.BitOffset >= Event.BitSize + Event.BitOffset)
				{
					SelectedEvent.Set(FilteredDrawState->Events[PrevEventIndex]);
					OnSelectedEventChanged();
					break;
				}
			}
			break;
		}
	}

	BringEventIntoView(SelectedEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::FindNextLevel()
{
	if (!SelectedEvent.IsValid())
	{
		FindLastEvent();
		return;
	}

	const int32 EventCount = FilteredDrawState->Events.Num();
	for (int32 EventIndex = 0; EventIndex < EventCount; ++EventIndex)
	{
		const FNetworkPacketEvent& Event = FilteredDrawState->Events[EventIndex];
		if (Event.Equals(SelectedEvent.Event))
		{
			for (int32 NextEventIndex = EventIndex + 1; NextEventIndex < EventCount; ++NextEventIndex)
			{
				const FNetworkPacketEvent& NextEvent = FilteredDrawState->Events[NextEventIndex];
				if (NextEvent.Level > Event.Level &&
					NextEvent.BitOffset >= Event.BitOffset &&
					NextEvent.BitSize + NextEvent.BitOffset <= Event.BitSize + Event.BitOffset)
				{
					SelectedEvent.Set(FilteredDrawState->Events[NextEventIndex]);
					OnSelectedEventChanged();
					break;
				}
			}
			break;
		}
	}

	BringEventIntoView(SelectedEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState SPacketContentView::FilterByNetId_IsChecked() const
{
	return bFilterByNetId ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::FilterByNetId_OnCheckStateChanged(ECheckBoxState NewState)
{
	bFilterByNetId = (NewState == ECheckBoxState::Checked);
	bIsStateDirty = true;

	TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = ProfilerWindowWeakPtr.Pin();
	if (ProfilerWindow.IsValid())
	{
		TSharedPtr<SPacketView> PacketView = ProfilerWindow->GetPacketView();
		if (PacketView.IsValid())
		{
			PacketView->InvalidateState();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SPacketContentView::GetFilterNetIdText() const
{
	return FText::AsNumber(FilterNetId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::FilterNetId_OnTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	if (InNewText.IsNumeric())
	{
		uint64 NewNetId = 0;
		TTypeFromString<uint64>::FromString(NewNetId, *InNewText.ToString());
		SetFilterNetId(NewNetId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState SPacketContentView::FilterByEventType_IsChecked() const
{
	return bFilterByEventType ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::FilterByEventType_OnCheckStateChanged(ECheckBoxState NewState)
{
	bFilterByEventType = (NewState == ECheckBoxState::Checked);
	bIsStateDirty = true;

	TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = ProfilerWindowWeakPtr.Pin();
	if (ProfilerWindow.IsValid())
	{
		TSharedPtr<SPacketView> PacketView = ProfilerWindow->GetPacketView();
		if (PacketView.IsValid())
		{
			PacketView->InvalidateState();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState SPacketContentView::HighlightFilteredEvents_IsChecked() const
{
	return bHighlightFilteredEvents ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::HighlightFilteredEvents_OnCheckStateChanged(ECheckBoxState NewState)
{
	bHighlightFilteredEvents = (NewState == ECheckBoxState::Checked);
	bIsStateDirty = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (ThisGeometry != AllottedGeometry || bIsViewportDirty)
	{
		bIsViewportDirty = false;
		const float ViewWidth = static_cast<float>(AllottedGeometry.GetLocalSize().X);
		const float ViewHeight = static_cast<float>(AllottedGeometry.GetLocalSize().Y);
		Viewport.SetSize(ViewWidth, ViewHeight);
		bIsStateDirty = true;
	}

	ThisGeometry = AllottedGeometry;

	const float FontScale = AllottedGeometry.Scale;
	Tooltip.SetFontScale(FontScale);

	FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();

	if (!bIsScrolling)
	{
		// Elastic snap to horizontal limits.
		if (ViewportX.UpdatePosWithinLimits())
		{
			bIsStateDirty = true;
		}
	}

	if (bIsStateDirty)
	{
		bIsStateDirty = false;
		UpdateState(FontScale);
		AdjustForSplitContent();
	}

	Tooltip.Update();
	if (!MousePosition.IsZero())
	{
		Tooltip.SetPosition(MousePosition, 0.0f, Viewport.GetWidth(), 0.0f, Viewport.GetHeight() - 12.0f);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::ResetPacket()
{
	GameInstanceIndex = 0;
	ConnectionIndex = 0;
	ConnectionMode = TraceServices::ENetProfilerConnectionMode::Outgoing;
	PacketIndex = 0;
	PacketSequence = 0;
	PacketBitSize = 0;

	FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.SetMinMaxValueInterval(0.0, 0.0);
	ViewportX.CenterOnValue(0.0);
	UpdateHorizontalScrollBar();

	DrawState->Reset();
	FilteredDrawState->Reset();
	bIsStateDirty = true;

	HoveredEvent.Reset();
	SelectedEvent.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::SetPacket(uint32 InGameInstanceIndex, uint32 InConnectionIndex, TraceServices::ENetProfilerConnectionMode InConnectionMode, uint32 InPacketIndex, int64 InPacketBitSize)
{
	GameInstanceIndex = InGameInstanceIndex;
	ConnectionIndex = InConnectionIndex;
	ConnectionMode = InConnectionMode;
	PacketIndex = InPacketIndex;
	PacketSequence = GetPacketSequence(InPacketIndex);
	PacketBitSize = InPacketBitSize;

	FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.SetMinMaxValueInterval(0.0, static_cast<double>(InPacketBitSize));
	ViewportX.CenterOnValueInterval(0.0, static_cast<double>(InPacketBitSize));
	UpdateHorizontalScrollBar();

	DrawState->Reset();
	FilteredDrawState->Reset();
	bIsStateDirty = true;

	HoveredEvent.Reset();
	SelectedEvent.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::SetFilterNetId(const uint64 InNetId)
{
	FilterNetId = InNetId;

	if (bFilterByNetId)
	{
		bIsStateDirty = true;

		TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = ProfilerWindowWeakPtr.Pin();
		if (ProfilerWindow.IsValid())
		{
			TSharedPtr<SPacketView> PacketView = ProfilerWindow->GetPacketView();
			if (PacketView.IsValid())
			{
				PacketView->InvalidateState();
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::SetFilterEventType(const uint32 InEventTypeIndex, const FText& InEventName)
{
	FilterEventTypeIndex = InEventTypeIndex;
	FilterEventName = InEventName;

	if (bFilterByEventType)
	{
		bIsStateDirty = true;

		TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = ProfilerWindowWeakPtr.Pin();
		if (ProfilerWindow.IsValid())
		{
			TSharedPtr<SPacketView> PacketView = ProfilerWindow->GetPacketView();
			if (PacketView.IsValid())
			{
				PacketView->InvalidateState();
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::EnableFilterEventType(const uint32 InEventTypeIndex)
{
	FText EventName;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
		if (NetProfilerProvider)
		{
			NetProfilerProvider->ReadEventType(InEventTypeIndex, [&EventName](const TraceServices::FNetProfilerEventType& EventType)
			{
				EventName = FText::FromString(EventType.Name);
			});
		}
	}

	bFilterByEventType = true;
	SetFilterEventType(InEventTypeIndex, EventName);
	bHighlightFilteredEvents = true;
	bIsStateDirty = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 SPacketContentView::GetPacketSequence(int32 InPacketIndex) const
{
	uint32 NewSequenceNumber = 0U;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
		if (NetProfilerProvider)
		{
			NetProfilerProvider->EnumeratePackets(ConnectionIndex, ConnectionMode, InPacketIndex, InPacketIndex, [&NewSequenceNumber](const TraceServices::FNetProfilerPacket& Packet)
			{
				NewSequenceNumber = Packet.SequenceNumber;
			});
		}
	}

	return NewSequenceNumber;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::DisableFilterEventType()
{
	bFilterByEventType = false;
	bHighlightFilteredEvents = false;
	bIsStateDirty = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::UpdateState(float FontScale)
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	if (PacketBitSize > 0)
	{
		FPacketContentViewDrawStateBuilder Builder(*DrawState, Viewport, FontScale);
		FPacketContentViewDrawStateBuilder FilteredDrawStateBuilder(*FilteredDrawState, Viewport, FontScale);

		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
			if (NetProfilerProvider)
			{
				// Count all events in packet, including split data
				const uint32 StartPos = 0U;
				const uint32 EndPos = ~0U;
				uint32 EndNetIdMatchPos = ~0U;
				uint32 EndEventTypeMatchPos = ~0U;
				uint32 HighestEndPos = 0;

				NetProfilerProvider->EnumeratePacketContentEventsByPosition(ConnectionIndex, ConnectionMode, PacketIndex, StartPos, EndPos, [this, &Builder, &FilteredDrawStateBuilder, NetProfilerProvider, &EndNetIdMatchPos, &EndEventTypeMatchPos, &HighestEndPos](const TraceServices::FNetProfilerContentEvent& Event)
				{
					const TCHAR* Name = nullptr;

					if (Event.EndPos > HighestEndPos)
					{
						HighestEndPos = Event.EndPos;
					}

					uint32 NameIndex = Event.NameIndex;
					uint64 NetId = 0;
					if (Event.ObjectInstanceIndex != 0)
					{
						NetProfilerProvider->ReadObject(GameInstanceIndex, Event.ObjectInstanceIndex, [&NetId, &NameIndex](const TraceServices::FNetProfilerObjectInstance& ObjectInstance)
						{
							NameIndex = ObjectInstance.NameIndex;
							NetId = ObjectInstance.NetObjectId;
						});
					}

					NetProfilerProvider->ReadName(NameIndex, [&Name](const TraceServices::FNetProfilerName& NetProfilerName)
					{
						Name = NetProfilerName.Name;
					});

					Builder.AddEvent(Event, Name, NetId);

					// Include events and sub-events matching event type
					if (bFilterByEventType)
					{
						if (Event.EndPos > EndEventTypeMatchPos)
						{
							EndEventTypeMatchPos = ~0U;
						}
						if (EndEventTypeMatchPos == ~0U && FilterEventTypeIndex == Event.EventTypeIndex)
						{
							EndEventTypeMatchPos = Event.EndPos;
						}
					}

					// Include events and sub-events matching net id
					if (bFilterByNetId)
					{
						if (Event.EndPos > EndNetIdMatchPos)
						{
							EndNetIdMatchPos = ~0U;
						}
						if (EndNetIdMatchPos == ~0U && (Event.ObjectInstanceIndex != 0 && FilterNetId == NetId))
						{
							EndNetIdMatchPos = Event.EndPos;
						}
					}

					if ((!bFilterByNetId || EndNetIdMatchPos != ~0U) && (!bFilterByEventType || EndEventTypeMatchPos != ~0U))
					{
						FilteredDrawStateBuilder.AddEvent(Event, Name, NetId);
					}
				});

				FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();
				if (HighestEndPos > ViewportX.GetMaxValue())
				{
					PacketBitSize = HighestEndPos;
					ViewportX.SetMinMaxValueInterval(0.0, static_cast<double>(PacketBitSize));
					ViewportX.CenterOnValueInterval(0.0, static_cast<double>(PacketBitSize));
					UpdateHorizontalScrollBar();
				}
			}
		}

		Builder.Flush();
		FilteredDrawStateBuilder.Flush();
	}

	Stopwatch.Stop();
	UpdateDurationHistory.AddValue(Stopwatch.AccumulatedTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::UpdateHoveredEvent()
{
	HoveredEvent = GetEventAtMousePosition(static_cast<float>(MousePosition.X), static_cast<float>(MousePosition.Y));
	//if (!HoveredEvent.IsValid())
	//{
	//	HoveredEvent = GetEventAtMousePosition(MousePosition.X - 1.0f, MousePosition.Y);
	//}
	//if (!HoveredEvent.IsValid())
	//{
	//	HoveredEvent = GetEventAtMousePosition(MousePosition.X + 1.0f, MousePosition.Y);
	//}

	if (HoveredEvent.IsValid())
	{
		// Init the tooltip's content.
		Tooltip.ResetContent();

		const FNetworkPacketEvent& Event = HoveredEvent.Event;
		FString Name(TEXT("?"));
		TraceServices::FNetProfilerEventType EventType;
		TraceServices::FNetProfilerObjectInstance ObjectInstance;

		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
			if (NetProfilerProvider)
			{
				NetProfilerProvider->ReadEventType(Event.EventTypeIndex, [&EventType](const TraceServices::FNetProfilerEventType& InEventType)
				{
					EventType = InEventType;
				});

				NetProfilerProvider->ReadName(EventType.NameIndex, [&Name](const TraceServices::FNetProfilerName& NetProfilerName)
				{
					Name = NetProfilerName.Name;
				});

				if (Event.ObjectInstanceIndex != 0)
				{
					NetProfilerProvider->ReadObject(GameInstanceIndex, Event.ObjectInstanceIndex, [&ObjectInstance](const TraceServices::FNetProfilerObjectInstance& InObjectInstance)
					{
						ObjectInstance = InObjectInstance;
					});
				}
			}
		}

		Tooltip.AddTitle(Name);

		if (Event.ObjectInstanceIndex != 0)
		{
			Tooltip.AddNameValueTextLine(TEXT("Net Id:"), FText::AsNumber(ObjectInstance.NetObjectId).ToString());
			Tooltip.AddNameValueTextLine(TEXT("Type Id:"), FString::Printf(TEXT("0x%016" UINT64_x_FMT), ObjectInstance.TypeId));
			Tooltip.AddNameValueTextLine(TEXT("Obj. LifeTime:"), FString::Format(TEXT("from {0} to {1}"),
				{ FormatTimeAuto(ObjectInstance.LifeTime.Begin), FormatTimeAuto(ObjectInstance.LifeTime.End)}));
		}

		Tooltip.AddNameValueTextLine(TEXT("Offset:"), FString::Format(TEXT("bit {0}"), { FText::AsNumber(Event.BitOffset).ToString() }));
		if (Event.BitSize == 1)
		{
			Tooltip.AddNameValueTextLine(TEXT("Size:"), TEXT("1 bit"));
		}
		else
		{
			Tooltip.AddNameValueTextLine(TEXT("Size:"), FString::Format(TEXT("{0} bits"), { FText::AsNumber(Event.BitSize).ToString() }));
		}

		Tooltip.AddNameValueTextLine(TEXT("Level:"), FText::AsNumber(Event.Level).ToString());

		Tooltip.UpdateLayout();

		Tooltip.SetDesiredOpacity(1.0f);
	}
	else
	{
		Tooltip.SetDesiredOpacity(0.0f);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::OnSelectedEventChanged()
{
	TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = ProfilerWindowWeakPtr.Pin();
	if (ProfilerWindow.IsValid())
	{
		if (SelectedEvent.IsValid())
		{
			// Select the node coresponding to net event type of selected net event instance.
			ProfilerWindow->SetSelectedEventTypeIndex(SelectedEvent.Event.EventTypeIndex);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::SelectHoveredEvent()
{
	SelectedEvent = HoveredEvent;
	OnSelectedEventChanged();
	BringEventIntoView(SelectedEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FNetworkPacketEventRef SPacketContentView::GetEventAtMousePosition(float X, float Y)
{
	if (!bIsStateDirty)
	{
		for (const FNetworkPacketEvent& Event : DrawState->Events)
		{
			const FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();

			const float EventX1 = ViewportX.GetRoundedOffsetForValue(static_cast<double>(Event.BitOffset));
			const float EventX2 = ViewportX.GetRoundedOffsetForValue(static_cast<double>(Event.BitOffset + Event.BitSize));

			const float EventY = Viewport.GetTopEventPosY() + (Viewport.GetEventHeight() + Viewport.GetEventDY()) * static_cast<float>(Event.Level);

			constexpr float ToleranceX = 1.0f;

			if (X >= EventX1 - ToleranceX && X <= EventX2 &&
				Y >= EventY - Viewport.GetEventDY() / 2 && Y < EventY + Viewport.GetEventHeight() + Viewport.GetEventDY() / 2)
			{
				return FNetworkPacketEventRef(Event);
			}
		}
	}
	return FNetworkPacketEventRef();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 SPacketContentView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const bool bEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	FDrawContext DrawContext(AllottedGeometry, MyCullingRect, InWidgetStyle, DrawEffects, OutDrawElements, LayerId);

	const float ViewWidth = static_cast<float>(AllottedGeometry.Size.X);
	const float ViewHeight = static_cast<float>(AllottedGeometry.Size.Y);

	//////////////////////////////////////////////////
	{
		FStopwatch Stopwatch;
		Stopwatch.Start();

		FPacketContentViewDrawHelper Helper(DrawContext, Viewport);

		Helper.DrawBackground();

		if (bHighlightFilteredEvents && (bFilterByNetId || bFilterByEventType))
		{
			Helper.Draw(*DrawState, 0.1f);
			Helper.Draw(*FilteredDrawState);
		}
		else
		{
			// Draw the events contained by the network packet using the cached draw state.
			Helper.Draw(*DrawState);
		}

		if (!FNetworkPacketEventRef::AreEquals(SelectedEvent, HoveredEvent))
		{
			// Highlight the selected event (if any).
			if (SelectedEvent.IsValid())
			{
				Helper.DrawEventHighlight(SelectedEvent.Event, FPacketContentViewDrawHelper::EHighlightMode::Selected);
			}

			// Highlight the hovered event (if any).
			if (HoveredEvent.IsValid())
			{
				Helper.DrawEventHighlight(HoveredEvent.Event, FPacketContentViewDrawHelper::EHighlightMode::Hovered);
			}
		}
		else
		{
			// Highlight the selected and hovered event (if any).
			if (SelectedEvent.IsValid())
			{
				Helper.DrawEventHighlight(SelectedEvent.Event, FPacketContentViewDrawHelper::EHighlightMode::SelectedAndHovered);
			}
		}

		// Draw tooltip for hovered Event.
		Tooltip.Draw(DrawContext);

		Stopwatch.Stop();
		DrawDurationHistory.AddValue(Stopwatch.AccumulatedTime);
	}
	//////////////////////////////////////////////////

	const bool bShouldDisplayDebugInfo = FInsightsManager::Get()->IsDebugInfoEnabled();
	if (bShouldDisplayDebugInfo)
	{
		const FSlateBrush* WhiteBrush = FAppStyle::Get().GetBrush("WhiteBrush");
		FSlateFontInfo SummaryFont = FAppStyle::Get().GetFontStyle("SmallFont");

		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const float FontScale = DrawContext.Geometry.Scale;
		const float MaxFontCharHeight = static_cast<float>(FontMeasureService->Measure(TEXT("!"), SummaryFont, FontScale).Y / FontScale);
		const float DbgDY = MaxFontCharHeight;

		const float DbgW = 280.0f;
		const float DbgH = DbgDY * 5 + 3.0f;
		const float DbgX = ViewWidth - DbgW - 20.0f;
		float DbgY = 7.0f;

		DrawContext.LayerId++;

		DrawContext.DrawBox(DbgX - 2.0f, DbgY - 2.0f, DbgW, DbgH, WhiteBrush, FLinearColor(1.0f, 1.0f, 1.0f, 0.9f));
		DrawContext.LayerId++;

		FLinearColor DbgTextColor(0.0f, 0.0f, 0.0f, 0.9f);

		// Time interval since last OnPaint call.
		const uint64 CurrentTime = FPlatformTime::Cycles64();
		const uint64 OnPaintDuration = CurrentTime - LastOnPaintTime;
		LastOnPaintTime = CurrentTime;
		OnPaintDurationHistory.AddValue(OnPaintDuration);
		const uint64 AvgOnPaintDuration = OnPaintDurationHistory.ComputeAverage();
		const uint64 AvgOnPaintDurationMs = FStopwatch::Cycles64ToMilliseconds(AvgOnPaintDuration);
		const double AvgOnPaintFps = AvgOnPaintDurationMs != 0 ? 1.0 / FStopwatch::Cycles64ToSeconds(AvgOnPaintDuration) : 0.0;

		const uint64 AvgUpdateDurationMs = FStopwatch::Cycles64ToMilliseconds(UpdateDurationHistory.ComputeAverage());
		const uint64 AvgDrawDurationMs = FStopwatch::Cycles64ToMilliseconds(DrawDurationHistory.ComputeAverage());

		// Draw performance info.
		DrawContext.DrawText
		(
			DbgX, DbgY,
			FString::Printf(TEXT("U: %" UINT64_FMT " ms    D: %" UINT64_FMT " ms + %" UINT64_FMT " ms = %" UINT64_FMT " ms (%" INT64_FMT " fps)"),
				AvgUpdateDurationMs, // average duration of UpdateState calls
				AvgDrawDurationMs, // drawing time
				AvgOnPaintDurationMs - AvgDrawDurationMs, // other overhead to OnPaint calls
				AvgOnPaintDurationMs, // average time between two OnPaint calls
				FMath::RoundToInt(AvgOnPaintFps)), // framerate of OnPaint calls
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		// Draw "the update stats".
		DrawContext.DrawText
		(
			DbgX, DbgY,
			FString::Printf(TEXT("U: %s events"),
				*FText::AsNumber(DrawState->Events.Num()).ToString()),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		// Draw "the draw stats".
		DrawContext.DrawText
		(
			DbgX, DbgY,
			FString::Printf(TEXT("D: %s boxes, %s borders (%s merged), %s texts"),
				*FText::AsNumber(DrawState->Boxes.Num()).ToString(),
				*FText::AsNumber(DrawState->Borders.Num()).ToString(),
				*FText::AsNumber(DrawState->GetNumMergedBoxes()).ToString(),
				*FText::AsNumber(DrawState->Texts.Num()).ToString()),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		// Draw viewport's horizontal info.
		DrawContext.DrawText
		(
			DbgX, DbgY,
			Viewport.GetHorizontalAxisViewport().ToDebugString(TEXT("X")),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		// Draw packet info.
		DrawContext.DrawText
		(
			DbgX, DbgY,
			FString::Printf(TEXT("Game Instance %d, Connection %d (%s), Packet %d"),
				GameInstanceIndex,
				ConnectionIndex,
				(ConnectionMode == TraceServices::ENetProfilerConnectionMode::Outgoing) ? TEXT("Outgoing") : TEXT("Incoming"),
				PacketIndex),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;
	}

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled && IsEnabled());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SPacketContentView::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	MousePositionOnButtonDown = MousePosition;

	ViewportPosXOnButtonDown = Viewport.GetHorizontalAxisViewport().GetPos();

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bIsLMB_Pressed = true;

		// Capture mouse.
		Reply = FReply::Handled().CaptureMouse(SharedThis(this));
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		bIsRMB_Pressed = true;

		// Capture mouse, so we can drag outside this widget.
		Reply = FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SPacketContentView::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	MousePositionOnButtonUp = MousePosition;

	const bool bIsValidForMouseClick = MousePositionOnButtonUp.Equals(MousePositionOnButtonDown, MOUSE_SNAP_DISTANCE);

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (bIsLMB_Pressed)
		{
			if (bIsScrolling)
			{
				bIsScrolling = false;
				CursorType = ECursorType::Default;
			}
			else if (bIsValidForMouseClick)
			{
				// Select the hovered timing event (if any).
				UpdateHoveredEvent();
				SelectHoveredEvent();
			}

			// Release mouse as we no longer drag.
			Reply = FReply::Handled().ReleaseMouseCapture();

			bIsLMB_Pressed = false;
		}
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (bIsRMB_Pressed)
		{
			if (bIsScrolling)
			{
				bIsScrolling = false;
				CursorType = ECursorType::Default;
			}
			else if (bIsValidForMouseClick)
			{
				//ShowContextMenu(MouseEvent);
			}

			// Release mouse as we no longer drag.
			Reply = FReply::Handled().ReleaseMouseCapture();

			bIsRMB_Pressed = false;
		}
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SPacketContentView::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	if (!MouseEvent.GetCursorDelta().IsZero())
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) ||
			MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
		{
			if (HasMouseCapture())
			{
				if (!bIsScrolling)
				{
					bIsScrolling = true;
					CursorType = ECursorType::Hand;

					HoveredEvent.Reset();
					Tooltip.SetDesiredOpacity(0.0f);
				}

				FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();
				const float PosX = ViewportPosXOnButtonDown + static_cast<float>(MousePositionOnButtonDown.X - MousePosition.X);
				ViewportX.ScrollAtPos(PosX);
				UpdateHorizontalScrollBar();
				bIsStateDirty = true;
			}
		}
		else
		{
			UpdateHoveredEvent();
		}

		Reply = FReply::Handled();
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	if (!HasMouseCapture())
	{
		// No longer dragging (unless we have mouse capture).
		bIsScrolling = false;

		bIsLMB_Pressed = false;
		bIsRMB_Pressed = false;

		MousePosition = FVector2D::ZeroVector;

		HoveredEvent.Reset();
		Tooltip.SetDesiredOpacity(0.0f);

		CursorType = ECursorType::Default;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SPacketContentView::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	//if (MouseEvent.GetModifierKeys().IsShiftDown())
	//{
	//}
	//else //if (MouseEvent.GetModifierKeys().IsControlDown())
	{
		// Zoom in/out horizontally.
		const float Delta = MouseEvent.GetWheelDelta();
		ZoomHorizontally(Delta, static_cast<float>(MousePosition.X));
	}

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SPacketContentView::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Select the hovered timing event (if any).
	UpdateHoveredEvent();
	SelectHoveredEvent();

	if (SelectedEvent.IsValid())
	{
		EnableFilterEventType(SelectedEvent.Event.EventTypeIndex);
	}
	else
	{
		DisableFilterEventType();
	}

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FCursorReply SPacketContentView::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	FCursorReply CursorReply = FCursorReply::Unhandled();

	if (CursorType == ECursorType::Arrow)
	{
		CursorReply = FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}
	else if (CursorType == ECursorType::Hand)
	{
		CursorReply = FCursorReply::Cursor(EMouseCursor::GrabHand);
	}

	return CursorReply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SPacketContentView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Left)
	{
		if (InKeyEvent.GetModifierKeys().IsShiftDown())
		{
			FindFirstEvent();
		}
		else if (InKeyEvent.GetModifierKeys().IsControlDown() ||
				 InKeyEvent.GetModifierKeys().IsCommandDown())
		{
			TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = ProfilerWindowWeakPtr.Pin();
			if (ProfilerWindow.IsValid())
			{
				const TSharedPtr<SPacketView> PacketView = ProfilerWindow->GetPacketView();
				if (PacketView.IsValid())
				{
					PacketView->SelectPreviousPacket();
				}
			}
		}
		else
		{
			FindPreviousEvent(EEventNavigationType::SameLevel);
		}
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Right)
	{
		if (InKeyEvent.GetModifierKeys().IsShiftDown())
		{
			FindLastEvent();
		}
		else if (InKeyEvent.GetModifierKeys().IsControlDown() ||
				 InKeyEvent.GetModifierKeys().IsCommandDown())
		{
			TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = ProfilerWindowWeakPtr.Pin();
			if (ProfilerWindow.IsValid())
			{
				const TSharedPtr<SPacketView> PacketView = ProfilerWindow->GetPacketView();
				if (PacketView.IsValid())
				{
					PacketView->SelectNextPacket();
				}
			}
		}
		else
		{
			FindNextEvent(EEventNavigationType::SameLevel);
		}
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Up)
	{
		FindPreviousLevel();
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Down)
	{
		FindNextLevel();
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Equals ||
			 InKeyEvent.GetKey() == EKeys::Add)
	{
		ZoomHorizontally(1.0f, static_cast<float>(MousePosition.X));
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Hyphen ||
			 InKeyEvent.GetKey() == EKeys::Subtract)
	{
		ZoomHorizontally(-1.0f, static_cast<float>(MousePosition.X));
		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::BindCommands()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::HorizontalScrollBar_OnUserScrolled(float ScrollOffset)
{
	FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.OnUserScrolled(HorizontalScrollBar, ScrollOffset);
	bIsStateDirty = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::UpdateHorizontalScrollBar()
{
	FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.UpdateScrollBar(HorizontalScrollBar);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::ZoomHorizontally(const float Delta, const float X)
{
	FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.RelativeZoomWithFixedOffset(Delta, X);
	UpdateHorizontalScrollBar();
	bIsStateDirty = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::BringIntoView(const float X1, const float X2)
{
	FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();

	// Increase interval with 8% (of view size) on each side.
	const float DX = ViewportX.GetSize() * 0.08f;

	float NewPos = ViewportX.GetPos();

	const float MinPos = X2 + DX - ViewportX.GetSize();
	if (NewPos < MinPos)
	{
		NewPos = MinPos;
	}

	const float MaxPos = X1 - DX;
	if (NewPos > MaxPos)
	{
		NewPos = MaxPos;
	}

	if (NewPos != ViewportX.GetPos())
	{
		ViewportX.ScrollAtPos(NewPos);
		UpdateHorizontalScrollBar();
		bIsStateDirty = true;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::BringEventIntoView(const FNetworkPacketEventRef& EventRef)
{
	if (EventRef.IsValid())
	{
		const FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();
		const float X1 = ViewportX.GetPosForValue(static_cast<double>(EventRef.Event.BitOffset));
		const float X2 = ViewportX.GetPosForValue(static_cast<double>(EventRef.Event.BitOffset + SelectedEvent.Event.BitSize));
		BringIntoView(X1, X2);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketContentView::AdjustForSplitContent()
{
	if (FilteredDrawState->Events.Num() > 0)
	{
		FAxisViewportDouble& ViewportX = Viewport.GetHorizontalAxisViewport();
		const FNetworkPacketEvent& LastEvent = FilteredDrawState->Events.Last();
		const uint32 LastBit = LastEvent.BitOffset + LastEvent.BitSize;
		if (LastBit > PacketBitSize)
		{
			PacketBitSize = LastBit;
			ViewportX.SetMinMaxValueInterval(0.0, static_cast<double>(PacketBitSize));
			ViewportX.CenterOnValueInterval(0.0, static_cast<double>(PacketBitSize));
			UpdateHorizontalScrollBar();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
