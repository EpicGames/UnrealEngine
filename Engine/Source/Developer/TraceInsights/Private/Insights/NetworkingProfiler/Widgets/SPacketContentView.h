// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Geometry.h"
#include "Rendering/RenderingCommon.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"

// TraceServices
#include "TraceServices/Model/NetProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/FixedCircularBuffer.h"

// TraceInsights
#include "Insights/NetworkingProfiler/ViewModels/PacketContentViewDrawHelper.h"
#include "Insights/NetworkingProfiler/ViewModels/PacketContentViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"

class SScrollBar;

namespace UE::Insights::NetworkingProfiler
{

class SNetworkingProfilerWindow;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FNetworkPacketEventRef
{
	FNetworkPacketEvent Event;
	bool bIsValid;

	FNetworkPacketEventRef()
		: Event()
		, bIsValid(false)
	{
	}

	FNetworkPacketEventRef(const FNetworkPacketEvent& InEvent)
		: Event(InEvent)
		, bIsValid(true)
	{
	}

	FNetworkPacketEventRef(const FNetworkPacketEventRef& Other)
		: Event(Other.Event)
		, bIsValid(Other.bIsValid)
	{
	}

	FNetworkPacketEventRef& operator=(const FNetworkPacketEventRef& Other)
	{
		Event = Other.Event;
		bIsValid = Other.bIsValid;
		return *this;
	}

	void Set(const FNetworkPacketEvent& InEvent)
	{
		Event = InEvent;
		bIsValid = true;
	}

	void Reset()
	{
		bIsValid = false;
	}

	bool IsValid() const
	{
		return bIsValid;
	}

	bool Equals(const FNetworkPacketEventRef& Other) const
	{
		return bIsValid == Other.bIsValid && Event.Equals(Other.Event);
	}

	static bool AreEquals(const FNetworkPacketEventRef& A, const FNetworkPacketEventRef& B)
	{
		return A.Equals(B);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Widget used to present content of a network packet.
 */
class SPacketContentView : public SCompoundWidget
{
private:
	struct FAggregationModeItem
	{
		/** Conversion constructor. */
		FAggregationModeItem(const TraceServices::ENetProfilerAggregationMode& InMode)
			: Mode(InMode)
		{}

		FText GetText() const;
		FText GetTooltipText() const;

		TraceServices::ENetProfilerAggregationMode Mode;
	};

public:
	/** Number of pixels. */
	static constexpr float MOUSE_SNAP_DISTANCE = 2.0f;

	enum class ECursorType
	{
		Default,
		Arrow,
		Hand,
	};

	enum class EEventNavigationType { AnyLevel, SameLevel };

public:
	/** Default constructor. */
	SPacketContentView();

	/** Virtual destructor. */
	virtual ~SPacketContentView();

	/** Resets internal widget's data to the default one. */
	void Reset();

	SLATE_BEGIN_ARGS(SPacketContentView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<SNetworkingProfilerWindow> InProfilerWindow);

	/**
	 * Ticks this widget. Override in derived classes, but always call the parent implementation.
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	void ResetPacket();
	void SetPacket(uint32 InGameInstanceIndex, uint32 InConnectionIndex, TraceServices::ENetProfilerConnectionMode InConnectionMode, uint32 InPacketIndex, int64 InPacketBitSize);

	bool IsFilterByNetIdEnabled() const { return bFilterByNetId; }
	uint64 GetFilterNetId() const { return FilterNetId; }
	void SetFilterNetId(const uint64 InNetId);

	bool IsFilterByEventTypeEnabled() const { return bFilterByEventType; }
	uint32 GetFilterEventTypeIndex() const { return FilterEventTypeIndex; }
	const FText& GetFilterEventName() const { return FilterEventName; }
	void SetFilterEventType(const uint32 InEventTypeIndex, const FText& InEventName);
	void EnableFilterEventType(const uint32 InEventTypeIndex);
	void DisableFilterEventType();

	TraceServices::ENetProfilerAggregationMode GetSelectedFilterEventAggregationMode() const { return SelectedAggregationMode ? SelectedAggregationMode->Mode : TraceServices::ENetProfilerAggregationMode::Aggregate; }

	void FindFirstEvent();
	void FindPreviousEvent(EEventNavigationType NavigationType);
	void FindNextEvent(EEventNavigationType NavigationType);
	void FindLastEvent();
	void FindPreviousLevel();
	void FindNextLevel();

private:
	void FindPreviousPacket();
	void FindNextPacket();

	FText GetPacketText() const;
	void Packet_OnTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

	ECheckBoxState FilterByNetId_IsChecked() const;
	void FilterByNetId_OnCheckStateChanged(ECheckBoxState NewState);
	FText GetFilterNetIdText() const;
	void FilterNetId_OnTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

	ECheckBoxState FilterByEventType_IsChecked() const;
	void FilterByEventType_OnCheckStateChanged(ECheckBoxState NewState);
	FText GetFilterEventTypeText() const { return FilterEventName; }

	ECheckBoxState HighlightFilteredEvents_IsChecked() const;
	void HighlightFilteredEvents_OnCheckStateChanged(ECheckBoxState NewState);

	//void ShowContextMenu(const FPointerEvent& MouseEvent);

	/** Binds our UI commands to delegates. */
	void BindCommands();

	////////////////////////////////////////////////////////////////////////////////////////////////////

	/**
	 * Called when the user scrolls the horizontal scrollbar.
	 * @param ScrollOffset Scroll offset as a fraction between 0 and 1.
	 */
	void HorizontalScrollBar_OnUserScrolled(float ScrollOffset);
	void UpdateHorizontalScrollBar();

	void ZoomHorizontally(const float Delta, const float X);
	void BringIntoView(const float X1, const float X2);
	void BringEventIntoView(const FNetworkPacketEventRef& EventRef);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	uint32 GetPacketSequence(int32 PacketIndex) const;

	void UpdateState(float FontScale);

	void UpdateHoveredEvent();
	FNetworkPacketEventRef GetEventAtMousePosition(float X, float Y);

	void OnSelectedEventChanged();
	void SelectHoveredEvent();

	void AdjustForSplitContent();

	TSharedRef<SWidget> CreateAggregationModeComboBox();
	TSharedRef<SWidget> AggregationMode_OnGenerateWidget(TSharedPtr<FAggregationModeItem> InAggregationMode) const;
	void AggregationMode_OnSelectionChanged(TSharedPtr<FAggregationModeItem> NewAggregationMode, ESelectInfo::Type SelectInfo);
	FText AggregationMode_GetSelectedText() const;
	FText AggregationMode_GetSelectedTooltipText() const;

private:
	TWeakPtr<SNetworkingProfilerWindow> ProfilerWindowWeakPtr;

	/** The track's viewport. Encapsulates info about position and scale. */
	FPacketContentViewport Viewport;
	bool bIsViewportDirty;

	uint32 GameInstanceIndex;
	uint32 ConnectionIndex;
	TraceServices::ENetProfilerConnectionMode ConnectionMode;
	uint32 PacketIndex;
	uint32 PacketSequence;
	int64 PacketBitSize; // total number of bits; [bit]

	bool bFilterByEventType;
	uint32 FilterEventTypeIndex;
	FText FilterEventName;

	bool bFilterByNetId;
	uint64 FilterNetId;

	bool bHighlightFilteredEvents;

	/** Cached draw state of the packet content. */
	TSharedRef<FPacketContentViewDrawState> DrawState;
	TSharedRef<FPacketContentViewDrawState> FilteredDrawState;
	bool bIsStateDirty;

	//////////////////////////////////////////////////

	TSharedPtr<SScrollBar> HorizontalScrollBar;

	TSharedPtr<SComboBox<TSharedPtr<FAggregationModeItem>>> AggregationModeComboBox;
	TArray<TSharedPtr<FAggregationModeItem>> AvailableAggregationModes;
	TSharedPtr<FAggregationModeItem> SelectedAggregationMode;

	//////////////////////////////////////////////////
	// Panning and Zooming behaviors

	/** The current mouse position. */
	FVector2D MousePosition;

	/** Mouse position during the call on mouse button down. */
	FVector2D MousePositionOnButtonDown;
	float ViewportPosXOnButtonDown;

	/** Mouse position during the call on mouse button up. */
	FVector2D MousePositionOnButtonUp;

	bool bIsLMB_Pressed;
	bool bIsRMB_Pressed;

	/** True, if the user is currently interactively scrolling the view (ex.: by holding the left mouse button and dragging). */
	bool bIsScrolling;

	//////////////////////////////////////////////////
	// Selection

	FNetworkPacketEventRef HoveredEvent;
	FNetworkPacketEventRef SelectedEvent;

	FTooltipDrawState Tooltip;

	//////////////////////////////////////////////////
	// Misc

	FGeometry ThisGeometry;

	/** Cursor type. */
	ECursorType CursorType;

	// Debug stats
	TFixedCircularBuffer<uint64, 32> UpdateDurationHistory;
	mutable TFixedCircularBuffer<uint64, 32> DrawDurationHistory;
	mutable TFixedCircularBuffer<uint64, 32> OnPaintDurationHistory;
	mutable uint64 LastOnPaintTime;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler
