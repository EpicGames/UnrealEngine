// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPacketView.h"

#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformTime.h"
#include "Rendering/DrawElements.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SScrollBar.h"

// TraceServices
#include "TraceServices/Model/Frames.h"
#include "TraceServices/Model/NetProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/NetworkingProfiler/NetworkingProfilerManager.h"
#include "Insights/NetworkingProfiler/Widgets/SNetworkingProfilerWindow.h"
#include "Insights/NetworkingProfiler/Widgets/SPacketContentView.h"

#include <limits>

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::NetworkingProfiler::SPacketView"

namespace UE::Insights::NetworkingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

SPacketView::SPacketView()
	: ProfilerWindowWeakPtr()
	, PacketSeries(MakeShared<FNetworkPacketSeries>())
{
	Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SPacketView::~SPacketView()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::Reset()
{
	//ProfilerWindowWeakPtr

	GameInstanceIndex = 0;
	ConnectionIndex = 0;
	ConnectionMode = TraceServices::ENetProfilerConnectionMode::Outgoing;

	Viewport.Reset();
	FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.SetScaleLimits(0.0001f, 16.0f); // 10000 [sample/px] to 16 [px/sample]
	ViewportX.SetScale(5.0f);
	FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();
	ViewportY.SetScaleLimits(0.0001, 50.0);
	ViewportY.SetScale(0.02);
	bIsViewportDirty = true;

	PacketSeries->Reset();

	bIsStateDirty = true;

	bIsAutoZoomEnabled = true;
	AutoZoomViewportPos = ViewportX.GetPos();
	AutoZoomViewportScale = ViewportX.GetScale();
	AutoZoomViewportSize = 0.0f;

	AnalysisSyncNextTimestamp = 0;
	ConnectionChangeCount = 0;

	MousePosition = FVector2D::ZeroVector;

	MousePositionOnButtonDown = FVector2D::ZeroVector;
	ViewportPosXOnButtonDown = 0.0f;

	MousePositionOnButtonUp = FVector2D::ZeroVector;

	bIsLMB_Pressed = false;
	bIsRMB_Pressed = false;

	bIsScrolling = false;

	SelectionStartPacketIndex = 0;
	SelectionEndPacketIndex = 0;
	LastSelectedPacketIndex = 0;
	SelectedTimeSpan = 0.0;

	SelectedSample.Reset();
	HoveredSample.Reset();
	TooltipDesiredOpacity = 0.9f;
	TooltipOpacity = 0.0f;

	//ThisGeometry

	CursorType = ECursorType::Default;

	NumUpdatedPackets = 0;
	UpdateDurationHistory.Reset();
	DrawDurationHistory.Reset();
	OnPaintDurationHistory.Reset();
	LastOnPaintTime = FPlatformTime::Cycles64();

	TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = ProfilerWindowWeakPtr.Pin();
	if (ProfilerWindow.IsValid())
	{
		SetConnection(ProfilerWindow->GetSelectedGameInstanceIndex(), ProfilerWindow->GetSelectedConnectionIndex(), ProfilerWindow->GetSelectedConnectionMode());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::SetConnection(uint32 InGameInstanceIndex, uint32 InConnectionIndex, TraceServices::ENetProfilerConnectionMode InConnectionMode)
{
	GameInstanceIndex = InGameInstanceIndex;
	ConnectionIndex = InConnectionIndex;
	ConnectionMode = InConnectionMode;

	HoveredSample.Reset();

	SelectedSample.Reset();
	SelectionStartPacketIndex = 0;
	SelectionEndPacketIndex = 0;
	LastSelectedPacketIndex = 0;
	OnSelectionChanged();

	bIsStateDirty = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::Construct(const FArguments& InArgs, TSharedRef<SNetworkingProfilerWindow> InProfilerWindow)
{
	ProfilerWindowWeakPtr = InProfilerWindow;

	ChildSlot
	[
		SNew(SOverlay)
		.Visibility(EVisibility::SelfHitTestInvisible)

		+ SOverlay::Slot()
		.VAlign(VAlign_Top)
		.Padding(FMargin(0, 0, 0, 0))
		[
			SAssignNew(HorizontalScrollBar, SScrollBar)
			.Orientation(Orient_Horizontal)
			.AlwaysShowScrollbar(false)
			.Visibility(EVisibility::Visible)
			.Thickness(FVector2D(5.0f, 5.0f))
			.RenderOpacity(0.75)
			.OnUserScrolled(this, &SPacketView::HorizontalScrollBar_OnUserScrolled)
		]
	];

	UpdateHorizontalScrollBar();

	BindCommands();

	Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (ThisGeometry != AllottedGeometry || bIsViewportDirty)
	{
		bIsViewportDirty = false;
		const float ViewWidth = static_cast<float>(AllottedGeometry.GetLocalSize().X);
		const float ViewHeight = static_cast<float>(AllottedGeometry.GetLocalSize().Y);
		Viewport.SetSize(ViewWidth, ViewHeight);
		bIsStateDirty = true;
	}

	ThisGeometry = AllottedGeometry;

	FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

	if (!bIsScrolling)
	{
		// Elastic snap to horizontal limits.
		if (ViewportX.UpdatePosWithinLimits())
		{
			bIsStateDirty = true;
		}
	}

	// Disable auto-zoom if viewport's position or scale has changed.
	if (AutoZoomViewportPos != ViewportX.GetPos() ||
		AutoZoomViewportScale != ViewportX.GetScale())
	{
		bIsAutoZoomEnabled = false;
	}

	// Update auto-zoom if viewport size has changed.
	bool bAutoZoom = bIsAutoZoomEnabled && AutoZoomViewportSize != ViewportX.GetSize();

	const uint64 Time = FPlatformTime::Cycles64();
	if (Time > AnalysisSyncNextTimestamp)
	{
		const uint64 WaitTime = static_cast<uint64>(0.1 / FPlatformTime::GetSecondsPerCycle64()); // 100ms
		AnalysisSyncNextTimestamp = Time + WaitTime;

		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
			if (NetProfilerProvider)
			{
				const uint32 NewConnectionChangeCount = NetProfilerProvider->GetConnectionChangeCount();
				if (NewConnectionChangeCount != ConnectionChangeCount)
				{
					ConnectionChangeCount = NewConnectionChangeCount;
					bIsStateDirty = true;

					if (bIsAutoZoomEnabled)
					{
						bAutoZoom = true;
					}
				}
			}
		}
	}

	if (bAutoZoom)
	{
		AutoZoom();
	}

	if (bIsStateDirty)
	{
		bIsStateDirty = false;
		UpdateState();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SPacketView::IsConnectionValid(const TraceServices::INetProfilerProvider& NetProfilerProvider, const uint32 InGameInstanceIndex, const uint32 InConnectionIndex, const TraceServices::ENetProfilerConnectionMode InConnectionMode)
{
	//TODO: return NetProfilerProvider.IsConnectionValid(GameInstanceIndex, ConnectionIndex, ConnectionMode);

	const uint32 GameInstanceCount = NetProfilerProvider.GetGameInstanceCount();
	if (InGameInstanceIndex >= GameInstanceCount)
	{
		return false;
	}

	bool bIsValidConnection = false;
	NetProfilerProvider.ReadConnections(InGameInstanceIndex, [InConnectionIndex, InConnectionMode, &bIsValidConnection](const TraceServices::FNetProfilerConnection& Connection)
	{
		if (InConnectionIndex == Connection.ConnectionIndex)
		{
			if ((InConnectionMode == TraceServices::ENetProfilerConnectionMode::Outgoing && Connection.bHasOutgoingData) ||
				(InConnectionMode == TraceServices::ENetProfilerConnectionMode::Incoming && Connection.bHasIncomingData))
			{
				bIsValidConnection = true;
			}
		}
	});
	return bIsValidConnection;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::UpdateState()
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	//////////////////////////////////////////////////

	struct FPacketFilter
	{
		bool bByNetId = false;
		uint64 NetId = 0;
		bool bByEventType = false;
		uint32 EventTypeIndex = 0;
		TraceServices::ENetProfilerAggregationMode AggregationMode = TraceServices::ENetProfilerAggregationMode::None;
	};

	FPacketFilter Filter;

	TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = ProfilerWindowWeakPtr.Pin();
	if (ProfilerWindow)
	{
		TSharedPtr<SPacketContentView> PacketContentView = ProfilerWindow->GetPacketContentView();
		if (PacketContentView)
		{
			Filter.bByNetId = PacketContentView->IsFilterByNetIdEnabled();
			Filter.NetId = PacketContentView->GetFilterNetId();

			Filter.bByEventType = PacketContentView->IsFilterByEventTypeEnabled();
			Filter.EventTypeIndex = PacketContentView->GetFilterEventTypeIndex();
			Filter.AggregationMode = PacketContentView->GetSelectedFilterEventAggregationMode();
		}
	}

	//////////////////////////////////////////////////

	FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

	const int32 PreviousMaxValue = ViewportX.GetMaxValue();
	ViewportX.SetMinMaxInterval(0, 0);

	// Reset stats.
	PacketSeries->NumAggregatedPackets = 0;
	NumUpdatedPackets = 0;

	FNetworkPacketSeriesBuilder Builder(*PacketSeries, Viewport);

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
		if (NetProfilerProvider && IsConnectionValid(*NetProfilerProvider, GameInstanceIndex, ConnectionIndex, ConnectionMode))
		{
			const uint32 NumPackets = NetProfilerProvider->GetPacketCount(ConnectionIndex, ConnectionMode);

			ViewportX.SetMinMaxInterval(0, NumPackets);

			if (NumPackets > 0)
			{
				const int32 MinIndex = ViewportX.GetValueAtOffset(0.0f);
				const int32 MaxIndex = ViewportX.GetValueAtOffset(ViewportX.GetSize());

				const uint32 PacketStartIndex = static_cast<uint32>(FMath::Max(0, MinIndex));
				const uint32 PacketEndIndex = FMath::Min(NumPackets - 1, static_cast<uint32>(FMath::Max(0, MaxIndex)));

				if (PacketStartIndex <= PacketEndIndex)
				{
					int32 PacketIndex = PacketStartIndex;
					int32 FilterMatchEventTypeIndex = -1;
					NetProfilerProvider->EnumeratePackets(ConnectionIndex, ConnectionMode, PacketStartIndex, PacketEndIndex, [this, &Builder, &PacketIndex, &Filter, NetProfilerProvider, &FilterMatchEventTypeIndex](const TraceServices::FNetProfilerPacket& Packet)
					{
						FNetworkPacketAggregatedSample* SamplePtr = Builder.AddPacket(PacketIndex++, Packet);

						if (SamplePtr)
						{
							if (SamplePtr->NumPackets == 1)
							{
								SamplePtr->bAtLeastOnePacketMatchesFilter = !Filter.bByNetId && !Filter.bByEventType;
							}

							if ((!SamplePtr->bAtLeastOnePacketMatchesFilter || Filter.AggregationMode != TraceServices::ENetProfilerAggregationMode::None) && (Filter.bByNetId || Filter.bByEventType))
							{
								bool bFilterMatch = false;
								bool bOldEventMatchesFilter = false;
								uint32 FilterMatchAggregatedEventSizeInBits = 0U;
								uint32 FilterMatchMaxEventSizeBits = 0U;

								// Filter all events in packet, including split data
								const uint32 StartPos = 0;
								const uint32 EndPos = ~0U;
								uint32 EndNetIdMatchPos = ~0U;
								uint32 EndEventTypeMatchPos = ~0U;
								uint32 LastMatchingLevel = ~0U;

								NetProfilerProvider->EnumeratePacketContentEventsByPosition(ConnectionIndex, ConnectionMode, PacketIndex - 1, StartPos, EndPos, [this, &LastMatchingLevel, &bFilterMatch, &bOldEventMatchesFilter, &Filter, NetProfilerProvider, &FilterMatchAggregatedEventSizeInBits,&FilterMatchMaxEventSizeBits,  &FilterMatchEventTypeIndex, &EndNetIdMatchPos, &EndEventTypeMatchPos](const TraceServices::FNetProfilerContentEvent& Event)
								{
									if (!bFilterMatch || (Filter.AggregationMode != TraceServices::ENetProfilerAggregationMode::None))
									{
										// Include events and sub-events matching event type
										if (Filter.bByEventType)
										{
											if (Event.EndPos > EndEventTypeMatchPos)
											{
												EndEventTypeMatchPos = ~0U;
											}
											if (EndEventTypeMatchPos == ~0U && Filter.EventTypeIndex == Event.EventTypeIndex)
											{
												EndEventTypeMatchPos = Event.EndPos;
											}
										}

										// Include events and sub-events matching net id
										if (Filter.bByNetId)
										{
											uint64 NetId = uint64(-1);
											if (Event.ObjectInstanceIndex != 0)
											{
												NetProfilerProvider->ReadObject(GameInstanceIndex, Event.ObjectInstanceIndex, [&NetId](const TraceServices::FNetProfilerObjectInstance& ObjectInstance)
												{
													NetId = ObjectInstance.NetObjectId;
												});
											}

											if (Event.EndPos > EndNetIdMatchPos)
											{
												EndNetIdMatchPos = ~0U;
											}
											if (EndNetIdMatchPos == ~0U && (Event.ObjectInstanceIndex != 0 && Filter.NetId == NetId))
											{
												EndNetIdMatchPos = Event.EndPos;
											}
										}

										// Check if all conditions are fulfilled but only aggregate stats for top-level event.
										const bool bEventMatchesFilter = (!Filter.bByNetId || EndNetIdMatchPos != ~0U) && (!Filter.bByEventType || EndEventTypeMatchPos != ~0U);										
										if (bEventMatchesFilter && (!bOldEventMatchesFilter || LastMatchingLevel == Event.Level))
										{
											const uint32 EventSize = static_cast<uint32>(Event.EndPos - Event.StartPos);
											FilterMatchAggregatedEventSizeInBits += EventSize;
											FilterMatchMaxEventSizeBits = FMath::Max(FilterMatchMaxEventSizeBits, EventSize);
											LastMatchingLevel = Event.Level;

											if (!bFilterMatch)
											{
												FilterMatchEventTypeIndex = Event.NameIndex;
												bFilterMatch = true;
											}
										}
										bOldEventMatchesFilter = bEventMatchesFilter;
									}

								});

								if (bFilterMatch)
								{
									if (Filter.AggregationMode == TraceServices::ENetProfilerAggregationMode::Aggregate)
									{
										SamplePtr->FilterMatchHighlightSizeInBits = FMath::Max(SamplePtr->FilterMatchHighlightSizeInBits, FilterMatchAggregatedEventSizeInBits);
									}
									else if (Filter.AggregationMode == TraceServices::ENetProfilerAggregationMode::InstanceMax)
									{
										SamplePtr->FilterMatchHighlightSizeInBits = FMath::Max(SamplePtr->FilterMatchHighlightSizeInBits, FilterMatchMaxEventSizeBits);
									}
									
									SamplePtr->bAtLeastOnePacketMatchesFilter = true;
								}
							}
						}
					});

					Builder.SetHighlightEventTypeIndex(FilterMatchEventTypeIndex);
				}
			}
		}
	}

	// Init series with mock data.
	#if 0
	{
		constexpr int32 NumPackets = 100000;

		ViewportX.SetMinMaxInterval(0, NumPackets);

		const int32 StartIndex = FMath::Max(0, ViewportX.GetValueAtOffset(0.0f));
		const int32 EndIndex = FMath::Min(NumPackets, ViewportX.GetValueAtOffset(ViewportX.GetSize()));

		for (int32 PacketIndex = StartIndex; PacketIndex < EndIndex; ++PacketIndex)
		{
			FRandomStream RandomStream((PacketIndex * PacketIndex * PacketIndex) ^ 0x2c2c57ed);
			const uint32 Size = static_cast<uint32>(RandomStream.RandRange(0, 2000));

			TraceServices::ENetProfilerDeliveryStatus Status = TraceServices::ENetProfilerDeliveryStatus::Unknown;
			const float Fraction = RandomStream.GetFraction();
			if (Fraction < 0.01) // 1%
			{
				Status = TraceServices::ENetProfilerDeliveryStatus::Dropped;
			}
			else if (Fraction < 0.05) // 4%
			{
				Status = TraceServices::ENetProfilerDeliveryStatus::Delivered;
			}

			const double Timestamp = ((double)PacketIndex * 100.0) / (double)NumPackets + RandomStream.GetFraction() * 0.1;

			TraceServices::FNetProfilerPacket Packet;
			Packet.TimeStamp = static_cast<TraceServices::FNetProfilerTimeStamp>(Timestamp);
			Packet.SequenceNumber = PacketIndex;
			Packet.ContentSizeInBits = Size;
			Packet.TotalPacketSizeInBytes = (Size + 7) / 8;
			Packet.DeliveryStatus = Status;
			Builder.AddPacket(PacketIndex, Packet);
		}
	}
	#endif

	NumUpdatedPackets += Builder.GetNumAddedPackets();

	if (bIsAutoZoomEnabled && ViewportX.GetMaxValue() != PreviousMaxValue)
	{
		// Forces auto-zoom to be updated in the next Tick(), like after a viewport resize.
		AutoZoomViewportSize = 0.0f;
	}

	Stopwatch.Stop();
	UpdateDurationHistory.AddValue(Stopwatch.AccumulatedTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FNetworkPacketSampleRef SPacketView::GetSample(const int32 InPacketIndex)
{
	FNetworkPacketSampleRef SampleRef;
	SampleRef.Series = PacketSeries;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
		if (NetProfilerProvider && IsConnectionValid(*NetProfilerProvider, GameInstanceIndex, ConnectionIndex, ConnectionMode))
		{
			const uint32 NumPackets = NetProfilerProvider->GetPacketCount(ConnectionIndex, ConnectionMode);
			if (InPacketIndex >= 0 && InPacketIndex < static_cast<int32>(NumPackets))
			{
				NetProfilerProvider->EnumeratePackets(ConnectionIndex, ConnectionMode, InPacketIndex, InPacketIndex, [InPacketIndex, &SampleRef](const TraceServices::FNetProfilerPacket& Packet)
				{
					SampleRef.Sample = MakeShared<FNetworkPacketAggregatedSample>();
					SampleRef.Sample->AddPacket(InPacketIndex, Packet);
				});
			}
		}
	}

	return SampleRef;
}

void SPacketView::UpdateSelectedTimeSpan()
{
	SelectedTimeSpan = 0.0;

	if (SelectionEndPacketIndex == SelectionStartPacketIndex + 1)
	{		
		return;
	}

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
		if (NetProfilerProvider && IsConnectionValid(*NetProfilerProvider, GameInstanceIndex, ConnectionIndex, ConnectionMode))
		{
			double StartTimeStamp = 0.0f;
			double EndTimeStamp = 0.0f;

			const int32 LastPacketIndex = SelectionEndPacketIndex - 1;

			const uint32 NumPackets = NetProfilerProvider->GetPacketCount(ConnectionIndex, ConnectionMode);
			if (SelectionStartPacketIndex >= 0 && SelectionStartPacketIndex < static_cast<int32>(NumPackets) && 
				LastPacketIndex >= 0 && LastPacketIndex < static_cast<int32>(NumPackets))
			{
				NetProfilerProvider->EnumeratePackets(ConnectionIndex, ConnectionMode, SelectionStartPacketIndex, SelectionStartPacketIndex, [&StartTimeStamp](const TraceServices::FNetProfilerPacket& Packet)
				{
					StartTimeStamp = Packet.TimeStamp;
				});
				NetProfilerProvider->EnumeratePackets(ConnectionIndex, ConnectionMode, LastPacketIndex, LastPacketIndex, [&EndTimeStamp](const TraceServices::FNetProfilerPacket& Packet)
				{
					EndTimeStamp = Packet.TimeStamp;
				});
			
				SelectedTimeSpan = FMath::Max(0.0, EndTimeStamp - StartTimeStamp);
			}
		}
	}

}

////////////////////////////////////////////////////////////////////////////////////////////////////

FNetworkPacketSampleRef SPacketView::GetSampleAtMousePosition(double X, double Y)
{
	if (!bIsStateDirty)
	{
		float SampleW = Viewport.GetSampleWidth();
		int32 SampleIndex = FMath::FloorToInt(static_cast<float>(X) / SampleW);
		if (SampleIndex >= 0)
		{
			if (PacketSeries->NumAggregatedPackets > 0 &&
				SampleIndex < PacketSeries->Samples.Num())
			{
				const FNetworkPacketAggregatedSample& Sample = PacketSeries->Samples[SampleIndex];
				if (Sample.NumPackets > 0)
				{
					const FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();

					const float ViewHeight = FMath::RoundToFloat(Viewport.GetHeight());
					const float BaselineY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(0.0));

					//const float ValueY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(static_cast<double>(Sample.LargestPacket.ContentSizeInBits)));
					const float ValueY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(static_cast<double>(Sample.LargestPacket.TotalSizeInBytes * 8)));

					constexpr float ToleranceY = 3.0f; // [pixels]

					const float BottomY = FMath::Min(ViewHeight, ViewHeight - BaselineY + ToleranceY);
					const float TopY = FMath::Max(0.0f, ViewHeight - ValueY - ToleranceY);

					const float MY = static_cast<float>(Y);

					if (MY >= TopY && MY < BottomY)
					{
						return FNetworkPacketSampleRef(PacketSeries, MakeShared<FNetworkPacketAggregatedSample>(Sample));
					}
				}
			}
		}
	}
	return FNetworkPacketSampleRef(nullptr, nullptr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::SelectSampleAtMousePosition(double X, double Y, const FPointerEvent& MouseEvent)
{
	FNetworkPacketSampleRef SampleRef = GetSampleAtMousePosition(X, Y);
	if (!SampleRef.IsValid())
	{
		SampleRef = GetSampleAtMousePosition(X - 1.0, Y);
	}
	if (!SampleRef.IsValid())
	{
		SampleRef = GetSampleAtMousePosition(X + 1.0, Y);
	}

	bool bRaiseSelectionChanged = false;

	if (SampleRef.IsValid())
	{
		const int32 SelectedPacketIndex = SampleRef.Sample->LargestPacket.Index;

		if (MouseEvent.GetModifierKeys().IsShiftDown())
		{
			if (SelectedPacketIndex >= LastSelectedPacketIndex)
			{
				// Extend selection toward right.
				if (SelectionStartPacketIndex != LastSelectedPacketIndex ||
					SelectionEndPacketIndex != SelectedPacketIndex + 1)
				{
					SelectionStartPacketIndex = LastSelectedPacketIndex;
					SelectionEndPacketIndex = SelectedPacketIndex + 1;
					bRaiseSelectionChanged = true;
				}
				LastSelectedPacketIndex = SelectionStartPacketIndex;
			}
			else
			{
				// Extend selection toward left.
				if (SelectionEndPacketIndex != SelectedPacketIndex + 1)
				{
					SelectionStartPacketIndex = SelectedPacketIndex;
					SelectionEndPacketIndex = LastSelectedPacketIndex + 1;
					bRaiseSelectionChanged = true;
				}
				LastSelectedPacketIndex = SelectionEndPacketIndex - 1;
			}
		}
		else
		{
			if (SelectionStartPacketIndex != SelectedPacketIndex ||
				SelectionEndPacketIndex != SelectedPacketIndex + 1)
			{
				SelectionStartPacketIndex = SelectedPacketIndex;
				SelectionEndPacketIndex = SelectedPacketIndex + 1;
				bRaiseSelectionChanged = true;
			}
			LastSelectedPacketIndex = SelectedPacketIndex;
		}
	}
	else
	{
		if (SelectionStartPacketIndex != 0 ||
			SelectionEndPacketIndex != 0)
		{
			SelectionStartPacketIndex = 0;
			SelectionEndPacketIndex = 0;
			LastSelectedPacketIndex = 0;
			bRaiseSelectionChanged = true;
		}
	}

	if (SelectionEndPacketIndex == SelectionStartPacketIndex + 1)
	{
		if (!SelectedSample.Equals(SampleRef))
		{
			SelectedSample = SampleRef;
		}
	}
	else
	{
		SelectedSample.Reset();
	}

	if (bRaiseSelectionChanged)
	{
		OnSelectionChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::OnSelectionChanged()
{
	TSharedPtr<SNetworkingProfilerWindow> ProfilerWindow = ProfilerWindowWeakPtr.Pin();
	if (ProfilerWindow.IsValid())
	{
		// Update selected time range
		UpdateSelectedTimeSpan();

		if (SelectedSample.IsValid())
		{
			const uint32 BitSize = SelectedSample.Sample->LargestPacket.TotalSizeInBytes * 8;
			ProfilerWindow->SetSelectedPacket(SelectedSample.Sample->LargestPacket.Index, SelectedSample.Sample->LargestPacket.Index + 1, BitSize);
			ProfilerWindow->SetSelectedBitRange(0, BitSize);
		}
		else
		{
			ProfilerWindow->SetSelectedPacket(SelectionStartPacketIndex, SelectionEndPacketIndex);
			ProfilerWindow->SetSelectedBitRange(0, 0);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* StatusToString(TraceServices::ENetProfilerDeliveryStatus Status)
{
	switch (Status)
	{
		case TraceServices::ENetProfilerDeliveryStatus::Delivered:  return TEXT("Delivered");
		case TraceServices::ENetProfilerDeliveryStatus::Dropped:    return TEXT("Dropped");
		case TraceServices::ENetProfilerDeliveryStatus::Unknown:
		default:                                                    return TEXT("Unknown");
	};
}

const TCHAR* AggregatedStatusToString(TraceServices::ENetProfilerDeliveryStatus Status)
{
	switch (Status)
	{
		case TraceServices::ENetProfilerDeliveryStatus::Delivered:  return TEXT("all packets are Delivered");
		case TraceServices::ENetProfilerDeliveryStatus::Dropped:    return TEXT("at least one Dropped packet");
		case TraceServices::ENetProfilerDeliveryStatus::Unknown:
		default:                                                    return TEXT("Unknown");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 SPacketView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const bool bEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	FDrawContext DrawContext(AllottedGeometry, MyCullingRect, InWidgetStyle, DrawEffects, OutDrawElements, LayerId);

	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	FSlateFontInfo SummaryFont = FAppStyle::Get().GetFontStyle("SmallFont");
	const float FontScale = AllottedGeometry.Scale;

	const FSlateBrush* WhiteBrush = FAppStyle::Get().GetBrush("WhiteBrush");

	const float ViewWidth = static_cast<float>(AllottedGeometry.Size.X);
	const float ViewHeight = static_cast<float>(AllottedGeometry.Size.Y);

	int32 NumDrawSamples = 0;

	//////////////////////////////////////////////////
	{
		FStopwatch Stopwatch;
		Stopwatch.Start();

		FPacketViewDrawHelper Helper(DrawContext, Viewport);

		Helper.DrawBackground();

		// Draw the horizontal axis grid.
		DrawHorizontalAxisGrid(DrawContext, WhiteBrush, SummaryFont);

		Helper.DrawCached(*PacketSeries);

		NumDrawSamples = Helper.GetNumDrawSamples();

		// Highlight the selected and/or hovered sample.
		bool bIsSelectedAndHovered = SelectedSample.Equals(HoveredSample);
		if (SelectedSample.IsValid())
		{
			Helper.DrawSampleHighlight(*SelectedSample.Sample, bIsSelectedAndHovered ? FPacketViewDrawHelper::EHighlightMode::SelectedAndHovered : FPacketViewDrawHelper::EHighlightMode::Selected);
		}
		if (HoveredSample.IsValid() && !bIsSelectedAndHovered)
		{
			Helper.DrawSampleHighlight(*HoveredSample.Sample, FPacketViewDrawHelper::EHighlightMode::Hovered);
		}
		if (SelectionEndPacketIndex > SelectionStartPacketIndex + 1)
		{
			
			Helper.DrawSelection(SelectionStartPacketIndex, SelectionEndPacketIndex, SelectedTimeSpan);
		}

		// Draw the vertical axis grid.
		DrawVerticalAxisGrid(DrawContext, WhiteBrush, SummaryFont);

		// Draw tooltip for hovered sample.
		if (HoveredSample.IsValid())
		{
			if (TooltipOpacity < TooltipDesiredOpacity)
			{
				TooltipOpacity = TooltipOpacity * 0.9f + TooltipDesiredOpacity * 0.1f;
			}
			else
			{
				TooltipOpacity = TooltipDesiredOpacity;
			}

			const double Precision = 0.01; // 10ms
			int NumLines;
			FString Text;
			uint32 UnusedBits = HoveredSample.Sample->LargestPacket.TotalSizeInBytes * 8 - HoveredSample.Sample->LargestPacket.ContentSizeInBits;
			TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
			uint64 EngineFrameNumber = 0;
			if (Session.IsValid())
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
				const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*Session.Get());
				EngineFrameNumber = FrameProvider.GetFrameNumberForTimestamp(ETraceFrameType::TraceFrameType_Game, HoveredSample.Sample->LargestPacket.TimeStamp);
			}
			if (HoveredSample.Sample->NumPackets == 1)
			{
				NumLines = 7;
				Text = FString::Format(TEXT("Sequence Number: {0}\n"
											"Content Size: {1} bits\n"
											"Total Size: {2} bytes ({3} unused bits)\n"
											"Timestamp: {4}\n"
											"Status: {5}\n"
											"Connection State: {6}\n"
											"Engine Frame Number: {7}"),
					{						
						FText::AsNumber(HoveredSample.Sample->LargestPacket.SequenceNumber).ToString(),
						FText::AsNumber(HoveredSample.Sample->LargestPacket.ContentSizeInBits).ToString(),
						FText::AsNumber(HoveredSample.Sample->LargestPacket.TotalSizeInBytes).ToString(),
						FText::AsNumber(UnusedBits).ToString(),
						FormatTimeHMS(HoveredSample.Sample->LargestPacket.TimeStamp, Precision),
						StatusToString(HoveredSample.Sample->LargestPacket.Status),
						LexToString(HoveredSample.Sample->LargestPacket.ConnectionState),
						EngineFrameNumber > 0 ?
						FText::AsNumber(EngineFrameNumber).ToString() :
						TEXT("N/A")
					});
			}
			else
			{
				NumLines = 10;
				Text = FString::Format(TEXT("{0} network packets\n"
											"({1})\n"
											"Largest Packet\n"
											"    Sequence Number: {2}\n"
											"    Content Size: {3} bits\n"
											"    Total Size: {4} bytes ({5} unused bits)\n"
											"    Timestamp: {6}\n"
											"    Status: {7}\n"
											"    Connection State: {8}\n"
											"    Engine Frame Number: {9}"),
					{
						HoveredSample.Sample->NumPackets,
						AggregatedStatusToString(HoveredSample.Sample->AggregatedStatus),
						FText::AsNumber(HoveredSample.Sample->LargestPacket.SequenceNumber).ToString(),
						FText::AsNumber(HoveredSample.Sample->LargestPacket.ContentSizeInBits).ToString(),
						FText::AsNumber(HoveredSample.Sample->LargestPacket.TotalSizeInBytes).ToString(),
						FText::AsNumber(UnusedBits).ToString(),
						FormatTimeHMS(HoveredSample.Sample->LargestPacket.TimeStamp, Precision),
						StatusToString(HoveredSample.Sample->LargestPacket.Status),
						LexToString(HoveredSample.Sample->LargestPacket.ConnectionState),
						EngineFrameNumber > 0 ?
						FText::AsNumber(EngineFrameNumber).ToString() :
						TEXT("N/A")
					});
			}

			const FVector2D TextSize = FontMeasureService->Measure(Text, SummaryFont, FontScale) / FontScale;

			const float DX = 2.0f;
			const float W2 = static_cast<float>(TextSize.X) / 2 + DX;

			const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

			float X1 = ViewportX.GetOffsetForValue(HoveredSample.Sample->LargestPacket.Index);
			float CX = X1 + FMath::RoundToFloat(Viewport.GetSampleWidth() / 2);
			if (CX + W2 > ViewportX.GetSize())
			{
				CX = FMath::RoundToFloat(ViewportX.GetSize() - W2);
			}
			if (CX - W2 < 0)
			{
				CX = W2;
			}

			const float Y = 10.0f;
			const float H = 2.0f + 13.0f * static_cast<float>(NumLines);
			DrawContext.DrawBox(CX - W2, Y, 2 * W2, H, WhiteBrush, FLinearColor(0.7f, 0.7f, 0.7f, TooltipOpacity));
			DrawContext.LayerId++;
			DrawContext.DrawText(CX - W2 + DX, Y + 1.0f, Text, SummaryFont, FLinearColor(0.0f, 0.0f, 0.0f, TooltipOpacity));
			DrawContext.LayerId++;
		}

		Stopwatch.Stop();
		DrawDurationHistory.AddValue(Stopwatch.AccumulatedTime);
	}
	//////////////////////////////////////////////////

	const bool bShouldDisplayDebugInfo = FInsightsManager::Get()->IsDebugInfoEnabled();
	if (bShouldDisplayDebugInfo)
	{
		const float MaxFontCharHeight = static_cast<float>(FontMeasureService->Measure(TEXT("!"), SummaryFont, FontScale).Y / FontScale);
		const float DbgDY = MaxFontCharHeight;

		const float DbgW = 280.0f;
		const float DbgH = DbgDY * 5 + 3.0f;
		const float DbgX = ViewWidth - DbgW - 20.0f;
		float DbgY = 7.0f;

		DrawContext.LayerId++;

		DrawContext.DrawBox(DbgX - 2.0f, DbgY - 2.0f, DbgW, DbgH, WhiteBrush, FLinearColor(1.0f, 1.0f, 1.0f, 0.9f));
		DrawContext.LayerId++;

		FLinearColor DbgTextColor(0.0f, 0.0f, 0.0f, 0.9f);

		// Time interval since last OnPaint call.
		const uint64 CurrentTime = FPlatformTime::Cycles64();
		const uint64 OnPaintDuration = CurrentTime - LastOnPaintTime;
		LastOnPaintTime = CurrentTime;
		OnPaintDurationHistory.AddValue(OnPaintDuration); // saved for last 32 OnPaint calls
		const uint64 AvgOnPaintDuration = OnPaintDurationHistory.ComputeAverage();
		const uint64 AvgOnPaintDurationMs = FStopwatch::Cycles64ToMilliseconds(AvgOnPaintDuration);
		const double AvgOnPaintFps = AvgOnPaintDurationMs != 0 ? 1.0 / FStopwatch::Cycles64ToSeconds(AvgOnPaintDuration) : 0.0;

		const uint64 AvgUpdateDurationMs = FStopwatch::Cycles64ToMilliseconds(UpdateDurationHistory.ComputeAverage());
		const uint64 AvgDrawDurationMs = FStopwatch::Cycles64ToMilliseconds(DrawDurationHistory.ComputeAverage());

		// Draw performance info.
		DrawContext.DrawText
		(
			DbgX, DbgY,
			FString::Printf(TEXT("U: %" UINT64_FMT " ms, D: %" UINT64_FMT " ms + %" UINT64_FMT " ms = %" UINT64_FMT " ms (%" INT64_FMT " fps)"),
				AvgUpdateDurationMs, // caching time
				AvgDrawDurationMs, // drawing time
				AvgOnPaintDurationMs - AvgDrawDurationMs, // other overhead to OnPaint calls
				AvgOnPaintDurationMs, // average time between two OnPaint calls
				FMath::RoundToInt(AvgOnPaintFps)), // framerate of OnPaint calls
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		// Draw number of draw calls.
		DrawContext.DrawText
		(
			DbgX, DbgY,
			FString::Printf(TEXT("U: %s packets, D: %s samples"),
				*FText::AsNumber(NumUpdatedPackets).ToString(),
				*FText::AsNumber(NumDrawSamples).ToString()),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		// Draw viewport's horizontal info.
		DrawContext.DrawText
		(
			DbgX, DbgY,
			Viewport.GetHorizontalAxisViewport().ToDebugString(TEXT("X"), TEXT("packet")),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		// Draw viewport's vertical info.
		DrawContext.DrawText
		(
			DbgX, DbgY,
			Viewport.GetVerticalAxisViewport().ToDebugString(TEXT("Y")),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		// Draw connection info.
		DrawContext.DrawText
		(
			DbgX, DbgY,
			FString::Printf(TEXT("Game Instance %d, Connection %d (%s)"),
				GameInstanceIndex,
				ConnectionIndex,
				(ConnectionMode == TraceServices::ENetProfilerConnectionMode::Outgoing) ? TEXT("Outgoing") : TEXT("Incoming")),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;
	}

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled && IsEnabled());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::DrawVerticalAxisGrid(FDrawContext& DrawContext, const FSlateBrush* Brush, const FSlateFontInfo& Font) const
{
	const FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();

	const float RoundedViewHeight = FMath::RoundToFloat(ViewportY.GetSize());

	constexpr float MinDY = 32.0f; // min vertical distance between horizontal grid lines

	const double TopValue = ViewportY.GetValueAtOffset(RoundedViewHeight);
	const double GridValue = ViewportY.GetValueAtOffset(MinDY);
	const double BottomValue = ViewportY.GetValueAtOffset(0.0f);
	const double Delta = GridValue - BottomValue;

	if (Delta > 0.0)
	{
		int64 DeltaBits = static_cast<int64>(Delta);
		if (DeltaBits <= 0)
		{
			DeltaBits = 1;
		}

		// Compute rounding based on magnitude of visible range of values (Delta).
		int64 Power10 = 1;
		int64 Delta10 = DeltaBits;
		while (Delta10 > 0)
		{
			Delta10 /= 10;
			Power10 *= 10;
		}
		if (Power10 >= 100)
		{
			Power10 /= 100;
		}
		else
		{
			Power10 = 1;
		}

		const double Grid = static_cast<double>(((DeltaBits + Power10 - 1) / Power10) * Power10); // next value divisible with a multiple of 10

		const double StartValue = FMath::GridSnap(BottomValue, Grid);

		const float ViewWidth = Viewport.GetWidth();

		const FLinearColor GridColor(0.0f, 0.0f, 0.0f, 0.1f);
		const FLinearColor TextBgColor(0.05f, 0.05f, 0.05f, 1.0f);
		const FLinearColor TextColor(1.0f, 1.0f, 1.0f, 1.0f);

		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const float FontScale = DrawContext.Geometry.Scale;

		for (double Value = StartValue; Value < TopValue; Value += Grid)
		{
			const float Y = RoundedViewHeight - FMath::RoundToFloat(ViewportY.GetOffsetForValue(Value));

			// Draw horizontal grid line.
			DrawContext.DrawBox(0, Y, ViewWidth, 1, Brush, GridColor);

			const int64 ValueBits = static_cast<int64>(Value);
			const FString Text = (ValueBits == 0) ? TEXT("0") : FString::Format(TEXT("{0} bits"), { FText::AsNumber(ValueBits).ToString() });
			const FVector2D TextSize = FontMeasureService->Measure(Text, Font, FontScale) / FontScale;
			const float TextW = static_cast<float>(TextSize.X);
			constexpr float TextH = 14.0f;

			// Draw background for value text.
			DrawContext.DrawBox(ViewWidth - TextW - 4.0f, Y - TextH, TextW + 4.0f, TextH, Brush, TextBgColor);

			// Draw value text.
			DrawContext.DrawText(ViewWidth - TextW - 2.0f, Y - TextH + 1.0f, Text, Font, TextColor);
		}
		DrawContext.LayerId++;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::DrawHorizontalAxisGrid(FDrawContext& DrawContext, const FSlateBrush* Brush, const FSlateFontInfo& Font) const
{
	const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

	const float RoundedViewWidth = FMath::RoundToFloat(ViewportX.GetSize());

	constexpr float MinDX = 100.0f; // min horizontal distance between vertical grid lines

	const int32 LeftIndex = ViewportX.GetValueAtOffset(0.0f);
	const int32 GridIndex = ViewportX.GetValueAtOffset(MinDX);
	const int32 RightIndex = ViewportX.GetValueAtOffset(RoundedViewWidth);
	const int32 Delta = GridIndex - LeftIndex;

	if (Delta > 0)
	{
		// Compute rounding based on magnitude of visible range of samples (Delta).
		int32 Power10 = 1;
		int32 Delta10 = Delta;
		while (Delta10 > 0)
		{
			Delta10 /= 10;
			Power10 *= 10;
		}
		if (Power10 >= 100)
		{
			Power10 /= 100;
		}
		else
		{
			Power10 = 1;
		}

		const int32 Grid = ((Delta + Power10 - 1) / Power10) * Power10; // next value divisible with a multiple of 10

		// Skip grid lines for negative indices.
		int32 StartIndex = ((LeftIndex + Grid - 1) / Grid) * Grid;
		while (StartIndex < 0)
		{
			StartIndex += Grid;
		}

		const float ViewHeight = Viewport.GetHeight();

		const FLinearColor GridColor(0.0f, 0.0f, 0.0f, 0.1f);
		//const FLinearColor TextBgColor(0.05f, 0.05f, 0.05f, 1.0f);
		//const FLinearColor TextColor(1.0f, 1.0f, 1.0f, 1.0f);
		const FLinearColor TopTextColor(1.0f, 1.0f, 1.0f, 0.7f);

		//const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		//const float FontScale = DrawContext.Geometry.Scale;

		for (int32 Index = StartIndex; Index < RightIndex; Index += Grid)
		{
			const float X = FMath::RoundToFloat(ViewportX.GetOffsetForValue(Index));

			// Draw vertical grid line.
			DrawContext.DrawBox(X, 0, 1, ViewHeight, Brush, GridColor);

			const FString Text = FText::AsNumber(Index).ToString();
			//const FVector2D TextSize = FontMeasureService->Measure(Text, Font, FontScale) / FontScale;
			//constexpr float TextH = 14.0f;

			// Draw background for index text.
			//DrawContext.DrawBox(X, ViewHeight - TextH, TextSize.X + 4.0f, TextH, Brush, TextBgColor);

			// Draw index text.
			//DrawContext.DrawText(X + 2.0f, ViewHeight - TextH + 1.0f, Text, Font, TextColor);

			DrawContext.DrawText(X + 2.0f, 10.0f, Text, Font, TopTextColor);
		}
		DrawContext.LayerId++;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SPacketView::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	MousePositionOnButtonDown = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	ViewportPosXOnButtonDown = Viewport.GetHorizontalAxisViewport().GetPos();

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bIsLMB_Pressed = true;

		// Capture mouse.
		Reply = FReply::Handled().CaptureMouse(SharedThis(this));
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		bIsRMB_Pressed = true;

		// Capture mouse, so we can scroll outside this widget.
		Reply = FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SPacketView::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	MousePositionOnButtonUp = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	const bool bIsValidForMouseClick = MousePositionOnButtonUp.Equals(MousePositionOnButtonDown, MOUSE_SNAP_DISTANCE);

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (bIsLMB_Pressed)
		{
			if (bIsScrolling)
			{
				bIsScrolling = false;
				CursorType = ECursorType::Default;
			}
			else if (bIsValidForMouseClick)
			{
				SelectSampleAtMousePosition(static_cast<float>(MousePositionOnButtonUp.X), static_cast<float>(MousePositionOnButtonUp.Y), MouseEvent);
			}

			bIsLMB_Pressed = false;

			// Release the mouse.
			Reply = FReply::Handled().ReleaseMouseCapture();
		}
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (bIsRMB_Pressed)
		{
			if (bIsScrolling)
			{
				bIsScrolling = false;
				CursorType = ECursorType::Default;
			}
			else if (bIsValidForMouseClick)
			{
				ShowContextMenu(MouseEvent);
			}

			bIsRMB_Pressed = false;

			// Release mouse as we no longer scroll.
			Reply = FReply::Handled().ReleaseMouseCapture();
		}
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SPacketView::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	if (!MouseEvent.GetCursorDelta().IsZero())
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) ||
			MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
		{
			if (HasMouseCapture())
			{
				if (!bIsScrolling)
				{
					bIsScrolling = true;
					CursorType = ECursorType::Hand;

					HoveredSample.Reset();
				}

				FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
				const float PosX = ViewportPosXOnButtonDown + static_cast<float>(MousePositionOnButtonDown.X - MousePosition.X);
				ViewportX.ScrollAtValue(ViewportX.GetValueAtPos(PosX)); // align viewport position with sample
				UpdateHorizontalScrollBar();
				bIsStateDirty = true;
			}
		}
		else
		{
			if (!HoveredSample.IsValid())
			{
				TooltipOpacity = 0.0f;
			}
			HoveredSample = GetSampleAtMousePosition(MousePosition.X, MousePosition.Y);
			if (!HoveredSample.IsValid())
			{
				HoveredSample = GetSampleAtMousePosition(MousePosition.X - 1.0, MousePosition.Y);
			}
			if (!HoveredSample.IsValid())
			{
				HoveredSample = GetSampleAtMousePosition(MousePosition.X + 1.0, MousePosition.Y);
			}
		}

		Reply = FReply::Handled();
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	if (!HasMouseCapture())
	{
		bIsLMB_Pressed = false;
		bIsRMB_Pressed = false;

		HoveredSample.Reset();

		CursorType = ECursorType::Default;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SPacketView::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	if (MouseEvent.GetModifierKeys().IsShiftDown())
	{
		FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();

		// Zoom in/out vertically.
		const double Delta = MouseEvent.GetWheelDelta();
		constexpr double ZoomStep = 0.25; // as percent
		double ScaleY;

		if (Delta > 0)
		{
			ScaleY = ViewportY.GetScale() * FMath::Pow(1.0 + ZoomStep, Delta);
		}
		else
		{
			ScaleY = ViewportY.GetScale() * FMath::Pow(1.0 / (1.0 + ZoomStep), -Delta);
		}

		ViewportY.SetScale(ScaleY);
		//UpdateVerticalScrollBar();
	}
	else //if (MouseEvent.GetModifierKeys().IsControlDown())
	{
		// Zoom in/out horizontally.
		const float Delta = MouseEvent.GetWheelDelta();
		ZoomHorizontally(Delta, static_cast<float>(MousePosition.X));
	}

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SPacketView::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FCursorReply SPacketView::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	FCursorReply CursorReply = FCursorReply::Unhandled();

	if (CursorType == ECursorType::Arrow)
	{
		CursorReply = FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}
	else if (CursorType == ECursorType::Hand)
	{
		CursorReply = FCursorReply::Cursor(EMouseCursor::GrabHand);
	}

	return CursorReply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SPacketView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::A)
	{
		if (InKeyEvent.GetModifierKeys().IsControlDown())
		{
			const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
			const int32 LastPacketIndex = ViewportX.GetMaxValue();

			// Select all packets.
			SelectionStartPacketIndex = 0;
			SelectionEndPacketIndex = LastPacketIndex;

			LastSelectedPacketIndex = 0;

			SelectedSample.Reset();
			if (SelectionEndPacketIndex == SelectionStartPacketIndex + 1)
			{
				SelectedSample = GetSample(SelectionStartPacketIndex);
			}
			OnSelectionChanged();

			return FReply::Handled();
		}
	}
	else if (InKeyEvent.GetKey() == EKeys::Left)
	{
		if (InKeyEvent.GetModifierKeys().IsShiftDown())
		{
			if (InKeyEvent.GetModifierKeys().IsControlDown())
			{
				ShrinkRightSideOfSelectedInterval();
			}
			else
			{
				ExtendLeftSideOfSelectedInterval();
			}
		}
		else
		{
			SelectPreviousPacket();
		}
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Right)
	{
		if (InKeyEvent.GetModifierKeys().IsShiftDown())
		{
			if (InKeyEvent.GetModifierKeys().IsControlDown())
			{
				ShrinkLeftSideOfSelectedInterval();
			}
			else
			{
				ExtendRightSideOfSelectedInterval();
			}
		}
		else
		{
			SelectNextPacket();
		}
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Equals ||
			 InKeyEvent.GetKey() == EKeys::Add)
	{
		if (InKeyEvent.GetModifierKeys().IsShiftDown())
		{
			FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();
			const double ScaleY = ViewportY.GetScale() * 1.25;
			ViewportY.SetScale(ScaleY);
		}
		else
		{
			ZoomHorizontally(1.0f, static_cast<float>(MousePosition.X));
		}
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Hyphen ||
			 InKeyEvent.GetKey() == EKeys::Subtract)
	{
		if (InKeyEvent.GetModifierKeys().IsShiftDown())
		{
			FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();
			const double ScaleY = ViewportY.GetScale() * 0.8;
			ViewportY.SetScale(ScaleY);
		}
		else
		{
			ZoomHorizontally(-1.0f, static_cast<float>(MousePosition.X));
		}
		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::EnsurePacketIsVisible(const int InPacketIndex)
{
	FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	const float LeftX = ViewportX.GetPosForValue(InPacketIndex);
	const float RightX = LeftX + ViewportX.GetSampleSize();

	if (RightX > ViewportX.GetPos() + ViewportX.GetSize())
	{
		const int32 VisibleSampleCount = FMath::FloorToInt(ViewportX.GetSize() / ViewportX.GetSampleSize());
		ViewportX.ScrollAtValue(InPacketIndex - VisibleSampleCount + 1);
		UpdateHorizontalScrollBar();
		bIsStateDirty = true;
	}

	if (LeftX < ViewportX.GetPos())
	{
		ViewportX.ScrollAtValue(InPacketIndex);
		UpdateHorizontalScrollBar();
		bIsStateDirty = true;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::SetSelectedPacket(const int32 InPacketIndex)
{
	const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	const int32 PacketCount = ViewportX.GetMaxValue();
	if (PacketCount > 0)
	{
		const int32 PacketIndex = FMath::Min(FMath::Max(0, InPacketIndex), PacketCount - 1);
		SelectionStartPacketIndex = PacketIndex;
		SelectionEndPacketIndex = PacketIndex + 1;
		LastSelectedPacketIndex = PacketIndex;
		EnsurePacketIsVisible(LastSelectedPacketIndex);
		UpdateSelectedSample();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::SelectPacketBySequenceNumber(const uint32 InSequenceNumber)
{
	// Find the PacketIndex from sequence number
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::INetProfilerProvider* NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());
		if (NetProfilerProvider)
		{
			const int32 PacketId = NetProfilerProvider->FindPacketIndexFromPacketSequence(ConnectionIndex, ConnectionMode, InSequenceNumber);
			if (PacketId != -1)
			{
				SetSelectedPacket(PacketId);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::SelectPreviousPacket()
{
	const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	const int32 PacketCount = ViewportX.GetMaxValue();

	if (PacketCount > 0)
	{
		if (SelectionStartPacketIndex >= SelectionEndPacketIndex) // no selection?
		{
			// Select the first packet.
			SelectionStartPacketIndex = 0;
			SelectionEndPacketIndex = 1;
		}
		else
		{
			if (SelectionStartPacketIndex > 0)
			{
				// Select the previous packet.
				SelectionStartPacketIndex--;
			}
			SelectionEndPacketIndex = SelectionStartPacketIndex + 1;
		}

		LastSelectedPacketIndex = SelectionStartPacketIndex;
		EnsurePacketIsVisible(LastSelectedPacketIndex);
		UpdateSelectedSample();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::SelectNextPacket()
{
	const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	const int32 PacketCount = ViewportX.GetMaxValue();

	if (PacketCount > 0)
	{
		if (SelectionStartPacketIndex >= SelectionEndPacketIndex) // no selection?
		{
			// Select the last packet.
			SelectionStartPacketIndex = PacketCount - 1;
			SelectionEndPacketIndex = PacketCount;
		}
		else
		{
			if (SelectionEndPacketIndex < PacketCount)
			{
				// Select the next packet.
				SelectionEndPacketIndex++;
			}
			SelectionStartPacketIndex = SelectionEndPacketIndex - 1;
		}

		LastSelectedPacketIndex = SelectionStartPacketIndex;
		EnsurePacketIsVisible(LastSelectedPacketIndex);
		UpdateSelectedSample();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::ExtendLeftSideOfSelectedInterval()
{
	const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	const int32 PacketCount = ViewportX.GetMaxValue();

	if (PacketCount > 0)
	{
		bool bSelectionChanged = false;

		if (SelectionStartPacketIndex >= SelectionEndPacketIndex) // no selection?
		{
			// Select the first packet.
			SelectionStartPacketIndex = 0;
			SelectionEndPacketIndex = 1;
			bSelectionChanged = true;
		}
		else
		{
			if (SelectionStartPacketIndex > 0)
			{
				// Extend left side of selected interval.
				SelectionStartPacketIndex--;
				bSelectionChanged = true;
			}
		}

		if (bSelectionChanged)
		{
			LastSelectedPacketIndex = SelectionStartPacketIndex;
			EnsurePacketIsVisible(LastSelectedPacketIndex);
			UpdateSelectedSample();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::ShrinkLeftSideOfSelectedInterval()
{
	const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	const int32 PacketCount = ViewportX.GetMaxValue();

	if (PacketCount > 0)
	{
		if (SelectionStartPacketIndex + 1 < SelectionEndPacketIndex)
		{
			// Shrink left side of selected interval.
			SelectionStartPacketIndex++;

			LastSelectedPacketIndex = SelectionStartPacketIndex;
			EnsurePacketIsVisible(LastSelectedPacketIndex);
			UpdateSelectedSample();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::ExtendRightSideOfSelectedInterval()
{
	const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	const int32 PacketCount = ViewportX.GetMaxValue();

	if (PacketCount > 0)
	{
		bool bSelectionChanged = false;

		if (SelectionStartPacketIndex >= SelectionEndPacketIndex) // no selection?
		{
			// Select the last packet.
			SelectionStartPacketIndex = PacketCount - 1;
			SelectionEndPacketIndex = PacketCount;
			bSelectionChanged = true;
		}
		else
		{
			if (SelectionEndPacketIndex < PacketCount)
			{
				// Extend right side of selected interval.
				SelectionEndPacketIndex++;
				bSelectionChanged = true;
			}
		}

		if (bSelectionChanged)
		{
			LastSelectedPacketIndex = SelectionEndPacketIndex - 1;
			EnsurePacketIsVisible(LastSelectedPacketIndex);
			UpdateSelectedSample();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::ShrinkRightSideOfSelectedInterval()
{
	const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	const int32 PacketCount = ViewportX.GetMaxValue();

	if (PacketCount > 0)
	{
		if (SelectionEndPacketIndex - 1 > SelectionStartPacketIndex)
		{
			// Shrink right side of selected interval.
			SelectionEndPacketIndex--;

			LastSelectedPacketIndex = SelectionEndPacketIndex - 1;
			EnsurePacketIsVisible(LastSelectedPacketIndex);
			UpdateSelectedSample();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::UpdateSelectedSample()
{
	SelectedSample.Reset();
	if (SelectionEndPacketIndex == SelectionStartPacketIndex + 1)
	{
		SelectedSample = GetSample(SelectionStartPacketIndex);
	}
	OnSelectionChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::ShowContextMenu(const FPointerEvent& MouseEvent)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, NULL);

	MenuBuilder.BeginSection("Misc");
	{
		FUIAction Action_AutoZoom
		(
			FExecuteAction::CreateSP(this, &SPacketView::ContextMenu_AutoZoom_Execute),
			FCanExecuteAction::CreateSP(this, &SPacketView::ContextMenu_AutoZoom_CanExecute),
			FIsActionChecked::CreateSP(this, &SPacketView::ContextMenu_AutoZoom_IsChecked)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_AutoZoom", "Auto Zoom"),
			LOCTEXT("ContextMenu_AutoZoom_Desc", "Enable auto zoom. Makes entire graph series to fit into view."),
			FSlateIcon(),
			Action_AutoZoom,
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	TSharedRef<SWidget> MenuWidget = MenuBuilder.MakeWidget();

	FWidgetPath EventPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
	const FVector2D ScreenSpacePosition = MouseEvent.GetScreenSpacePosition();
	FSlateApplication::Get().PushMenu(SharedThis(this), EventPath, MenuWidget, ScreenSpacePosition, FPopupTransitionEffect::ContextMenu);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::ContextMenu_AutoZoom_Execute()
{
	bIsAutoZoomEnabled = !bIsAutoZoomEnabled;

	if (bIsAutoZoomEnabled)
	{
		AutoZoom();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SPacketView::ContextMenu_AutoZoom_CanExecute()
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SPacketView::ContextMenu_AutoZoom_IsChecked()
{
	return bIsAutoZoomEnabled;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::AutoZoom()
{
	FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

	AutoZoomViewportPos = ViewportX.GetMinPos();
	ViewportX.ScrollAtPos(AutoZoomViewportPos);

	AutoZoomViewportSize = ViewportX.GetSize();

	if (AutoZoomViewportSize > 0.0f &&
		ViewportX.GetMaxValue() - ViewportX.GetMinValue() > 0)
	{
		float DX = ViewportX.GetMaxPos() - ViewportX.GetMinPos();

		// Auto zoom in.
		while (DX < AutoZoomViewportSize)
		{
			const float OldScale = ViewportX.GetScale();
			ViewportX.RelativeZoomWithFixedOffset(+0.1f, 0.0f);
			ViewportX.ScrollAtPos(AutoZoomViewportPos);
			DX = ViewportX.GetMaxPos() - ViewportX.GetMinPos();
			if (OldScale == ViewportX.GetScale())
			{
				break;
			}
		}

		// Auto zoom out (until entire session frame range fits into view).
		while (DX > AutoZoomViewportSize)
		{
			const float OldScale = ViewportX.GetScale();
			ViewportX.RelativeZoomWithFixedOffset(-0.1f, 0.0f);
			ViewportX.ScrollAtPos(AutoZoomViewportPos);
			DX = ViewportX.GetMaxPos() - ViewportX.GetMinPos();
			if (OldScale == ViewportX.GetScale())
			{
				break;
			}
		}
	}

	AutoZoomViewportScale = ViewportX.GetScale();

	UpdateHorizontalScrollBar();
	bIsStateDirty = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::BindCommands()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::HorizontalScrollBar_OnUserScrolled(float ScrollOffset)
{
	FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.OnUserScrolled(HorizontalScrollBar, ScrollOffset);
	bIsStateDirty = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::UpdateHorizontalScrollBar()
{
	FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.UpdateScrollBar(HorizontalScrollBar);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SPacketView::ZoomHorizontally(const float Delta, const float X)
{
	FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.RelativeZoomWithFixedOffset(Delta, X);
	ViewportX.ScrollAtValue(ViewportX.GetValueAtPos(ViewportX.GetPos())); // align viewport position with sample
	UpdateHorizontalScrollBar();
	bIsStateDirty = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::NetworkingProfiler

#undef LOCTEXT_NAMESPACE
