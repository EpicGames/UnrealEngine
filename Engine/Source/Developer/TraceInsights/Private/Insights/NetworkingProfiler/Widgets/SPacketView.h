// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Geometry.h"
#include "Rendering/RenderingCommon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

// TraceServices
#include "TraceServices/Model/NetProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/FixedCircularBuffer.h"

// TraceInsights
#include "Insights/NetworkingProfiler/ViewModels/PacketViewDrawHelper.h"
#include "Insights/NetworkingProfiler/ViewModels/PacketViewport.h"

class SScrollBar;

namespace UE::Insights::NetworkingProfiler
{

class SNetworkingProfilerWindow;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FNetworkPacketSampleRef
{
	TSharedPtr<FNetworkPacketSeries> Series;
	TSharedPtr<FNetworkPacketAggregatedSample> Sample;

	FNetworkPacketSampleRef()
		: Series(), Sample()
	{
	}

	FNetworkPacketSampleRef(TSharedPtr<FNetworkPacketSeries> InSeries, TSharedPtr<FNetworkPacketAggregatedSample> InSample)
		: Series(InSeries), Sample(InSample)
	{
	}

	FNetworkPacketSampleRef(const FNetworkPacketSampleRef& Other)
		: Series(Other.Series), Sample(Other.Sample)
	{
	}

	FNetworkPacketSampleRef& operator=(const FNetworkPacketSampleRef& Other)
	{
		Series = Other.Series;
		Sample = Other.Sample;
		return *this;
	}

	void Reset()
	{
		Series.Reset();
		Sample.Reset();
	}

	bool IsValid() const
	{
		return Series.IsValid() && Sample.IsValid();
	}

	bool Equals(const FNetworkPacketSampleRef& Other) const
	{
		return Series == Other.Series
			&& ((Sample == Other.Sample) || (Sample.IsValid() && Other.Sample.IsValid() && Sample->Equals(*Other.Sample)));
	}

	static bool AreEquals(const FNetworkPacketSampleRef& A, const FNetworkPacketSampleRef& B)
	{
		return A.Equals(B);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Widget used to present the network packets as a bar track.
 */
class SPacketView : public SCompoundWidget
{
public:
	/** Number of pixels. */
	static constexpr float MOUSE_SNAP_DISTANCE = 2.0f;

	enum class ECursorType
	{
		Default,
		Arrow,
		Hand,
	};

public:
	/** Default constructor. */
	SPacketView();

	/** Virtual destructor. */
	virtual ~SPacketView();

	/** Resets internal widget's data to the default one. */
	void Reset();

	void SetConnection(uint32 GameInstanceIndex, uint32 ConnectionIndex, TraceServices::ENetProfilerConnectionMode ConnectionMode);

	SLATE_BEGIN_ARGS(SPacketView)
	{
		_Clipping = EWidgetClipping::ClipToBounds;
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<SNetworkingProfilerWindow> InProfilerWindow);

	/**
	 * Ticks this widget. Override in derived classes, but always call the parent implementation.
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	void EnsurePacketIsVisible(const int InPacketIndex);
	void SetSelectedPacket(const int32 InPacketIndex);
	void SelectPacketBySequenceNumber(const uint32 InSequenceNumber);
	void SelectPreviousPacket();
	void SelectNextPacket();
	void ExtendLeftSideOfSelectedInterval();
	void ShrinkLeftSideOfSelectedInterval();
	void ExtendRightSideOfSelectedInterval();
	void ShrinkRightSideOfSelectedInterval();

	void InvalidateState() { bIsStateDirty = true; }

private:
	void UpdateSelectedSample();

	bool IsConnectionValid(const TraceServices::INetProfilerProvider& NetProfilerProvider, const uint32 InGameInstanceIndex, const uint32 InConnectionIndex, const TraceServices::ENetProfilerConnectionMode InConnectionMode);
	void UpdateState();

	void DrawHorizontalAxisGrid(FDrawContext& DrawContext, const FSlateBrush* Brush, const FSlateFontInfo& Font) const;
	void DrawVerticalAxisGrid(FDrawContext& DrawContext, const FSlateBrush* Brush, const FSlateFontInfo& Font) const;

	FNetworkPacketSampleRef GetSample(const int32 InPacketIndex);
	FNetworkPacketSampleRef GetSampleAtMousePosition(double X, double Y);
	void SelectSampleAtMousePosition(double X, double Y, const FPointerEvent& MouseEvent);
	void OnSelectionChanged();

	void ShowContextMenu(const FPointerEvent& MouseEvent);

	void ContextMenu_AutoZoom_Execute();
	bool ContextMenu_AutoZoom_CanExecute();
	bool ContextMenu_AutoZoom_IsChecked();
	void AutoZoom();

	/** Binds our UI commands to delegates. */
	void BindCommands();

	/**
	 * Called when the user scrolls the horizontal scrollbar.
	 * @param ScrollOffset Scroll offset as a fraction between 0 and 1.
	 */
	void HorizontalScrollBar_OnUserScrolled(float ScrollOffset);
	void UpdateHorizontalScrollBar();

	void ZoomHorizontally(const float Delta, const float X);

private:
	void UpdateSelectedTimeSpan();

	TWeakPtr<SNetworkingProfilerWindow> ProfilerWindowWeakPtr;

	uint32 GameInstanceIndex;
	uint32 ConnectionIndex;
	TraceServices::ENetProfilerConnectionMode ConnectionMode;

	/** The track's viewport. Encapsulates info about position and scale. */
	FPacketViewport Viewport;
	bool bIsViewportDirty;

	/** Cached info for the packet series. */
	TSharedRef<FNetworkPacketSeries> PacketSeries;
	bool bIsStateDirty;

	bool bIsAutoZoomEnabled;
	float AutoZoomViewportPos;
	float AutoZoomViewportScale;
	float AutoZoomViewportSize;

	uint64 AnalysisSyncNextTimestamp;
	uint32 ConnectionChangeCount;

	//////////////////////////////////////////////////

	TSharedPtr<SScrollBar> HorizontalScrollBar;

	//////////////////////////////////////////////////
	// Panning and Zooming behaviors

	/** The current mouse position. */
	FVector2D MousePosition;

	/** Mouse position during the call on mouse button down. */
	FVector2D MousePositionOnButtonDown;
	float ViewportPosXOnButtonDown;

	/** Mouse position during the call on mouse button up. */
	FVector2D MousePositionOnButtonUp;

	bool bIsLMB_Pressed;
	bool bIsRMB_Pressed;

	/** True, if the user is currently interactively scrolling the view (ex.: by holding the left mouse button and dragging). */
	bool bIsScrolling;

	//////////////////////////////////////////////////
	// Selection

	int32 SelectionStartPacketIndex;
	int32 SelectionEndPacketIndex;
	int32 LastSelectedPacketIndex;
	double SelectedTimeSpan;

	FNetworkPacketSampleRef SelectedSample;
	FNetworkPacketSampleRef HoveredSample;

	float TooltipDesiredOpacity;
	mutable float TooltipOpacity;

	//////////////////////////////////////////////////
	// Misc

	FGeometry ThisGeometry;

	/** Cursor type. */
	ECursorType CursorType;

	// Debug stats
	int32 NumUpdatedPackets;
	TFixedCircularBuffer<uint64, 32> UpdateDurationHistory;
	mutable TFixedCircularBuffer<uint64, 32> DrawDurationHistory;
	mutable TFixedCircularBuffer<uint64, 32> OnPaintDurationHistory;
	mutable uint64 LastOnPaintTime;
};

} // namespace UE::Insights::NetworkingProfiler
