// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetInfoProvider.h"

#include "Insights/InsightsStyle.h"
#include "Insights/ObjectProfiler/ObjectProfilerManager.h"

#if UE_INSIGHTS_WITH_CUSTOM_ASSET_INFO

#define LOCTEXT_NAMESPACE "UE::Insights::ObjectProfiler::AssetInfoProvider"

namespace UE::Insights::ObjectProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FAssetInfoProvider
////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetInfoProvider::FAssetInfoProvider()
{
	InitCategories();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetInfoProvider::~FAssetInfoProvider()
{
	ReleaseCategories();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetInfoProvider::FAssetInfoCategory* FAssetInfoProvider::AddCategory(
	const FAssetInfoCategory* InParentCategory,
	const FName& InClassName,
	const FText& InDisplayName,
	const FLinearColor& InColor,
	const FSlateBrush* InIcon)
{
	FAssetInfoCategory* Category = new FAssetInfoCategory();

	Category->ClassName = InClassName;
	Category->DisplayName = InDisplayName;

	Category->Color = InColor;
	if (InColor.A == 0.0f)
	{
		uint32 Seed = InParentCategory ? GetTypeHash(InParentCategory->GetClassName()) : 0;
		Category->Color = FLinearColor(static_cast<float>(Seed % 3600) / 10.0f, 0.7f, 1.0f, 1.0f).HSVToLinearRGB();
	}

	Category->Icon = InIcon ? InIcon : InParentCategory ? InParentCategory->GetIcon() : nullptr;
	Category->ParentCategory = InParentCategory;

	Categories.Add(Category);
	Classes.Add(InClassName, Category);

	return Category;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetInfoProvider::FAssetInfoCategory* FAssetInfoProvider::AddCategory(
	const UTF8CHAR* InParentCategoryClassName,
	const UTF8CHAR* InCategoryClassName,
	const UTF8CHAR* InCategoryDisplayName,
	FColor InCategoryColor,
	const UTF8CHAR* InCategoryIcon)
{
	const FAssetInfoCategory* ParentCategory = nullptr;
	if (InParentCategoryClassName)
	{
		ParentCategory = Classes.FindRef(FName(InParentCategoryClassName));
		if (!ParentCategory)
		{
			UE_LOGF(LogObjectProfiler, Warning, "[Obj] Unknown parent category (%s) -- when adding class (%s) and category (%s)!",
				InParentCategoryClassName, InCategoryClassName, InCategoryDisplayName);
		}
	}
	const FSlateBrush* IconBrush = InCategoryIcon ? FInsightsStyle::GetBrush(FName(InCategoryIcon)) : nullptr;
	return AddCategory(ParentCategory, FName(InCategoryClassName), FText::FromString(InCategoryDisplayName), InCategoryColor, IconBrush);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetInfoProvider::FAssetInfoCategory* FAssetInfoProvider::AddCategory(
	const ANSICHAR* InParentCategoryClassName,
	const ANSICHAR* InCategoryClassName,
	const ANSICHAR* InCategoryDisplayName,
	FColor InCategoryColor,
	const ANSICHAR* InCategoryIcon)
{
	const FAssetInfoCategory* ParentCategory = nullptr;
	if (InParentCategoryClassName)
	{
		ParentCategory = Classes.FindRef(FName(InParentCategoryClassName));
		if (!ParentCategory)
		{
			UE_LOGF(LogObjectProfiler, Warning, "[Obj] Unknown parent category (%s) -- when adding class (%s) and category (%s)!",
				InParentCategoryClassName, InCategoryClassName, InCategoryDisplayName);
		}
	}
	const FSlateBrush* IconBrush = InCategoryIcon ? FInsightsStyle::GetBrush(FName(InCategoryIcon)) : nullptr;
	return AddCategory(ParentCategory, FName(InCategoryClassName), FText::FromString(InCategoryDisplayName), InCategoryColor, IconBrush);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetInfoProvider::AddClass(const FAssetInfoProvider::FAssetInfoCategory* InCategory, const FName& InClassName)
{
	Classes.Add(InClassName, InCategory);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetInfoProvider::AddClass(const UTF8CHAR* InCategoryClassName, const UTF8CHAR* InClassName)
{
	const FAssetInfoCategory* Category = Classes.FindRef(FName(InCategoryClassName));
	if (Category)
	{
		Classes.Add(FName(InClassName), Category);
	}
	else
	{
		UE_LOGF(LogObjectProfiler, Warning, "[Obj] Unknown category (%s) -- when adding class (%s)!",
			InCategoryClassName, InClassName);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetInfoProvider::AddClass(const ANSICHAR* InCategoryClassName, const ANSICHAR* InClassName)
{
	const FAssetInfoCategory* Category = Classes.FindRef(FName(InCategoryClassName));
	if (Category)
	{
		Classes.Add(FName(InClassName), Category);
	}
	else
	{
		UE_LOGF(LogObjectProfiler, Warning, "[Obj] Unknown category (%s) -- when adding class (%s)!",
			InCategoryClassName, InClassName);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetInfoProvider::InitCategories()
{
	DefaultCategory.Color = FLinearColor(0.9f, 0.6f, 0.2f, 1.0f);

	AddCategory(nullptr, "Actor", "Actor", FColor::Green, "Icons.AActor.TreeItem");
	AddCategory(nullptr, "ActorComponent", "Actor Component", FColor::Green, "Icons.AActor.TreeItem");

	AddCategory(nullptr, "AI", "Artificial Intelligence", FColor::White, nullptr);
	AddClass("AI", "LearningAgentsNeuralNetwork");

	AddCategory(nullptr, "Animation", "Animation", FColor(80, 123, 72), nullptr);
	AddClass("Animation", "AnimBoneCompressionSettings");
	AddClass("Animation", "AnimCurveCompressionSettings");
	AddClass("Animation", "VariableFrameStrippingSettings");
	AddClass("Animation", "AnimInstance");
	AddClass("Animation", "AnimationAsset");

	AddCategory(nullptr, "Audio", "Audio", FColor(128, 255, 128), nullptr);
	AddClass("Audio", "SoundBase");
	AddCategory("Audio", "MetaSoundSource", "MetaSound Source", FColor(64, 255, 64), nullptr);
	AddCategory("Audio", "SoundWave", "Sound Wave", FColor(128, 255, 64), nullptr);
	AddCategory("Audio", "SoundAttenuation", "Sound Attenuation", FColor(64, 255, 128), nullptr);

	AddCategory(nullptr, "Basic", "Basic", FColor::White, nullptr);

	AddCategory(nullptr, "Blueprint", "Blueprint", FColor(63, 127, 255), nullptr);
	AddCategory("Blueprint", "BlueprintGeneratedClass", "Blueprint Generated Class", FColor(133, 173, 255), nullptr);
	AddCategory("Blueprint", "CompiledBlueprintClass", "Compiled Blueprint Class", FColor(133, 173, 255), nullptr);

	AddCategory(nullptr, "BodySetup", "Body Setup", FColor::White, nullptr);
	AddCategory(nullptr, "Cinematics", "Cinematics", FColor::White, nullptr);
	AddCategory(nullptr, "Engine", "Engine", FColor::White, nullptr);
	AddCategory(nullptr, "Foliage", "Foliage", FColor::White, nullptr);

	AddCategory(nullptr, "Font", "Font", FColor::White, nullptr);
	AddCategory("Font", "FontFace", "FontFace", FColor::White, nullptr);

	AddCategory(nullptr, "Mass", "Mass", FColor::White, nullptr);
	AddClass("Mass", "MassProcessor");
	AddClass("Mass", "MassEntitySubsystem");
	AddClass("Mass", "MassEntitySettings");

	AddCategory(nullptr, "FX", "FX", FColor::White, nullptr);
	AddCategory("FX", "ParticleSystem", "Particle System", FColor::White, nullptr);
	AddCategory("FX", "Niagara", "Niagara", FColor::White, nullptr);
	AddClass("Niagara", "NiagaraScript");
	AddClass("Niagara", "NiagaraSystem");
	AddClass("Niagara", "NiagaraEmitter");
	AddClass("Niagara", "NiagaraRendererProperties");
	AddClass("Niagara", "NiagaraParameterCollection");
	AddClass("Niagara", "NiagaraNotifyOnChanged");
	AddClass("Niagara", "NiagaraDataChannelAsset");

	AddCategory(nullptr, "Gameplay", "Gameplay", FColor::White, nullptr);
	AddClass("Gameplay", "GameplayAbility");
	AddClass("Gameplay", "GameplayEffect");
	AddClass("Gameplay", "GameplayTagsManager");
	AddClass("Gameplay", "GameplayTagsList");
	AddClass("Gameplay", "GameplayTagsSettings");
	AddClass("Gameplay", "RestrictedGameplayTagsList");
	AddClass("Gameplay", "EditableGameplayTagQuery");
	AddClass("Gameplay", "EditableGameplayTagQueryExpression");

	AddCategory(nullptr, "Input", "Input", FColor::White, nullptr);

	AddCategory(nullptr, "Material", "Material", FColor(0, 172, 172), "Icons.UMaterial.TreeItem");
	AddClass("Material", "MaterialInterface");
	AddClass("Material", "MaterialExpression");
	AddClass("Material", "MaterialParameterCollection");
	AddCategory("Material", "MaterialFunction", "Material Function", FColor(0, 172, 172), nullptr);
	AddClass("MaterialFunction", "MaterialFunctionInterface");
	AddCategory("Material", "MaterialInstance", "Material Instance", FColor(0, 172, 172), nullptr);
	AddClass("MaterialInstance", "MaterialInstanceConstant");
	AddClass("MaterialInstance", "MaterialInstanceDynamic");
	AddClass("MaterialInstance", "LandscapeMaterialInstanceConstant");

	AddCategory(nullptr, "Media", "Media", FColor::White, nullptr);
	AddClass("Media", "MovieSceneSignedObject");

	//////////////////////////////////////////////////
	// Misc

	AddCategory(nullptr, "Misc", "Miscellaneous", FColor::Red, nullptr);

	AddCategory("Misc", "Field", "Field", FColor::White, "Icons.UField.TreeItem");
	AddCategory("Misc", "Enum", "Enum", FColor::White, "Icons.UEnum.TreeItem");
	AddCategory("Misc", "Property", "Property", FColor::White, "Icons.UProperty.TreeItem");
	AddCategory("Misc", "Struct", "Struct", FColor::White, "Icons.UStruct.TreeItem");
	AddCategory("Misc", "Class", "Class", FColor::White, "Icons.UClass.TreeItem");

	AddCategory("Misc", "Curve", "Curve", FColor(100, 172, 172), nullptr);
	AddClass("Curve", "CurveBase");
	AddClass("Curve", "LinearColorRamp");
	AddClass("Curve", "CurveTable");
	AddClass("Curve", "CompositeCurveTable");
	AddClass("Curve", "CurveLinearColor");
	AddClass("Curve", "CurveFloat");
	AddClass("Curve", "CurveVector");

	AddCategory("Misc", "DeveloperSettings", "Developer Settings", FColor(64, 192, 255), nullptr);
	AddCategory("Misc", "DeletedObjectPlaceholder", "Deleted Object Placeholder", FColor::Red, nullptr);
	AddCategory("Misc", "DataAsset", "Data Asset", FColor(195, 28, 82), nullptr);
	AddCategory("Misc", "DataTable", "Data Table", FColor(102, 64, 192), nullptr);
	AddCategory("Misc", "DeviceProfile", "Device Profile", FColor::Cyan, nullptr);
	AddCategory("Misc", "EntityPrefab", "Entity Prefab", FColor(102, 64, 192), nullptr);

	AddCategory("Misc", "Function", "Function", FColor(64, 192, 64), "Icons.UFunction.TreeItem");
	AddCategory("Function", "DelegateFunction", "Delegate Function", FColor(64, 192, 64), nullptr);

	AddCategory("Misc", "Interface", "Interface", FColor(255, 192, 255), nullptr);

	AddCategory("Misc", "ScriptStruct", "Script Struct", FColor::Cyan, nullptr);
	AddClass("ScriptStruct", "PropertyBag");

	AddCategory("Misc", "Python", "Python", FColor::Green, nullptr);
	AddClass("Python", "PythonCallableForDelegate");

	//////////////////////////////////////////////////

	AddCategory(nullptr, "SkeletalMesh", "Skeletal Mesh", FColor::White, "Icons.USkeletalMesh.TreeItem");
	AddCategory(nullptr, "StaticMesh", "Static Mesh", FColor::White, "Icons.UStaticMesh.TreeItem");

	AddCategory(nullptr, "NavCollision", "Collision System", FColor::White, nullptr);
	AddClass("NavCollision", "NavCollisionBase");
	AddClass("NavCollision", "NavCollisionBox");
	AddClass("NavCollision", "NavCollisionCylinder");

	AddCategory(nullptr, "Physics", "Physics", FColor(200, 162, 108), nullptr);
	AddClass("Physics", "PhysicalMaterialMask");
	AddClass("Physics", "PhysicsAsset");

	AddCategory(nullptr, "PCG", "PCG", FColor(0, 204, 172), nullptr);
	AddClass("PCG", "PCGGraphInterface");
	AddClass("PCG", "PCGData");

	AddCategory(nullptr, "RigVM", "RigVM", FColor(192, 192, 255), nullptr);
	AddClass("RigVM", "RigVMActionStack");
	AddClass("RigVM", "RigVMBuildData");
	AddClass("RigVM", "RigVMCompiler");
	AddClass("RigVM", "RigVMController");
	AddClass("RigVM", "RigVMControllerSettings");
	AddClass("RigVM", "RigVMEditorAsset");
	AddClass("RigVM", "RigVMGraph");
	AddClass("RigVM", "RigVMInjectionInfo");
	AddClass("RigVM", "RigVMLink");
	AddClass("RigVM", "RigVMNode");
	AddClass("RigVM", "RigVMPin");
	AddClass("RigVM", "RigVMSchema");
	AddClass("RigVM", "RigVMUserWorkflowRegistry");

	//////////////////////////////////////////////////
	// Texture

	AddCategory(nullptr, "Texture", "Texture", FColor(255, 64, 64), "Icons.UTexture.TreeItem");
	AddClass("Texture", "VirtualTextureAdapter");
	AddClass("Texture", "MaterialCacheVirtualTexture");

	AddCategory("Texture", "Texture2D", "Texture 2D", FColor(255, 64, 64), nullptr);
	AddClass("Texture2D", "CurveLinearColorAtlas");
	AddClass("Texture2D", "LightMapTexture2D");
	AddClass("Texture2D", "ShadowMapTexture2D");
	AddClass("Texture2D", "TextureLightProfile");
	AddClass("Texture2D", "LightMapVirtualTexture2D");
	AddClass("Texture2D", "MeshPaintVirtualTexture");
	AddClass("Texture2D", "RuntimeVirtualTextureStreamingProxy");
	AddClass("Texture2D", "VirtualTexture2D");

	AddCategory("Texture", "TextureRenderTarget", "Texture Render Target", FColor(255, 128, 64), nullptr);
	AddClass("TextureRenderTarget", "CanvasRenderTarget2D");
	AddClass("TextureRenderTarget", "TextureRenderTarget2D");
	AddClass("TextureRenderTarget", "TextureRenderTarget2DArray");
	AddClass("TextureRenderTarget", "TextureRenderTargetCube");
	AddClass("TextureRenderTarget", "TextureRenderTargetVolume");

	AddCategory("Texture", "Texture2DArray", "Texture 2D Array", FColor(255, 64, 64), nullptr);
	AddCategory("Texture", "Texture2DDynamic", "Texture 2D Dynamic", FColor(255, 64, 64), nullptr);
	AddCategory("Texture", "TextureCube", "Texture Cube", FColor(255, 64, 128), nullptr);
	AddCategory("Texture", "TextureCubeArray", "Texture Cube Array", FColor(255, 64, 128), nullptr);
	AddCategory("Texture", "VolumeTexture", "Volume Texture", FColor(255, 64, 128), nullptr);
	AddCategory("Texture", "SparseVolumeTexture", "Sparse Volume Texture", FColor(255, 64, 128), nullptr);
	AddCategory("Texture", "TextureLODSettings", "Texture LOD Settings", FColor(255, 64, 64), nullptr);

	//////////////////////////////////////////////////
	// UI

	AddCategory(nullptr, "UI", "User Interface", FColor::Cyan, nullptr);

	AddCategory("UI", "Visual", "Visual", FColor::Cyan, nullptr);
	AddClass("Visual", "Widget");
	AddClass("Visual", "PanelSlot");

	AddClass("UI", "ToolMenus");
	AddClass("UI", "WidgetBlueprint");
	AddClass("UI", "WidgetBlueprintGeneratedClass");
	AddClass("UI", "EditorUtilityWidgetBlueprint");

	//////////////////////////////////////////////////

	AddCategory(nullptr, "VectorField", "Vector Field", FColor(255, 128, 128), nullptr);
	AddClass("VectorField", "VectorFieldAnimated");
	AddClass("VectorField", "VectorFieldStatic");

	AddCategory(nullptr, "Verse", "Verse", FColor(255, 255, 64), nullptr);
	AddClass("Verse", "VerseDebugData");
	AddClass("Verse", "VerseDigest");
	AddClass("Verse", "VerseClass");
	AddClass("Verse", "VerseStruct");
	AddClass("Verse", "VersePersistentVarWeakMapKey");
	AddClass("Verse", "VerseMovableObjectBase");
	AddClass("Verse", "VerseAsset");

	AddCategory(nullptr, "World", "World", FColor(192, 208, 255), "Icons.UWorld.TreeItem");
	AddCategory(nullptr, "Level", "Level", FColor(246, 150, 0), "Icons.ULevel.TreeItem");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetInfoProvider::ReleaseCategories()
{
	for (FAssetInfoCategory* Category : Categories)
	{
		delete Category;
	}
	Categories.Reset();
	Classes.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IAssetInfoCategory& FAssetInfoProvider::GetClassCategory(const FName InClassName) const
{
	const FAssetInfoCategory* FoundCategory = Classes.FindRef(InClassName);
	if (FoundCategory)
	{
		return *FoundCategory;
	}
	return DefaultCategory;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IAssetInfoCategory& FAssetInfoProvider::GetObjectCategory(const FName InClassName, const TCHAR* InObjectName, const TCHAR* InObjectPath) const
{
	return GetClassCategory(InClassName);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FAssetInfoProvider::GetDisplayName(const FAssetData& InAssetData, const FName InClassName) const
{
	return GetClassCategory(InClassName).GetDisplayName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FAssetInfoProvider::GetColor(const FAssetData& InAssetData, const FName InClassName) const
{
	return GetClassCategory(InClassName).GetColor();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* FAssetInfoProvider::GetIcon(const FAssetData& InAssetData, const FName InClassName) const
{
	return GetClassCategory(InClassName).GetIcon();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* FAssetInfoProvider::GetThumbnail(const FAssetData& InAssetData, const FName InClassName) const
{
	return GetClassCategory(InClassName).GetIcon();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler

#undef LOCTEXT_NAMESPACE

#endif // UE_INSIGHTS_WITH_CUSTOM_ASSET_INFO
