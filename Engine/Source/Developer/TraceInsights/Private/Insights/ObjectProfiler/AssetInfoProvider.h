// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Insights/ObjectProfiler/IAssetInfoProvider.h"

#ifndef UE_INSIGHTS_WITH_CUSTOM_ASSET_INFO
#if defined(__INTELLISENSE__)
#define UE_INSIGHTS_WITH_CUSTOM_ASSET_INFO 1
#else
#define UE_INSIGHTS_WITH_CUSTOM_ASSET_INFO !WITH_EDITOR
#endif
#endif

#if UE_INSIGHTS_WITH_CUSTOM_ASSET_INFO

namespace UE::Insights::ObjectProfiler
{

class FAssetInfoProvider : public IAssetInfoProvider
{
private:
	class FAssetInfoCategory : public IAssetInfoCategory
	{
		friend class FAssetInfoProvider;

	public:
		FAssetInfoCategory() {}
		virtual ~FAssetInfoCategory() = default;

		virtual uint32 GetId() const override { return ClassName.GetComparisonIndex().ToUnstableInt(); }
		virtual FText GetDisplayName() const override { return DisplayName; }
		virtual FLinearColor GetColor() const override { return Color; }
		virtual const FSlateBrush* GetIcon() const override { return Icon; }

		const FName& GetClassName() const { return ClassName; }
		const FAssetInfoCategory* GetParentCategory() const { return ParentCategory; }

	private:
		FName ClassName;
		FText DisplayName;
		FLinearColor Color;
		const FSlateBrush* Icon = nullptr;
		const FAssetInfoCategory* ParentCategory = nullptr;
	};

public:
	FAssetInfoProvider();
	virtual ~FAssetInfoProvider();

	virtual const IAssetInfoCategory& GetClassCategory(const FName InClassName) const override;
	virtual const IAssetInfoCategory& GetObjectCategory(const FName InClassName, const TCHAR* InObjectName, const TCHAR* InObjectPath = nullptr) const override;

	virtual FText GetDisplayName(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual FLinearColor GetColor(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual const FSlateBrush* GetIcon(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual const FSlateBrush* GetThumbnail(const FAssetData& InAssetData, const FName InClassName) const override;

private:
	FAssetInfoCategory* AddCategory(
		const FAssetInfoCategory* InParentCategory,
		const FName& InClassName,
		const FText& InDisplayName,
		const FLinearColor& InColor,
		const FSlateBrush* InIcon);

	FAssetInfoCategory* AddCategory(
		const UTF8CHAR* InParentCategoryClassName,
		const UTF8CHAR* InCategoryClassName,
		const UTF8CHAR* InCategoryDisplayName,
		FColor InCategoryColor,
		const UTF8CHAR* InCategoryIcon);

	FAssetInfoCategory* AddCategory(
		const ANSICHAR* InParentCategoryClassName,
		const ANSICHAR* InCategoryClassName,
		const ANSICHAR* InCategoryDisplayName,
		FColor InCategoryColor,
		const ANSICHAR* InCategoryIcon);

	void AddClass(const FAssetInfoCategory* InCategory, const FName& InClassName);
	void AddClass(const UTF8CHAR* InCategoryClassName, const UTF8CHAR* InClassName);
	void AddClass(const ANSICHAR* InCategoryClassName, const ANSICHAR* InClassName);

	void InitCategories();
	void ReleaseCategories();

private:
	TArray<FAssetInfoCategory*> Categories;
	TMap<FName, const FAssetInfoCategory*> Classes; // ClassName -> Category
	FAssetInfoCategory DefaultCategory;
};

} // namespace UE::Insights::ObjectProfiler

#endif // UE_INSIGHTS_WITH_CUSTOM_ASSET_INFO
