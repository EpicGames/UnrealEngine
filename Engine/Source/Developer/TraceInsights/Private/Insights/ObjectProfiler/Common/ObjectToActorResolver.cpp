// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectToActorResolver.h"

#if WITH_EDITOR || defined(__INTELLISENSE__)

#include "AssetRegistry/AssetData.h"
#include "Containers/StridedView.h"
#include "HAL/FileManager.h"
#include "Logging/StructuredLog.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/NamePermissionList.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "String/Join.h"
#include "UObject/GCObjectInfo.h"

#include "AssetToolsModule.h"
#include "Editor.h"
#include "Engine/World.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY_STATIC(LogObjectToActorResolver, Warning, All);

#define LOCTEXT_NAMESPACE "UE::MemorySnapsnot::FObjectToActorResolver"

namespace UE::Insights::ObjectProfiler
{

static constexpr bool GPerformInternalActorSearchDuringDFS = false;

static bool GPerformInternalActorSearch = true;
static FAutoConsoleVariableRef CVarPerformInternalActorSearch(
	TEXT("Insights.MemorySnapshot.PerformInternalActorSearch"),
	GPerformInternalActorSearch,
	TEXT("Whether to perform batched internal actor search after DFS completes."));

static int32 GReferenceChainSearchMode = (int32)EReferenceChainSearchMode::Default;
static FAutoConsoleVariableRef CVarReferenceChainSearchMode(
	TEXT("Insights.MemorySnapshot.ReferenceChainSearchMode"),
	GReferenceChainSearchMode,
	TEXT("Reference chain search mode for internal actor search."));

FBuildPackageDependencyMapBuilder::FBuildPackageDependencyMapBuilder(TSoftObjectPtr<UWorld> InRootPackage, const TArray<FName>& InTargetPackageNames)
	: RootPackage(InRootPackage)
	, TargetPackageNames(InTargetPackageNames)
	, AssetRegistry(IAssetRegistry::GetChecked())
	, NumTargetPackages(InTargetPackageNames.Num())
{
	TargetPackageActorInfos.SetNum(NumTargetPackages);
	TargetPackageIndices.Reserve(NumTargetPackages);
	for (int32 Index = 0; Index < NumTargetPackages; ++Index)
	{
		TargetPackageIndices.Add(InTargetPackageNames[Index], Index);
	}
}

FActorScanResult FBuildPackageDependencyMapBuilder::ScanChainForExternalActors()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ScanChainForExternalActors);
	// External actors. Skip the first element in the stack as it is assumed to be the root package.
	for (int32 ChainIndex = 1; ChainIndex < Stack.Num(); ++ChainIndex)
	{
		const FAssetIdentifier& Ident = Stack[ChainIndex].AssetId;
		FDependencyChainActorInfo& ElemInfo = Stack[ChainIndex].ActorInfo;
		if (!EnumHasAnyFlags(ElemInfo.ScanState, EScanState::AssetRegistry))
		{
			// Important to do a case-insensitive comparison here - paths can differ in case.
			if (FNameBuilder(Ident.PackageName).ToView().Contains(ExternalActorsFolderName, ESearchCase::IgnoreCase))
			{
				Assets.Reset();
				AssetRegistry.GetAssetsByPackageName(Ident.PackageName, Assets, true);
				for (const FAssetData& AssetData : Assets)
				{
					FSoftObjectPath ActorPath = AssetData.GetSoftObjectPath();
					FGuid ActorGuid;
#if WITH_EDITOR
					if (const TUniquePtr<FWorldPartitionActorDesc> ActorDesc =
						FWorldPartitionActorDescUtils::GetActorDescriptorFromAssetData(AssetData))
					{
						ActorGuid = ActorDesc->GetGuid();
					}
#endif // WITH_EDITOR
					const int32 ActorIdx = InternActorInfo(FActorInfo(MoveTemp(ActorPath), MoveTemp(ActorGuid)));
					ElemInfo.Info.AddUnique(ActorIdx);
				}
			}
			ElemInfo.ScanState |= EScanState::AssetRegistry;
		}
		if (!ElemInfo.Info.IsEmpty())
		{
			UE_LOGFMT(LogObjectToActorResolver, VeryVerbose,
				"Found {NumActors} external actor(s) at chain index {ChainIndex}, package {PackageName}",
				ElemInfo.Info.Num(),
				ChainIndex,
				Ident);
			return { .SourceChainIndex = ChainIndex, .ActorRefs = ElemInfo.Info };
		}
	}
	return {};
}

FActorScanResult FBuildPackageDependencyMapBuilder::ScanChainForInternalActors()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ScanChainForInternalActors);
	// Internal actors. Skip the first element in the stack as it is assumed to be the root package.
	for (int32 ChainIndex = 1; ChainIndex < Stack.Num(); ++ChainIndex)
	{
		const FAssetIdentifier& Ident = Stack[ChainIndex].AssetId;
		FDependencyChainActorInfo& ElemInfo = Stack[ChainIndex].ActorInfo;
		if (!EnumHasAnyFlags(ElemInfo.ScanState, EScanState::ReferenceChainSearch))
		{
			Assets.Reset();
			AssetRegistry.GetAssetsByPackageName(Ident.PackageName, Assets, true);
			AssetObjects.Reset();
			for (const FAssetData& AssetData : Assets)
			{
				if (UObject* Obj = AssetData.FastGetAsset())
				{
					AssetObjects.Add(Obj);
				}
			}
			UE_LOGFMT(LogObjectToActorResolver, Verbose,
				"Performing FReferenceChainSearch for package {PackageName} (chain index {ChainIndex}, {NumAssets} assets)",
				Ident.PackageName,
				ChainIndex,
				AssetObjects.Num());
			FReferenceChainSearch Search(AssetObjects, EReferenceChainSearchMode::Default);
			const int32 NumChains = Search.GetReferenceChains().Num();
			UE_LOGFMT(LogObjectToActorResolver, VeryVerbose,
				"FReferenceChainSearch completed: {NumChains} reference chains found",
				NumChains);
			for (const FReferenceChainSearch::FReferenceChain* RefChain : Search.GetReferenceChains())
			{
				// Loop back through the reference chain, from the root to the target. We only want reference chains which
				// include our world, so filter by this before checking if links objects are actors.
				bool bFoundWorld = false;
				for (int32 Node = RefChain->Num() - 1; Node >= 0; --Node)
				{
					const FReferenceChainSearch::FGraphNode* Link = RefChain->GetNode(Node);
					if (UObject* Obj = Link->ObjectInfo->TryResolveObject())
					{
						if (!bFoundWorld)
						{
							bFoundWorld = (Obj == RootWorld);
							continue;
						}
						if (AActor* Actor = Cast<AActor>(Obj))
						{
							FSoftObjectPath ActorPath = Obj;
							FGuid ActorGuid;
#if WITH_EDITOR
							ActorGuid = Actor->GetActorGuid();
#endif // WITH_EDITOR
							const int32 ActorIdx = InternActorInfo(FActorInfo(MoveTemp(ActorPath), MoveTemp(ActorGuid)));
							ElemInfo.Info.AddUnique(ActorIdx);
							break;
						}
					}
				}
			}
			ElemInfo.ScanState |= EScanState::ReferenceChainSearch;
		}
		if (!ElemInfo.Info.IsEmpty())
		{
			UE_LOGFMT(LogObjectToActorResolver, VeryVerbose,
				"Found {NumActors} internal actor(s) at chain index {ChainIndex}, package {PackageName}",
				ElemInfo.Info.Num(),
				ChainIndex,
				Ident);
			return { .SourceChainIndex = ChainIndex, .ActorRefs = ElemInfo.Info };
		}
	}
	return {};
}

FActorScanResult FBuildPackageDependencyMapBuilder::ScanChainForActors()
{
	FActorScanResult Result = ScanChainForExternalActors();
	if (Result.ActorRefs.IsEmpty() && GPerformInternalActorSearchDuringDFS)
	{
		UE_LOGFMT(LogObjectToActorResolver, Verbose,
			"No external actors found, starting internal reference chain search (chain depth: {ChainDepth})",
			Stack.Num() - 1);
		Result = ScanChainForInternalActors();
	}
	return Result;
}

int32 FBuildPackageDependencyMapBuilder::InternActorInfo(FActorInfo Info)
{
	if (const int32* Existing = ActorInfoToIndex.Find(Info))
	{
		return *Existing;
	}
	const int32 Idx = ActorInfoPool.Add(Info);
	ActorInfoToIndex.Add(MoveTemp(Info), Idx);
	return Idx;
}

/**
	* Attempts to extract an actor from a reference chain.
	* Walks the chain from root to target, looking for RootWorld followed by an Actor.
	* @param RefChain The reference chain to process
	* @param RootWorld The world to search within
	* @return Actor info if found, empty optional otherwise
	*/
TOptional<FActorInfo> FBuildPackageDependencyMapBuilder::TryExtractActorFromChain(const FReferenceChainSearch::FReferenceChain& RefChain, const UWorld* RootWorld)
{
	if (RefChain.Num() == 0)
	{
		return NullOpt;
	}

	// Loop through the reference chain from root to target to find actors
	bool bFoundWorld = false;
	for (int32 Node = RefChain.Num() - 1; Node >= 0; --Node)
	{
		const FReferenceChainSearch::FGraphNode* Link = RefChain.GetNode(Node);
		UObject* Obj = Link->ObjectInfo ? Link->ObjectInfo->TryResolveObject() : nullptr;
		if (!Obj)
		{
			continue;
		}

		if (!bFoundWorld)
		{
			bFoundWorld = (Obj == RootWorld);
			continue;
		}

		if (const AActor* Actor = Cast<AActor>(Obj))
		{
			FSoftObjectPath ActorPath = Actor;
			FGuid ActorGuid;
#if WITH_EDITOR
			ActorGuid = Actor->GetActorGuid();
#endif
			return FActorInfo(MoveTemp(ActorPath), MoveTemp(ActorGuid));
		}
	}

	return NullOpt;
}

/**
	* Performs batched internal actor search using FReferenceChainSearch for all targets without actors.
	* This builds a complete reference graph and searches backwards from target packages to find actor references.
	*/
void FBuildPackageDependencyMapBuilder::PerformBatchedReferenceChainSearch(TSet<int32>& TargetIndicesNeedingSearch)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PerformBatchedReferenceChainSearch);

	const int32 InitialTargetsNeedingSearch = TargetIndicesNeedingSearch.Num();

	// Collect UPackage objects for all targets needing search
	TArray<UObject*> TargetPackageObjects;
	TargetPackageObjects.Reserve(TargetIndicesNeedingSearch.Num());

	int32 NumPackagesNotFound = 0;

	for (int32 TargetIndex : TargetIndicesNeedingSearch)
	{
		// Find the package name for this target index
		const FName PackageName = TargetPackageNames[TargetIndex];
		if (UPackage* Package = FindObjectFast<UPackage>(nullptr, PackageName))
		{
			TargetPackageObjects.Add(Package);
		}
		else
		{
			++NumPackagesNotFound;
			UE_LOGFMT(LogObjectToActorResolver, Verbose,
				"Package not found for internal actor search: {PackageName}",
				PackageName);
		}
	}

	UE_CLOGFMT(NumPackagesNotFound > 0, LogObjectToActorResolver, Log,
		"Could not find {NumNotFound}/{NumTotal} package(s) for internal actor search",
		NumPackagesNotFound,
		TargetIndicesNeedingSearch.Num());

	if (TargetPackageObjects.IsEmpty())
	{
		return;
	}

	const EReferenceChainSearchMode SearchMode = static_cast<EReferenceChainSearchMode>(GReferenceChainSearchMode);
	UE_LOGFMT(LogObjectToActorResolver, Verbose,
		"Starting FReferenceChainSearch with {NumPackages} package object(s), search mode: {SearchMode}",
		TargetPackageObjects.Num(),
		GReferenceChainSearchMode);

	FReferenceChainSearch Search(TargetPackageObjects, SearchMode);
	const int32 NumChains = Search.GetReferenceChains().Num();

	UE_LOGFMT(LogObjectToActorResolver, VeryVerbose,
		"FReferenceChainSearch completed: {NumChains} reference chain(s) found",
		NumChains);

	// Create a separate slow task scope for processing chains
	FScopedSlowTask ChainProcessingTask(static_cast<float>(NumChains),
		FText::FormatOrdered(NSLOCTEXT("ObjectToAssetResolver", "PerformBatchedReferenceChainSearch_SlowTask", "Processing {0} reference chains"), FText::AsNumber(NumChains)));
	ChainProcessingTask.MakeDialog(true);

	int32 LastProgressUpdate = 0;

	// Process reference chains to extract actors
	for (auto RefChainIter = Search.GetReferenceChains().CreateConstIterator(); RefChainIter; ++RefChainIter)
	{
		const int32 ChainIndex = RefChainIter.GetIndex();

		// Update progress periodically (every 1000 chains to reduce overhead)
		if ((ChainIndex - LastProgressUpdate) >= 1000)
		{
			const float WorkDone = static_cast<float>(ChainIndex - LastProgressUpdate);
			ChainProcessingTask.EnterProgressFrame(WorkDone);
			LastProgressUpdate = ChainIndex;
		}

		if (ChainProcessingTask.ShouldCancel())
		{
			UE_LOGFMT(LogObjectToActorResolver, Warning, "Internal actor search cancelled by user");
			break;
		}

		if (TargetIndicesNeedingSearch.IsEmpty())
		{
			UE_LOGFMT(LogObjectToActorResolver, VeryVerbose, "Early termination: Found actors for all targets");
			break;
		}

		const FReferenceChainSearch::FReferenceChain* RefChain = *RefChainIter;
		if (!RefChain || RefChain->Num() == 0)
		{
			continue;
		}

		const FReferenceChainSearch::FGraphNode* TargetNode = RefChain->GetNode(0);
		if (!TargetNode || !TargetNode->ObjectInfo)
		{
			continue;
		}

		const FName TargetPackageName = TargetNode->ObjectInfo->GetName();
		const int32* TargetPackageIndexPtr = TargetPackageIndices.Find(TargetPackageName);
		if (!TargetPackageIndexPtr)
		{
			continue;
		}

		// Skip if we've already found actors for this target
		const int32 TargetIndex = *TargetPackageIndexPtr;
		if (!TargetIndicesNeedingSearch.Contains(TargetIndex))
		{
			continue;
		}

		// Try to extract an actor from this chain
		if (TOptional<FActorInfo> ActorInfo = TryExtractActorFromChain(*RefChain, RootWorld))
		{
			TargetPackageActorInfos[TargetIndex].Add(InternActorInfo(MoveTemp(*ActorInfo)));

			// Remove this target from the set since we found at least one actor
			TargetIndicesNeedingSearch.Remove(TargetIndex);
		}
	}

	// Complete any remaining progress
	const float RemainingWork = ChainProcessingTask.TotalAmountOfWork - ChainProcessingTask.CompletedWork - ChainProcessingTask.CurrentFrameScope;
	if (RemainingWork > 0.0f)
	{
		ChainProcessingTask.EnterProgressFrame(RemainingWork);
	}

	UE_LOGFMT(LogObjectToActorResolver, Verbose,
		"Batched internal search found actors for {NumTargetsFound}/{NumTargetsSearched} target(s)",
		InitialTargetsNeedingSearch - TargetIndicesNeedingSearch.Num(),
		InitialTargetsNeedingSearch);
}

int32 FBuildPackageDependencyMapBuilder::FindOrCreateHitMaskIndex(const FTargetHitMask& HitMask)
{
	uint32 HitMaskHash = GetTypeHash(HitMask.Num());
	for (int32 Bit : HitMask)
	{
		HitMaskHash = HashCombine(HitMaskHash, GetTypeHash(Bit));
	}

	TArray<int32>& Bucket = TargetHitMaskHashBuckets.FindOrAdd(HitMaskHash);
	for (int32 HitMaskIndex : Bucket)
	{
		if (TargetHitMasksCache[HitMaskIndex] == HitMask)
		{
			return HitMaskIndex;
		}
	}
	return Bucket.Add_GetRef(TargetHitMasksCache.Add(HitMask));
}

TMap<FName, TSet<FActorInfo>> FBuildPackageDependencyMapBuilder::Execute()
{
	const FAssetIdentifier RootAssetIdentifier(RootPackage.GetLongPackageFName());
	// We don't want to bother recording actor references to the root world package.
	TargetPackageIndices.Remove(RootAssetIdentifier.PackageName);

	UE_LOGFMT(LogObjectToActorResolver, Verbose,
		"Starting dependency map build: Root={RootPackage}, NumTargets={NumTargets}",
		RootPackage.ToSoftObjectPath(),
		NumTargetPackages);

	FScopedSlowTask ParentTask(2.0f);
	ParentTask.MakeDialog(/*bShowCancelButton=*/true);

	ParentTask.EnterProgressFrame(1.0f, NSLOCTEXT("ObjectToActorResolver", "BuildPackageDependencyMap_SlowTask", "Collecting package dependency information"));
	TOptional<FScopedSlowTask> SlowTask;
	SlowTask.Emplace(0.0f);

	// Initialize with root package
	{
		const int32 StartDepsIndex = Dependencies.Num();
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GetDependencies);
			AssetRegistry.GetDependencies(RootAssetIdentifier, Dependencies);
		}
		const int32 EndDepsIndex = Dependencies.Num();
		Stack.Push({
			.AssetId = RootAssetIdentifier,
			.StartIndex = StartDepsIndex,
			.CurrIndex = StartDepsIndex,
			.EndIndex = EndDepsIndex,
			.TargetHitMask = {},
			});
		CurrentPath.Add(RootAssetIdentifier);
		SlowTask->TotalAmountOfWork += static_cast<float>(EndDepsIndex - StartDepsIndex);
	}

	RootWorld = RootPackage.Get();
	if (!RootWorld)
	{
		UE_LOGFMT(LogObjectToActorResolver, Warning,
			"Root world package {PackageName} not loaded, actor references may be incomplete",
			RootPackage.ToSoftObjectPath());
	}

	while (!Stack.IsEmpty())
	{
		FPackageDependencyStackFrame& Frame = Stack.Last();

		// When a target package is found, add the appropriate actor reference:
		if (const int32* TargetPackageIndex = TargetPackageIndices.Find(Frame.AssetId.PackageName))
		{
			// Always record a hit, regardless of whether we find an actor in the chain or not.
			Frame.bHit = true;
			Frame.TargetHitMask.Add(*TargetPackageIndex);

			const FActorScanResult Scan = ScanChainForActors();
			if (!Scan.ActorRefs.IsEmpty())
			{
				FDependencyChainActorInfo& Source = Stack[Scan.SourceChainIndex].ActorInfo;

				// Mark this target as contributed for this source frame; if it was already there, skip the work.
				bool bAlreadyContributed = false;
				Source.ContributedTargets.Add(*TargetPackageIndex, &bAlreadyContributed);
				if (!bAlreadyContributed)
				{
					TSet<int32>& TargetSet = TargetPackageActorInfos[*TargetPackageIndex];
					for (int32 ActorIdx : Scan.ActorRefs)
					{
						TargetSet.Add(ActorIdx);
					}
				}
			}
		}

		// If there are no more children/dependencies to process:
		if (Frame.CurrIndex >= Frame.EndIndex || SlowTask->ShouldCancel())
		{
			VisitedNodes.Add(Frame.AssetId, Frame.bHit ? FindOrCreateHitMaskIndex(Frame.TargetHitMask) : INDEX_NONE);
			if (Frame.bHit && Stack.Num() > 1)
			{
				// Propagate hit-mask and hit flag to parent
				FPackageDependencyStackFrame& ParentFrame = Stack.Last(1);
				ParentFrame.bHit = true;
				ParentFrame.TargetHitMask.Append(Frame.TargetHitMask);
			}
			Dependencies.SetNum(Frame.StartIndex, EAllowShrinking::No);
			CurrentPath.Remove(Frame.AssetId);
			Stack.RemoveAt(Stack.Num() - 1, EAllowShrinking::No);
			continue;
		}

		SlowTask->CompletedWork += 1.0f;
		SlowTask->TickProgress();
		const FAssetIdentifier& Node = Dependencies[Frame.CurrIndex++];
		const uint32 NodeHash = GetTypeHash(Node);

		// If node is on the current path, skip (this prevents cycles)
		if (CurrentPath.ContainsByHash(NodeHash, Node))
		{
			continue;
		}

		// Check if we've already visited this node:
		if (const int32* HitMaskIndex = VisitedNodes.FindByHash(NodeHash, Node))
		{
			// If there were hits recorded for this node, propagate them to our actor chain.
			if (TargetHitMasksCache.IsValidIndex(*HitMaskIndex))
			{
				const FTargetHitMask& CachedHitMask = TargetHitMasksCache[*HitMaskIndex];

				// Record these hits as our own too.
				Frame.bHit = true;
				Frame.TargetHitMask.Append(CachedHitMask);

				// For each target package hit by the cached result, add the source frame's actor(s)
				// to the per-target actor infos. Add+test on ContributedTargets in one step:
				// if the target was already contributed by this source frame, skip the per-actor
				// loop. This avoids both a separate Contains check and a bulk Append at the end.
				const FActorScanResult Scan = ScanChainForActors();
				if (!Scan.ActorRefs.IsEmpty())
				{
					FDependencyChainActorInfo& Source = Stack[Scan.SourceChainIndex].ActorInfo;
					for (int32 TargetIndex : CachedHitMask)
					{
						bool bAlreadyContributed = false;
						Source.ContributedTargets.Add(TargetIndex, &bAlreadyContributed);
						if (bAlreadyContributed)
						{
							continue;
						}
						TSet<int32>& TargetSet = TargetPackageActorInfos[TargetIndex];
						for (int32 ActorIdx : Scan.ActorRefs)
						{
							TargetSet.Add(ActorIdx);
						}
					}
				}
			}
			continue;
		}

		// Fetch dependencies. We assume that GetDependencies appends to Dependencies, and doesn't reset it.
		const int32 StartDepsIndex = Dependencies.Num();
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GetDependencies);
			AssetRegistry.GetDependencies(Node, Dependencies);
		}
		const int32 EndDepsIndex = Dependencies.Num();
		Stack.Add({
			.AssetId = Node,
			.StartIndex = StartDepsIndex,
			.CurrIndex = StartDepsIndex,
			.EndIndex = EndDepsIndex,
			.TargetHitMask = {},
			});
		CurrentPath.Add(Node);
		SlowTask->TotalAmountOfWork += static_cast<float>(EndDepsIndex - StartDepsIndex);
	}

	SlowTask.Reset();

	ParentTask.EnterProgressFrame(1.0f, NSLOCTEXT("ObjectToActorResolver", "SearchingReferenceChains_SlowTask", "Searching for reference chains"));

	// Perform batched internal actor search for targets without actors found during DFS
	TSet<int32> TargetIndicesNeedingSearch;
	for (const TPair<FName, int32>& Pair : TargetPackageIndices)
	{
		if (TargetPackageActorInfos[Pair.Value].IsEmpty())
		{
			TargetIndicesNeedingSearch.Add(Pair.Value);
		}
	}

	if (ParentTask.ShouldCancel())
	{
		UE_LOGFMT(LogObjectToActorResolver, Log,
			"Skipping internal actor search for {NumTargets} target(s) (cancelled by user)",
			TargetIndicesNeedingSearch.Num());
	}
	else if (!TargetIndicesNeedingSearch.IsEmpty())
	{
		if (GPerformInternalActorSearch)
		{
			UE_LOGFMT(LogObjectToActorResolver, Verbose,
				"Performing internal actor search for {NumTargets} target(s) without external actors",
				TargetIndicesNeedingSearch.Num());

			PerformBatchedReferenceChainSearch(TargetIndicesNeedingSearch);
		}
		else
		{
			UE_LOGFMT(LogObjectToActorResolver, Log,
				"Skipping internal actor search for {NumTargets} target(s) (disabled by GPerformInternalActorSearch)",
				TargetIndicesNeedingSearch.Num());
		}
	}

	TMap<FName, TSet<FActorInfo>> PackageToActorInfos;
	PackageToActorInfos.Reserve(TargetPackageIndices.Num());
	int32 NumTargetsWithActors = 0;
	int32 NumTargetsWithoutActors = 0;
	for (const TPair<FName, int32>& Pair : TargetPackageIndices)
	{
		const TSet<int32>& ActorIndices = TargetPackageActorInfos[Pair.Value];
		if (!ActorIndices.IsEmpty())
		{
			++NumTargetsWithActors;
		}
		else
		{
			++NumTargetsWithoutActors;
		}

		// Materialize FActorInfos from interned indices for the public boundary.
		TSet<FActorInfo> OutActors;
		OutActors.Reserve(ActorIndices.Num());
		for (int32 ActorIdx : ActorIndices)
		{
			OutActors.Emplace(ActorInfoPool[ActorIdx]);
		}
		PackageToActorInfos.Add(Pair.Key, MoveTemp(OutActors));
	}

	UE_LOGFMT(LogObjectToActorResolver, Verbose,
		"Completed dependency map build: {NumTargetsWithActors}/{NumTotalTargets} targets found actors, {NumTargetsWithoutActors} without actors",
		NumTargetsWithActors,
		TargetPackageIndices.Num(),
		NumTargetsWithoutActors);

	return PackageToActorInfos;
}
} // namespace UE::Insights::ObjectProfiler

#undef LOCTEXT_NAMESPACE

#endif	// WITH_EDITOR || defined(__INTELLISENSE__)