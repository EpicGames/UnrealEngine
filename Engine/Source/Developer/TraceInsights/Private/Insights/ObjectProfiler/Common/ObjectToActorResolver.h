// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR || defined(__INTELLISENSE__)

#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/SortedSet.h"
#include "UObject/ReferenceChainSearch.h"

#include "Insights/ObjectProfiler/IAssetInfoProvider.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::ObjectProfiler
{

enum class EScanState : uint8
{
	None = 0,
	AssetRegistry = 1 << 0,
	ReferenceChainSearch = 1 << 1,
};

ENUM_CLASS_FLAGS(EScanState);

// Sparse set of target package indices, sorted ascending. Replaces TBitArray<>(NumTargetPackages) for memory and op-cost reasons at large NumTargetPackages.
using FTargetHitMask = TSortedSet<int32, TInlineAllocator<8>>;

struct FDependencyChainActorInfo
{
	EScanState ScanState = EScanState::None;
	TArray<int32, TInlineAllocator<1>> Info;   // indices into FBuildPackageDependencyMapBuilder::ActorInfoPool. Typically one element (OFPA), occasionally several (OFMA / non-WP packages).
	FTargetHitMask ContributedTargets;         // targets that this frame's Info has already been pushed into. Short-circuits redundant work across cache-hits.

	void Reset()
	{
		ScanState = EScanState::None;
		Info.Reset();
		ContributedTargets.Empty();
	}
};

// Result of a chain scan: the chain frame the actors came from (so the caller can address its ContributedTargets), and a view of the actor indices.
struct FActorScanResult
{
	int32 SourceChainIndex = INDEX_NONE;
	TConstArrayView<int32> ActorRefs;
};

struct FPackageDependencyStackFrame
{
	FDependencyChainActorInfo ActorInfo;
	FAssetIdentifier AssetId;
	int32 StartIndex = INDEX_NONE;
	int32 CurrIndex = INDEX_NONE;
	int32 EndIndex = INDEX_NONE;
	FTargetHitMask TargetHitMask;
	bool bHit = false;
};
	
/**
 * Helper class for building a package dependency map.
 * See BuildPackageDependencyMap() for details.
 */
class FBuildPackageDependencyMapBuilder
{
public:
	FBuildPackageDependencyMapBuilder(TSoftObjectPtr<UWorld> InRootPackage, const TArray<FName>& InTargetPackageNames);

	TMap<FName, TSet<FActorInfo>> Execute();

private:
	TSoftObjectPtr<UWorld> RootPackage;
	const TArray<FName> TargetPackageNames;

	IAssetRegistry& AssetRegistry;
	const int32 NumTargetPackages;

	// Interned FActorInfo storage. Each unique actor is stored once; per-target sets and
	// chain-frame Info arrays carry int32 indices into this pool. Saves redundant FString
	// allocations when the same actor appears in many per-target sets.
	TArray<FActorInfo> ActorInfoPool;
	TMap<FActorInfo, int32> ActorInfoToIndex;

	TArray<TSet<int32>> TargetPackageActorInfos;   // values are indices into ActorInfoPool
	TMap<FName, int32> TargetPackageIndices;

	TArray<FAssetData> Assets;                            // Scratch for FAssetData results for registry queries during actor scans
	TArray<UObject*> AssetObjects;                        // Scratch for resolved UObject* instances for internal reference-chain scans
	TArray<FAssetIdentifier> Dependencies;                // Flat list of all dependencies discovered on the current path
	TArray<FPackageDependencyStackFrame> Stack;           // DFS stack frames, one per package on the current path
	TSet<FAssetIdentifier> CurrentPath;                   // Set of nodes on the current DFS path (for cycle detection)
	TArray<FTargetHitMask> TargetHitMasksCache;           // Cache of unique hit-mask patterns (one sorted set of target indices per visited pattern)
	TMap<uint32, TArray<int32>> TargetHitMaskHashBuckets; // Hash buckets to speed up lookups into TargetHitMasksCache.

	// Records visit status per AssetId:
	//   INDEX_NONE = no targets in this subtree,
	//   else = index into TargetHitMasksCache of which targets were hit.
	TMap<FAssetIdentifier, int32> VisitedNodes;

	UWorld* RootWorld = nullptr;
	const FStringView ExternalActorsFolderName = FPackagePath::GetExternalActorsFolderName();

	FActorScanResult ScanChainForExternalActors();

	FActorScanResult ScanChainForInternalActors();

	FActorScanResult ScanChainForActors();

	// Returns an interned index for the actor (existing if previously seen; else newly added to ActorInfoPool).
	int32 InternActorInfo(FActorInfo Info);

	/**
		* Attempts to extract an actor from a reference chain.
		* Walks the chain from root to target, looking for RootWorld followed by an Actor.
		* @param RefChain The reference chain to process
		* @param RootWorld The world to search within
		* @return Actor info if found, empty optional otherwise
		*/
	static TOptional<FActorInfo> TryExtractActorFromChain(const FReferenceChainSearch::FReferenceChain& RefChain, const UWorld* RootWorld);

	/**
		* Performs batched internal actor search using FReferenceChainSearch for all targets without actors.
		* This builds a complete reference graph and searches backwards from target packages to find actor references.
		*/
	void PerformBatchedReferenceChainSearch(TSet<int32>& TargetIndicesNeedingSearch);

	int32 FindOrCreateHitMaskIndex(const FTargetHitMask& HitMask);
};

} // namespace UE::Insights::ObjectProfiler

#undef UE_API

#endif // WITH_EDITOR || defined(__INTELLISENSE__)