// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorAssetInfoProvider.h"

#if UE_INSIGHTS_WITH_EDITOR_ASSET_INFO

#include "Styling/SlateIconFinder.h"

// CoreUObject
#include "AssetRegistry/AssetData.h"
#include "UObject/UObjectGlobals.h"

// AssetDefinition
#include "Misc/AssetCategoryPath.h"
#include "AssetDefinitionRegistry.h"

// AssetRegistry
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"

// ContentBrowser
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"

// Editor
#include "UnrealEdGlobals.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
extern UNREALED_API class UEditorEngine* GEditor;

// TraceInsights

#include "Insights/ObjectProfiler/Common/ObjectToActorResolver.h"
#include "Insights/ObjectProfiler/ObjectProfilerManager.h"
#include "Insights/ObjectProfiler/ViewModels/AssetInfoNode.h"
#include "Insights/ObjectProfiler/ViewModels/ObjectNode.h"

namespace UE::Insights::ObjectProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FEditorAssetInfoCategory
////////////////////////////////////////////////////////////////////////////////////////////////////

FEditorAssetInfoProvider::FEditorAssetInfoCategory::FEditorAssetInfoCategory()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FEditorAssetInfoProvider::FEditorAssetInfoCategory::GetDisplayName() const
{
	return Def ? Def->GetAssetDisplayName() : FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FEditorAssetInfoProvider::FEditorAssetInfoCategory::GetColor() const
{
	return Def ? Def->GetAssetColor() : FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* FEditorAssetInfoProvider::FEditorAssetInfoCategory::GetIcon() const
{
	return Icon;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FEditorAssetInfoProvider
////////////////////////////////////////////////////////////////////////////////////////////////////

FEditorAssetInfoProvider::FEditorAssetInfoProvider()
	: AssetRegistryModule(FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry"))
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FEditorAssetInfoProvider::~FEditorAssetInfoProvider()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IAssetInfoCategory& FEditorAssetInfoProvider::GetClassCategory(FName InClassName) const
{
	if (uint32* FoundIndex = CategoryByClassName.Find(InClassName))
	{
		return *FoundIndex < uint32(Categories.Num()) ? Categories[*FoundIndex] : DefaultCategory;
	}

	const UClass* Class = FindFirstObject<UClass>(InClassName.ToString());
	const UAssetDefinition* Def = Class ? UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(Class) : nullptr;
	if (!Def)
	{
		CategoryByClassName.Add(InClassName, ~0u);
		return DefaultCategory;
	}

	if (uint32* FoundIndex = CategoriesByDef.Find(Def))
	{
		CategoryByClassName.Add(InClassName, *FoundIndex);
		return Categories[*FoundIndex];
	}

	const uint32 CategoryIndex = uint32(Categories.Num());
	CategoriesByDef.Add(Def, CategoryIndex);

	FEditorAssetInfoCategory& Category = Categories.AddDefaulted_GetRef();
	Category.Id = CategoryIndex;
	Category.Def = Def;
	Category.Icon = FSlateIconFinder::FindIconBrushForClass(Class);

	CategoryByClassName.Add(InClassName, CategoryIndex);
	return Category;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IAssetInfoCategory& FEditorAssetInfoProvider::GetObjectCategory(const FName InClassName, const TCHAR* InObjectName, const TCHAR* InObjectPath) const
{
	return GetClassCategory(InClassName);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FEditorAssetInfoProvider::GetDisplayName(const FAssetData& InAssetData, const FName InClassName) const
{
	return GetClassCategory(InClassName).GetDisplayName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FEditorAssetInfoProvider::GetColor(const FAssetData& InAssetData, const FName InClassName) const
{
	return GetClassCategory(InClassName).GetColor();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* FEditorAssetInfoProvider::GetIcon(const FAssetData& InAssetData, const FName InClassName) const
{
	const FEditorAssetInfoCategory& Category = static_cast<const FEditorAssetInfoCategory&>(GetClassCategory(InClassName));
	if (Category.Def)
	{
		return Category.Def->GetIconBrush(InAssetData, InClassName);
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* FEditorAssetInfoProvider::GetThumbnail(const FAssetData& InAssetData, const FName InClassName) const
{
	const FSlateBrush* Brush = nullptr;

	const FEditorAssetInfoCategory& Category = static_cast<const FEditorAssetInfoCategory&>(GetClassCategory(InClassName));
	if (Category.Def)
	{
		// Try getting the thumbnail brush unique for the given asset data.
		Brush = Category.Def->GetThumbnailBrush(InAssetData, InClassName);
		if (!Brush)
		{
			const FName AssetClassName = InAssetData.AssetClassPath.GetAssetName();
			if (!AssetClassName.IsNone() && AssetClassName != InClassName)
			{
				Brush = Category.Def->GetThumbnailBrush(InAssetData, AssetClassName);
			}
		}
	}

	if (!Brush)
	{
		// Get the class thumbnail brush.
		if (const UClass* Class = FindFirstObject<UClass>(InClassName.ToString()))
		{
			Brush = FSlateIconFinder::FindIconBrushForClass(Class);
		}
		if (!Brush && Category.Def && Category.Def->GetClass())
		{
			Brush = FSlateIconFinder::FindIconBrushForClass(Category.Def->GetClass());
		}
	}

	return Brush;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FEditorAssetInfoProvider::GetAssetData(const FObjectNode& InObjectNode, FAssetData& OutAsset) const
{
	if (FCString::Strncmp(InObjectNode.GetObjectName(), TEXT("Default__"), 9) == 0)
	{
		// Exclude the Class Default Objects.
		return false;
	}
	FString PathStr = InObjectNode.GetObjectPath();
	FSoftObjectPath SoftObjectPath(PathStr);
	OutAsset = AssetRegistryModule.Get().GetAssetByObjectPath(SoftObjectPath, false, false);
	TSharedPtr<FObjectNode> ClassNode = InObjectNode.GetClass();
	if (ClassNode)
	{
		OutAsset.AssetClassPath.TrySetPath(ClassNode->GetObjectPath());
	}
	return OutAsset.IsValid();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FEditorAssetInfoProvider::FindMatchedAsset(const TArray<FTableTreeNodePtr>& InSelectedNodes, FAssetData& OutAsset) const
{
	for (const FTableTreeNodePtr& SelectedNode : InSelectedNodes)
	{
		if (SelectedNode->Is<FObjectNode>())
		{
			const FObjectNode& ObjectNode = SelectedNode->As<FObjectNode>();
			if (GetAssetData(ObjectNode, OutAsset))
			{
				return true;
			}
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#if 0
bool FEditorAssetInfoProvider::GetActors(const FAssetData& InAssetData, TArray<FSoftObjectPath>& OutActors) const
{
	TArray<FName> Names;
	Names.Add(InAssetData.PackageName);

	FBuildPackageDependencyMapBuilder Builder(GEditor->GetEditorWorldContext().World(), Names);

	auto Map = Builder.Execute();
	for (auto& Entry : Map)
	{
		for (auto& Actor : Entry.Value)
		{
			OutActors.Add(Actor.GetPath());
		}
	}

	return !OutActors.IsEmpty();
}
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FEditorAssetInfoProvider::BrowseToAsset(const FAssetInfoNode& InAssetInfo) const
{
	if (InAssetInfo.AssetData.IsValid())
	{
		if (FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser"))
		{
			ContentBrowserModule->Get().SyncBrowserToAssets({ InAssetInfo.AssetData });
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FEditorAssetInfoProvider::BrowseToActor(const FAssetInfoNode& InAssetInfo) const
{
	const FSoftObjectPath& ActorPath = InAssetInfo.Actor.GetPath();
	if (ActorPath.IsValid())
	{
		UObject* Obj = ActorPath.ResolveObject();
		if (!Obj)
		{
			Obj = ActorPath.TryLoad();
		}
		if (AActor* AsActor = Cast<AActor>(Obj))
		{
			GEditor->SelectNone(false, true);
			GEditor->SelectActor(AsActor, /*InSelected=*/true, /*bNotify=*/false, /*bSelectEvenIfHidden=*/true);
			GEditor->NoteSelectionChange();
			GEditor->MoveViewportCamerasToActor(*AsActor, false);
			GUnrealEd->ShowActorProperties();
			GUnrealEd->UpdateFloatingPropertyWindows();
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TMap<FName, TSharedRef<FActorSet>> FEditorAssetInfoProvider::MatchNamesToActors(const TArray<FName>& PackageNames) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorAssetInfoProvider::MatchNamesToActors);

	FBuildPackageDependencyMapBuilder Builder(GEditor->GetEditorWorldContext().World(), PackageNames);
	TMap<FName, TSet<FActorInfo>> RawMap = Builder.Execute();

	TRACE_CPUPROFILER_EVENT_SCOPE(MatchNamesToActors_BuildMap);

	TMap<FName, TSharedRef<FActorSet>> SharedMap;
	SharedMap.Reserve(RawMap.Num());
	for (TPair<FName, TSet<FActorInfo>>& Pair : RawMap)
	{
		SharedMap.Add(Pair.Key, MakeShared<FActorSet>(MoveTemp(Pair.Value)));
	}
	return SharedMap;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FEditorAssetInfoProvider::ConvertRuntimePathToEditorPath(FString& ObjectPath) const
{
	//
	// Logic taken from ULiveEditObjectSubsystemState::ConvertRuntimePathToEditorPath
	// Try to convert from /.../WorldName/_Generated_/MainGrid_L0_X0_Y0_DL0.WorldName:PersistentLevel.ActorName
	//                  to /.../WorldName.WorldName:PersistentLevel.ActorName
	//
	// This can only be done optimistically by analyzing the path to see if it's a generated package and perform some string
	// magic to extract the world name and reconstruct the editor path, assuming that the world's package always has the same
	// name as the world object (which is enforced in UWorld::PostLoad())
	//
	const FStringView PathView = ObjectPath;
	if (int32 GeneratedPos = PathView.Find(TEXTVIEW("/_Generated_/"), 0); GeneratedPos != INDEX_NONE)
	{
		if (int32 NextDotPos = PathView.Find(TEXTVIEW("."), GeneratedPos); NextDotPos != INDEX_NONE)
		{
			if (int32 NextColonPos = PathView.Find(TEXTVIEW(":"), NextDotPos); NextColonPos != INDEX_NONE)
			{
				ObjectPath.RemoveAt(GeneratedPos, NextDotPos - GeneratedPos);
			}
		}
	}

	RemapObjectPluginPath(ObjectPath);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler

#endif // UE_INSIGHTS_WITH_EDITOR_ASSET_INFO
