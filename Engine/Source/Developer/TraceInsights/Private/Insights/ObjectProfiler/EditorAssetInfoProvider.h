// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Insights/ObjectProfiler/IAssetInfoProvider.h"

#ifndef UE_INSIGHTS_WITH_EDITOR_ASSET_INFO
#if defined(__INTELLISENSE__)
#define UE_INSIGHTS_WITH_EDITOR_ASSET_INFO 1
#else
#define UE_INSIGHTS_WITH_EDITOR_ASSET_INFO WITH_EDITOR
#endif
#endif

#if UE_INSIGHTS_WITH_EDITOR_ASSET_INFO

#define UE_API TRACEINSIGHTS_API

// AssetDefinition
//#include "Misc/AssetCategoryPath.h"

class UAssetDefinition;
class FAssetRegistryModule;

namespace UE::Insights::ObjectProfiler
{

class FEditorAssetInfoProvider : public IAssetInfoProvider
{
private:
	class FEditorAssetInfoCategory : public IAssetInfoCategory
	{
		friend class FEditorAssetInfoProvider;

	public:
		FEditorAssetInfoCategory();
		virtual ~FEditorAssetInfoCategory() = default;

		virtual uint32 GetId() const override { return Id; }

		virtual FText GetDisplayName() const override;
		virtual FLinearColor GetColor() const override;
		virtual const FSlateBrush* GetIcon() const override;

	private:
		uint32 Id = ~0u;
		TObjectPtr<const UAssetDefinition> Def = nullptr;
		const FSlateBrush* Icon = nullptr;
	};

public:
	UE_API FEditorAssetInfoProvider();
	UE_API virtual ~FEditorAssetInfoProvider();

	UE_API virtual const IAssetInfoCategory& GetClassCategory(const FName InClassName) const override;
	UE_API virtual const IAssetInfoCategory& GetObjectCategory(const FName InClassName, const TCHAR* InObjectName, const TCHAR* InObjectPath = nullptr) const override;

	UE_API virtual FText GetDisplayName(const FAssetData& InAssetData, const FName InClassName) const override;
	UE_API virtual FLinearColor GetColor(const FAssetData& InAssetData, const FName InClassName) const override;
	UE_API virtual const FSlateBrush* GetIcon(const FAssetData& InAssetData, const FName InClassName) const override;
	UE_API virtual const FSlateBrush* GetThumbnail(const FAssetData& InAssetData, const FName InClassName) const override;

	UE_API virtual bool GetAssetData(const FObjectNode& InObjectNode, FAssetData& OutAsset) const override;
	UE_API virtual bool FindMatchedAsset(const TArray<TSharedPtr<FTableTreeNode>>& InSelectedNodes, FAssetData& OutAsset) const override;
	UE_API virtual TMap<FName, TSharedRef<FActorSet>> MatchNamesToActors(const TArray<FName>& PackageNames) const override;

	virtual bool GetActors(const FAssetData& InAssetData, TArray<FSoftObjectPath>& OutActors) const { return false; }
	UE_API virtual bool BrowseToAsset(const FAssetInfoNode& InAssetInfo) const override;
	UE_API virtual bool BrowseToActor(const FAssetInfoNode& InAssetInfo) const override;

protected:
	/**
	 * Attempts to remap a path using the editor plugin mappings. Default implementation is a no-op.
	 * Returns true if the path was modified.
	 */
	virtual bool RemapObjectPluginPath(FString& InOutPath) const { return false; }

	UE_API virtual void ConvertRuntimePathToEditorPath(FString& ObjectPath) const override;

private:
	FAssetRegistryModule& AssetRegistryModule;
	mutable TArray<FEditorAssetInfoCategory> Categories;
	FEditorAssetInfoCategory DefaultCategory;
	mutable TMap<FName, uint32> CategoryByClassName;
	mutable TMap<TObjectPtr<const UAssetDefinition>, uint32> CategoriesByDef;
};

} // namespace UE::Insights::ObjectProfiler

#undef UE_API

#endif // UE_INSIGHTS_WITH_EDITOR_ASSET_INFO
