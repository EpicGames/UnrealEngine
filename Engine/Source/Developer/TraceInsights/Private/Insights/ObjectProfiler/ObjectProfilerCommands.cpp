// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectProfilerCommands.h"

#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/ObjectProfiler/ObjectProfilerManager.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::ObjectProfiler"

namespace UE::Insights::ObjectProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FObjectProfilerCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectProfilerCommands::FObjectProfilerCommands()
: TCommands<FObjectProfilerCommands>(
	TEXT("ObjectProfilerCommands"),
	NSLOCTEXT("Contexts", "ObjectProfilerCommands", "Insights - Object Insights"),
	NAME_None,
	FInsightsStyle::GetStyleSetName())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectProfilerCommands::~FObjectProfilerCommands()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// UI_COMMAND takes long for the compiler to optimize
UE_DISABLE_OPTIMIZATION_SHIP
void FObjectProfilerCommands::RegisterCommands()
{
	UI_COMMAND(ToggleObjectTableTreeViewVisibility,
		"Objects",
		"Toggles the visibility of the main Objects table/tree view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleObjectDetailsViewVisibility,
		"Object Details",
		"Toggles the visibility of the Object Details view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////
// Toggle Commands
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FObjectProfilerActionManager, This, ToggleObjectTableTreeViewVisibility, IsObjectTableTreeViewVisible, ShowHideObjectTableTreeView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FObjectProfilerActionManager, This, ToggleObjectDetailsViewVisibility, IsObjectDetailsViewVisible, ShowHideObjectDetailsView)

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler

#undef LOCTEXT_NAMESPACE
