// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/Commands.h"

// TraceInsights
#include "Insights/Common/CommandUtils.h"

namespace UE::Insights::ObjectProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class that holds all profiler commands.
 */
class FObjectProfilerCommands : public TCommands<FObjectProfilerCommands>
{
public:
	FObjectProfilerCommands();
	virtual ~FObjectProfilerCommands();
	virtual void RegisterCommands() override;

public:
	/** Toggles visibility for the Objects table/tree view. */
	TSharedPtr<FUICommandInfo> ToggleObjectTableTreeViewVisibility;

	/** Toggles visibility for the Object Details view. */
	TSharedPtr<FUICommandInfo> ToggleObjectDetailsViewVisibility;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class that provides helper functions for the commands to avoid cluttering profiler manager with many small functions.
 * Can't contain any variables. Directly operates on the profiler manager instance.
 */
class FObjectProfilerActionManager
{
	friend class FObjectProfilerManager;

private:
	/** Private constructor. */
	FObjectProfilerActionManager(class FObjectProfilerManager* Instance)
		: This(Instance)
	{}

	//////////////////////////////////////////////////
	// Toggle Commands

	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleObjectTableTreeViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleObjectDetailsViewVisibility)

	//////////////////////////////////////////////////

protected:
	/** Reference to the global instance of the profiler manager. */
	class FObjectProfilerManager* This;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler
