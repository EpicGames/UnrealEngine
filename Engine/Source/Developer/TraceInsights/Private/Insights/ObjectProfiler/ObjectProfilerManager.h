// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Ticker.h"
#include "Framework/Commands/UICommandList.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Common/AvailabilityCheck.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/ObjectProfiler/ObjectProfilerCommands.h"

namespace UE::Insights::ObjectProfiler
{

class IAssetInfoProvider;
class SObjectProfilerWindow;

DECLARE_LOG_CATEGORY_EXTERN(LogObjectProfiler, Log, All);

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * This class manages the Object Profiler state and settings.
 */
class FObjectProfilerManager : public TSharedFromThis<FObjectProfilerManager>, public IInsightsComponent
{
	friend class FObjectProfilerActionManager;

public:
	/** Creates the Object Profiler manager, only one instance can exist. */
	FObjectProfilerManager(TSharedRef<FUICommandList> InCommandList);

	/** Virtual destructor. */
	virtual ~FObjectProfilerManager();

	/** Creates an instance of the Object Profiler manager. */
	static TSharedPtr<FObjectProfilerManager> CreateInstance();

	/**
	 * @return the global instance of the Object Profiler manager.
	 * This is an internal singleton and cannot be used outside TraceInsights.
	 * For external use:
	 *     IUnrealInsightsModule& Module = FModuleManager::Get().LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	 *     Module.GetObjectProfilerManager();
	 */
	static TSharedPtr<FObjectProfilerManager> Get();

	bool IsAvailable() const { return bIsAvailable; }

	//////////////////////////////////////////////////
	// IInsightsComponent

	virtual void Initialize(IUnrealInsightsModule& InsightsModule) override;
	virtual void Shutdown() override;
	virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) override;
	virtual void UnregisterMajorTabs() override;
	virtual void OnWindowClosedEvent() override;
	virtual bool Exec(const TCHAR* Cmd, FOutputDevice& Ar) override;

	//////////////////////////////////////////////////

	/** @return UI command list for the Object Profiler manager. */
	const TSharedRef<FUICommandList> GetCommandList() const;

	/** @return an instance of the Object Profiler commands. */
	static const FObjectProfilerCommands& GetCommands();

	/** @return an instance of the Object Profiler action manager. */
	static FObjectProfilerActionManager& GetActionManager();

	/**
	 * Converts profiler window weak pointer to a shared pointer and returns it.
	 * Make sure the returned pointer is valid before trying to dereference it.
	 */
	TSharedPtr<SObjectProfilerWindow> GetProfilerWindow() const
	{
		return ProfilerWindowWeakPtr.Pin();
	}

	const FName& GetMajorTabId() const { return MajorTabId; }
	const FName& GetLogListingName() const { return LogListingName; }

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Getters and setters used by Toggle Commands.

	/** @return true, if the Objects table/tree view is visible */
	bool IsObjectTableTreeViewVisible() const { return bIsObjectTableTreeViewVisible; }
	void SetObjectTableTreeViewVisible(bool bIsVisible) { bIsObjectTableTreeViewVisible = bIsVisible; }
	void ShowHideObjectTableTreeView(bool bIsVisible);

	/** @return true, if the Object Details view is visible */
	bool IsObjectDetailsViewVisible() const { return bIsObjectDetailsViewVisible; }
	void SetObjectDetailsViewVisible(bool bIsVisible) { bIsObjectDetailsViewVisible = bIsVisible; }
	void ShowHideObjectDetailsView(bool bIsVisible);

	/** @return true, if the Segmented Bar Graph is visible (in toolbar) */
	bool IsSegmentedBarGraphVisible() const { return bIsSegmentedBarGraphVisible; }
	void SetSegmentedBarGraphVisible(bool bIsVisible) { bIsSegmentedBarGraphVisible = bIsVisible; }
	void ShowHideSegmentedBarGraph(bool bIsVisible);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void OnSessionChanged();

private:
	/** Binds our UI commands to delegates. */
	void BindCommands();

	/** Called to spawn the Object Profiler major tab. */
	TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args);

	bool CanSpawnTab(const FSpawnTabArgs& Args) const;

	/** Callback called when the Object Profiler major tab is closed. */
	void OnTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	/** Updates this manager, done through FCoreTicker. */
	bool Tick(float DeltaTime);

	void AssignProfilerWindow(const TSharedRef<SObjectProfilerWindow>& InProfilerWindow)
	{
		ProfilerWindowWeakPtr = InProfilerWindow;
	}

	void RemoveProfilerWindow()
	{
		ProfilerWindowWeakPtr.Reset();
	}

private:
	bool bIsInitialized = false;
	bool bIsAvailable = false;

	/** If the tab is registered in the tab manager. */
	bool bIsTabRegistered = false;

	FAvailabilityCheck AvailabilityCheck;

	/** The delegate to be invoked when this manager ticks. */
	FTickerDelegate OnTick;

	/** Handle to the registered OnTick. */
	FTSTicker::FDelegateHandle OnTickHandle;

	/** List of UI commands for this manager. This will be filled by this and corresponding classes. */
	TSharedRef<FUICommandList> CommandList;

	/** An instance of the Object Profiler action manager. */
	FObjectProfilerActionManager ActionManager;

	/** A weak pointer to the Object Insights window. */
	TWeakPtr<SObjectProfilerWindow> ProfilerWindowWeakPtr;

	/** The Asset Info Provider to be used by the Object Insights window. */
	TSharedPtr<IAssetInfoProvider> AssetInfoProvider;

	/** If the Objects table/tree view is visible or hidden. */
	bool bIsObjectTableTreeViewVisible = false;

	/** If the Object Details view is visible or hidden. */
	bool bIsObjectDetailsViewVisible = false;

	/** If the Segmented Bar Graph is visible or hidden. */
	bool bIsSegmentedBarGraphVisible = false;

	/** The id of the Object Insights major tab. */
	FName MajorTabId;

	/** The name of the Object Profiler log listing. */
	FName LogListingName;

	/** A shared pointer to the global instance of the Object Profiler manager. */
	static TSharedPtr<FObjectProfilerManager> Instance;
};

} // namespace UE::Insights::ObjectProfiler
