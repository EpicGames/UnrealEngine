// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "AssetRegistry/AssetData.h"
#include "UObject/NameTypes.h"

#include "Insights/ObjectProfiler/IAssetInfoProvider.h"

namespace UE::Insights::ObjectProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Holds asset registry data for a single matched asset entry shown in the object details view.
 */
struct FAssetInfoNode
{
	FActorInfo Actor;
	FAssetData AssetData;
	FName AssetClassName;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler
