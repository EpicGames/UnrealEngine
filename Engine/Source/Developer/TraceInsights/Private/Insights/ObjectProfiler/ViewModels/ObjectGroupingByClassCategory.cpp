// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectGroupingByClassCategory.h"

#include "Containers/BitArray.h"
#include "Containers/Ticker.h"
#include "Internationalization/Internationalization.h"

// TraceInsightsCore
#include "InsightsCore/Common/AsyncOperationProgress.h"
#include "InsightsCore/Widgets/SSegmentedBarGraph.h"

// TraceInsights
#include "Insights/ObjectProfiler/ObjectProfilerManager.h"
#include "Insights/ObjectProfiler/ViewModels/ObjectNode.h"
#include "Insights/ObjectProfiler/Widgets/SObjectTableTreeView.h"
#include "Insights/ObjectProfiler/Widgets/SObjectProfilerWindow.h"

#define LOCTEXT_NAMESPACE "UE::Insights::ObjectProfiler::FObjectNode"

namespace UE::Insights::ObjectProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FObjClassBarGraphSegment
////////////////////////////////////////////////////////////////////////////////////////////////////

class FObjClassBarGraphSegment : public UE::Insights::IBarGraphSegment
{
	friend class FObjectGroupingByClass;

public:
	FObjClassBarGraphSegment() {}
	virtual ~FObjClassBarGraphSegment() {}

	virtual double GetSize() const { return Size; }
	virtual FText GetText() const { return Text; }
	virtual FText GetToolTipText() const { return ToolTipText; }
	virtual FLinearColor GetColor() const { return Color; }
	virtual FLinearColor GetTextColor() const { return TextColor; }

private:
	double Size;
	FText Text;
	FText ToolTipText;
	FLinearColor Color;
	FLinearColor TextColor;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FClassObjectGroupNode
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FClassObjectGroupNode)

////////////////////////////////////////////////////////////////////////////////////////////////////

FClassObjectGroupNode::FClassObjectGroupNode(TWeakPtr<FTable> InParentTable, const FText& InDisplayName, uint32 InClassId)
	: FCustomTableTreeNode(InParentTable, InDisplayName)
	, ClassId(InClassId)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FClassObjectGroupNode::~FClassObjectGroupNode()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FClassObjectGroupNode::UpdateEstimatedMemory(int64 TotalMemorySize, bool bUseTotalEstimatedMemory)
{
	if (bIsUpdated)
	{
		return;
	}
	bIsUpdated = true;

	SystemMemorySize = 0;
	VideoMemorySize = 0;
	Impact = 0.0;

	for (const auto& Child : GetChildren())
	{
		if (Child->Is<FObjectNode>())
		{
			FObjectNode& ObjNode = Child->As<FObjectNode>();
			if (bUseTotalEstimatedMemory)
			{
				SystemMemorySize += ObjNode.GetTotalSystemMemorySize();
				VideoMemorySize += ObjNode.GetTotalVideoMemorySize();
			}
			else
			{
				SystemMemorySize += ObjNode.GetSystemMemorySize();
				VideoMemorySize += ObjNode.GetVideoMemorySize();
			}
		}
		else if (Child->Is<FClassObjectGroupNode>())
		{
			FClassObjectGroupNode& ObjClassGroupNode = Child->As<FClassObjectGroupNode>();
			ObjClassGroupNode.UpdateEstimatedMemory(TotalMemorySize, bUseTotalEstimatedMemory);
			SystemMemorySize += ObjClassGroupNode.GetSystemMemorySize();
			VideoMemorySize += ObjClassGroupNode.GetVideoMemorySize();
		}
	}

	if (TotalMemorySize > 0)
	{
		Impact = double(SystemMemorySize + VideoMemorySize) / double(TotalMemorySize);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FObjectGroupingByClassImpl
////////////////////////////////////////////////////////////////////////////////////////////////////

class FObjectGroupingByClassImpl final
{
	friend class FObjectGroupingByClass;

public:
	FObjectGroupingByClassImpl(FTableTreeNode& InParentGroup, TWeakPtr<FTable> InParentTable, TSharedPtr<SObjectTableTreeView> InObjectTableTreeView)
		: ParentGroup(InParentGroup)
		, ParentTable(InParentTable)
		, ObjectTableTreeView(InObjectTableTreeView)
	{
		if (ObjectTableTreeView)
		{
			AssetInfoProvider = ObjectTableTreeView->GetAssetInfoProvider();
			bUseTotalEstimatedMemory = ObjectTableTreeView->HasTotalEstimatedMemory();
		}
	}
	~FObjectGroupingByClassImpl() {}

	void AddObject(FObjectNode& ObjNode);
	FTableTreeNode& GetParentGroupNode(uint32 ClassId, TSharedPtr<FObjectNode> ClassNode);
	void UpdateEstimatedMemory();

private:
	const IAssetInfoCategory& GetCategoryFromClassNode(const FObjectNode& ClassNode);

private:
	FTableTreeNode& ParentGroup;
	TWeakPtr<FTable> ParentTable;
	TSharedPtr<SObjectTableTreeView> ObjectTableTreeView;
	TSharedPtr<IAssetInfoProvider> AssetInfoProvider;
	TMap<uint32, TSharedPtr<FClassObjectGroupNode>> AddedClasses;
	TMap<uint32, TSharedPtr<FClassObjectGroupNode>> AddedCategories;
	TSharedPtr<FClassObjectGroupNode> InvalidClassGroupNode;
	TSharedPtr<FClassObjectGroupNode> UncategorizedGroupNode;
	int64 TotalSystemMemorySize = 0;
	int64 TotalVideoMemorySize = 0;
	bool bShouldCreateCategoryGroups = false;
	bool bShouldHideObjectBaseClass = true;
	bool bUseTotalEstimatedMemory = false;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectGroupingByClassImpl::AddObject(FObjectNode& ObjNode)
{
	if (bUseTotalEstimatedMemory)
	{
		TotalSystemMemorySize += ObjNode.GetTotalSystemMemorySize();
		TotalVideoMemorySize += ObjNode.GetTotalVideoMemorySize();
	}
	else
	{
		TotalSystemMemorySize += ObjNode.GetSystemMemorySize();
		TotalVideoMemorySize += ObjNode.GetVideoMemorySize();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const IAssetInfoCategory& FObjectGroupingByClassImpl::GetCategoryFromClassNode(const FObjectNode& ClassNode)
{
	checkSlow(AssetInfoProvider);

	const FName ClassName(ClassNode.GetDisplayName().ToString());
	const IAssetInfoCategory* Category = &AssetInfoProvider->GetClassCategory(ClassName);

	if (Category->GetDisplayName().IsEmpty() &&
		ClassNode.Is<FClassObjectNode>() &&
		ensure(ObjectTableTreeView))
	{
		const FClassObjectNode& Class = ClassNode.As<FClassObjectNode>();
		if (Class.GetSuperId() != FObjectNode::InvalidObjectId)
		{
			TSharedPtr<FObjectNode> SuperClassNode = ObjectTableTreeView->GetObjectNode(Class.GetSuperId());
			if (SuperClassNode.IsValid())
			{
				return GetCategoryFromClassNode(*SuperClassNode);
			}
		}
	}

	return *Category;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTableTreeNode& FObjectGroupingByClassImpl::GetParentGroupNode(uint32 ClassId, TSharedPtr<FObjectNode> ClassNode)
{
	if (TSharedPtr<FClassObjectGroupNode>* FoundClassGroupNode = AddedClasses.Find(ClassId))
	{
		return **FoundClassGroupNode;
	}

	if (!ClassNode)
	{
		if (!InvalidClassGroupNode)
		{
			const FText InvalidClassDisplayName = LOCTEXT("InvalidClassDisplayName", "<Invalid>");
			InvalidClassGroupNode = MakeShared<FClassObjectGroupNode>(ParentTable, InvalidClassDisplayName, FObjectNode::InvalidObjectId);
			InvalidClassGroupNode->SetColor(FLinearColor(0.7f, 0.7f, 0.7f, 1.0f));
			ParentGroup.AddChildAndSetParent(InvalidClassGroupNode);
			AddedClasses.Add(FObjectNode::InvalidObjectId, InvalidClassGroupNode);
		}
		if (ClassId != FObjectNode::InvalidObjectId)
		{
			AddedClasses.Add(ClassId, InvalidClassGroupNode);
		}
		return *InvalidClassGroupNode;
	}

	if (bShouldCreateCategoryGroups)
	{
		// Group by Class Category

		if (AssetInfoProvider)
		{
			auto& Category = GetCategoryFromClassNode(*ClassNode);
			if (TSharedPtr<FClassObjectGroupNode>* FoundCategoryGroupNode = AddedCategories.Find(Category.GetId()))
			{
				return **FoundCategoryGroupNode;
			}

			if (Category.GetDisplayName().IsEmpty())
			{
				if (!UncategorizedGroupNode)
				{
					const FText UncategorizedDisplayName = LOCTEXT("UncategorizedDisplayName", "<Uncategorized>");
					UncategorizedGroupNode = MakeShared<FClassObjectGroupNode>(ParentTable, UncategorizedDisplayName, FObjectNode::InvalidObjectId);
					UncategorizedGroupNode->SetColor(FLinearColor(0.7f, 0.7f, 0.7f, 1.0f));
					ParentGroup.AddChildAndSetParent(UncategorizedGroupNode);
				}
				AddedCategories.Add(Category.GetId(), UncategorizedGroupNode);
				AddedClasses.Add(ClassId, UncategorizedGroupNode);
				return *UncategorizedGroupNode;
			}

			TSharedRef<FClassObjectGroupNode> ClassGroupNode = MakeShared<FClassObjectGroupNode>(ParentTable, Category.GetDisplayName(), ClassId);
			ClassGroupNode->SetColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
			ClassGroupNode->SetIconColor(Category.GetColor());
			if (Category.GetIcon())
			{
				ClassGroupNode->SetIcon(Category.GetIcon());
			}

			ParentGroup.AddChildAndSetParent(ClassGroupNode);
			AddedCategories.Add(Category.GetId(), ClassGroupNode);
			AddedClasses.Add(ClassId, ClassGroupNode);
			return *ClassGroupNode;
		}
		else
		{
			// Just create a group for each class.
			TSharedPtr<FClassObjectGroupNode> ClassGroupNode = MakeShared<FClassObjectGroupNode>(ParentTable, ClassNode->GetDisplayName(), ClassId);
			ClassGroupNode->SetColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
			ParentGroup.AddChildAndSetParent(ClassGroupNode);
			AddedClasses.Add(ClassId, ClassGroupNode);
			return *ClassGroupNode;
		}
	}
	else
	{
		// Group by Class Hierarchy

		if (bShouldHideObjectBaseClass)
		{
			// Hides the root "Object" class.
			if (FCString::Strcmp(ClassNode->GetObjectName(), TEXT("Object")) == 0)
			{
				return ParentGroup;
			}
		}

		FTableTreeNode* ParentGroupPtr = &ParentGroup;

		if (ensure(ClassNode->Is<FClassObjectNode>()))
		{
			// Create class hierarchy.
			FClassObjectNode& ClassObjectNode = ClassNode->As<FClassObjectNode>();
			if (ClassObjectNode.GetSuperId() != FObjectNode::InvalidObjectId)
			{
				if (ensure(ObjectTableTreeView))
				{
					TSharedPtr<FObjectNode> SuperClassNode = ObjectTableTreeView->GetObjectNode(ClassObjectNode.GetSuperId());
					ParentGroupPtr = &GetParentGroupNode(ClassObjectNode.GetSuperId(), SuperClassNode);
				}
			}
		}

		TSharedRef<FClassObjectGroupNode> ClassGroupNode = MakeShared<FClassObjectGroupNode>(ParentTable, ClassNode->GetDisplayName(), ClassId);
		ClassGroupNode->SetColor(FLinearColor(1.0f, 1.0f, 0.3f, 1.0f));

		if (AssetInfoProvider)
		{
			auto& Category = GetCategoryFromClassNode(*ClassNode);
			ClassGroupNode->SetIconColor(Category.GetColor());
			if (Category.GetIcon())
			{
				ClassGroupNode->SetIcon(Category.GetIcon());
			}
		}

		ParentGroupPtr->AddChildAndSetParent(ClassGroupNode);
		AddedClasses.Add(ClassId, ClassGroupNode);
		return *ClassGroupNode;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectGroupingByClassImpl::UpdateEstimatedMemory()
{
	// Update Estimated Memory (System + Video Memory) and Impact for each ClassGroupNode.
	const int64 TotalMemorySize = TotalSystemMemorySize + TotalVideoMemorySize;
	for (auto& KV : AddedClasses)
	{
		FClassObjectGroupNode& ClassGroupNode = *KV.Value;
		ClassGroupNode.UpdateEstimatedMemory(TotalMemorySize, bUseTotalEstimatedMemory);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FObjectGroupingByClass
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FObjectGroupingByClass)

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectGroupingByClass::FObjectGroupingByClass(TWeakPtr<SObjectTableTreeView> InTreeView)
	: FTreeNodeGrouping(FText::GetEmpty(), FText::GetEmpty(), FText::GetEmpty(), nullptr)
	, WeakObjectTableTreeView(InTreeView)
{
	SetColor(FLinearColor(1.0f, 1.0f, 0.3f, 1.0f));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectGroupingByClass::~FObjectGroupingByClass()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectGroupingByClass::GroupNodes(
	const TArray<FTableTreeNodePtr>& Nodes,
	FTableTreeNode& ParentGroup,
	TWeakPtr<FTable> InParentTable,
	IAsyncOperationProgress& InAsyncOperationProgress) const
{
	ParentGroup.ClearChildren();

	FObjectGroupingByClassImpl Impl(ParentGroup, InParentTable, WeakObjectTableTreeView.Pin());
	Impl.bShouldCreateCategoryGroups = IsCategoryGrouping();
	Impl.bShouldHideObjectBaseClass = bShouldHideObjectBaseClass;

	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			return;
		}

		if (!NodePtr->Is<FObjectNode>())
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		FObjectNode& ObjNode = NodePtr->As<FObjectNode>();
		if (ObjNode.GetObjectId() == FObjectNode::InvalidObjectId)
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		Impl.AddObject(ObjNode);

		FTableTreeNode& ParentGroupNode = Impl.GetParentGroupNode(ObjNode.GetClassId(), ObjNode.GetClass());
		ParentGroupNode.AddChildAndSetParent(NodePtr);
	}

	Impl.UpdateEstimatedMemory();

	if (bShouldUpdateSegmentedBarGraph)
	{
		UpdateSegmentedBarGraph(ParentGroup);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectGroupingByClass::UpdateSegmentedBarGraph(FTableTreeNode& ParentGroup) const
{
	if (ParentGroup.GetDisplayName().CompareTo(LOCTEXT("GroupName_All", "All")) != 0)
	{
		return;
	}

	TSharedPtr<FObjectProfilerManager> ObjectProfilerManager = FObjectProfilerManager::Get();
	if (!ObjectProfilerManager)
	{
		return;
	}

	TSharedPtr<SObjectProfilerWindow> Window = ObjectProfilerManager->GetProfilerWindow();
	if (!Window)
	{
		return;
	}

	TArray<TSharedPtr<IBarGraphSegment>>& Segments = Window->GetSegments();
	Segments.Reset();

	uint32 NumOtherGroups = 0;
	double TotalImpact = 0.0;
	for (const auto& Child : ParentGroup.GetChildren())
	{
		if (Child->Is<FClassObjectGroupNode>())
		{
			FClassObjectGroupNode& ObjClassGroupNode = Child->As<FClassObjectGroupNode>();
			double Impact = ObjClassGroupNode.GetEstimatedMemoryImpact();
			if (Impact >= 0.01) // >= 1%
			{
				TotalImpact += Impact;

				auto Segment = MakeShared<FObjClassBarGraphSegment>();
				Segment->Size = Impact * 100.0;
				Segment->Text = ObjClassGroupNode.GetDisplayName();
				Segment->ToolTipText = FText::Format(LOCTEXT("BarGraphSegmentToolTipFmt", "{0} ({1}) -- {2}"),
					ObjClassGroupNode.GetDisplayName(),
					FText::AsMemory(ObjClassGroupNode.GetEstimatedMemorySize()),
					FText::AsPercent(Impact));
				Segment->Color = ObjClassGroupNode.GetIconColor();
				const bool bIsDarkColor = (Segment->Color.GetLuminance() < 0.4f);
				Segment->TextColor = bIsDarkColor ? FLinearColor(1.0f, 1.0f, 1.0f, 1.0f) : FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);

				Segments.Add(Segment);
			}
			else
			{
				++NumOtherGroups;
			}
		}
	}

	Segments.Sort([](const TSharedPtr<IBarGraphSegment>& A, const TSharedPtr<IBarGraphSegment>& B) { return A->GetSize() > B->GetSize(); });

	if (TotalImpact < 1.0)
	{
		auto Segment = MakeShared<FObjClassBarGraphSegment>();
		Segment->Size = (1.0 - TotalImpact) * 100.0;
		Segment->Text = LOCTEXT("Other", "Other");
		Segment->ToolTipText = FText::Format(LOCTEXT("OtherTootTipFmt", "Other Object Categories ({0} categories) -- {1}"),
			FText::AsNumber(NumOtherGroups),
			FText::AsPercent(1.0 - TotalImpact));
		Segment->Color = FLinearColor(0.3f, 0.3f, 0.3f, 1.0f);
		Segment->TextColor = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);

		Segments.Add(Segment);
	}

	TSharedPtr<SSegmentedBarGraph> SegmentedBarGraph = Window->GetSegmentedBarGraph();
	if (SegmentedBarGraph)
	{
		ExecuteOnGameThread(TEXT("RefreshSegmentedBarGraph"), [SegmentedBarGraph]() { SegmentedBarGraph->RequestGraphRefresh(); });
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FObjectGroupingByClassCategory
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FObjectGroupingByClassCategory)

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectGroupingByClassCategory::FObjectGroupingByClassCategory(TWeakPtr<SObjectTableTreeView> InTreeView)
	: FObjectGroupingByClass(InTreeView)
{
	ShortName = LOCTEXT("Grouping_ByClassCategory_ShortName", "Class Category");
	TitleName = LOCTEXT("Grouping_ByClassCategory_TitleName", "By Class Category");
	Description = LOCTEXT("Grouping_ByClassCategory_Desc", "Creates a tree based on registered class categories.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectGroupingByClassCategory::~FObjectGroupingByClassCategory()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FObjectGroupingByClassHierarchy
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FObjectGroupingByClassHierarchy)

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectGroupingByClassHierarchy::FObjectGroupingByClassHierarchy(TWeakPtr<SObjectTableTreeView> InTreeView)
	: FObjectGroupingByClass(InTreeView)
{
	ShortName = LOCTEXT("Grouping_ByClassHierarchy_ShortName", "Class Hierarchy");
	TitleName = LOCTEXT("Grouping_ByClassHierarchy_TitleName", "By Class Hierarchy");
	Description = LOCTEXT("Grouping_ByClassHierarchy_Desc", "Creates a tree based on object's class hierarchy.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectGroupingByClassHierarchy::~FObjectGroupingByClassHierarchy()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler

#undef LOCTEXT_NAMESPACE
