// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

namespace UE::Insights::ObjectProfiler
{

class FObjectNode;
class SObjectTableTreeView;

class FObjectGroupingByClass : public FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FObjectGroupingByClass, FTreeNodeGrouping)

public:
	FObjectGroupingByClass(TWeakPtr<SObjectTableTreeView> InTreeView);
	virtual ~FObjectGroupingByClass() override;

	virtual bool IsCategoryGrouping() const = 0;

	bool ShouldHideObjectBaseClass() const { return bShouldHideObjectBaseClass; }
	void SetShouldHideObjectBaseClass(bool bOnOff) { bShouldHideObjectBaseClass = bOnOff; }

	bool ShouldUpdateSegmentedBarGraph() const { return bShouldUpdateSegmentedBarGraph; }
	void SetShouldUpdateSegmentedBarGraph(bool bOnOff) { bShouldUpdateSegmentedBarGraph = bOnOff; }

	virtual void GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const override;

private:
	void UpdateSegmentedBarGraph(FTableTreeNode& ParentGroup) const;

private:
	TWeakPtr<SObjectTableTreeView> WeakObjectTableTreeView;
	bool bShouldHideObjectBaseClass = true;
	bool bShouldUpdateSegmentedBarGraph = true;
};

class FObjectGroupingByClassCategory : public FObjectGroupingByClass
{
	INSIGHTS_DECLARE_RTTI(FObjectGroupingByClassCategory, FObjectGroupingByClass)

public:
	FObjectGroupingByClassCategory(TWeakPtr<SObjectTableTreeView> InTreeView);
	virtual ~FObjectGroupingByClassCategory() override;

	virtual bool IsCategoryGrouping() const override { return true; }

private:
};

class FObjectGroupingByClassHierarchy : public FObjectGroupingByClass
{
	INSIGHTS_DECLARE_RTTI(FObjectGroupingByClassHierarchy, FObjectGroupingByClass)

public:
	FObjectGroupingByClassHierarchy(TWeakPtr<SObjectTableTreeView> InTreeView);
	virtual ~FObjectGroupingByClassHierarchy() override;

	virtual bool IsCategoryGrouping() const override { return false; }

private:
};

class FClassObjectGroupNode : public FCustomTableTreeNode
{
	friend class FObjectGroupingByClassCategory;

	INSIGHTS_DECLARE_RTTI(FClassObjectGroupNode, FCustomTableTreeNode)

public:
	FClassObjectGroupNode(TWeakPtr<FTable> InParentTable, const FText& InDisplayName, uint32 InClassId);
	virtual ~FClassObjectGroupNode() override;

	int64 GetSystemMemorySize() const { return SystemMemorySize; }
	int64 GetVideoMemorySize() const { return VideoMemorySize; }
	int64 GetEstimatedMemorySize() const { return SystemMemorySize + VideoMemorySize; }
	double GetEstimatedMemoryImpact() const { return Impact; }

	void UpdateEstimatedMemory(int64 TotalMemorySize, bool bUseTotalEstimatedMemory);

private:
	uint32 ClassId;
	int64 SystemMemorySize = 0;
	int64 VideoMemorySize = 0;
	double Impact = 0.0;
	bool bIsUpdated = false;
};

} // namespace UE::Insights::ObjectProfiler
