// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectGroupingByObjectName.h"

#include "Internationalization/Internationalization.h"

// TraceInsightsCore
#include "InsightsCore/Common/AsyncOperationProgress.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/ObjectProfiler/ViewModels/ObjectNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::ObjectProfiler::FObjectNode"

namespace UE::Insights::ObjectProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FObjectGroupingByObjectName)

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectGroupingByObjectName::FObjectGroupingByObjectName()
	: FTreeNodeGrouping(
		LOCTEXT("Grouping_ByObjectName_ShortName", "Name"),
		LOCTEXT("Grouping_ByObjectName_TitleName", "By Object Name"),
		LOCTEXT("Grouping_ByObjectName_Desc", "Group together objects with same name."),
		nullptr)
{
	SetColor(FLinearColor(1.0f, 0.45f, 0.6f, 1.0f));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectGroupingByObjectName::~FObjectGroupingByObjectName()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectGroupingByObjectName::GroupNodes(
	const TArray<FTableTreeNodePtr>& Nodes,
	FTableTreeNode& ParentGroup,
	TWeakPtr<FTable> InParentTable,
	IAsyncOperationProgress& InAsyncOperationProgress) const
{
	ParentGroup.ClearChildren();

	TArray<TSharedPtr<FObjectNode>> ObjectNodes;

	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			break;
		}

		if (!NodePtr->Is<FObjectNode>())
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		FObjectNode& ObjNode = NodePtr->As<FObjectNode>();
		if (ObjNode.GetObjectId() == FObjectNode::InvalidObjectId)
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		ObjectNodes.Add(StaticCastSharedPtr<FObjectNode>(NodePtr));
	}

	ObjectNodes.Sort([](const TSharedPtr<FObjectNode>& A, const TSharedPtr<FObjectNode>& B)
		{
			return FCString::Strcmp(A->GetObjectName(), B->GetObjectName()) < 0;
		});

	int32 ObjIndex = 0;
	while (ObjIndex < ObjectNodes.Num())
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			break;
		}

		TSharedPtr<FObjectNode>& ObjNode = ObjectNodes[ObjIndex];

		int32 ObjEndIndex = ObjIndex + 1;
		while (ObjEndIndex < ObjectNodes.Num() &&
			FCString::Strcmp(ObjNode->GetObjectName(), ObjectNodes[ObjEndIndex]->GetObjectName()) == 0)
		{
			if (InAsyncOperationProgress.ShouldCancelAsyncOp())
			{
				break;
			}
			++ObjEndIndex;
		}

		if (ObjEndIndex - ObjIndex > 1)
		{
			const FText DisplayName = ObjNode->GetDisplayName();
			auto ObjectGroupNode = MakeShared<FCustomTableTreeNode>(ObjNode->GetParentTable(), DisplayName);
			ObjectGroupNode->SetColor(FBaseTreeNode::GetDefaultIconColor(false));
			ObjectGroupNode->SetIconColor(FBaseTreeNode::GetDefaultIconColor(true));
			ObjectGroupNode->SetIcon(FInsightsStyle::GetBrush("Icons.UObject.Group.TreeItem"));

			for (int32 Index = ObjIndex; Index < ObjEndIndex; ++Index)
			{
				ObjectGroupNode->AddChildAndSetParent(ObjectNodes[Index]);
			}
			ParentGroup.AddChildAndSetParent(ObjectGroupNode);
		}
		else
		{
			ParentGroup.AddChildAndSetParent(ObjNode);
		}

		ObjIndex = ObjEndIndex;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler

#undef LOCTEXT_NAMESPACE
