// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

namespace UE::Insights::ObjectProfiler
{

class FObjectNode;

class FObjectGroupingByObjectName : public FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FObjectGroupingByObjectName, FTreeNodeGrouping)

public:
	FObjectGroupingByObjectName();
	virtual ~FObjectGroupingByObjectName() override;

	virtual void GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const override;
};

} // namespace UE::Insights::ObjectProfiler
