// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectGroupingByOuter.h"

#include "Containers/BitArray.h"
#include "Internationalization/Internationalization.h"

// TraceInsightsCore
#include "InsightsCore/Common/AsyncOperationProgress.h"

// TraceInsights
#include "Insights/ObjectProfiler/ViewModels/ObjectNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::ObjectProfiler::FObjectNode"

namespace UE::Insights::ObjectProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FObjectGroupingByOuter)

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectGroupingByOuter::FObjectGroupingByOuter()
	: FTreeNodeGrouping(
		LOCTEXT("Grouping_ByOuter_ShortName", "Outer"),
		LOCTEXT("Grouping_ByOuter_TitleName", "By Outer"),
		LOCTEXT("Grouping_ByOuter_Desc", "Creates a tree based on object hierarchy."),
		nullptr)
{
	SetColor(FLinearColor(1.0f, 0.45f, 0.6f, 1.0f));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectGroupingByOuter::~FObjectGroupingByOuter()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectGroupingByOuter::GroupNodes(
	const TArray<FTableTreeNodePtr>& Nodes,
	FTableTreeNode& ParentGroup,
	TWeakPtr<FTable> InParentTable,
	IAsyncOperationProgress& InAsyncOperationProgress) const
{
	ParentGroup.ClearChildren();
	ParentGroupPtr = &ParentGroup;

	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			break;
		}

		if (!NodePtr->Is<FObjectNode>())
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		FObjectNode& ObjNode = NodePtr->As<FObjectNode>();
		if (ObjNode.GetObjectId() == FObjectNode::InvalidObjectId)
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		TSharedPtr<FObjectNode> NewNode = AddObject(ObjNode);
		NewNode->ResetIsMissing();
	}

	ObjectNodes.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FObjectNode> FObjectGroupingByOuter::AddObject(FObjectNode& ObjNode) const
{
	const uint32 ObjId = ObjNode.GetObjectId();
	if (ObjId >= uint32(ObjectNodes.Num()))
	{
		ObjectNodes.AddDefaulted(ObjId - ObjectNodes.Num() + 1);
	}
	if (ObjectNodes[ObjId].IsValid())
	{
		return ObjectNodes[ObjId];
	}

	TSharedPtr<FObjectNode> NewNode = FObjectNode::Clone(ObjNode);
	NewNode->SetIsMissing();
	ObjectNodes[ObjId] = NewNode;

	if (bCreateChildrenSubGroups)
	{
		TSharedPtr<FObjectNode> Outer = ObjNode.GetOuter();
		if (!Outer.IsValid())
		{
			// Create the Object group node (root object)
			auto ObjectGroupNode = CreateObjectGroupNode(ObjNode);
			ObjectGroupNode->AddChildAndSetParent(NewNode);
			ParentGroupPtr->AddChildAndSetParent(ObjectGroupNode);
			return NewNode;
		}

		// Get or create the "Children" group node
		TSharedPtr<FObjectNode> OuterClone = AddObject(*Outer);
		auto OuterParentGroup = OuterClone->GetParent();
		check(OuterParentGroup.IsValid() && OuterParentGroup->Is<FCustomTableTreeNode>());
		if (OuterParentGroup->GetChildrenCount() == 1)
		{
			auto ChildrenGroupNode = MakeShared<FCustomTableTreeNode>(ObjNode.GetParentTable(), LOCTEXT("GroupName_Children", "Children"));
			OuterParentGroup->AddChildAndSetParent(ChildrenGroupNode);
		}
		check(OuterParentGroup->GetChildrenCount() > 1);
		auto ChildrenGroupNode = OuterParentGroup->GetChildren()[1];
		check(ChildrenGroupNode->Is<FCustomTableTreeNode>());

		// Create the Object group node (child object)
		auto ObjectGroupNode = CreateObjectGroupNode(ObjNode);
		ObjectGroupNode->AddChildAndSetParent(NewNode);
		ChildrenGroupNode->AddChildAndSetParent(ObjectGroupNode);
		return NewNode;
	}
	else
	{
		TSharedPtr<FObjectNode> Outer = ObjNode.GetOuter();
		if (!Outer.IsValid())
		{
			ParentGroupPtr->AddChildAndSetParent(NewNode);
			return NewNode;
		}

		TSharedPtr<FObjectNode> OuterClone = AddObject(*Outer);
		OuterClone->InitGroupData();
		OuterClone->AddChildAndSetParent(NewNode);
		return NewNode;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FCustomTableTreeNode> FObjectGroupingByOuter::CreateObjectGroupNode(FObjectNode& ObjNode) const
{
	const FText DisplayName = ObjNode.GetDisplayName();
	auto ObjectGroupNode = MakeShared<FCustomTableTreeNode>(ObjNode.GetParentTable(), DisplayName);
	ObjectGroupNode->SetColor(FLinearColor(0.9f, 0.9f, 1.0f, 1.0f));
	return ObjectGroupNode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler

#undef LOCTEXT_NAMESPACE
