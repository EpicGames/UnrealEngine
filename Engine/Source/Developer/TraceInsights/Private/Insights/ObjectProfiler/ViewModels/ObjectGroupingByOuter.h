// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

namespace UE::Insights::ObjectProfiler
{

class FObjectNode;

class FObjectGroupingByOuter : public FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FObjectGroupingByOuter, FTreeNodeGrouping)

public:
	FObjectGroupingByOuter();
	virtual ~FObjectGroupingByOuter() override;

	virtual void GroupNodes(const TArray<FTableTreeNodePtr>& Nodes, FTableTreeNode& ParentGroup, TWeakPtr<FTable> InParentTable, IAsyncOperationProgress& InAsyncOperationProgress) const override;

private:
	TSharedPtr<FObjectNode> AddObject(FObjectNode& ObjNode) const;
	TSharedPtr<FCustomTableTreeNode> CreateObjectGroupNode(FObjectNode& ObjNode) const;

	bool bCreateChildrenSubGroups = false;
	mutable FTableTreeNode* ParentGroupPtr = nullptr;
	mutable TArray<TSharedPtr<FObjectNode>> ObjectNodes;
};

} // namespace UE::Insights::ObjectProfiler
