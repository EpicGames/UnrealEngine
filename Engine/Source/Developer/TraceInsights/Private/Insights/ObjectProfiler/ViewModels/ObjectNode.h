// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "CoreTypes.h"

// TraceServices
#include "TraceServices/Model/ObjectProvider.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

// TraceInsights
#include "Insights/ObjectProfiler/IAssetInfoProvider.h"
#include "Insights/ObjectProfiler/Common/ObjectToActorResolver.h"

namespace TraceServices
{
	struct FObjectInfo;
}

namespace UE::Insights::ObjectProfiler
{

class FObjectTable;
class SObjectTableTreeView;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class used to store information about a UObject node (used in the SObjectTableTreeView).
 */
class FObjectNode : public FTableTreeNode
{
	friend class SObjectTableTreeView;
	friend class FObjectGroupingByOuter;

	INSIGHTS_DECLARE_RTTI(FObjectNode, FTableTreeNode)

private:
	enum class EInternalFlags
	{
		None = 0,
		IsMissing = 1 << 0, // missing intermediate node in the Outer hierarchy
	};

public:
	/** Initialization constructor for the UObject node. */
	explicit FObjectNode(TWeakPtr<FObjectTable> InParentTable, const TraceServices::FObjectInfo& ObjectInfo);

	explicit FObjectNode(const FObjectNode& Clone);

	/** Initialization constructor for the group node. */
	explicit FObjectNode(TWeakPtr<FObjectTable> InParentTable);

	virtual const FText GetDisplayName() const override;
	virtual const FText GetExtraDisplayName() const override;
	virtual bool HasExtraDisplayName() const override;
	virtual const FText GetTooltipText() const override;

	virtual const FSlateBrush* GetIcon() const override;
	virtual FLinearColor GetIconColor() const override;
	virtual FLinearColor GetColor() const override;

	uint32 GetObjectId() const { return ObjectId; }

	uint32 GetClassId() const { return ClassId; }
	uint32 GetOuterId() const { return OuterId; }

	uint32 GetObjectFlags() const { return ObjectFlags; }

	TSharedPtr<FObjectNode> GetClass() const { return Class.Pin(); }
	TSharedPtr<FObjectNode> GetOuter() const { return Outer.Pin(); }

	FName GetClassName() const { return ClassName; }

	TSharedPtr<FObjectNode> GetPackage() const;

	const TCHAR* GetObjectName() const { return ObjectName; }

	void InitObjectPath();
	void ConvertObjectPath(const TSharedPtr<IAssetInfoProvider>& Provider);
	const FString& GetObjectPath() const { return ObjectPath; }

	const TCHAR* GetVersePath() const { return VersePath; }

	virtual int32 GetStructureSize() const { return 0; }

	//////////////////////////////////////////////////
	// Exclusive Estimated Memory Size
	// (excluding sub-objects)

	int64 GetSystemMemorySize() const { return SystemMemorySize; }
	int64 GetVideoMemorySize() const { return VideoMemorySize; }

	static int64 GetEstimatedMemorySize(const FBaseTreeNode& Node);
	int64 GetEstimatedMemorySize() const { return SystemMemorySize + VideoMemorySize; }

	static TOptional<double> GetEstimatedMemoryImpact(const FBaseTreeNode& Node);
	double GetEstimatedMemoryImpact() const;

	//////////////////////////////////////////////////
	// Total Estimated Memory Size
	// (including sub-objects)

	int64 GetTotalSystemMemorySize() const { return TotalSystemMemorySize; }
	int64 GetTotalVideoMemorySize() const { return TotalVideoMemorySize; }

	static int64 GetTotalEstimatedMemorySize(const FBaseTreeNode& Node);
	int64 GetTotalEstimatedMemorySize() const { return TotalSystemMemorySize + TotalVideoMemorySize; }

	static TOptional<double> GetTotalEstimatedMemoryImpact(const FBaseTreeNode& Node);
	double GetTotalEstimatedMemoryImpact() const;

	//////////////////////////////////////////////////

	static uint32 GetNumReferences(const FBaseTreeNode& Node);
	virtual uint32 GetNumReferences() const { return MatchedActors.IsValid() ? MatchedActors->Num() : 0; }

	bool IsMissing() const { return (uint32(InternalFlags) & uint32(EInternalFlags::IsMissing)) != 0; }

	virtual bool IsField() const { return false; }
	virtual bool IsStruct() const { return false; }
	virtual bool IsClass() const { return false; }
	virtual bool IsFunction() const { return false; }
	virtual bool IsPackage() const { return false; }

	FAssetData MakeAssetData();

	void SetMatchedAsset(FAssetData&& InMatchedAsset) { MatchedAsset = MoveTemp(InMatchedAsset); }
	const FAssetData& GetMatchedAsset() const { return MatchedAsset; }

	void SetMatchedActors(const TSharedRef<FActorSet>& InMatchedActors) { MatchedActors = InMatchedActors; }
	const FActorSet* GetMatchedActors() const { return MatchedActors.Get(); }

	bool IsIdentityMasked() const { return bIdentityMasked; }
	void SetIdentityMasked(bool bInIdentityMasked) { bIdentityMasked = bInIdentityMasked; }

	bool IsOwnedByCurrentProject() const { return bIsOwnedByCurrentProject; }
	void SetOwnedByCurrentProject(bool bInOwnedByCurrentProject) { bIsOwnedByCurrentProject = bInOwnedByCurrentProject; }

	virtual const TCHAR* GetSourcePackageName() const { return nullptr; }

	static TSharedRef<FObjectNode> Clone(FObjectNode& InNode);

protected:
	void SetIsMissing()
	{
		InternalFlags = EInternalFlags(uint32(InternalFlags) | uint32(EInternalFlags::IsMissing));
	}

	void ResetIsMissing()
	{
		InternalFlags = EInternalFlags(uint32(InternalFlags) & ~uint32(EInternalFlags::IsMissing));
	}

	void SetClass(TWeakPtr<FObjectNode> InClass)
	{
		Class = InClass;
		TSharedPtr<FObjectNode> ClassPtr = InClass.Pin();
		ClassName = ClassPtr ? FName(ClassPtr->GetObjectName()) : NAME_None;
	}

	void SetOuter(TWeakPtr<FObjectNode> InOuter)
	{
		Outer = InOuter;
	}

public:
	static constexpr uint32 InvalidObjectId = uint32(-1);

private:
	uint32 ObjectId = InvalidObjectId;

	uint32 ClassId = InvalidObjectId;
	uint32 OuterId = InvalidObjectId;

	uint32 ObjectFlags = 0;

	FName ClassName;
	TWeakPtr<FObjectNode> Class;
	TWeakPtr<FObjectNode> Outer;

	const TCHAR* ObjectName = nullptr; // persistent string (session lifetime)
	const TCHAR* VersePath = nullptr; // persistent string (session lifetime)
	FString ObjectPath;

	int64 SystemMemorySize = 0;
	int64 VideoMemorySize = 0;
	int64 TotalSystemMemorySize = 0;
	int64 TotalVideoMemorySize = 0;

	EInternalFlags InternalFlags = EInternalFlags::None;

	FAssetData MatchedAsset;
	TSharedPtr<FActorSet> MatchedActors;

	bool bIdentityMasked = false;
	bool bIsOwnedByCurrentProject = false;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFieldObjectNode : public FObjectNode
{
	friend class SObjectTableTreeView;

	INSIGHTS_DECLARE_RTTI(FFieldObjectNode, FObjectNode)

public:
	/** Initialization constructor for the UField node. */
	explicit FFieldObjectNode(TWeakPtr<FObjectTable> InParentTable, const TraceServices::FObjectInfo& ObjectInfo);

	explicit FFieldObjectNode(const FFieldObjectNode& Clone);

	virtual const FSlateBrush* GetIcon() const override;
	virtual FLinearColor GetIconColor() const override;

	virtual bool IsField() const override { return true; }
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FStructObjectNode : public FFieldObjectNode
{
	friend class SObjectTableTreeView;

	INSIGHTS_DECLARE_RTTI(FStructObjectNode, FFieldObjectNode)

public:
	/** Initialization constructor for the UStruct node. */
	explicit FStructObjectNode(TWeakPtr<FObjectTable> InParentTable, const TraceServices::FObjectInfo& ObjectInfo);

	explicit FStructObjectNode(const FStructObjectNode& Clone);

	virtual const FSlateBrush* GetIcon() const override;
	virtual FLinearColor GetIconColor() const override;

	virtual bool IsStruct() const override { return true; }

	uint32 GetSuperId() const { return SuperId; }
	uint32 GetInheritanceSuperId() const { return InheritanceSuperId; }
	virtual int32 GetStructureSize() const override { return StructureSize; }

private:
	uint32 SuperId = InvalidObjectId;
	uint32 InheritanceSuperId = InvalidObjectId;
	int32 StructureSize = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FClassObjectNode : public FStructObjectNode
{
	friend class SObjectTableTreeView;

	INSIGHTS_DECLARE_RTTI(FClassObjectNode, FStructObjectNode)

	virtual const FSlateBrush* GetIcon() const override;
	virtual FLinearColor GetIconColor() const override;

	virtual bool IsClass() const override { return true; }

public:
	/** Initialization constructor for the UClass node. */
	explicit FClassObjectNode(TWeakPtr<FObjectTable> InParentTable, const TraceServices::FObjectInfo& ObjectInfo);

	explicit FClassObjectNode(const FClassObjectNode& Clone);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFunctionObjectNode : public FStructObjectNode
{
	friend class SObjectTableTreeView;

	INSIGHTS_DECLARE_RTTI(FFunctionObjectNode, FStructObjectNode)

	virtual const FSlateBrush* GetIcon() const override;
	virtual FLinearColor GetIconColor() const override;

	virtual bool IsFunction() const override { return true; }

	uint32 GetFunctionFlags() const { return FunctionFlags; }
	uint32 GetFunctionNumParms() const { return uint32(FunctionNumParms); }
	uint32 GetFunctionParmsSize() const { return uint32(FunctionParmsSize); }

public:
	/** Initialization constructor for the UFunction node. */
	explicit FFunctionObjectNode(TWeakPtr<FObjectTable> InParentTable, const TraceServices::FObjectInfo& ObjectInfo);

	explicit FFunctionObjectNode(const FFunctionObjectNode& Clone);

private:
	uint32 FunctionFlags = 0;
	uint8 FunctionNumParms = 0;
	uint16 FunctionParmsSize = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FPackageObjectNode : public FObjectNode
{
	friend class SObjectTableTreeView;

	INSIGHTS_DECLARE_RTTI(FPackageObjectNode, FObjectNode)

public:
	/** Initialization constructor for the UPackage node. */
	explicit FPackageObjectNode(TWeakPtr<FObjectTable> InParentTable, const TraceServices::FObjectInfo& ObjectInfo);

	explicit FPackageObjectNode(const FPackageObjectNode& Clone);

	virtual const FSlateBrush* GetIcon() const override;
	virtual FLinearColor GetIconColor() const override;

	virtual bool IsPackage() const override { return true; }

	uint64 GetPackageId() const { return PackageId; }
	const TCHAR* GetPath() const { return Path; }
	const TCHAR* GetSourcePackageName() const override { return SourcePackageName; }

private:
	uint64 PackageId = 0;
	const TCHAR* Path = nullptr; // persistent string (session lifetime)
	const TCHAR* SourcePackageName = nullptr; // editor source package name when it differs from the runtime name; persistent string (session lifetime)
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler
