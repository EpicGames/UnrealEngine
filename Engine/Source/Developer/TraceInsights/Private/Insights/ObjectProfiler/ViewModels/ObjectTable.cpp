// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTable.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/ObjectProfiler/ViewModels/ObjectNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::ObjectProfiler::FObjectTable"

namespace UE::Insights::ObjectProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Column identifiers

const FName FObjectTableColumns::ObjectCountColumnId("ObjectCount");
const FName FObjectTableColumns::ObjectNameColumnId("ObjectName");
const FName FObjectTableColumns::ObjectIdColumnId("ObjectId");
const FName FObjectTableColumns::ClassNameColumnId("ClassName");
const FName FObjectTableColumns::ClassIdColumnId("ClassId");
const FName FObjectTableColumns::OuterNameColumnId("OuterName");
const FName FObjectTableColumns::OuterIdColumnId("OuterId");
const FName FObjectTableColumns::PackageNameColumnId("PackageName");
const FName FObjectTableColumns::PackageIdColumnId("PackageId");

const FName FObjectTableColumns::PackageUniqueIdColumnId("PackageUniqueId");
const FName FObjectTableColumns::PackagePathColumnId("PackagePath");
const FName FObjectTableColumns::SourcePackageNameColumnId("SourcePackageName");

const FName FObjectTableColumns::ObjectFlagsColumnId("ObjectFlags");

const FName FObjectTableColumns::ObjectPathColumnId("ObjectPath");
const FName FObjectTableColumns::VersePathColumnId("VersePath");

const FName FObjectTableColumns::SuperIdColumnId("SuperId");
const FName FObjectTableColumns::InheritanceSuperIdColumnId("InheritanceSuperId");
const FName FObjectTableColumns::StructureSizeColumnId("StructureSize");

const FName FObjectTableColumns::SystemMemorySizeColumnId("SystemMemorySize");
const FName FObjectTableColumns::VideoMemorySizeColumnId("VideoMemorySize");
const FName FObjectTableColumns::EstimatedSizeColumnId("EstimatedSize");
const FName FObjectTableColumns::ImpactColumnId("Impact");

const FName FObjectTableColumns::TotalSystemMemorySizeColumnId("TotalSystemMemorySize");
const FName FObjectTableColumns::TotalVideoMemorySizeColumnId("TotalVideoMemorySize");
const FName FObjectTableColumns::TotalEstimatedSizeColumnId("TotalEstimatedSize");
const FName FObjectTableColumns::TotalImpactColumnId("TotalImpact");

const FName FObjectTableColumns::ReferencesColumnId("References");

////////////////////////////////////////////////////////////////////////////////////////////////////
// FObjectTable
////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectTable::FObjectTable(bool bInIsSimplifiedMode, bool bInCanShowReferencingActors)
	: bIsSimplifiedMode(bInIsSimplifiedMode)
	, bCanShowReferencingActors(bInCanShowReferencingActors)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FObjectTable::~FObjectTable()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectTable::Reset()
{
	FTable::Reset();

	AddDefaultColumns();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FObjectTable::AddDefaultColumns()
{
	class FAggregatedValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const
		{
			return TOptional<FTableCellValue>();
		}

		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			if (Node.Is<FObjectNode>())
			{
				return GetNodeValue(Node);
			}
			if (Node.IsGroup() && Node.Is<FTableTreeNode>())
			{
				const FTableTreeNode& TableTreeNode = Node.As<FTableTreeNode>();
				if (TableTreeNode.HasAggregatedValue(Column.GetId()))
				{
					return TableTreeNode.GetAggregatedValue(Column.GetId());
				}
			}
			return GetNodeValue(Node);
		}
	};

	//////////////////////////////////////////////////
	// Hierarchy Column

	{
		const int32 HierarchyColumnIndex = -1;
		const TCHAR* HierarchyColumnName = nullptr;
		AddHierarchyColumn(HierarchyColumnIndex, HierarchyColumnName);

		FTableColumn& HierarchyColumn = *GetColumns()[0];
		HierarchyColumn.SetInitialWidth(200.0f);
		HierarchyColumn.SetShortName(LOCTEXT("HierarchyColumnName", "Hierarchy"));
		HierarchyColumn.SetTitleName(LOCTEXT("HierarchyColumnTitle", "UObject Hierarchy"));
		HierarchyColumn.SetDescription(LOCTEXT("HierarchyColumnDesc", "Hierarchy of the UObject's tree"));
	}

	int32 ColumnIndex = 0;

	//////////////////////////////////////////////////
	// Object Count Column

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::ObjectCountColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("ObjectCountColumnName", "Count"));
		Column.SetTitleName(LOCTEXT("ObjectCountColumnTitle", "Object Count"));
		Column.SetDescription(LOCTEXT("ObjectCountColumnDesc", "The number of the UObject instances"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(60.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.IsGroup() && Node.Is<FTableTreeNode>())
				{
					const FTableTreeNode& TableTreeNode = Node.As<FTableTreeNode>();
					if (TableTreeNode.HasAggregatedValue(FObjectTableColumns::ObjectCountColumnId))
					{
						return TableTreeNode.GetAggregatedValue(FObjectTableColumns::ObjectCountColumnId);
					}
				}
				if (Node.Is<FObjectNode>())
				{
					return FTableCellValue(static_cast<int64>(1));
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Object Name Column

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::ObjectNameColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("ObjectNameColumnName", "Name"));
		Column.SetTitleName(LOCTEXT("ObjectNameColumnTitle", "Object Name"));
		Column.SetDescription(LOCTEXT("ObjectNameColumnDesc", "The name of the UObject instance"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(120.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					return FTableCellValue(ObjectNode.GetObjectName());
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);

		// Set the same "by name" sorter also for the hierarchy column.
		//FTableColumn& HierarchyColumn = *GetColumns()[0];
		//HierarchyColumn.SetValueSorter(Sorter);
	}
	//////////////////////////////////////////////////
	// Object Id Column

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::ObjectIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("ObjectIdColumnName", "Object Id"));
		Column.SetTitleName(LOCTEXT("ObjectIdColumnTitle", "Object Id"));
		Column.SetDescription(LOCTEXT("ObjectIdColumnDesc", "The id of the UObject instance"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
						ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(60.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					const uint32 Id = ObjectNode.GetObjectId();
					if (Id != uint32(-1))
					{
						return FTableCellValue(int64(Id));
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::None);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Class Name Column

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::ClassNameColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("ClassNameColumnName", "Class"));
		Column.SetTitleName(LOCTEXT("ClassNameColumnTitle", "Class Name"));
		Column.SetDescription(LOCTEXT("ClassNameColumnDesc", "The name of the UClass of the current UObject instance"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
						ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(120.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					TSharedPtr<FObjectNode> ClassNode = ObjectNode.GetClass();
					if (ClassNode.IsValid())
					{
						return FTableCellValue(ClassNode->GetObjectName());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Class Id Column

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::ClassIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("ClassIdColumnName", "Class Id"));
		Column.SetTitleName(LOCTEXT("ClassIdColumnTitle", "Class Id"));
		Column.SetDescription(LOCTEXT("ClassIdColumnDesc", "The id of the UClass of the current UObject instance"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(60.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					const uint32 Id = ObjectNode.GetClassId();
					if (Id != uint32(-1))
					{
						return FTableCellValue(int64(Id));
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Outer Name Column

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::OuterNameColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("OuterNameColumnName", "Outer"));
		Column.SetTitleName(LOCTEXT("OuterNameColumnTitle", "Outer Object Name"));
		Column.SetDescription(LOCTEXT("OuterNameColumnDesc", "The name of the outer UObject of the current UObject instance"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
						ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(120.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					TSharedPtr<FObjectNode> OuterNode = ObjectNode.GetOuter();
					if (OuterNode.IsValid())
					{
						return FTableCellValue(OuterNode->GetObjectName());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Outer Id Column

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::OuterIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("OuterIdColumnName", "Outer Id"));
		Column.SetTitleName(LOCTEXT("OuterIdColumnTitle", "Outer Id"));
		Column.SetDescription(LOCTEXT("OuterIdColumnDesc", "The id of the outer UObject of current UObject instance"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(60.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					const uint32 Id = ObjectNode.GetOuterId();
					if (Id != uint32(-1))
					{
						return FTableCellValue(int64(Id));
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Package Name Column

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::PackageNameColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("PackageNameColumnName", "Package"));
		Column.SetTitleName(LOCTEXT("PackageNameColumnTitle", "Package Name"));
		Column.SetDescription(LOCTEXT("PackageNameColumnDesc", "The name of the UPackage of the current UObject instance"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
						ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(120.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					TSharedPtr<FObjectNode> PackageNode = ObjectNode.GetPackage();
					if (PackageNode.IsValid())
					{
						return FTableCellValue(PackageNode->GetObjectName());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Outer Package Object Id Column

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::PackageIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("PackageIdColumnName", "Package Id"));
		Column.SetTitleName(LOCTEXT("PackageIdColumnTitle", "Package Object Id"));
		Column.SetDescription(LOCTEXT("PackageIdColumnDesc", "The object id of the outer package of the current object instance"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(60.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					TSharedPtr<FObjectNode> Package = ObjectNode.GetPackage();
					if (Package.IsValid() && Package->GetObjectId() != uint32(-1))
					{
						return FTableCellValue(int64(Package->GetObjectId()));
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Package Unique Id Column (UPackage)

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::PackageUniqueIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("PackageUniqueIdColumnName", "Package UId"));
		Column.SetTitleName(LOCTEXT("PackageUniqueIdColumnTitle", "Package Unique Id"));
		Column.SetDescription(LOCTEXT("PackageUniqueIdColumnDesc", "The unique id of a UPackage object"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(60.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FPackageObjectNode>())
				{
					const FPackageObjectNode& PackageNode = Node.As<FPackageObjectNode>();
					const uint64 PackageUniqueId = PackageNode.GetPackageId();
					return FTableCellValue(int64(PackageUniqueId));
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		class FValueFormatter : public FTableCellValueFormatter
		{
		public:
			virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override
			{
				return InValue.IsSet() ?
					FText::FromString(FString::Printf(TEXT("0x%llX"), static_cast<uint64>(InValue.GetValue().Int64))) :
					FText::GetEmpty();
			}
		};
		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FValueFormatter>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::None);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Package Path Column (UPackage)

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::PackagePathColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("PackagePathColumnName", "Package Path"));
		Column.SetTitleName(LOCTEXT("PackagePathColumnTitle", "Package Path"));
		Column.SetDescription(LOCTEXT("PackagePathColumnDesc", "The path of a UPackage object"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(120.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FPackageObjectNode>())
				{
					const FPackageObjectNode& PackageNode = Node.As<FPackageObjectNode>();
					if (PackageNode.GetPath())
					{
						return FTableCellValue(PackageNode.GetPath());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Source Package Name Column
	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::SourcePackageNameColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SourcePackageNameColumnName", "Source Pkg"));
		Column.SetTitleName(LOCTEXT("SourcePackageNameColumnTitle", "Source Package Name"));
		Column.SetDescription(LOCTEXT("SourcePackageNameColumnDesc", "The editor source package name; differs from the object path for streamed/instanced level packages"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(200.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FPackageObjectNode>())
				{
					const FPackageObjectNode& PackageNode = Node.As<FPackageObjectNode>();
					if (PackageNode.IsIdentityMasked())
					{
						return FTableCellValue(TEXT(""));
					}
					if (PackageNode.GetSourcePackageName())
					{
						return FTableCellValue(PackageNode.GetSourcePackageName());
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Object Flags Column

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::ObjectFlagsColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("ObjectFlagsColumnName", "Flags"));
		Column.SetTitleName(LOCTEXT("ObjectFlagsColumnTitle", "Flags"));
		Column.SetDescription(LOCTEXT("ObjectFlagsColumnDesc", "The flags of the UObject instance"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
						ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(140.0f);

		Column.SetDataType(ETableCellDataType::Text);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					FString FlagsStr = LexToString(EObjectFlags(ObjectNode.GetObjectFlags()));
					return FTableCellValue(FText::FromString(FlagsStr));
				}
				return TOptional<FTableCellValue>();
			}
			virtual uint64 GetValueId(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					return uint64(ObjectNode.GetObjectFlags());
				}
				return 0;
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByTextValueWithId>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Object Path Column

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::ObjectPathColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("ObjectPathColumnName", "Path"));
		Column.SetTitleName(LOCTEXT("ObjectPathColumnTitle", "Object Path"));
		Column.SetDescription(LOCTEXT("ObjectPathColumnDesc", "The path of the UObject instance"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(200.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					return FTableCellValue(*ObjectNode.GetObjectPath());
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Verse Path Column

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::VersePathColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("VersePathColumnName", "Verse Path"));
		Column.SetTitleName(LOCTEXT("VersePathColumnTitle", "Verse Path"));
		Column.SetDescription(LOCTEXT("VersePathColumnDesc", "The Verse path of the UObject instance"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(200.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					return FTableCellValue(ObjectNode.GetVersePath());
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Super Id Column (UStruct and UClass)

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::SuperIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SuperIdColumnName", "Super Id"));
		Column.SetTitleName(LOCTEXT("SuperIdColumnTitle", "Super Id"));
		Column.SetDescription(LOCTEXT("SuperIdColumnDesc", "The id of the super class/struct"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(60.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FStructObjectNode>())
				{
					const FStructObjectNode& StructNode = Node.As<FStructObjectNode>();
					const uint32 Id = StructNode.GetSuperId();
					if (Id != uint32(-1))
					{
						return FTableCellValue(int64(Id));
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Inheritance Super Id Column (UStruct and UClass)

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::InheritanceSuperIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("InheritanceSuperIdColumnName", "Inheritance"));
		Column.SetTitleName(LOCTEXT("InheritanceSuperIdColumnTitle", "Inheritance Super Id"));
		Column.SetDescription(LOCTEXT("InheritanceSuperIdColumnDesc", "The id of the super class/struct (inheritance)"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(60.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FStructObjectNode>())
				{
					const FStructObjectNode& StructNode = Node.As<FStructObjectNode>();
					const uint32 Id = StructNode.GetInheritanceSuperId();
					if (Id != uint32(-1))
					{
						return FTableCellValue(int64(Id));
					}
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Structure Size Column

	if (!bIsSimplifiedMode)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::StructureSizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("StructureSizeColumnName", "Structure Size"));
		Column.SetTitleName(LOCTEXT("StructureSizeColumnTitle", "Structure Size"));
		Column.SetDescription(LOCTEXT("StructureSizeColumnDesc", "The estimated structure size (bytes) of the UStruct instance"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
						ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FStructObjectNode>())
				{
					const FStructObjectNode& StructNode = Node.As<FStructObjectNode>();
					return FTableCellValue(int64(StructNode.GetStructureSize()));
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<FSorterByInt64Value> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Sorter->SetShouldSortGroupsFirst(false);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// System Memory Size Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::SystemMemorySizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SystemMemorySizeColumnName", "System Mem"));
		Column.SetTitleName(LOCTEXT("SystemMemorySizeColumnTitle", "System Memory Size"));
		Column.SetDescription(LOCTEXT("SystemMemorySizeColumnDesc", "The estimated system memory size (bytes) of the UObject instance"));

		if (bIsSimplifiedMode)
		{
			Column.SetFlags(ETableColumnFlags::CanBeHidden |
							ETableColumnFlags::CanBeFiltered);
		}
		else
		{
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
							ETableColumnFlags::CanBeHidden |
							ETableColumnFlags::CanBeFiltered);
		}

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					return FTableCellValue(int64(ObjectNode.GetSystemMemorySize()));
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<FSorterByInt64Value> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Sorter->SetShouldSortGroupsFirst(false);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Video Memory Size Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::VideoMemorySizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("VideoMemorySizeColumnName", "Video Mem"));
		Column.SetTitleName(LOCTEXT("VideoMemorySizeColumnTitle", "Video Memory Size"));
		Column.SetDescription(LOCTEXT("VideoMemorySizeColumnDesc", "The estimated video memory size (bytes) of the UObject instance"));

		if (bIsSimplifiedMode)
		{
			Column.SetFlags(ETableColumnFlags::CanBeHidden |
							ETableColumnFlags::CanBeFiltered);
		}
		else
		{
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
							ETableColumnFlags::CanBeHidden |
							ETableColumnFlags::CanBeFiltered);
		}

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					return FTableCellValue(int64(ObjectNode.GetVideoMemorySize()));
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<FSorterByInt64Value> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Sorter->SetShouldSortGroupsFirst(false);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Estimated Size Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::EstimatedSizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("EstimatedSizeColumnName", "Estimated Size"));
		Column.SetTitleName(LOCTEXT("EstimatedSizeColumnTitle", "Estimated Size (MiB)"));
		Column.SetDescription(LOCTEXT("EstimatedSizeColumnDesc", "The estimated memory size (system memory + video memory), in MiB"));

		if (bIsSimplifiedMode)
		{
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
							ETableColumnFlags::CanBeHidden |
							ETableColumnFlags::CanBeFiltered |
							ETableColumnFlags::IsDynamic);
		}
		else
		{
			Column.SetFlags(ETableColumnFlags::CanBeHidden |
							ETableColumnFlags::CanBeFiltered |
							ETableColumnFlags::IsDynamic);
		}

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					return FTableCellValue(static_cast<int64>(ObjectNode.GetEstimatedMemorySize()));
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		class FEstimatedSizeFormatter : public FTableCellValueFormatter
		{
		public:
			virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override
			{
				if (InValue.IsSet())
				{
					const int64 Size = InValue.GetValue().Int64;
					if (Size > 0)
					{
						int64 SizeMB = Size / (1024 * 1024);
						if (SizeMB == 0)
						{
							return LOCTEXT("LessThan1MB", "< 1 MiB");
						}
						return FText::Format(LOCTEXT("EstimatedSizeFmt", "{0} MiB"), FText::AsNumber(SizeMB));
					}
				}
				return FText::GetEmpty();
			}
			virtual FText FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const override
			{
				if (InValue.IsSet())
				{
					const int64 Size = InValue.GetValue().Int64;
					if (Size == 0)
					{
						return FText::GetEmpty(); // LOCTEXT("ZeroMB", "0");
					}
					FNumberFormattingOptions Options;
					Options.MaximumFractionalDigits = 2;
					return FText::Format(LOCTEXT("EstimatedSizeTooltipFmt", "{0} ({1} bytes)"),
						FText::AsMemory(Size, &Options),
						FText::AsNumber(Size));
				}
				return FText::GetEmpty();
			}
		};

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FEstimatedSizeFormatter>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<FSorterByInt64Value> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Sorter->SetShouldSortGroupsFirst(false);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Impact Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::ImpactColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("ImpactColumnName", "Impact"));
		Column.SetTitleName(LOCTEXT("ImpactColumnTitle", "Impact (%)"));
		Column.SetDescription(LOCTEXT("ImpactColumnDesc", "The impact (percentage) of estimated memory size"));

		if (bIsSimplifiedMode)
		{
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
							ETableColumnFlags::CanBeHidden |
							ETableColumnFlags::CanBeFiltered |
							ETableColumnFlags::IsDynamic);
		}
		else
		{
			Column.SetFlags(ETableColumnFlags::CanBeHidden |
							ETableColumnFlags::CanBeFiltered |
							ETableColumnFlags::IsDynamic);
		}

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(60.0f);

		Column.SetDataType(ETableCellDataType::Double);

		class FValueGetter : public FTableCellValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				TOptional<double> Impact = FObjectNode::GetEstimatedMemoryImpact(Node);
				if (Impact.IsSet())
				{
					return FTableCellValue(static_cast<double>(Impact.GetValue()));
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		class FImpactFormatter : public FTableCellValueFormatter
		{
		public:
			virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override
			{
				if (InValue.IsSet())
				{
					const double Ratio = InValue.GetValue().Double;
					int64 Percent = FMath::RoundToInt(Ratio * 100.0);
					return FText::Format(LOCTEXT("ImpactFmt", "{0}%"), FText::AsNumber(Percent));
				}
				return FText::GetEmpty();
			}
			virtual FText FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const override
			{
				if (InValue.IsSet())
				{
					const double Ratio = InValue.GetValue().Double;
					FNumberFormattingOptions Options;
					Options.MaximumFractionalDigits = 2;
					return FText::Format(LOCTEXT("ImpactTooltipFmt", "{0}"), FText::AsPercent(Ratio, &Options));
				}
				return FText::GetEmpty();
			}
		};
		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FImpactFormatter>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<FSorterByDoubleValue> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Sorter->SetShouldSortGroupsFirst(false);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		Column.SetAggregation(ETableColumnAggregation::None);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Total System Memory Size Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::TotalSystemMemorySizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TotalSystemMemorySizeColumnName", "Total System Mem"));
		Column.SetTitleName(LOCTEXT("TotalSystemMemorySizeColumnTitle", "Total System Memory Size"));
		Column.SetDescription(LOCTEXT("TotalSystemMemorySizeColumnDesc", "The total estimated system memory size (bytes) of the UObject instance (including sub-objects)"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					return FTableCellValue(int64(ObjectNode.GetTotalSystemMemorySize()));
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<FSorterByInt64Value> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Sorter->SetShouldSortGroupsFirst(false);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Total Video Memory Size Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::TotalVideoMemorySizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TotalVideoMemorySizeColumnName", "Total Video Mem"));
		Column.SetTitleName(LOCTEXT("TotalVideoMemorySizeColumnTitle", "Total Video Memory Size"));
		Column.SetDescription(LOCTEXT("TotalVideoMemorySizeColumnDesc", "The total estimated video memory size (bytes) of the UObject instance (including references)"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden |
						ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					return FTableCellValue(int64(ObjectNode.GetTotalVideoMemorySize()));
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<FSorterByInt64Value> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Sorter->SetShouldSortGroupsFirst(false);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Total Estimated Size Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::TotalEstimatedSizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TotalEstimatedSizeColumnName", "Total Estimated Size"));
		Column.SetTitleName(LOCTEXT("TotalEstimatedSizeColumnTitle", "Total Estimated Size (MiB)"));
		Column.SetDescription(LOCTEXT("TotalEstimatedSizeColumnDesc", "The total estimated memory size (system memory + video memory, including sub-objects), in MiB"));

		if (bIsSimplifiedMode)
		{
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
							ETableColumnFlags::CanBeHidden |
							ETableColumnFlags::CanBeFiltered |
							ETableColumnFlags::IsDynamic);
		}
		else
		{
			Column.SetFlags(ETableColumnFlags::CanBeHidden |
							ETableColumnFlags::CanBeFiltered |
							ETableColumnFlags::IsDynamic);
		}

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					return FTableCellValue(static_cast<int64>(ObjectNode.GetTotalEstimatedMemorySize()));
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		class FEstimatedSizeFormatter : public FTableCellValueFormatter
		{
		public:
			virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override
			{
				if (InValue.IsSet())
				{
					const int64 Size = InValue.GetValue().Int64;
					if (Size > 0)
					{
						int64 SizeMB = Size / (1024 * 1024);
						if (SizeMB == 0)
						{
							return LOCTEXT("LessThan1MB", "< 1 MiB");
						}
						return FText::Format(LOCTEXT("EstimatedSizeFmt", "{0} MiB"), FText::AsNumber(SizeMB));
					}
				}
				return FText::GetEmpty();
			}
			virtual FText FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const override
			{
				if (InValue.IsSet())
				{
					const int64 Size = InValue.GetValue().Int64;
					if (Size == 0)
					{
						return FText::GetEmpty(); // LOCTEXT("ZeroMB", "0");
					}
					FNumberFormattingOptions Options;
					Options.MaximumFractionalDigits = 2;
					return FText::Format(LOCTEXT("EstimatedSizeTooltipFmt", "{0} ({1} bytes)"),
						FText::AsMemory(Size, &Options),
						FText::AsNumber(Size));
				}
				return FText::GetEmpty();
			}
		};

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FEstimatedSizeFormatter>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<FSorterByInt64Value> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Sorter->SetShouldSortGroupsFirst(false);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Total Impact Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::TotalImpactColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TotalImpactColumnName", "Total Impact"));
		Column.SetTitleName(LOCTEXT("TotalImpactColumnTitle", "Total Impact (%)"));
		Column.SetDescription(LOCTEXT("TotalImpactColumnDesc", "The impact (percentage) of total estimated memory size"));

		if (bIsSimplifiedMode)
		{
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
							ETableColumnFlags::CanBeHidden |
							ETableColumnFlags::CanBeFiltered |
							ETableColumnFlags::IsDynamic);
		}
		else
		{
			Column.SetFlags(ETableColumnFlags::CanBeHidden |
							ETableColumnFlags::CanBeFiltered |
							ETableColumnFlags::IsDynamic);
		}

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(60.0f);

		Column.SetDataType(ETableCellDataType::Double);

		class FValueGetter : public FTableCellValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				TOptional<double> Impact = FObjectNode::GetTotalEstimatedMemoryImpact(Node);
				if (Impact.IsSet())
				{
					return FTableCellValue(static_cast<double>(Impact.GetValue()));
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		class FImpactFormatter : public FTableCellValueFormatter
		{
		public:
			virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override
			{
				if (InValue.IsSet())
				{
					const double Ratio = InValue.GetValue().Double;
					int64 Percent = FMath::RoundToInt(Ratio * 100.0);
					return FText::Format(LOCTEXT("ImpactFmt", "{0}%"), FText::AsNumber(Percent));
				}
				return FText::GetEmpty();
			}
			virtual FText FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const override
			{
				if (InValue.IsSet())
				{
					const double Ratio = InValue.GetValue().Double;
					FNumberFormattingOptions Options;
					Options.MaximumFractionalDigits = 2;
					return FText::Format(LOCTEXT("ImpactTooltipFmt", "{0}"), FText::AsPercent(Ratio, &Options));
				}
				return FText::GetEmpty();
			}
		};
		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FImpactFormatter>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<FSorterByDoubleValue> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Sorter->SetShouldSortGroupsFirst(false);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		Column.SetAggregation(ETableColumnAggregation::None);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// References Column

	if (bCanShowReferencingActors)
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FObjectTableColumns::ReferencesColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("ReferencesColumnName", "References"));
		Column.SetTitleName(LOCTEXT("ReferencesColumnTitle", "References"));
		Column.SetDescription(LOCTEXT("ReferencesColumnDesc", "The number of asset references"));

		if (bIsSimplifiedMode)
		{
			Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
							ETableColumnFlags::CanBeHidden |
							ETableColumnFlags::CanBeFiltered);
		}
		else
		{
			Column.SetFlags(ETableColumnFlags::CanBeHidden |
							ETableColumnFlags::CanBeFiltered);
		}

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(60.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FValueGetter : public FAggregatedValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetNodeValue(const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FObjectNode>())
				{
					const FObjectNode& ObjectNode = Node.As<FObjectNode>();
					const uint32 NumReferences = ObjectNode.GetNumReferences();
					return FTableCellValue(static_cast<int64>(NumReferences));
				}
				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<FSorterByInt64Value> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Sorter->SetShouldSortGroupsFirst(false);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		Column.SetAggregation(ETableColumnAggregation::None);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler

#undef LOCTEXT_NAMESPACE
