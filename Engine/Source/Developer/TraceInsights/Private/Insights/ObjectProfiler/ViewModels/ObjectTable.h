// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/NameTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/Table.h"

namespace UE::Insights::ObjectProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

// Column identifiers
struct FObjectTableColumns
{
	static const FName ObjectCountColumnId;
	static const FName ObjectNameColumnId;
	static const FName ObjectIdColumnId;
	static const FName ClassNameColumnId;
	static const FName ClassIdColumnId;
	static const FName OuterNameColumnId;
	static const FName OuterIdColumnId;
	static const FName PackageNameColumnId;
	static const FName PackageIdColumnId;

	static const FName PackageUniqueIdColumnId;
	static const FName PackagePathColumnId;
	static const FName SourcePackageNameColumnId;

	static const FName ObjectFlagsColumnId;

	static const FName ObjectPathColumnId;
	static const FName VersePathColumnId;
	
	static const FName SuperIdColumnId;
	static const FName InheritanceSuperIdColumnId;
	static const FName StructureSizeColumnId;

	static const FName SystemMemorySizeColumnId;
	static const FName VideoMemorySizeColumnId;
	static const FName EstimatedSizeColumnId;
	static const FName ImpactColumnId;

	static const FName TotalSystemMemorySizeColumnId;
	static const FName TotalVideoMemorySizeColumnId;
	static const FName TotalEstimatedSizeColumnId;
	static const FName TotalImpactColumnId;

	static const FName ReferencesColumnId;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FObjectTable : public FTable
{
public:
	FObjectTable(bool bInIsSimplifiedMode = false, bool bInCanShowReferencingActors = true);
	virtual ~FObjectTable();

	virtual void Reset();

private:
	void AddDefaultColumns();

private:
	bool bIsSimplifiedMode = false;
	bool bCanShowReferencingActors = false;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler
