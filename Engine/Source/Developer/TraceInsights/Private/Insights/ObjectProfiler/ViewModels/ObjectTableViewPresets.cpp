// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTableViewPresets.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"
#include "InsightsCore/Table/Widgets/STableTreeView.h"

// TraceInsights
#include "Insights/ObjectProfiler/ViewModels/ObjectGroupingByClassCategory.h"
#include "Insights/ObjectProfiler/ViewModels/ObjectGroupingByObjectName.h"
#include "Insights/ObjectProfiler/ViewModels/ObjectGroupingByOuter.h"
#include "Insights/ObjectProfiler/ViewModels/ObjectTable.h"
#include "Insights/ObjectProfiler/Widgets/SObjectTableTreeView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::ObjectProfiler::SObjectTableTreeView"

namespace UE::Insights::ObjectProfiler
{

TSharedRef<ITableTreeViewPreset> FObjectTableViewPresets::CreateAssetViewPreset(SObjectTableTreeView& TableTreeView)
{
	class FAssetViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("Assets_PresetName", "Assets");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("Assets_PresetToolTip", "Assets View\nConfigure the tree view to show memory estimations for assets.");
		}
		virtual FName GetSortColumn() const override
		{
			return FObjectTableColumns::EstimatedSizeColumnId;
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Descending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings.Num() > 0);
			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);

			auto GroupingByClassCategory = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FObjectGroupingByClassCategory>();
				});
			if (GroupingByClassCategory)
			{
				InOutCurrentGroupings.Add(*GroupingByClassCategory);
			}

			auto GroupingByObjectName = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FObjectGroupingByObjectName>();
				});
			if (GroupingByObjectName)
			{
				InOutCurrentGroupings.Add(*GroupingByObjectName);
			}
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),                     true, 300.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectCountColumnId,           !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectNameColumnId,            !true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectIdColumnId,              !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ClassNameColumnId,             !true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ClassIdColumnId,               !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::OuterNameColumnId,             !true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::OuterIdColumnId,               !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::PackageNameColumnId,           !true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::PackageIdColumnId,             !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectFlagsColumnId,           !true, 140.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectPathColumnId,            !true, 200.0f });
			InOutConfigSet.Add({ FObjectTableColumns::VersePathColumnId,             !true, 200.0f });
			InOutConfigSet.Add({ FObjectTableColumns::StructureSizeColumnId,         !true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::SystemMemorySizeColumnId,      !true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::VideoMemorySizeColumnId,       !true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::EstimatedSizeColumnId,         true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ImpactColumnId,                true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalSystemMemorySizeColumnId, !true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalVideoMemorySizeColumnId,  !true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalEstimatedSizeColumnId,    true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalImpactColumnId,           true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ReferencesColumnId,            true, 60.0f });
		}
		virtual void OnAppliedToView(STableTreeView& TableTreeView) const override
		{
			SObjectTableTreeView& ObjectTreeView = (SObjectTableTreeView&)TableTreeView;
			ObjectTreeView.PostApplyAssetsViewPreset();
		}
	};
	return MakeShared<FAssetViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Objects View

TSharedRef<ITableTreeViewPreset> FObjectTableViewPresets::CreateObjectViewPreset(SObjectTableTreeView& TableTreeView)
{
	class FObjectViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("Objects_PresetName", "Objects");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("Objects_PresetToolTip", "Objects View\nConfigure the tree view to show the all UObject instances.");
		}
		virtual FName GetSortColumn() const override
		{
			return FObjectTableColumns::ObjectIdColumnId;
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Ascending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings.Num() > 0);
			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),                     true, 400.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectCountColumnId,           true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectNameColumnId,            !true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectIdColumnId,              !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ClassNameColumnId,             true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ClassIdColumnId,               !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::OuterNameColumnId,             !true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::OuterIdColumnId,               !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::PackageNameColumnId,           !true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::PackageIdColumnId,             !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectFlagsColumnId,           true, 140.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectPathColumnId,            true, 200.0f });
			InOutConfigSet.Add({ FObjectTableColumns::VersePathColumnId,             !true, 200.0f });
			InOutConfigSet.Add({ FObjectTableColumns::StructureSizeColumnId,         true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::SystemMemorySizeColumnId,      true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::VideoMemorySizeColumnId,       true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::EstimatedSizeColumnId,         !true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ImpactColumnId,                !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalSystemMemorySizeColumnId, true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalVideoMemorySizeColumnId,  true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalEstimatedSizeColumnId,    !true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalImpactColumnId,           !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ReferencesColumnId,            !true, 80.0f });
		}
		virtual void OnAppliedToView(STableTreeView& TableTreeView) const override
		{
			//SObjectTableTreeView& ObjectTreeView = (SObjectTableTreeView&)TableTreeView;
		}
	};
	return MakeShared<FObjectViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Class View

TSharedRef<ITableTreeViewPreset> FObjectTableViewPresets::CreateClassViewPreset(SObjectTableTreeView& TableTreeView)
{
	class FClassViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("Class_PresetName", "Classes");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("Class_PresetToolTip", "Classes View\nConfigure the tree view to show the all UClass instances.");
		}
		virtual FName GetSortColumn() const override
		{
			return FObjectTableColumns::ObjectCountColumnId;
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Descending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings.Num() > 0);
			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);

			auto GroupingByClassHierarchy = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FObjectGroupingByClassHierarchy>();
				});
			if (GroupingByClassHierarchy)
			{
				InOutCurrentGroupings.Add(*GroupingByClassHierarchy);
			}

			auto GroupingByObjectName = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FObjectGroupingByObjectName>();
				});
			if (GroupingByObjectName)
			{
				InOutCurrentGroupings.Add(*GroupingByObjectName);
			}
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),                     true, 400.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectCountColumnId,           true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectNameColumnId,            !true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectIdColumnId,              !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ClassNameColumnId,             !true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ClassIdColumnId,               !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::OuterNameColumnId,             !true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::OuterIdColumnId,               !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::PackageNameColumnId,           !true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::PackageIdColumnId,             !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectFlagsColumnId,           true, 140.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectPathColumnId,            true, 200.0f });
			InOutConfigSet.Add({ FObjectTableColumns::VersePathColumnId,             !true, 200.0f });
			InOutConfigSet.Add({ FObjectTableColumns::StructureSizeColumnId,         true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::SystemMemorySizeColumnId,      true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::VideoMemorySizeColumnId,       true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::EstimatedSizeColumnId,         !true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ImpactColumnId,                !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalSystemMemorySizeColumnId, true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalVideoMemorySizeColumnId,  true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalEstimatedSizeColumnId,    !true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalImpactColumnId,           !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ReferencesColumnId,            !true, 80.0f });
		}
		virtual void OnAppliedToView(STableTreeView& TableTreeView) const override
		{
			//SObjectTableTreeView& ObjectTreeView = (SObjectTableTreeView&)TableTreeView;
		}
	};
	return MakeShared<FClassViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Outer View

TSharedRef<ITableTreeViewPreset> FObjectTableViewPresets::CreateOuterViewPreset(SObjectTableTreeView& TableTreeView)
{
	class FOuterViewPreset : public ITableTreeViewPreset
	{
	public:
		virtual FText GetName() const override
		{
			return LOCTEXT("Outer_PresetName", "Object Hierarchy");
		}
		virtual FText GetToolTip() const override
		{
			return LOCTEXT("Outer_PresetToolTip", "Object Hierarchy View\nConfigure the tree view to show the all UObject instances in a hierarchical view.");
		}
		virtual FName GetSortColumn() const override
		{
			return FObjectTableColumns::ObjectNameColumnId;
		}
		virtual EColumnSortMode::Type GetSortMode() const override
		{
			return EColumnSortMode::Type::Ascending;
		}
		virtual void SetCurrentGroupings(const TArray<TSharedPtr<FTreeNodeGrouping>>& InAvailableGroupings, TArray<TSharedPtr<FTreeNodeGrouping>>& InOutCurrentGroupings) const override
		{
			InOutCurrentGroupings.Reset();

			check(InAvailableGroupings.Num() > 0);
			check(InAvailableGroupings[0]->Is<FTreeNodeGroupingFlat>());
			InOutCurrentGroupings.Add(InAvailableGroupings[0]);

			auto GroupingByOuter = InAvailableGroupings.FindByPredicate(
				[](TSharedPtr<FTreeNodeGrouping>& Grouping)
				{
					return Grouping->Is<FObjectGroupingByOuter>();
				});
			if (GroupingByOuter)
			{
				InOutCurrentGroupings.Add(*GroupingByOuter);
			}
		}
		virtual void GetColumnConfigSet(TArray<FTableColumnConfig>& InOutConfigSet) const override
		{
			InOutConfigSet.Add({ FTable::GetHierarchyColumnId(),                     true, 400.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectCountColumnId,           true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectNameColumnId,            !true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectIdColumnId,              !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ClassNameColumnId,             true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ClassIdColumnId,               !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::OuterNameColumnId,             !true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::OuterIdColumnId,               !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::PackageNameColumnId,           !true, 120.0f });
			InOutConfigSet.Add({ FObjectTableColumns::PackageIdColumnId,             !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectFlagsColumnId,           true, 140.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ObjectPathColumnId,            true, 200.0f });
			InOutConfigSet.Add({ FObjectTableColumns::VersePathColumnId,             !true, 200.0f });
			InOutConfigSet.Add({ FObjectTableColumns::StructureSizeColumnId,         true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::SystemMemorySizeColumnId,      true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::VideoMemorySizeColumnId,       true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::EstimatedSizeColumnId,         !true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ImpactColumnId,                !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalSystemMemorySizeColumnId, true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalVideoMemorySizeColumnId,  true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalEstimatedSizeColumnId,    !true, 100.0f });
			InOutConfigSet.Add({ FObjectTableColumns::TotalImpactColumnId,           !true, 60.0f });
			InOutConfigSet.Add({ FObjectTableColumns::ReferencesColumnId,            !true, 80.0f });
		}
		virtual void OnAppliedToView(STableTreeView& TableTreeView) const override
		{
			//SObjectTableTreeView& ObjectTreeView = (SObjectTableTreeView&)TableTreeView;
		}
	};
	return MakeShared<FOuterViewPreset>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler

#undef LOCTEXT_NAMESPACE
