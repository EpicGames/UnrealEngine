// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Templates/SharedPointer.h"

namespace UE::Insights { class ITableTreeViewPreset; }

namespace UE::Insights::ObjectProfiler
{

class SObjectTableTreeView;

class FObjectTableViewPresets
{
public:
	static TSharedRef<ITableTreeViewPreset> CreateAssetViewPreset(SObjectTableTreeView& TableTreeView);
	static TSharedRef<ITableTreeViewPreset> CreateObjectViewPreset(SObjectTableTreeView& TableTreeView);
	static TSharedRef<ITableTreeViewPreset> CreateClassViewPreset(SObjectTableTreeView& TableTreeView);
	static TSharedRef<ITableTreeViewPreset> CreateOuterViewPreset(SObjectTableTreeView& TableTreeView);
};

} // namespace UE::Insights::ObjectProfiler
