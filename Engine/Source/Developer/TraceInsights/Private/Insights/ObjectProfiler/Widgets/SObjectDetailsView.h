// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

// TraceInsights
#include "Insights/ObjectProfiler/ViewModels/AssetInfoNode.h"

#if WITH_EDITOR
class FAssetThumbnail;
class FAssetThumbnailPool;
#endif

namespace UE::Insights
{
	class FBaseTreeNode;
}

namespace UE::Insights::ObjectProfiler
{

class IAssetInfoProvider;
class SObjectTableTreeView;
class FObjectNode;

////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A widget that displays a list of matched asset names for the current object selection.
 */
class SObjectDetailsView : public SCompoundWidget
{
public:
	/** Default constructor. */
	SObjectDetailsView();

	/** Virtual destructor. */
	virtual ~SObjectDetailsView();

	SLATE_BEGIN_ARGS(SObjectDetailsView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void Reset() {}

	void OnClose();

	void SetAssetInfoProvider(TSharedPtr<IAssetInfoProvider> InAssetInfoProvider)
	{
		WeakAssetInfoProvider = InAssetInfoProvider.ToWeakPtr();
	}

	void SetTableTreeView(TSharedPtr<SObjectTableTreeView> InTableTreeView)
	{
		WeakObjectTableTreeView = InTableTreeView.ToWeakPtr();
	}

	void SetSelectedNodes(const TArray<TSharedPtr<FTableTreeNode>>& InSelectedNodes);

private:
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FAssetInfoNode> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	FText GetSelectedObjectDetailedToolTip() const;

	FSlateColor GetTitleColor() const;
	FText GetTitleText() const;
	FText GetTitleToolTipText() const;

	FText GetSubTitleText() const;
	FText GetSubTitleToolTipText() const;

	EVisibility GetThumbnailVisibility() const;
	FText GetThumbnailToolTipText() const;
	const FSlateBrush* GetThumbnail() const;

	EVisibility GetActorsVisibility() const;

	void GatherObjectNodesRec(TSharedPtr<FBaseTreeNode> Node);
	void AddObjectNode(TSharedPtr<FObjectNode> InObjectNode);

private:
	TWeakPtr<IAssetInfoProvider> WeakAssetInfoProvider;

	TWeakPtr<SObjectTableTreeView> WeakObjectTableTreeView;

	TArray<TSharedPtr<FTableTreeNode>> SelectedNodes;
	TArray<TSharedPtr<FObjectNode>> SelectedObjectNodes;

	TSharedPtr<FObjectNode> SelectedObjectNode;
	FAssetData SelectedAssetData;
	FName SelectedAssetClassName;

	TSet<const FActorSet*> UniqueActorSets;

	/** List of matched asset info nodes shown in the list view. */
	TArray<TSharedPtr<FAssetInfoNode>> AssetEntries;

	TSharedPtr<SListView<TSharedPtr<FAssetInfoNode>>> ListView;
	TSharedPtr<SScrollBar> ExternalScrollbar;

#if WITH_EDITOR
	TSharedPtr<FAssetThumbnail> AssetThumbnail;
	TSharedPtr<FAssetThumbnailPool> AssetThumbnailPool;
#endif
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler
