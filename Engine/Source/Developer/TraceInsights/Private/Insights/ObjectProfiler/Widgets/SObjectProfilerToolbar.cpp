// Copyright Epic Games, Inc. All Rights Reserved.

#include "SObjectProfilerToolbar.h"

#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "ToolMenus.h"
#include "ToolMenuDelegates.h"
#include "Widgets/SBoxPanel.h"

// TraceInsights
#include "Insights/InsightsCommands.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/IUnrealInsightsModule.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::ObjectProfiler"

namespace UE::Insights::ObjectProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

SObjectProfilerToolbar::SObjectProfilerToolbar()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SObjectProfilerToolbar::~SObjectProfilerToolbar()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SObjectProfilerToolbar::Construct(const FArguments& InArgs, const FInsightsMajorTabConfig& Config)
{
	const FName ToolBarName = FName("ObjectProfiler.MainToolBar");
	const FName RightSideToolBarName = FName("ObjectProfiler.RightToolBar");
	const UToolMenus* ToolMenus = UToolMenus::Get();

	{
		FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
		if (!ToolMenus->IsMenuRegistered(ToolBarName))
		{
			UToolMenu* MainToolBar = UToolMenus::Get()->RegisterMenu(ToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
			MainToolBar->AddSection(FName("Main"));
		}
		if (!ToolMenus->IsMenuRegistered(RightSideToolBarName))
		{
			UToolMenu* RightSideToolBar = UToolMenus::Get()->RegisterMenu(RightSideToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
			RightSideToolBar->AddSection(FName("SnapshotInfo"));
		}
	}

	FToolMenuContext MenuContext(nullptr);

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.FillWidth(1.0)
		.Padding(0.0f)
		[
			UToolMenus::Get()->GenerateWidget(ToolBarName, MenuContext)
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0.0f)
		[
			UToolMenus::Get()->GenerateWidget(RightSideToolBarName, MenuContext)
		]
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler

#undef LOCTEXT_NAMESPACE
