// Copyright Epic Games, Inc. All Rights Reserved.

#include "SObjectProfilerWindow.h"

#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SSplitter.h"

// TraceInsightsCore
#include "InsightsCore/Widgets/SSegmentedBarGraph.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/ObjectProfiler/IObjectProfilerExtender.h"
#include "Insights/ObjectProfiler/ObjectProfilerManager.h"
#include "Insights/ObjectProfiler/Widgets/SObjectDetailsView.h"
#include "Insights/ObjectProfiler/Widgets/SObjectProfilerToolbar.h"
#include "Insights/ObjectProfiler/Widgets/SObjectTableTreeView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::ObjectProfiler"

namespace UE::Insights::ObjectProfiler
{

const FName FObjectProfilerTabs::ObjectTableTreeViewId("ObjectTableTreeView");
const FName FObjectProfilerTabs::ObjectDetailsViewId("ObjectDetailsView");
const FName ObjectProfilerTabId("ObjectProfiler");
const FName ObjectProfilerExtenderFeatureName("ObjectProfilerExtender");

////////////////////////////////////////////////////////////////////////////////////////////////////

SObjectProfilerWindow::SObjectProfilerWindow()
	: SMajorTabWindow(ObjectProfilerTabId)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SObjectProfilerWindow::~SObjectProfilerWindow()
{
	CloseAllOpenTabs();

	check(ObjectTableTreeView == nullptr);
	check(ObjectDetailsView == nullptr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* SObjectProfilerWindow::GetAnalyticsEventName() const
{
	return TEXT("Insights.Usage.ObjectProfiler");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SObjectProfilerWindow::Reset()
{
	if (ObjectTableTreeView)
	{
		ObjectTableTreeView->Reset();
	}

	if (ObjectDetailsView)
	{
		ObjectDetailsView->Reset();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SObjectProfilerWindow::SetAssetInfoProvider(TSharedPtr<IAssetInfoProvider> InAssetInfoProvider)
{
	AssetInfoProvider = InAssetInfoProvider;

	if (AssetInfoProvider)
	{
		if (ObjectTableTreeView)
		{
			ObjectTableTreeView->SetAssetInfoProvider(AssetInfoProvider);
		}

		if (ObjectDetailsView)
		{
			ObjectDetailsView->SetAssetInfoProvider(AssetInfoProvider);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SObjectProfilerWindow::SpawnTab_ObjectTableTreeView(const FSpawnTabArgs& Args)
{
	FObjectProfilerManager::Get()->SetObjectTableTreeViewVisible(true);

	check(ObjectTableTreeView == nullptr);
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(ObjectTableTreeView, SObjectTableTreeView)
		];
	check(ObjectTableTreeView != nullptr);

	ObjectTableTreeView->SetLogListingName(FObjectProfilerManager::Get()->GetLogListingName());

	if (AssetInfoProvider)
	{
		ObjectTableTreeView->SetAssetInfoProvider(AssetInfoProvider);
	}

	if (ObjectDetailsView)
	{
		ObjectTableTreeView->SetObjectDetailsView(ObjectDetailsView);
	}

	//SharedState->BindCommands();

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SObjectProfilerWindow::OnTabClosed_ObjectTableTreeView));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SObjectProfilerWindow::OnTabClosed_ObjectTableTreeView(TSharedRef<SDockTab> TabBeingClosed)
{
	FObjectProfilerManager::Get()->SetObjectTableTreeViewVisible(false);

	if (ObjectTableTreeView)
	{
		ObjectTableTreeView = nullptr;
	}

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SObjectProfilerWindow::SpawnTab_ObjectDetailsView(const FSpawnTabArgs& Args)
{
	FObjectProfilerManager::Get()->SetObjectDetailsViewVisible(true);

	check(ObjectDetailsView == nullptr);
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(ObjectDetailsView, SObjectDetailsView)
		];
	check(ObjectDetailsView != nullptr);

	if (AssetInfoProvider)
	{
		ObjectDetailsView->SetAssetInfoProvider(AssetInfoProvider);
	}

	if (ObjectTableTreeView)
	{
		ObjectTableTreeView->SetObjectDetailsView(ObjectDetailsView);
	}

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SObjectProfilerWindow::OnTabClosed_ObjectDetailsView));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SObjectProfilerWindow::OnTabClosed_ObjectDetailsView(TSharedRef<SDockTab> TabBeingClosed)
{
	FObjectProfilerManager::Get()->SetObjectDetailsViewVisible(false);

	if (ObjectDetailsView)
	{
		ObjectDetailsView = nullptr;
	}
	if (ObjectTableTreeView)
	{
		ObjectTableTreeView->SetObjectDetailsView(nullptr);
	}

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SObjectProfilerWindow::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	TSharedPtr<FObjectProfilerManager> ObjectProfilerManager = FObjectProfilerManager::Get();
	ensure(ObjectProfilerManager.IsValid());

	SetCommandList(ObjectProfilerManager->GetCommandList());

	SMajorTabWindow::FArguments Args;
	SMajorTabWindow::Construct(Args, ConstructUnderMajorTab, ConstructUnderWindow);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FWorkspaceItem> SObjectProfilerWindow::CreateWorkspaceMenuGroup()
{
	return GetTabManager()->AddLocalWorkspaceMenuCategory(LOCTEXT("ObjectProfilerMenuGroupName", "Object Insights"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SObjectProfilerWindow::RegisterTabSpawners()
{
	auto TabManagerPtr = GetTabManager();
	check(TabManagerPtr.IsValid());
	FTabManager& TabManagerRef = *TabManagerPtr.Get();

	check(GetWorkspaceMenuGroup().IsValid());
	const TSharedRef<FWorkspaceItem> Group = GetWorkspaceMenuGroup().ToSharedRef();

	IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(ObjectProfilerTabId);

	if (Config.ShouldRegisterMinorTab(FObjectProfilerTabs::ObjectTableTreeViewId))
	{
		TabManagerRef.RegisterTabSpawner(FObjectProfilerTabs::ObjectTableTreeViewId,
			FOnSpawnTab::CreateRaw(this, &SObjectProfilerWindow::SpawnTab_ObjectTableTreeView))
				.SetDisplayName(LOCTEXT("ObjectTableTreeViewTabTitle", "Objects"))
				.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ObjectTableTreeView"))
				.SetTooltipText(LOCTEXT("ObjectTableTreeViewTabToolTip", "Opens the Objects table/tree view."))
				.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FObjectProfilerTabs::ObjectDetailsViewId))
	{
		TabManagerRef.RegisterTabSpawner(FObjectProfilerTabs::ObjectDetailsViewId,
			FOnSpawnTab::CreateRaw(this, &SObjectProfilerWindow::SpawnTab_ObjectDetailsView))
				.SetDisplayName(LOCTEXT("ObjectDetailsViewTabTitle", "Object Details"))
				.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ObjectDetailsView"))
				.SetTooltipText(LOCTEXT("ObjectDetailsViewTabToolTip", "Opens the Object Details panel."))
				.SetGroup(Group);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<FTabManager::FLayout> SObjectProfilerWindow::CreateDefaultTabLayout() const
{
	return FTabManager::NewLayout("InsightsObjectProfilerLayout_v1.0")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.65f)
				->SetHideTabWell(true)
				->AddTab(FObjectProfilerTabs::ObjectTableTreeViewId, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.35f)
				->SetHideTabWell(true)
				->AddTab(FObjectProfilerTabs::ObjectDetailsViewId, ETabState::OpenedTab)
			)
		);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SObjectProfilerWindow::CreateToolbar(TSharedPtr<FExtender> Extender)
{
	IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(ObjectProfilerTabId);

#if 0 // debug code
	auto Segment1 = MakeShared<FCustomBarGraphSegment>();
	Segment1->Size = 50.0;
	Segment1->Text = LOCTEXT("Segment1", "Textures");
	Segment1->ToolTipText = LOCTEXT("Segment1_TT", "Textures...");
	Segment1->Color = FLinearColor(0.4f, 0.0f, 0.1f, 1.0f);
	Segment1->TextColor = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
	Segments.Add(Segment1);

	auto Segment2 = MakeShared<FCustomBarGraphSegment>();
	Segment2->Size = 30.0;
	Segment2->Text = LOCTEXT("Segment2", "Materials");
	Segment2->ToolTipText = LOCTEXT("Segment2_TT", "Materials...");
	Segment2->Color = FLinearColor(0.0f, 0.2f, 0.0f, 1.0f);
	Segment2->TextColor = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
	Segments.Add(Segment2);

	auto Segment3 = MakeShared<FCustomBarGraphSegment>();
	Segment3->Size = 20.0;
	Segment3->Text = LOCTEXT("Segment3_Verse", "Verse");
	Segment3->ToolTipText = LOCTEXT("Segment3_TT", "Verse objects");
	Segment3->Color = FLinearColor(0.1f, 0.0f, 0.4f, 1.0f);
	Segment3->TextColor = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
	Segments.Add(Segment3);
#endif

	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
		.BorderBackgroundColor(FLinearColor(0.014f, 0.014f, 0.014f, 1.0f))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.Padding(FMargin(4.0f, 4.0f, 4.0f, 0.0f))
			.AutoHeight()
			[
				SNew(SObjectProfilerToolbar, Config)
			]

			+ SVerticalBox::Slot()
			.Padding(FMargin(4.0f, 4.0f, 4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SegmentedBarGraphTitle", "Estimated Memory Breakdown"))
			]

			+ SVerticalBox::Slot()
			.Padding(FMargin(4.0f, 2.0f, 4.0f, 4.0f))
			[
				SNew(SBox)
				.HeightOverride(22.0f)
				[
					SAssignNew(SegmentedBarGraph, SSegmentedBarGraph)
					.SegmentsSource(&Segments)
				]
			]
		];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SObjectProfilerWindow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	TArray<IObjectProfilerExtender*> Extenders = IModularFeatures::Get().GetModularFeatureImplementations<IObjectProfilerExtender>(ObjectProfilerExtenderFeatureName);
	for (IObjectProfilerExtender* Extender : Extenders)
	{
		if (Extender)
		{
			FObjectProfilerExtenderTickParams Params(FInsightsManager::Get()->GetSession().Get());
			Params.CurrentTime = InCurrentTime;
			Params.DeltaTime = InDeltaTime;
			Extender->Tick(Params);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler

#undef LOCTEXT_NAMESPACE
