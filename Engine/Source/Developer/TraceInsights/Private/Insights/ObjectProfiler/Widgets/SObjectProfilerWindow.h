// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "UObject/NameTypes.h"

// TraceInsightsCore
#include "InsightsCore/ViewModels/IBarGraphSegment.h"

// TraceInsights
#include "Insights/Widgets/SMajorTabWindow.h"
#include "Insights/Widgets/IObjectProfilerMemoryView.h"

namespace UE::Insights
{
	class SSegmentedBarGraph;
}

namespace UE::Insights::ObjectProfiler
{

class SObjectTableTreeView;
class SObjectDetailsView;

struct FObjectProfilerTabs
{
	// Tab identifiers
	static const FName ObjectTableTreeViewId;
	static const FName ObjectDetailsViewId;
};

extern TRACEINSIGHTS_API const FName ObjectProfilerTabId;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FCustomBarGraphSegment : public UE::Insights::IBarGraphSegment
{
	friend class SObjectProfilerWindow;

public:
	FCustomBarGraphSegment() {}
	virtual ~FCustomBarGraphSegment() {}

	virtual double GetSize() const { return Size; }
	virtual FText GetText() const { return Text; }
	virtual FText GetToolTipText() const { return ToolTipText; }
	virtual FLinearColor GetColor() const { return Color; }
	virtual FLinearColor GetTextColor() const { return TextColor; }

private:
	double Size;
	FText Text;
	FText ToolTipText;
	FLinearColor Color;
	FLinearColor TextColor;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Implements the Object Insights window. */
class SObjectProfilerWindow : public ::Insights::SMajorTabWindow, public IObjectProfilerMemoryView
{
public:

	/** Default constructor. */
	SObjectProfilerWindow();

	/** Virtual destructor. */
	virtual ~SObjectProfilerWindow();

	SLATE_BEGIN_ARGS(SObjectProfilerWindow) {}
	SLATE_END_ARGS()

	virtual void Reset() override;

	/** Constructs this widget. */
	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);

	virtual void SetAssetInfoProvider(TSharedPtr<IAssetInfoProvider> InAssetInfoProvider) override;

	TSharedPtr<SObjectTableTreeView> GetObjectTableTreeView() const { return ObjectTableTreeView; }
	TSharedPtr<SObjectDetailsView> GetObjectDetailsView() const { return ObjectDetailsView; }

	void ShowHideSegmentedBarGraph(bool bOnOff) { /* TODO */ }

	TArray<TSharedPtr<IBarGraphSegment>>& GetSegments() { return Segments; }
	const TArray<TSharedPtr<IBarGraphSegment>>& GetSegments() const { return Segments; }
	TSharedPtr<SSegmentedBarGraph> GetSegmentedBarGraph() const { return SegmentedBarGraph; }

	const FName& GetLogListingName() const { return LogListingName; }

	virtual TSharedRef<SWidget> GetWidget() override { return SharedThis(this); }

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:
	virtual const TCHAR* GetAnalyticsEventName() const override;
	virtual TSharedRef<FWorkspaceItem> CreateWorkspaceMenuGroup() override;
	virtual void RegisterTabSpawners() override;
	virtual TSharedRef<FTabManager::FLayout> CreateDefaultTabLayout() const override;
	virtual TSharedRef<SWidget> CreateToolbar(TSharedPtr<FExtender> Extender);

private:
	TSharedRef<SDockTab> SpawnTab_ObjectTableTreeView(const FSpawnTabArgs& Args);
	void OnTabClosed_ObjectTableTreeView(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_ObjectDetailsView(const FSpawnTabArgs& Args);
	void OnTabClosed_ObjectDetailsView(TSharedRef<SDockTab> TabBeingClosed);

private:
	TSharedPtr<IAssetInfoProvider> AssetInfoProvider;

	/** The Objects table tree view widget */
	TSharedPtr<SObjectTableTreeView> ObjectTableTreeView;

	/** The Object Details panel widget */
	TSharedPtr<SObjectDetailsView> ObjectDetailsView;

	/** The name of the Object Insights log listing. */
	FName LogListingName;

	TSharedPtr<SSegmentedBarGraph> SegmentedBarGraph;
	bool bIsSegmentedBarGraphVisible = true;
	TArray<TSharedPtr<IBarGraphSegment>> Segments;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::ObjectProfiler
