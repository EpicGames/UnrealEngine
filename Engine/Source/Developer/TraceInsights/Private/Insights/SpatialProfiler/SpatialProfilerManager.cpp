// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpatialProfilerManager.h"

#include "Features/IModularFeatures.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Insights/Common/InsightsMenuBuilder.h"
#include "Insights/InsightsManager.h"
#include "Insights/SpatialProfiler/ISpatialPlotViewExtender.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SSpatialInsightsWindow.h"
#include "Widgets/SSpatialPlotView.h"
#include "Insights/Widgets/STimingView.h"
#include "Widgets/Text/STextBlock.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::SpatialProfiler::SpatialProfilerManager"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::SpatialProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FSpatialProfilerManager> FSpatialProfilerManager::Instance;

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FSpatialProfilerManager> FSpatialProfilerManager::Get()
{
	return Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FSpatialProfilerManager> FSpatialProfilerManager::CreateInstance()
{
	ensure(!Instance.IsValid());
	if (Instance.IsValid())
	{
		Instance.Reset();
	}

	Instance = MakeShared<FSpatialProfilerManager>();
	return Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSpatialProfilerManager::~FSpatialProfilerManager()
{
	ensure(!bIsInitialized);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSpatialProfilerManager::Initialize(IUnrealInsightsModule& InsightsModule)
{
	if (!ensure(!bIsInitialized))
	{
		return;
	}
	bIsInitialized = true;

	OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(this, &ThisClass::Tick), 0.0f);

	InsightsModule.OnRegisterMajorTabExtension(SpatialProfilerTabId).AddRaw(this, &ThisClass::RegisterExtenderControlPanels);

	if (TSharedPtr<::Insights::IInsightsManager> InsightsManager = InsightsModule.GetInsightsManager())
	{
		InsightsManager->GetSessionChangedEvent().AddSP(this, &ThisClass::InsightsSessionChangedHandler);
	}
	InsightsSessionChangedHandler();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSpatialProfilerManager::Shutdown()
{
	if (!ensure(bIsInitialized))
	{
		return;
	}
	bIsInitialized = false;

	if (IUnrealInsightsModule* UnrealInsightsModule = FModuleManager::GetModulePtr<IUnrealInsightsModule>("TraceInsights"))
	{
		UnrealInsightsModule->OnRegisterMajorTabExtension(SpatialProfilerTabId).RemoveAll(this);

		if (TSharedPtr<::Insights::IInsightsManager> InsightsManager = UnrealInsightsModule->GetInsightsManager())
		{
			InsightsManager->GetSessionChangedEvent().RemoveAll(this);
		}
	}

	FTSTicker::GetCoreTicker().RemoveTicker(OnTickHandle);

	FSpatialProfilerManager::Instance.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSpatialProfilerManager::OnWindowClosedEvent()
{
	if (TSharedPtr<SSpatialInsightsWindow> Wnd = WeakSpatialInsightsWindow.Pin())
	{
		TSharedPtr<TimingProfiler::STimingView> TimingView = Wnd->GetTimingView();
		if (TimingView)
		{
			TimingView->CloseQuickFindTab();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSpatialProfilerManager::RegisterMajorTabs(IUnrealInsightsModule& InsightsModule)
{
	const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(SpatialProfilerTabId);
	if (!Config.bIsAvailable)
	{
		return;
	}

	// Register tab spawner for Spatial Insights.
	FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		/*       TabId */ SpatialProfilerTabId,
		/*  OnSpawnTab */ FOnSpawnTab::CreateSP(AsShared(), &ThisClass::SpatialInsightsWindow_SpawnTab),
		/* CanSpawnTab */ FCanSpawnTab::CreateSP(AsShared(), &ThisClass::SpatialInsightsWindow_CanSpawnTab));
	TabSpawnerEntry.SetDisplayName(Config.TabLabel.Get(LOCTEXT("SpatialInsights_TabLabel", "Spatial Insights")));
	TabSpawnerEntry.SetTooltipText(Config.TabTooltip.Get(LOCTEXT("SpatialInsights_TabTooltip", "Open the Spatial Insights tab.")));

	TSharedRef<FWorkspaceItem> Group =
		Config.WorkspaceGroup
			? Config.WorkspaceGroup.ToSharedRef()
			: FInsightsManager::Get()->GetInsightsMenuBuilder()->GetInsightsToolsGroup();

	TabSpawnerEntry.SetGroup(Group);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSpatialProfilerManager::UnregisterMajorTabs()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(SpatialProfilerTabId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSpatialProfilerManager::InsightsSessionChangedHandler()
{
	bIsAvailable = false;

	if (IUnrealInsightsModule* UnrealInsightsModule = FModuleManager::GetModulePtr<IUnrealInsightsModule>("TraceInsights"))
	{
		if (UnrealInsightsModule->GetAnalysisSession())
		{
			AvailabilityCheck.Enable(1.0);
		}
		else
		{
			AvailabilityCheck.Disable();
		}
	}
	else
	{
		AvailabilityCheck.Disable();
	}

	TSharedPtr<SSpatialInsightsWindow> Wnd = WeakSpatialInsightsWindow.Pin();
	if (Wnd.IsValid())
	{
		Wnd->Reset();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FSpatialProfilerManager::SpatialInsightsWindow_CanSpawnTab(const FSpawnTabArgs& Args) const
{
	return bIsAvailable;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> FSpatialProfilerManager::SpatialInsightsWindow_SpawnTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.OnTabClosed_Raw(this, &ThisClass::SpatialInsightsWindow_TabClosed);

	DockTab->SetContent(SAssignNew(WeakSpatialInsightsWindow, SSpatialInsightsWindow, DockTab, Args.GetOwnerWindow()));
	return DockTab;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSpatialProfilerManager::SpatialInsightsWindow_TabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	if (TSharedPtr<SSpatialInsightsWindow> SpatialInsightsWindow = WeakSpatialInsightsWindow.Pin())
	{
		SpatialInsightsWindow->CloseAllOpenTabs();
	}
	WeakSpatialInsightsWindow.Reset();

	TabBeingClosed->SetOnTabClosed({});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FSpatialProfilerManager::Tick(float DeltaTime)
{
	if (!bIsAvailable && AvailabilityCheck.Tick())
	{
		const IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

		bool bHasSpatialData = false;

		TSharedPtr<const TraceServices::IAnalysisSession> Session = UnrealInsightsModule.GetAnalysisSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			if (Session->IsAnalysisComplete())
			{
				// Never check again during this session.
				AvailabilityCheck.Disable();
			}

			// Check if any registered extender reports data for the current session.
			if (!bHasSpatialData)
			{
				TArray<ISpatialPlotViewExtender*> Extenders = IModularFeatures::Get().GetModularFeatureImplementations<ISpatialPlotViewExtender>(SpatialPlotViewExtenderFeatureName);
				for (ISpatialPlotViewExtender* Extender : Extenders)
				{
					if (Extender->HasDataForSession(*Session))
					{
						bHasSpatialData = true;
						break;
					}
				}
			}
		}
		else
		{
			// Do not check again until the next session changed event (see OnSessionChanged).
			AvailabilityCheck.Disable();
		}

		if (bHasSpatialData)
		{
			bIsAvailable = true;

			if (FGlobalTabmanager::Get()->HasTabSpawner(SpatialProfilerTabId))
			{
				constexpr bool bInvokeAsInactive = true;
				FGlobalTabmanager::Get()->TryInvokeTab(SpatialProfilerTabId, bInvokeAsInactive);
			}
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FSpatialProfilerManager::RegisterExtenderControlPanels(FInsightsMajorTabExtender& InOutExtender)
{
	TArray<ISpatialPlotViewExtender*> Extenders = IModularFeatures::Get().GetModularFeatureImplementations<ISpatialPlotViewExtender>(SpatialPlotViewExtenderFeatureName);

	TSet<FName> SeenLayerNames;
	for (ISpatialPlotViewExtender* Extender : Extenders)
	{
		const FName LayerName = Extender->GetLayerName();

		if (SeenLayerNames.Contains(LayerName))
		{
			UE_LOGF(LogSpatialPlotView, Warning, "Duplicate ISpatialPlotViewExtender layer name: %ls. Skipping control panel registration.", *LayerName.ToString());
			continue;
		}
		SeenLayerNames.Add(LayerName);

		if (!Extender->HasControlPanel())
		{
			continue;
		}

		const FName TabId = FSpatialInsightsTabs::GetControlPanelTabId(LayerName);

		FMinorTabConfig& Config = InOutExtender.AddMinorTabConfig();
		Config.TabId = TabId;
		Config.TabLabel = Extender->GetLayerDisplayName();
		Config.TabIcon = Extender->GetLayerIcon();
		Config.WorkspaceGroup = InOutExtender.GetWorkspaceGroup();
		Config.OnSpawnTab = FOnSpawnTab::CreateLambda([LayerName](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
		{
			TSharedPtr<SWidget> ControlPanel;

			TArray<ISpatialPlotViewExtender*> Extenders = IModularFeatures::Get().GetModularFeatureImplementations<ISpatialPlotViewExtender>(SpatialPlotViewExtenderFeatureName);
			ISpatialPlotViewExtender** FoundExtender = Extenders.FindByPredicate([&LayerName](ISpatialPlotViewExtender* E) { return E->GetLayerName() == LayerName; });
			if (FoundExtender && (*FoundExtender)->HasControlPanel())
			{
				ControlPanel = (*FoundExtender)->CreateControlPanel();
			}

			if (!ControlPanel.IsValid())
			{
				ControlPanel = SNew(STextBlock).Text(LOCTEXT("ControlPanelUnavailable", "Control panel unavailable"));
			}

			return SNew(SDockTab)
				.ShouldAutosize(false)
				.TabRole(ETabRole::PanelTab)
				[
					ControlPanel.ToSharedRef()
				];
		});

		InOutExtender.GetLayoutExtender().ExtendStack(
			FSpatialInsightsTabs::RightPanelExtensionId,
			ELayoutExtensionPosition::After,
			FTabManager::FTab(TabId, ETabState::OpenedTab));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::SpatialProfiler

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE // UE::Insights::SpatialProfiler::SpatialProfilerManager

////////////////////////////////////////////////////////////////////////////////////////////////////
