// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Insights/IUnrealInsightsModule.h"
#include "InsightsCore/Common/AvailabilityCheck.h"

namespace UE::Insights::SpatialProfiler
{
	
class SSpatialInsightsWindow;

/**
 * The component that makes Spatial Insights available inside Unreal Insights.
 */
class FSpatialProfilerManager : public IInsightsComponent, public TSharedFromThis<FSpatialProfilerManager>
{
	using ThisClass = FSpatialProfilerManager;
	using Super = IInsightsComponent;
public:
	FSpatialProfilerManager() = default;
	virtual ~FSpatialProfilerManager();

	/** Creates an instance of the Spatial Profiler manager. */
	static TSharedPtr<FSpatialProfilerManager> CreateInstance();

	/** @return the global instance of the Spatial Profiler manager. */
	static TSharedPtr<FSpatialProfilerManager> Get();

	//~ Begin IInsightsComponent interface
	virtual void Initialize(IUnrealInsightsModule& InsightsModule) override;
	virtual void Shutdown() override;
	virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) override;
	virtual void UnregisterMajorTabs() override;
	virtual void OnWindowClosedEvent() override;
	//~ End IInsightsComponent interface

private:

	void InsightsSessionChangedHandler();
	void RegisterExtenderControlPanels(FInsightsMajorTabExtender& InOutExtender);

	bool SpatialInsightsWindow_CanSpawnTab(const FSpawnTabArgs& Args) const;
	TSharedRef<SDockTab> SpatialInsightsWindow_SpawnTab(const FSpawnTabArgs& Args);
	void SpatialInsightsWindow_TabClosed(TSharedRef<SDockTab> TabBeingClosed);

	bool Tick(float DeltaTime);

	bool bIsInitialized = false;
	bool bIsAvailable = false;
	FAvailabilityCheck AvailabilityCheck;

	TWeakPtr<SSpatialInsightsWindow> WeakSpatialInsightsWindow;

	FTSTicker::FDelegateHandle OnTickHandle;

	/** A shared pointer to the global instance of the SpatialProfiler manager. */
	static TSharedPtr<FSpatialProfilerManager> Instance;
};

} // namespace UE::Insights::SpatialProfiler
