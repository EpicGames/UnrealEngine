// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSpatialInsightsWindow.h"

#include "Features/IModularFeatures.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/SpatialProfiler/ISpatialPlotViewExtender.h"
#include "Insights/SpatialProfiler/SpatialProfilerManager.h"
#include "Insights/SpatialProfiler/Widgets/SSpatialPlotView.h"
#include "Insights/Widgets/STimingView.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "ToolMenus.h"
#include "ToolMenuWidgetCollectionContext.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "UE::Insights::SpatialProfiler::SSpatialInsightsWindow"

namespace UE::Insights::SpatialProfiler::Private
{

static constexpr FLazyName MainToolbarName("SpatialInsightsWindow.MainToolbar");
	
} // namespace UE::Insights::SpatialProfiler::Private

namespace UE::Insights::SpatialProfiler
{

const FName FSpatialInsightsTabs::TimingViewID("TimingView");
const FName FSpatialInsightsTabs::SpatialPlotViewID("SpatialPlotView");
const FName FSpatialInsightsTabs::RightPanelExtensionId("RightPanel");
const FName SpatialProfilerTabId("SpatialProfiler");

////////////////////////////////////////////////////////////////////////////////////////////////////

SLATE_IMPLEMENT_WIDGET(SSpatialInsightsWindow)

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSpatialInsightsWindow::PrivateRegisterAttributes(FSlateAttributeInitializer&)
{
	// Required by `SLATE_DECLARE_WIDGET(SSpatialInsightsWindow, ::Insights::SMajorTabWindow);`
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SSpatialInsightsWindow::SSpatialInsightsWindow()
	: SMajorTabWindow(SpatialProfilerTabId)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SSpatialInsightsWindow::~SSpatialInsightsWindow()
{
	CloseAllOpenTabs();

	check(!SpatialPlotView);
	check(!TimingView);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* SSpatialInsightsWindow::GetAnalyticsEventName() const
{
	return TEXT("Insights.Usage.SpatialInsights");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSpatialInsightsWindow::Reset()
{
	if (SpatialPlotView)
	{
		SpatialPlotView->Reset();
	}

	if (TimingView)
	{
		// STimingView::Reset() sets the marker to +infinity; override to 0 for CurrentTraceTime.
		TimingView->Reset();
		TimingView->SetTimeMarker(0.0);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSpatialInsightsWindow::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	Super::FArguments Args;
	Super::Construct(Args, ConstructUnderMajorTab, ConstructUnderWindow);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSpatialInsightsWindow::RegisterTabSpawners()
{
	check(GetTabManager().IsValid());
	FTabManager* TabManagerPtr = GetTabManager().Get();
	check(GetWorkspaceMenuGroup().IsValid());
	const TSharedRef<FWorkspaceItem> Group = GetWorkspaceMenuGroup().ToSharedRef();

	IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(SpatialProfilerTabId);

	if (Config.ShouldRegisterMinorTab(FSpatialInsightsTabs::TimingViewID))
	{
		FTabSpawnerEntry& TabSpawnerEntry = TabManagerPtr->RegisterTabSpawner(
			/*       TabId */ FSpatialInsightsTabs::TimingViewID,
			/*  OnSpawnTab */ FOnSpawnTab::CreateRaw(this, &SSpatialInsightsWindow::SpawnTab_TimingView));
		TabSpawnerEntry.SetDisplayName(LOCTEXT("TimingViewTabTitle", "Timing View"));
		TabSpawnerEntry.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TimingView"));
		TabSpawnerEntry.SetTooltipText(LOCTEXT("TimingViewTabToolTip", "Opens the Timing View."));
		TabSpawnerEntry.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FSpatialInsightsTabs::SpatialPlotViewID))
	{
		FTabSpawnerEntry& TabSpawnerEntry = TabManagerPtr->RegisterTabSpawner(
			/*       TabId */ FSpatialInsightsTabs::SpatialPlotViewID, 
			/*  OnSpawnTab */ FOnSpawnTab::CreateRaw(this, &SSpatialInsightsWindow::SpawnTab_SpatialPlotView));
		TabSpawnerEntry.SetDisplayName(LOCTEXT("SpatialPlotViewTabTitle", "Plot View"));
		TabSpawnerEntry.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.SpatialPlotView"));
		TabSpawnerEntry.SetTooltipText(LOCTEXT("SpatialPlotViewTabToolTip", "Opens the Plot View."));
		TabSpawnerEntry.SetGroup(Group);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<FTabManager::FLayout> SSpatialInsightsWindow::CreateDefaultTabLayout() const
{
	return FTabManager::NewLayout("SpatialInsightsLayout_v1.4")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.75f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.7f)
					->AddTab(FSpatialInsightsTabs::SpatialPlotViewID, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.3f)
					->AddTab(FSpatialInsightsTabs::TimingViewID, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.25f)
				->SetExtensionId(FSpatialInsightsTabs::RightPanelExtensionId)
			)
		);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SSpatialInsightsWindow::CreateToolbar(TSharedPtr<FExtender> Extender)
{
	RegisterToolBarMenus();

	FToolMenuContext MenuContext(nullptr, Extender);

	// Add this widget to the widget collection context object to allow dynamic menu creation logic to work,
	// as the registered menus are stateless.
	UToolMenuWidgetCollectionContext* WidgetCollection = UToolMenuWidgetCollectionContext::Get(MenuContext);
	WidgetCollection->AddWidget(SharedThis(this));

	return UToolMenus::Get()->GenerateWidget(Private::MainToolbarName, MenuContext);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSpatialInsightsWindow::RegisterToolBarMenus()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus || ToolMenus->IsMenuRegistered(Private::MainToolbarName))
	{
		return;
	}

	FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
	UToolMenu* MainToolbar = UToolMenus::Get()->RegisterMenu(Private::MainToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);

	FToolMenuSection& Section = MainToolbar->AddSection("ExtenderToggles");

	Section.AddDynamicEntry("ExtenderToggleEntries", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		UToolMenuWidgetCollectionContext* WidgetCollection = InSection.FindContext<UToolMenuWidgetCollectionContext>();
		TSharedPtr<SSpatialInsightsWindow> Window = WidgetCollection ? WidgetCollection->FindWidget<SSpatialInsightsWindow>() : nullptr;
		if (!Window)
		{
			return;
		}

		TWeakPtr<SSpatialInsightsWindow> WeakWindow = Window;

		TArray<ISpatialPlotViewExtender*> Extenders = IModularFeatures::Get().GetModularFeatureImplementations<ISpatialPlotViewExtender>(SpatialPlotViewExtenderFeatureName);

		TSet<FName> SeenLayerNames;
		for (ISpatialPlotViewExtender* Extender : Extenders)
		{
			const FName LayerName = Extender->GetLayerName();
			const FText DisplayName = Extender->GetLayerDisplayName();
			const FSlateIcon LayerIcon = Extender->GetLayerIcon();
			const bool bHasControlPanel = Extender->HasControlPanel();

			if (SeenLayerNames.Contains(LayerName))
			{
				UE_LOGF(LogSpatialPlotView, Warning, "Duplicate ISpatialPlotViewExtender layer name: %ls. Skipping toolbar entry.", *LayerName.ToString());
				continue;
			}
			SeenLayerNames.Add(LayerName);

			InSection.AddEntry(FToolMenuEntry::InitMenuEntry(
				LayerName,
				DisplayName,
				FText::Format(LOCTEXT("ToggleExtenderTooltip", "Show/Hide {0} layer"), DisplayName),
				LayerIcon,
				FToolUIAction(
					FToolMenuExecuteAction::CreateLambda([WeakWindow, LayerName, bHasControlPanel](const FToolMenuContext&)
					{
						TSharedPtr<SSpatialInsightsWindow> Wnd = WeakWindow.Pin();
						TSharedPtr<SSpatialPlotView> PlotView = Wnd ? Wnd->GetSpatialPlotView() : nullptr;
						if (!PlotView)
						{
							return;
						}

						const bool bNowVisible = !PlotView->IsExtenderVisible(LayerName);
						PlotView->SetExtenderVisible(LayerName, bNowVisible);

						if (!bHasControlPanel)
						{
							return;
						}

						TSharedPtr<FTabManager> TabMgr = Wnd->GetTabManager();
						if (!TabMgr)
						{
							return;
						}

						const FName ControlPanelTabId = FSpatialInsightsTabs::GetControlPanelTabId(LayerName);
						if (bNowVisible)
						{
							TabMgr->TryInvokeTab(ControlPanelTabId);
						}
						else if (TSharedPtr<SDockTab> Tab = TabMgr->FindExistingLiveTab(ControlPanelTabId))
						{
							Tab->RequestCloseTab();
						}
					}),
					FToolMenuCanExecuteAction(),
					FToolMenuGetActionCheckState::CreateLambda([WeakWindow, LayerName](const FToolMenuContext&) -> ECheckBoxState
					{
						TSharedPtr<SSpatialInsightsWindow> Wnd = WeakWindow.Pin();
						TSharedPtr<SSpatialPlotView> PlotView = Wnd ? Wnd->GetSpatialPlotView() : nullptr;
						return (PlotView && PlotView->IsExtenderVisible(LayerName)) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				),
				EUserInterfaceActionType::ToggleButton
			));
		}
	}));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SSpatialInsightsWindow::SpawnTab_TimingView(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		.OnTabClosed_Raw(this, &ThisClass::OnTimingViewTabClosed)
		[
			SAssignNew(TimingView, TimingProfiler::STimingView, SpatialProfilerTabId)
		];

	// STimingView::Reset() sets the marker to +infinity; override to 0 for CurrentTraceTime.
	TimingView->Reset(true);
	TimingView->SetTimeMarker(0.0);

	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSpatialInsightsWindow::OnTimingViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	TimingView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> SSpatialInsightsWindow::SpawnTab_SpatialPlotView(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		.OnTabClosed_Raw(this, &ThisClass::OnSpatialPlotViewTabClosed)
		[
			SAssignNew(SpatialPlotView, SSpatialPlotView)
			.CurrentTraceTime_Lambda([this]() { return TimingView.IsValid() ? TimingView->GetTimeMarker() : 0.0; })
		];

	SpatialPlotView->Reset(true);
	
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSpatialInsightsWindow::OnSpatialPlotViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	SpatialPlotView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::SpatialProfiler

#undef LOCTEXT_NAMESPACE // "UE::Insights::SpatialProfiler::SSpatialInsightsWindow"
