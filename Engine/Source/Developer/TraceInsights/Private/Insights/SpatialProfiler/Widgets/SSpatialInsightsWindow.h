// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/Widgets/SMajorTabWindow.h"

#define UE_API TRACEINSIGHTS_API

////////////////////////////////////////////////////////////////////////////////////////////////////

class UToolMenu;
namespace UE::Insights
{
	namespace SpatialProfiler
	{
		class SSpatialPlotView;
	} // namespace SpatialProfiler

	namespace TimingProfiler
	{
		class STimingView;
	} // namespace TimingProfiler
} // namespace UE::Insights

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::SpatialProfiler
{

struct FSpatialInsightsTabs
{
	// Tab identifiers
	static const FName TimingViewID;
	static const FName SpatialPlotViewID;

	// Layout extension identifiers
	static const FName RightPanelExtensionId;

	/** Returns the deterministic TabId for an extender's control panel. */
	static FName GetControlPanelTabId(FName InLayerName)
	{
		return FName(*(FString(TEXT("SpatialExtenderPanel_")) + InLayerName.ToString()));
	}
};

extern UE_API const FName SpatialProfilerTabId;

////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Implements the Spatial Insights window.
 */
class SSpatialInsightsWindow : public ::Insights::SMajorTabWindow
{
	SLATE_DECLARE_WIDGET(SSpatialInsightsWindow, ::Insights::SMajorTabWindow)

public:

	SSpatialInsightsWindow();
	virtual ~SSpatialInsightsWindow() override;

	SLATE_BEGIN_ARGS(SSpatialInsightsWindow) {}
	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs,
		const TSharedRef<SDockTab>& ConstructUnderMajorTab,
		const TSharedPtr<SWindow>& ConstructUnderWindow);

	//~ Begin SMajorTabWindow interface
	virtual void Reset() override;
	//~ End SMajorTabWindow interface

protected:

	//~ Begin SMajorTabWindow interface
	virtual const TCHAR* GetAnalyticsEventName() const override;
	virtual void RegisterTabSpawners() override;
	virtual TSharedRef<FTabManager::FLayout> CreateDefaultTabLayout() const override;
	virtual TSharedRef<SWidget> CreateToolbar(TSharedPtr<FExtender> Extender) override;
	//~ End SMajorTabWindow interface

private:

	static void RegisterToolBarMenus();

	TSharedRef<SDockTab> SpawnTab_TimingView(const FSpawnTabArgs& Args);
	void OnTimingViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_SpatialPlotView(const FSpawnTabArgs& Args);
	void OnSpatialPlotViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

public:
	TSharedPtr<SSpatialPlotView> GetSpatialPlotView() const { return SpatialPlotView; }
	TSharedPtr<TimingProfiler::STimingView> GetTimingView() const { return TimingView; }

private:
	TSharedPtr<TimingProfiler::STimingView> TimingView;
	TSharedPtr<SSpatialPlotView> SpatialPlotView;

};

} // namespace UE::Insights::SpatialProfiler

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
