// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSpatialPlotView.h"

#include "DesktopPlatformModule.h"
#include "Dom/JsonObject.h"
#include "Features/IModularFeatures.h"
#include "IImageWrapperModule.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/FileHelper.h"
#include "Rendering/DrawElements.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Styling/AppStyle.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/TimeUtils.h"

#include "Insights/IUnrealInsightsModule.h"
#include "Insights/SpatialProfiler/ISpatialPlotViewExtender.h"
#include "Modules/ModuleManager.h"
#include "TraceServices/Model/AnalysisSession.h"

DEFINE_LOG_CATEGORY(LogSpatialPlotView);

#define LOCTEXT_NAMESPACE "UE::Insights::SpatialProfiler::SSpatialPlotView"

namespace UE::Insights::SpatialProfiler::Private
{

constexpr float MinViewScale = 0.0001f;
constexpr float MaxViewScale = 1000.f;
constexpr float MarkerSize = 8.f;
constexpr float MarkerHitRadius = MarkerSize * 0.5f + 1.f; // Half visual extent + 1px tolerance
constexpr float MarkerHitRadiusSq = MarkerHitRadius * MarkerHitRadius;
constexpr float BugItGoDefaultZ = 1000.f;

template<typename T>
static UE::Math::TBox2<T> TransformBox2D(const UE::Math::TBox2<T>& Box, const TTransform2<T>& Transform)
{
	if (!Box.bIsValid)
	{
		return UE::Math::TBox2<T>(ForceInit);
	}

	UE::Math::TVector2<T> Centre, Extents;
	Box.GetCenterAndExtents(Centre, Extents);

	const UE::Math::TVector2<T> TransformedCentre = Transform.TransformPoint(Centre);

	const TMatrix2x2<T>& Matrix = Transform.GetMatrix();

	// Calculate the new extents by taking absolute values of transformed basis vectors
	T M00, M01, M10, M11;
	Matrix.GetMatrix(M00, M01, M10, M11);
	const UE::Math::TVector2<T> AbsTransformX(FMath::Abs(M00), FMath::Abs(M10));
	const UE::Math::TVector2<T> AbsTransformY(FMath::Abs(M01), FMath::Abs(M11));

	const UE::Math::TVector2<T> NewHalfExtents = AbsTransformX * Extents.X + AbsTransformY * Extents.Y;

	return UE::Math::TBox2<T>(TransformedCentre - NewHalfExtents, TransformedCentre + NewHalfExtents);
}

} // namespace UE::Insights::SpatialProfiler::Private

namespace UE::Insights::SpatialProfiler
{

const FName SpatialPlotViewExtenderFeatureName("SpatialPlotViewExtender");

void SSpatialPlotView::FSpatialViewState::ProduceTransforms(const FVector2f& InWidgetSize, FTransform2d& OutWorldToScreen, FTransform2d& OutScreenToWorld) const
{
	FTransform2d T = FTransform2d(FocusCenter);
	FTransform2d R = FTransform2d(FQuat2d(Rotation));
	FTransform2d S = FTransform2d(FScale2d(Scale), InWidgetSize * 0.5f);
	OutWorldToScreen = Concatenate(T, R, S);
	OutScreenToWorld = OutWorldToScreen.Inverse();
}

SLATE_IMPLEMENT_WIDGET(SSpatialPlotView)

void SSpatialPlotView::PrivateRegisterAttributes(FSlateAttributeInitializer&)
{
	// Required by `SLATE_DECLARE_WIDGET(SSpatialPlotView, SCompoundWidget);`
}

SSpatialPlotView::SSpatialPlotView() = default;
SSpatialPlotView::~SSpatialPlotView() = default;

void SSpatialPlotView::Construct(const FArguments& InArgs)
{
	CurrentTraceTime = InArgs._CurrentTraceTime;
}

void SSpatialPlotView::Reset(bool bIsFirstReset /* = false */)
{
	CachedExtenderDrawStates.Empty();
	HiddenExtenders.Empty();
	HoveredFingerprint = 0;
	Tooltip.SetDesiredOpacity(0.0f);
	SessionState = FSpatialViewState();
	bShouldAutoFrame = true;

	TArray<ISpatialPlotViewExtender*> Extenders = GetExtenders();

	if (!bIsFirstReset)
	{
		for (ISpatialPlotViewExtender* Extender : Extenders)
		{
			Extender->OnEndSession();
		}
	}

	TSharedPtr<const TraceServices::IAnalysisSession> AnalysisSession = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights").GetAnalysisSession();
	if (AnalysisSession.IsValid())
	{
		for (ISpatialPlotViewExtender* Extender : Extenders)
		{
			Extender->OnBeginSession(*AnalysisSession);
		}
	}

	ClearBackgroundImage();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Background Image
////////////////////////////////////////////////////////////////////////////////////////////////////

static TOptional<FBox2D> ParseMinimapBoundsJson(const FString& InJsonPath)
{
	FString JsonString;
	if (!FFileHelper::LoadFileToString(JsonString, *InJsonPath))
	{
		UE_LOGF(LogSpatialPlotView, Warning, "SSpatialPlotView: Failed to read sidecar JSON: %ls", *InJsonPath);
		return {};
	}

	TSharedPtr<FJsonObject> JsonObject;
	if (!FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(JsonString), JsonObject) || !JsonObject.IsValid())
	{
		UE_LOGF(LogSpatialPlotView, Warning, "SSpatialPlotView: Failed to parse JSON: %ls", *InJsonPath);
		return {};
	}

	const TArray<TSharedPtr<FJsonValue>>* BoundsMinArray = nullptr;
	const TArray<TSharedPtr<FJsonValue>>* BoundsMaxArray = nullptr;
	if (!JsonObject->TryGetArrayField(TEXT("worldBoundsMin"), BoundsMinArray) || BoundsMinArray->Num() < 2
		|| !JsonObject->TryGetArrayField(TEXT("worldBoundsMax"), BoundsMaxArray) || BoundsMaxArray->Num() < 2)
	{
		UE_LOGF(LogSpatialPlotView, Warning, "SSpatialPlotView: JSON missing worldBoundsMin/worldBoundsMax arrays: %ls", *InJsonPath);
		return {};
	}

	FBox2D Bounds;
	Bounds.Min = FVector2D((*BoundsMinArray)[0]->AsNumber(), (*BoundsMinArray)[1]->AsNumber());
	Bounds.Max = FVector2D((*BoundsMaxArray)[0]->AsNumber(), (*BoundsMaxArray)[1]->AsNumber());
	Bounds.bIsValid = true;
	return Bounds;
}

bool SSpatialPlotView::LoadBackgroundImage(const FString& InImagePath)
{
	ClearBackgroundImage();

	// Look for sidecar JSON.
	const FString JsonPath = InImagePath + TEXT(".json");
	TOptional<FBox2D> Bounds = ParseMinimapBoundsJson(JsonPath);
	if (!Bounds.IsSet())
	{
		UE_LOGF(LogSpatialPlotView, Warning, "SSpatialPlotView: No valid sidecar found at %ls. Use wp.Editor.ExportMinimapForInsights to export a minimap with metadata.", *JsonPath);
		return false;
	}

	// Load and decode image file (same pattern as MarkersTimingTrack screenshot loading).
	TArray<uint8> FileData;
	if (!FFileHelper::LoadFileToArray(FileData, *InImagePath))
	{
		UE_LOGF(LogSpatialPlotView, Warning, "Failed to read image file: %ls", *InImagePath);
		return false;
	}

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
	FImage Image;
	if (!ImageWrapperModule.DecompressImage(FileData.GetData(), FileData.Num(), Image))
	{
		UE_LOGF(LogSpatialPlotView, Warning, "Failed to decode image: %ls", *InImagePath);
		return false;
	}

	BackgroundImageBrush = FSlateDynamicImageBrush::CreateWithImageData(
		FName(*InImagePath),
		FVector2D(Image.SizeX, Image.SizeY),
		TArray<uint8>(Image.RawData));
	BackgroundWorldBounds = Bounds.GetValue();

	UE_LOGF(LogSpatialPlotView, Display, "SSpatialPlotView: Background image loaded (%dx%d): %ls", Image.SizeX, Image.SizeY, *InImagePath);
	return true;
}

void SSpatialPlotView::ClearBackgroundImage()
{
	BackgroundImageBrush.Reset();
	BackgroundWorldBounds.Reset();
}

int32 SSpatialPlotView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = PaintBackgroundImage(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	LayerId = PaintGrid(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	LayerId = PaintExtenders(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	LayerId = PaintLegends(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);

	FDrawContext DrawContext(AllottedGeometry, MyCullingRect, InWidgetStyle, ESlateDrawEffect::None, OutDrawElements, LayerId);
	Tooltip.Draw(DrawContext);

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect,
		OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

FReply SSpatialPlotView::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& Event)
{
	MousePositionOnButtonDown = MyGeometry.AbsoluteToLocal(Event.GetScreenSpacePosition());
	if (Event.GetEffectingButton() == EKeys::RightMouseButton)
	{
		bShouldAutoFrame = false;
		bIsPanning = true;
		LastPanCursor = MyGeometry.AbsoluteToLocal(Event.GetScreenSpacePosition());
		return FReply::Handled().CaptureMouse(AsShared());
	}
	return FReply::Unhandled();
}

FReply SSpatialPlotView::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& Event)
{
	FVector2D MousePosition = MyGeometry.AbsoluteToLocal(Event.GetScreenSpacePosition());

	if (HasMouseCapture())
	{
		if (bIsPanning)
		{
			SessionState.FocusCenter += (MousePosition - LastPanCursor) / SessionState.Scale;
			LastPanCursor = MousePosition;
			return FReply::Handled();
		}
	}

	if (!bIsPanning)
	{
		UpdateTooltip(MyGeometry, Event);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SSpatialPlotView::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& Event)
{
	MousePositionOnButtonUp = MyGeometry.AbsoluteToLocal(Event.GetScreenSpacePosition());
	bool bIsMouseClick = MousePositionOnButtonUp.Equals(MousePositionOnButtonDown, 2.0f);
	if (Event.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (bIsPanning)
		{
			bIsPanning = false;
		}

		if (bIsMouseClick)
		{
			ShowContextMenu(MyGeometry, Event);
		}

		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}

FReply SSpatialPlotView::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& Event)
{
	bShouldAutoFrame = false;
	float Delta = Event.GetWheelDelta() * 0.1f;
	// Zoom centered on cursor
	// Compute transforms before zoom
	FTransform2d WorldToScreenBefore, ScreenToWorldBefore;
	SessionState.ProduceTransforms(FVector2f(MyGeometry.GetLocalSize()), WorldToScreenBefore, ScreenToWorldBefore);
	FVector2f ScreenPos = MyGeometry.AbsoluteToLocal(Event.GetScreenSpacePosition());
	FVector2d WorldPosBefore = ScreenToWorldBefore.TransformPoint(FVector2d(ScreenPos));

	// Apply zoom
	SessionState.Scale = FMath::Clamp(SessionState.Scale * (1.f + Delta), Private::MinViewScale, Private::MaxViewScale);

	// Recompute transforms after zoom
	FTransform2d WorldToScreenAfter, ScreenToWorldAfter;
	SessionState.ProduceTransforms(FVector2f(MyGeometry.GetLocalSize()), WorldToScreenAfter, ScreenToWorldAfter);
	FVector2d WorldPosAfter = ScreenToWorldAfter.TransformPoint(FVector2d(ScreenPos));

	// Adjust focus so that world position under cursor stays fixed
	SessionState.FocusCenter += FVector2d(WorldPosAfter - WorldPosBefore);
	return FReply::Handled();
}

void SSpatialPlotView::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	bIsPanning = false;

	Super::OnMouseCaptureLost(CaptureLostEvent);
}

void SSpatialPlotView::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	Tooltip.SetDesiredOpacity(0.0f);
	HoveredFingerprint = 0;

	Super::OnMouseLeave(MouseEvent);
}

FCursorReply SSpatialPlotView::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return bIsPanning ? FCursorReply::Cursor(EMouseCursor::GrabHand)
		: FCursorReply::Unhandled();
}

void SSpatialPlotView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	FSpatialPlotViewExtenderTickParams TickParams;
	TickParams.CurrentTraceTime = CurrentTraceTime.Get();
	TickParams.DeltaTime = InDeltaTime;

	Tooltip.Update();

	TArray<ISpatialPlotViewExtender*> Extenders = GetExtenders();
	for (ISpatialPlotViewExtender* Extender : Extenders)
	{
		Extender->Tick(TickParams);
	}
	CacheExtenderDrawStates(Extenders);

	if (bShouldAutoFrame)
	{
		AutoFrameToContent(AllottedGeometry);
	}
}

TArray<ISpatialPlotViewExtender*> SSpatialPlotView::GetExtenders() const
{
	return IModularFeatures::Get().GetModularFeatureImplementations<ISpatialPlotViewExtender>(SpatialPlotViewExtenderFeatureName);
}

const SSpatialPlotView::FExtenderDrawState* SSpatialPlotView::FindCachedExtenderDrawState(const ISpatialPlotViewExtender* InExtender) const
{
	for (const FExtenderDrawState& State : CachedExtenderDrawStates)
	{
		if (State.Extender == InExtender)
		{
			return &State;
		}
	}
	return nullptr;
}

void SSpatialPlotView::SetExtenderVisible(FName InLayerName, bool bIsVisible)
{
	if (bIsVisible)
	{
		HiddenExtenders.Remove(InLayerName);
	}
	else
	{
		HiddenExtenders.Add(InLayerName);
	}
}

bool SSpatialPlotView::IsExtenderVisible(FName InLayerName) const
{
	return !HiddenExtenders.Contains(InLayerName);
}

void SSpatialPlotView::CacheExtenderDrawStates(const TArray<ISpatialPlotViewExtender*>& InExtenders)
{
	TArray<FExtenderDrawState> NewExtenderDrawStates;

	const double TraceTime = CurrentTraceTime.Get();
	const bool bTimeChanged = (TraceTime != CachedTraceTime);

	IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	TSharedPtr<const TraceServices::IAnalysisSession> CurrentSession = UnrealInsightsModule.GetAnalysisSession();
	if (CurrentSession.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*CurrentSession.Get());

		for (ISpatialPlotViewExtender* Extender : InExtenders)
		{
			if (HiddenExtenders.Contains(Extender->GetLayerName()))
			{
				continue;
			}

			FExtenderDrawState& NewExtenderDrawState = NewExtenderDrawStates.AddDefaulted_GetRef();
			NewExtenderDrawState.Extender = Extender;
			NewExtenderDrawState.CachedChangeSerial = Extender->GetChangeSerial();
			
			FExtenderDrawState* OldExtenderDrawStatePtr = CachedExtenderDrawStates.FindByPredicate([Extender](const FExtenderDrawState& DrawState)
			{
				return DrawState.Extender == Extender;
			});

			const bool bCanReuse = OldExtenderDrawStatePtr
				&& !bTimeChanged
				&& NewExtenderDrawState.CachedChangeSerial != 0
				&& NewExtenderDrawState.CachedChangeSerial == OldExtenderDrawStatePtr->CachedChangeSerial;

			if (bCanReuse)
			{
				NewExtenderDrawState.Regions = MoveTemp(OldExtenderDrawStatePtr->Regions);
				NewExtenderDrawState.Markers = MoveTemp(OldExtenderDrawStatePtr->Markers);
				NewExtenderDrawState.Legend = MoveTemp(OldExtenderDrawStatePtr->Legend);
			}
			else
			{
				Extender->EnumerateRegions([&NewExtenderDrawState](const FSpatialPlotRegion& Region)
				{
					NewExtenderDrawState.Regions.Add(Region);
				});

				Extender->EnumerateMarkers([&NewExtenderDrawState](const FSpatialPlotMarker& Marker)
				{
					NewExtenderDrawState.Markers.Add(Marker);
				});

				NewExtenderDrawState.Legend = Extender->GetLegend();

				HoveredFingerprint = 0;
			}
		}

		Swap(CachedExtenderDrawStates, NewExtenderDrawStates);
		CachedTraceTime = TraceTime;
	}
}

void SSpatialPlotView::AutoFrameToContent(const FGeometry& InAllottedGeometry)
{
	FBox2D ContentBounds(ForceInit);

	for (const FExtenderDrawState& DrawState : CachedExtenderDrawStates)
	{
		for (const FSpatialPlotRegion& Region : DrawState.Regions)
		{
			ContentBounds += Region.Bounds;
		}
		for (const FSpatialPlotMarker& Marker : DrawState.Markers)
		{
			ContentBounds += Marker.Position;
		}
	}

	if (!ContentBounds.bIsValid)
	{
		return; // No data yet, retry next Tick.
	}

	const FVector2D ContentSize = ContentBounds.GetSize();
	const FVector2D WidgetSize(InAllottedGeometry.GetLocalSize());

	if (WidgetSize.X <= 0.0 || WidgetSize.Y <= 0.0)
	{
		return;
	}

	// Pad content bounds so data doesn't touch widget edges.
	constexpr double AutoFramePaddingFactor = 1.2; // 10% padding on each side
	constexpr double MinExtent = 10000.0; // Minimum extent to avoid degenerate zoom on single-point data.
	const double PaddedWidth = FMath::Max(ContentSize.X, MinExtent) * AutoFramePaddingFactor;
	const double PaddedHeight = FMath::Max(ContentSize.Y, MinExtent) * AutoFramePaddingFactor;

	const double ScaleX = WidgetSize.X / PaddedWidth;
	const double ScaleY = WidgetSize.Y / PaddedHeight;

	SessionState.Scale = FMath::Clamp(static_cast<float>(FMath::Min(ScaleX, ScaleY)), Private::MinViewScale, Private::MaxViewScale);
	SessionState.FocusCenter = -ContentBounds.GetCenter();

	bShouldAutoFrame = false;
}

FSpatialPlotHitTestResult SSpatialPlotView::HitTestExtenderDrawState(const FExtenderDrawState& InExtenderDrawState, const FGeometry& InAllottedGeometry, const FPointerEvent& InMouseEvent) const
{
	FSpatialPlotHitTestResult HitTestResult;

	FVector2D MousePosition = InAllottedGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());
	FTransform2d WorldToScreen, ScreenToWorld;
	SessionState.ProduceTransforms(InAllottedGeometry.GetLocalSize(), WorldToScreen, ScreenToWorld);

	FVector2D WorldMousePosition = ScreenToWorld.TransformPoint(MousePosition);

	for (const FSpatialPlotRegion& Region : InExtenderDrawState.Regions)
	{
		if (Region.Bounds.IsInsideOrOn(WorldMousePosition))
		{
			HitTestResult.Regions.Add(Region);
		}
	}

	for (const FSpatialPlotMarker& Marker : InExtenderDrawState.Markers)
	{
		FVector2D ScreenMarkerPosition = WorldToScreen.TransformPoint(Marker.Position);
		if (FVector2D::DistSquared(ScreenMarkerPosition, MousePosition) <= Private::MarkerHitRadiusSq)
		{
			HitTestResult.Markers.Add(Marker);
		}
	}

	return HitTestResult;
}

void SSpatialPlotView::UpdateTooltip(const FGeometry& InAllottedGeometry, const FPointerEvent& InMouseEvent)
{
	// Iterate live extenders so we never dispatch to one that unregistered since the last Tick.
	const TArray<ISpatialPlotViewExtender*> LiveExtenders = GetExtenders();

	struct FExtenderHit
	{
		ISpatialPlotViewExtender* Extender = nullptr;
		FSpatialPlotHitTestResult Result;
	};
	TArray<FExtenderHit> Hits;
	Hits.Reserve(LiveExtenders.Num());

	uint32 NewFingerprint = 0;
	for (ISpatialPlotViewExtender* Extender : LiveExtenders)
	{
		if (const FExtenderDrawState* CachedExtenderDrawState = FindCachedExtenderDrawState(Extender))
		{
			FExtenderHit& Hit = Hits.AddDefaulted_GetRef();
			Hit.Extender = Extender;
			Hit.Result = HitTestExtenderDrawState(*CachedExtenderDrawState, InAllottedGeometry, InMouseEvent);
			for (const FSpatialPlotRegion& Region : Hit.Result.Regions)
			{
				NewFingerprint = HashCombine(NewFingerprint, GetTypeHash(Region.Id));
			}
			for (const FSpatialPlotMarker& Marker : Hit.Result.Markers)
			{
				NewFingerprint = HashCombine(NewFingerprint, GetTypeHash(Marker.Id));
			}
		}
	}

	if (NewFingerprint != HoveredFingerprint)
	{
		HoveredFingerprint = NewFingerprint;
		if (NewFingerprint == 0)
		{
			Tooltip.SetDesiredOpacity(0.0f);
			return;
		}

		Tooltip.ResetContent();

		bool bHasContent = false;
		for (const FExtenderHit& Hit : Hits)
		{
			bHasContent |= Hit.Extender->AppendTooltip(Tooltip, Hit.Result);
		}

		if (bHasContent)
		{
			Tooltip.UpdateLayout();
			Tooltip.SetDesiredOpacity(1.f);
		}
		else
		{
			Tooltip.SetDesiredOpacity(0.0f);
		}
	}

	FVector2D MousePosition = InAllottedGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());
	Tooltip.SetPosition(MousePosition, 0, InAllottedGeometry.GetLocalSize().X, 0, InAllottedGeometry.GetLocalSize().Y);
}

void SSpatialPlotView::ShowContextMenu(const FGeometry& InAllottedGeometry, const FPointerEvent& InMouseEvent)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	FVector2D MousePosition = InAllottedGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());
	FTransform2d WorldToScreen, ScreenToWorld;
	SessionState.ProduceTransforms(InAllottedGeometry.GetLocalSize(), WorldToScreen, ScreenToWorld);
	FVector2D WorldMousePosition = ScreenToWorld.TransformPoint(MousePosition);

	FString BugItGoCommand = FString::Printf(TEXT("BugItGo %.1f %.1f %.1f 0 0 0"), WorldMousePosition.X, WorldMousePosition.Y, Private::BugItGoDefaultZ);

	MenuBuilder.BeginSection(TEXT("Navigation"));
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CopyBugItGoCommand", "Copy BugItGo Command"),
		LOCTEXT("CopyBugItGoCommandTooltip", "Copy a BugItGo console command for this location to the clipboard"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([BugItGoCommand = MoveTemp(BugItGoCommand)]() { FPlatformApplicationMisc::ClipboardCopy(*BugItGoCommand); }))
	);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("BackgroundImage"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("LoadBackgroundImage", "Load Background Image..."),
			LOCTEXT("LoadBackgroundImageTooltip", "Load a minimap image exported with wp.Editor.ExportMinimapForInsights"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
				if (!DesktopPlatform)
				{
					return;
				}

				TArray<FString> OutFiles;
				const bool bOpened = DesktopPlatform->OpenFileDialog(
					FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
					LOCTEXT("LoadBackgroundImageDialogTitle", "Load Background Image").ToString(),
					FString(),
					FString(),
					TEXT("Image Files (*.png;*.jpg;*.jpeg)|*.png;*.jpg;*.jpeg"),
					EFileDialogFlags::None,
					OutFiles);

				if (bOpened && OutFiles.Num() > 0)
				{
					this->LoadBackgroundImage(OutFiles[0]);
				}
			}))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ClearBackgroundImage", "Clear Background Image"),
			LOCTEXT("ClearBackgroundImageTooltip", "Remove the background image"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]() { this->ClearBackgroundImage(); }),
				FCanExecuteAction::CreateLambda([this]() { return BackgroundImageBrush.IsValid(); }))
		);
	}
	MenuBuilder.EndSection();

	for (ISpatialPlotViewExtender* Extender : GetExtenders())
	{
		if (const FExtenderDrawState* CachedExtenderDrawState = FindCachedExtenderDrawState(Extender))
		{
			FSpatialPlotHitTestResult HitTestResult = HitTestExtenderDrawState(*CachedExtenderDrawState, InAllottedGeometry, InMouseEvent);
			Extender->ExtendContextMenu(MenuBuilder, HitTestResult);
		}
	}

	TSharedRef<SWidget> MenuWidget = MenuBuilder.MakeWidget();

	FWidgetPath EventPath = InMouseEvent.GetEventPath() != nullptr ? *InMouseEvent.GetEventPath() : FWidgetPath();
	const FVector2D ScreenSpacePosition = InMouseEvent.GetScreenSpacePosition();
	FSlateApplication::Get().PushMenu(SharedThis(this), EventPath, MenuWidget, ScreenSpacePosition, FPopupTransitionEffect::ContextMenu);
}

int32 SSpatialPlotView::PaintBackgroundImage(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	if (!BackgroundWorldBounds.IsSet() || !BackgroundImageBrush.IsValid())
	{
		return LayerId;
	}

	FTransform2d WorldToScreen, ScreenToWorld;
	SessionState.ProduceTransforms(AllottedGeometry.GetLocalSize(), WorldToScreen, ScreenToWorld);

	const FBox2D ScreenBounds = Private::TransformBox2D(BackgroundWorldBounds.GetValue(), WorldToScreen);
	const FPaintGeometry Geo = AllottedGeometry.ToPaintGeometry(
		ScreenBounds.GetSize(),
		FSlateLayoutTransform(ScreenBounds.Min));

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		Geo,
		BackgroundImageBrush.Get(),
		ESlateDrawEffect::None,
		FLinearColor::White);

	return LayerId + 1;
}

int32 SSpatialPlotView::PaintGrid(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements,
	int32 LayerId) const
{
	constexpr int32 BaseCellSize = 100;
	constexpr float WantedCellScreenSize = 16.0f;
	constexpr float BaseAlpha = 0.25f;

	FTransform2d WorldToScreen, ScreenToWorld;
	SessionState.ProduceTransforms(AllottedGeometry.GetLocalSize(), WorldToScreen, ScreenToWorld);

	// Calculate effective cell size using bit operations
	const FVector2f CellVector = WorldToScreen.TransformVector(FVector2f(BaseCellSize, BaseCellSize));
	const float BaseCellScreenSize = CellVector.GetMax();
	if (BaseCellScreenSize <= 0.0f)
	{
		return LayerId;
	}

	int32 EffectiveCellSize = BaseCellSize;
	if (BaseCellScreenSize <= WantedCellScreenSize)
	{
		const float InflationRatio = WantedCellScreenSize / BaseCellScreenSize;
		const uint32 Inflation = FMath::RoundUpToPowerOfTwo(FMath::CeilToInt(InflationRatio));
		EffectiveCellSize = BaseCellSize * Inflation;
	}

	const float CellScreenSize = WorldToScreen.TransformVector(FVector2f(static_cast<float>(EffectiveCellSize))).GetAbsMax();

	// Compute visible rect
	const FBox2f ViewRect(FVector2f::ZeroVector, AllottedGeometry.GetLocalSize());
	const FBox2d ViewRectWorld = Private::TransformBox2D(FBox2D(ViewRect), ScreenToWorld);

	// Early exit if no visible area
	if (ViewRectWorld.GetArea() <= 0.0)
	{
		return LayerId;
	}

	// Pre-calculate reusable values
	const FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry();

	struct FGridLevel
	{
		int32 Granularity;
		float Thickness;
	};

	// Define grid hierarchy
	constexpr FGridLevel GridLevels[] = {
		{ 4, 1.0f },  // Fine grid
		{ 2, 1.0f },  // Medium grid
		{ 1, 2.0f }   // Coarse grid (thicker)
	};

	for (const FGridLevel& Level : GridLevels)
	{
		const int64 CurrentCellSize = EffectiveCellSize / Level.Granularity;
		const float CurrentCellScreenSize = CellScreenSize / static_cast<float>(Level.Granularity);

		// Calculate alpha with smooth transitions
		float Alpha = FMath::Min(CurrentCellScreenSize / WantedCellScreenSize, 1.0f);

		// Fade for very fine grids
		if (CurrentCellScreenSize < WantedCellScreenSize * 0.5f)
		{
			Alpha *= FMath::Max(0.0f, (CurrentCellScreenSize - WantedCellScreenSize * 0.25f) / (WantedCellScreenSize * 0.25f));
		}

		// Conditional alpha adjustment based on resolution
		const bool bAtBaseResolution = (EffectiveCellSize >= BaseCellSize * Level.Granularity);
		if (!bAtBaseResolution)
		{
			Alpha *= 0.5f;
		}

		// Skip if alpha too low to be visible
		if (Alpha <= 0.01f)
		{
			continue;
		}

		const FLinearColor GridColor(0.1f, 0.1f, 0.1f, Alpha * BaseAlpha);

		// Calculate grid bounds using integer coordinates
		const double CellSizeD = static_cast<double>(CurrentCellSize);
		const int64 MinGridX = FMath::FloorToInt64(ViewRectWorld.Min.X / CellSizeD);
		const int64 MaxGridX = FMath::CeilToInt64(ViewRectWorld.Max.X / CellSizeD);
		const int64 MinGridY = FMath::FloorToInt64(ViewRectWorld.Min.Y / CellSizeD);
		const int64 MaxGridY = FMath::CeilToInt64(ViewRectWorld.Max.Y / CellSizeD);

		// Skip if no lines visible
		if (MinGridX >= MaxGridX || MinGridY >= MaxGridY)
		{
			continue;
		}

		// Pre-calculate world bounds
		const double WorldMinX = static_cast<double>(MinGridX) * CellSizeD;
		const double WorldMaxX = static_cast<double>(MaxGridX) * CellSizeD;
		const double WorldMinY = static_cast<double>(MinGridY) * CellSizeD;
		const double WorldMaxY = static_cast<double>(MaxGridY) * CellSizeD;

		// Draw horizontal lines
		for (int64 GridY = MinGridY; GridY <= MaxGridY; ++GridY)
		{
			const double Y = static_cast<double>(GridY) * CellSizeD;
			const FVector2f LineStart(WorldToScreen.TransformPoint(FVector2d(WorldMinX, Y)));
			const FVector2f LineEnd(WorldToScreen.TransformPoint(FVector2d(WorldMaxX, Y)));

			FSlateDrawElement::MakeLines(
				OutDrawElements, LayerId, PaintGeometry, { LineStart, LineEnd },
				ESlateDrawEffect::None, GridColor, false, Level.Thickness);
		}

		// Draw vertical lines
		for (int64 GridX = MinGridX; GridX <= MaxGridX; ++GridX)
		{
			const double X = static_cast<double>(GridX) * CellSizeD;
			const FVector2f LineStart(WorldToScreen.TransformPoint(FVector2d(X, WorldMinY)));
			const FVector2f LineEnd(WorldToScreen.TransformPoint(FVector2d(X, WorldMaxY)));

			FSlateDrawElement::MakeLines(
				OutDrawElements, LayerId, PaintGeometry, { LineStart, LineEnd },
				ESlateDrawEffect::None, GridColor, false, Level.Thickness);
		}
	}

	return LayerId + 1;
}

int32 SSpatialPlotView::PaintExtenders(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	if (CachedExtenderDrawStates.IsEmpty())
	{
		return LayerId;
	}

	const FSlateBrush* Brush = FAppStyle::Get().GetBrush("WhiteBrush");

	FTransform2d WorldToScreen, ScreenToWorld;
	SessionState.ProduceTransforms(AllottedGeometry.GetLocalSize(), WorldToScreen, ScreenToWorld);

	for (const FExtenderDrawState& ExtenderDrawState : CachedExtenderDrawStates)
	{
		for (const FSpatialPlotRegion& Region : ExtenderDrawState.Regions)
		{
			FBox2D ScreenBounds = Private::TransformBox2D(Region.Bounds, WorldToScreen);
			FPaintGeometry Geo = AllottedGeometry.ToPaintGeometry(ScreenBounds.Max - ScreenBounds.Min, FSlateLayoutTransform(ScreenBounds.Min));
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId, Geo, Brush, ESlateDrawEffect::None, Region.FillColor);

			const FVector2f BottomLeft(ScreenBounds.Min);
			const FVector2f TopRight(ScreenBounds.Max);
			TArray<FVector2f> BorderPoints = {
				BottomLeft,
				FVector2f(BottomLeft.X, TopRight.Y),
				TopRight,
				FVector2f(TopRight.X, BottomLeft.Y),
				BottomLeft
			};
			FSlateDrawElement::MakeLines(OutDrawElements, LayerId + 1, AllottedGeometry.ToPaintGeometry(), BorderPoints, ESlateDrawEffect::None, Region.BorderColor);
		}

		for (const FSpatialPlotMarker& Marker : ExtenderDrawState.Markers)
		{
			FVector2d ScreenPosition = WorldToScreen.TransformPoint(Marker.Position);
			FPaintGeometry Geo = AllottedGeometry.ToPaintGeometry(FVector2f(Private::MarkerSize, Private::MarkerSize), FSlateLayoutTransform(ScreenPosition - FVector2D(Private::MarkerSize * 0.5f, Private::MarkerSize * 0.5f)));
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId + 2, Geo, Brush, ESlateDrawEffect::None, Marker.Color);
		}
	}

	return LayerId + 3;
}

int32 SSpatialPlotView::PaintLegends(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	// Collect legends from all cached extender draw states.
	TArray<const FSpatialPlotLegend*> Legends;
	for (const FExtenderDrawState& DrawState : CachedExtenderDrawStates)
	{
		if (DrawState.Legend.IsSet())
		{
			Legends.Add(&DrawState.Legend.GetValue());
		}
	}

	if (Legends.IsEmpty())
	{
		return LayerId;
	}

	const FSlateBrush* WhiteBrush = FAppStyle::Get().GetBrush("WhiteBrush");
	const FSlateFontInfo Font = FAppStyle::Get().GetFontStyle("SmallFont");
	const FSlateFontInfo BoldFont = FAppStyle::Get().GetFontStyle("SmallFontBold");
	const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const float FontScale = AllottedGeometry.Scale;

	constexpr float Padding = 6.0f;
	constexpr float SwatchSize = 12.0f;
	constexpr float SwatchGap = 4.0f;
	constexpr float LineHeight = 16.0f;
	constexpr float TitleHeight = 16.0f;
	constexpr float TitleContentGap = 4.0f;
	constexpr float GradientBarWidth = 16.0f;
	constexpr float GradientBarHeight = 100.0f;
	constexpr float GradientLabelGap = 4.0f;
	constexpr float SectionSeparator = 6.0f;

	// --- Layout pass: compute total legend box dimensions ---
	float TotalHeight = Padding;
	float MaxWidth = 0.0f;

	for (int32 LegendIdx = 0; LegendIdx < Legends.Num(); ++LegendIdx)
	{
		const FSpatialPlotLegend& Legend = *Legends[LegendIdx];

		if (LegendIdx > 0)
		{
			TotalHeight += SectionSeparator;
		}

		// Title
		const FVector2D TitleSize = FontMeasure->Measure(Legend.Title.ToString(), BoldFont, FontScale) / FontScale;
		MaxWidth = FMath::Max(MaxWidth, Padding + static_cast<float>(TitleSize.X) + Padding);
		TotalHeight += TitleHeight + TitleContentGap;

		if (Legend.Type == ESpatialPlotLegendType::Discrete)
		{
			for (const FSpatialPlotLegendEntry& Entry : Legend.Entries)
			{
				const FString LabelStr = Entry.Label.ToString();
				const FVector2D LabelSize = FontMeasure->Measure(LabelStr, Font, FontScale) / FontScale;
				MaxWidth = FMath::Max(MaxWidth, Padding + SwatchSize + SwatchGap + static_cast<float>(LabelSize.X) + Padding);
				TotalHeight += LineHeight;
			}
		}
		else // Gradient
		{
			if (Legend.Entries.Num() >= 2)
			{
				float MaxLabelWidth = 0.0f;
				for (const FSpatialPlotLegendEntry& Entry : Legend.Entries)
				{
					if (!Entry.Label.IsEmpty())
					{
						const FString LabelStr = Entry.Label.ToString();
						const FVector2D LabelSize = FontMeasure->Measure(LabelStr, Font, FontScale) / FontScale;
						MaxLabelWidth = FMath::Max(MaxLabelWidth, static_cast<float>(LabelSize.X));
					}
				}
				MaxWidth = FMath::Max(MaxWidth, Padding + GradientBarWidth + GradientLabelGap + MaxLabelWidth + Padding);
				TotalHeight += GradientBarHeight;
			}
		}
	}
	TotalHeight += Padding;

	// --- Position: bottom-left corner ---
	const FVector2D WidgetSize = AllottedGeometry.GetLocalSize();
	const float BoxX = Padding;
	const float BoxY = static_cast<float>(WidgetSize.Y) - TotalHeight - Padding;

	// --- Draw background ---
	const FLinearColor BackgroundColor(0.05f, 0.05f, 0.05f, 0.85f);
	FSlateDrawElement::MakeBox(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(FVector2D(MaxWidth, TotalHeight), FSlateLayoutTransform(FVector2D(BoxX, BoxY))), WhiteBrush, ESlateDrawEffect::None, BackgroundColor);
	LayerId++;

	// --- Draw each legend section ---
	float CursorY = BoxY + Padding;

	for (int32 LegendIdx = 0; LegendIdx < Legends.Num(); ++LegendIdx)
	{
		const FSpatialPlotLegend& Legend = *Legends[LegendIdx];

		if (LegendIdx > 0)
		{
			// Separator line
			const FLinearColor SeparatorColor(0.3f, 0.3f, 0.3f, 0.6f);
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(FVector2D(MaxWidth - 2.0f * Padding, 1.0f), FSlateLayoutTransform(FVector2D(BoxX + Padding, CursorY + SectionSeparator * 0.5f - 0.5f))), WhiteBrush, ESlateDrawEffect::None, SeparatorColor);
			CursorY += SectionSeparator;
		}

		// Title
		const FLinearColor TitleColor(0.9f, 0.9f, 0.5f, 1.0f);
		FSlateDrawElement::MakeText(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(FSlateLayoutTransform(1.0f, FVector2D(BoxX + Padding, CursorY))), Legend.Title.ToString(), BoldFont, ESlateDrawEffect::None, TitleColor);
		CursorY += TitleHeight + TitleContentGap;

		if (Legend.Type == ESpatialPlotLegendType::Discrete)
		{
			for (const FSpatialPlotLegendEntry& Entry : Legend.Entries)
			{
				// Color swatch
				const float SwatchY = CursorY + (LineHeight - SwatchSize) * 0.5f;
				FSlateDrawElement::MakeBox(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(FVector2D(SwatchSize, SwatchSize), FSlateLayoutTransform(FVector2D(BoxX + Padding, SwatchY))), WhiteBrush, ESlateDrawEffect::None, Entry.Color);

				// Label
				const FLinearColor LabelColor(0.9f, 0.9f, 0.9f, 1.0f);
				FSlateDrawElement::MakeText(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(FSlateLayoutTransform(1.0f, FVector2D(BoxX + Padding + SwatchSize + SwatchGap, CursorY + 1.0f))), Entry.Label.ToString(), Font, ESlateDrawEffect::None, LabelColor);

				CursorY += LineHeight;
			}
		}
		else // Gradient
		{
			if (Legend.Entries.Num() >= 2)
			{
				const float BarX = BoxX + Padding;
				const float BarY = CursorY;

				// Build gradient stops (top to bottom).
				TArray<FSlateGradientStop> GradientStops;
				const int32 NumStops = Legend.Entries.Num();
				for (int32 StopIndex = 0; StopIndex < NumStops; ++StopIndex)
				{
					const float StopY = GradientBarHeight * (static_cast<float>(StopIndex) / static_cast<float>(NumStops - 1));
					GradientStops.Add(FSlateGradientStop(FVector2D(0.0f, StopY), Legend.Entries[StopIndex].Color));
				}

				// Orient_Horizontal = horizontal stripes = gradient runs top-to-bottom (uses Position.Y).
				FSlateDrawElement::MakeGradient(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(FVector2D(GradientBarWidth, GradientBarHeight), FSlateLayoutTransform(FVector2D(BarX, BarY))), GradientStops, Orient_Horizontal, ESlateDrawEffect::None);

				// Labels at stops with non-empty text
				const FLinearColor LabelColor(0.9f, 0.9f, 0.9f, 1.0f);
				const float LabelX = BarX + GradientBarWidth + GradientLabelGap;
				for (int32 StopIndex = 0; StopIndex < NumStops; ++StopIndex)
				{
					if (!Legend.Entries[StopIndex].Label.IsEmpty())
					{
						const FString LabelStr = Legend.Entries[StopIndex].Label.ToString();
						const FVector2D LabelSize = FontMeasure->Measure(LabelStr, Font, FontScale) / FontScale;
						const float StopY = GradientBarHeight * (static_cast<float>(StopIndex) / static_cast<float>(NumStops - 1));
						const float LabelY = BarY + StopY - static_cast<float>(LabelSize.Y) * 0.5f;
						FSlateDrawElement::MakeText(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(FSlateLayoutTransform(1.0f, FVector2D(LabelX, LabelY))), LabelStr, Font, ESlateDrawEffect::None, LabelColor);
					}
				}

				CursorY += GradientBarHeight;
			}
		}
	}

	return LayerId + 1;
}

} // namespace UE::Insights::SpatialProfiler

#undef LOCTEXT_NAMESPACE
