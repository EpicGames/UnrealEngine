// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Insights/SpatialProfiler/ISpatialPlotViewExtender.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Layout/Geometry.h"
#include "Rendering/RenderingCommon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Widgets/SCompoundWidget.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSpatialPlotView, Log, All);

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::SpatialProfiler
{

/**
 * 2D spatial visualization widget for the Spatial Profiler.
 */
class SSpatialPlotView : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SSpatialPlotView, SCompoundWidget)

public:

	SSpatialPlotView();
	virtual ~SSpatialPlotView() override;

	SLATE_BEGIN_ARGS(SSpatialPlotView)
	{
		_Clipping = EWidgetClipping::ClipToBounds;
	}
		SLATE_ATTRIBUTE(double, CurrentTraceTime);
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	void Reset(bool bIsFirstReset = false);

private:

	//~ Begin SWidget interface
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& Event) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& Event) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& Event) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& Event) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget interface

	struct FSpatialViewState
	{
		/** Current map focus center in world coords. */
		FVector2D FocusCenter = FVector2D::ZeroVector;

		/** Current zoom scale. */
		float Scale = 1.0f;

		/** Current rotation in radians. */
		float Rotation = 0.0f;

		/** Build both transforms (world->screen and screen->world). */
		void ProduceTransforms(const FVector2f& InWidgetSize, FTransform2d& OutWorldToScreen, FTransform2d& OutScreenToWorld) const;
	};

	int32 PaintBackgroundImage(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;
	int32 PaintGrid(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;
	int32 PaintExtenders(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;
	int32 PaintLegends(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	TArray<ISpatialPlotViewExtender*> GetExtenders() const;

	struct FExtenderDrawState
	{
		ISpatialPlotViewExtender* Extender = nullptr;
		uint32 CachedChangeSerial = 0;
		TArray<FSpatialPlotRegion> Regions;
		TArray<FSpatialPlotMarker> Markers;
		TOptional<FSpatialPlotLegend> Legend;
	};

	void CacheExtenderDrawStates(const TArray<ISpatialPlotViewExtender*>& InExtenders);
	void AutoFrameToContent(const FGeometry& InAllottedGeometry);
	FSpatialPlotHitTestResult HitTestExtenderDrawState(const FExtenderDrawState& InExtenderDrawState, const FGeometry& InAllottedGeometry, const FPointerEvent& InMouseEvent) const;

	void UpdateTooltip(const FGeometry& InAllottedGeometry, const FPointerEvent& InMouseEvent);
	void ShowContextMenu(const FGeometry& InAllottedGeometry, const FPointerEvent& InMouseEvent);

public:
	void SetExtenderVisible(FName InLayerName, bool bIsVisible);
	bool IsExtenderVisible(FName InLayerName) const;

private:
	/** Internal pan/zoom session state. */
	FSpatialViewState SessionState;

	TAttribute<double> CurrentTraceTime;

	/** Track panning. */
	bool bIsPanning = false;
	FVector2D LastPanCursor;

	FVector2D MousePositionOnButtonDown;
	FVector2D MousePositionOnButtonUp;

	TArray<FExtenderDrawState> CachedExtenderDrawStates;

	const FExtenderDrawState* FindCachedExtenderDrawState(const ISpatialPlotViewExtender* InExtender) const;

	FTooltipDrawState Tooltip;
	uint32 HoveredFingerprint = 0;
	double CachedTraceTime = -1.0;

	/** When true, the view will auto-frame to content on the next Tick with non-empty data. */
	bool bShouldAutoFrame = true;

	/** Extenders in this set are hidden from rendering (but still ticked). */
	TSet<FName> HiddenExtenders;

	/** Background image (minimap) state. */
	bool LoadBackgroundImage(const FString& InImagePath);
	void ClearBackgroundImage();

	TSharedPtr<FSlateDynamicImageBrush> BackgroundImageBrush;
	TOptional<FBox2D> BackgroundWorldBounds;
};

} // namespace UE::Insights::SpatialProfiler

////////////////////////////////////////////////////////////////////////////////////////////////////
