// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/Table/ViewModels/TableCommands.h"

#include "HAL/PlatformApplicationMisc.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"

namespace UE::Insights::Private
{

void CopyToClipboardImpl(TSharedRef<FTable> Table, TArray<FBaseTreeNodePtr> SelectedNodes, TSharedPtr<ITableCellValueSorter> Sorter, ESortMode ColumnSortMode)
{
	if (SelectedNodes.Num() == 0)
	{
		return;
	}

	FString ClipboardText;

	if (Sorter.IsValid())
	{
		Sorter->Sort(SelectedNodes, ColumnSortMode);
	}

	Table->GetVisibleColumnsData(SelectedNodes, NAME_None, TEXT('\t'), true, ClipboardText);

	if (ClipboardText.Len() > 0)
	{
		FPlatformApplicationMisc::ClipboardCopy(*ClipboardText);
	}
}

void CopyNameToClipboardImpl(TSharedRef<FTable> Table, TArray<FBaseTreeNodePtr> SelectedNodes, TSharedPtr<ITableCellValueSorter> Sorter, ESortMode ColumnSortMode)
{
	if (SelectedNodes.Num() == 0)
	{
		return;
	}

	if (Sorter.IsValid())
	{
		Sorter->Sort(SelectedNodes, ColumnSortMode);
	}

	FString ClipboardText = FString::JoinBy(
		SelectedNodes,
		TEXT("\n"),
		[](const FBaseTreeNodePtr& Node)
		{
			return Node->GetDisplayName().ToString();
		});

	if (ClipboardText.Len() > 0)
	{
		FPlatformApplicationMisc::ClipboardCopy(*ClipboardText);
	}
}

} // namespace UE::Insights::Private
