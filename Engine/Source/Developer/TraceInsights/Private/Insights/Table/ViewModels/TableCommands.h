// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Algo/Transform.h"
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "Templates/SharedPointer.h"

namespace UE::Insights
{

class FTable;
class ITableCellValueSorter;

namespace Private
{
	template<class TreeNodePtrType>
	TArray<FBaseTreeNodePtr> ConvertNodes(const TArray<TreeNodePtrType>& InNodes)
	{
		TArray<FBaseTreeNodePtr> BaseNodes;
		Algo::Transform(InNodes, BaseNodes, [](const TreeNodePtrType& Ptr)
		{
			return StaticCastSharedPtr<FBaseTreeNodePtr::ElementType>(Ptr);
		});
		return BaseNodes;
	}

	void CopyToClipboardImpl(TSharedRef<FTable> Table, TArray<FBaseTreeNodePtr> SelectedNodes, TSharedPtr<ITableCellValueSorter> Sorter, ESortMode ColumnSortMode);
	void CopyNameToClipboardImpl(TSharedRef<FTable> Table, TArray<FBaseTreeNodePtr> SelectedNodes, TSharedPtr<ITableCellValueSorter> Sorter, ESortMode ColumnSortMode);

} // Private

template<class TreeNodePtrType>
void CopyToClipboard(TSharedRef<FTable> Table, const TArray<TreeNodePtrType>& SelectedNodes, TSharedPtr<ITableCellValueSorter> Sorter, ESortMode ColumnSortMode)
{
	Private::CopyToClipboardImpl(Table, Private::ConvertNodes(SelectedNodes), MoveTemp(Sorter), ColumnSortMode);
}

template<class TreeNodePtrType>
void CopyNameToClipboard(TSharedRef<FTable> Table, const TArray<TreeNodePtrType>& SelectedNodes, TSharedPtr<ITableCellValueSorter> Sorter, ESortMode ColumnSortMode)
{
	Private::CopyNameToClipboardImpl(Table, Private::ConvertNodes(SelectedNodes), MoveTemp(Sorter), ColumnSortMode);
}

} // namespace UE::Insights