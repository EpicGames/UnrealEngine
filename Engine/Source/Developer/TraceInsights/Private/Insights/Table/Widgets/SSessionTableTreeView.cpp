// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSessionTableTreeView.h"

// TraceInsights
#include "Insights/InsightsManager.h"

#define LOCTEXT_NAMESPACE "UE::Insights::SSessionTableTreeView"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// SSessionTableTreeView
////////////////////////////////////////////////////////////////////////////////////////////////////

SSessionTableTreeView::~SSessionTableTreeView()
{
	// Remove ourselves from the Insights manager.
	if (FInsightsManager::Get().IsValid())
	{
		FInsightsManager::Get()->GetSessionChangedEvent().RemoveAll(this);
	}

	Session.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSessionTableTreeView::ConstructWidget(TSharedPtr<FTable> InTablePtr)
{
	UE::Insights::STableTreeView::ConstructWidget(InTablePtr);

	// Register ourselves with the Insights manager.
	FInsightsManager::Get()->GetSessionChangedEvent().AddSP(this, &SSessionTableTreeView::InsightsManager_OnSessionChanged);

	// Update the Session (i.e. when analysis session was already started).
	InsightsManager_OnSessionChanged();

	CreateGroupings();
	CreateSortings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSessionTableTreeView::InsightsManager_OnSessionChanged()
{
	TSharedPtr<const TraceServices::IAnalysisSession> NewSession = FInsightsManager::Get()->GetSession();
	if (NewSession != Session)
	{
		Session = NewSession;
		Reset();
	}
	else
	{
		UpdateTree();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
