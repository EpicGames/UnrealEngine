// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// TraceInsightsCore
#include "InsightsCore/Table/Widgets/STableTreeView.h"

namespace TraceServices
{
	class IAnalysisSession;
}

namespace UE::Insights
{

class FTable;

class SSessionTableTreeView : public STableTreeView
{
public:
	virtual ~SSessionTableTreeView() override;

protected:
	virtual void ConstructWidget(TSharedPtr<FTable> InTablePtr) override;

	/** Called when the analysis session has changed. */
	void InsightsManager_OnSessionChanged();

protected:
	/** The analysis session used to populate this widget. */
	TSharedPtr<const TraceServices::IAnalysisSession> Session;
};

} // namespace UE::Insights
