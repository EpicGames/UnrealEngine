// Copyright Epic Games, Inc. All Rights Reserved.

#include "TaskGraphProfilerManager.h"

#include "Async/TaskGraphInterfaces.h"
#include "Features/IModularFeatures.h"
#include "Framework/Docking/TabManager.h"
#include "Logging/MessageLog.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"

// TraceServices
#include "TraceServices/Model/TasksProfiler.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/TaskGraphProfiler/ViewModels/TaskGraphRelation.h"
#include "Insights/TaskGraphProfiler/ViewModels/TaskTable.h"
#include "Insights/TaskGraphProfiler/ViewModels/TaskTimingTrack.h"
#include "Insights/TaskGraphProfiler/Widgets/STaskTableTreeView.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Tracks/CpuTimingTrack.h"
#include "Insights/TimingProfiler/ViewModels/ThreadTimingSharedState.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/ViewModels/ThreadTrackEvent.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TaskGraphProfiler"

namespace UE::Insights::TaskGraphProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

const FName FTaskGraphProfilerTabs::TaskTableTreeViewTabID(TEXT("TaskTableTreeView"));

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTaskGraphProfilerManager> FTaskGraphProfilerManager::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTaskGraphProfilerManager> FTaskGraphProfilerManager::Get()
{
	return FTaskGraphProfilerManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTaskGraphProfilerManager> FTaskGraphProfilerManager::CreateInstance()
{
	ensure(!FTaskGraphProfilerManager::Instance.IsValid());
	if (FTaskGraphProfilerManager::Instance.IsValid())
	{
		FTaskGraphProfilerManager::Instance.Reset();
	}

	FTaskGraphProfilerManager::Instance = MakeShared<FTaskGraphProfilerManager>(FInsightsManager::Get()->GetCommandList());

	return FTaskGraphProfilerManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTaskGraphProfilerManager::FTaskGraphProfilerManager(TSharedRef<FUICommandList> InCommandList)
	: bIsInitialized(false)
	, bIsAvailable(false)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::Initialize(IUnrealInsightsModule& InsightsModule)
{
	ensure(!bIsInitialized);
	if (bIsInitialized)
	{
		return;
	}
	bIsInitialized = true;

	InitializeColorCode();

	// Register tick functions.
	OnTick = FTickerDelegate::CreateSP(this, &FTaskGraphProfilerManager::Tick);
	OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(OnTick, 0.0f);

	FOnRegisterMajorTabExtensions* TimingProfilerLayoutExtension = InsightsModule.FindMajorTabLayoutExtension(FInsightsManagerTabs::TimingProfilerTabId);
	if (TimingProfilerLayoutExtension)
	{
		TimingProfilerLayoutExtension->AddRaw(this, &FTaskGraphProfilerManager::RegisterTimingProfilerLayoutExtensions);
	}

	FInsightsManager::Get()->GetSessionChangedEvent().AddSP(this, &FTaskGraphProfilerManager::OnSessionChanged);
	OnSessionChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::Shutdown()
{
	if (!bIsInitialized)
	{
		return;
	}
	bIsInitialized = false;

	FInsightsManager::Get()->GetSessionChangedEvent().RemoveAll(this);

	// Unregister tick function.
	FTSTicker::RemoveTicker(OnTickHandle);

	FTaskGraphProfilerManager::Instance.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTaskGraphProfilerManager::~FTaskGraphProfilerManager()
{
	ensure(!bIsInitialized);

	if (TaskTimingSharedState.IsValid())
	{
		IModularFeatures::Get().UnregisterModularFeature(Timing::TimingViewExtenderFeatureName, TaskTimingSharedState.Get());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::RegisterMajorTabs(IUnrealInsightsModule& InsightsModule)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::UnregisterMajorTabs()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTaskGraphProfilerManager::Tick(float DeltaTime)
{
	// Check if session has task events (to spawn the tab), but not too often.
	if (!bIsAvailable && AvailabilityCheck.Tick())
	{
		bool bShouldBeAvailable = false;

		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			const TraceServices::ITasksProvider* TasksProvider = TraceServices::ReadTasksProvider(*Session.Get());
			TSharedPtr<FTabManager> TabManagerShared = TimingTabManager.Pin();
			if (TasksProvider && TasksProvider->GetNumTasks() > 0 && TabManagerShared.IsValid())
			{
				TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
				if (!TimingView.IsValid())
				{
					return true;
				}

				bIsAvailable = true;

				if (!TaskTimingSharedState.IsValid())
				{
					TaskTimingSharedState = MakeShared<FTaskTimingSharedState>(TimingView.Get());
					IModularFeatures::Get().RegisterModularFeature(Timing::TimingViewExtenderFeatureName, TaskTimingSharedState.Get());
				}
				TabManagerShared->TryInvokeTab(FTaskGraphProfilerTabs::TaskTableTreeViewTabID, true);
			}

			if (Session->IsAnalysisComplete())
			{
				// Never check again during this session.
				AvailabilityCheck.Disable();
			}
		}
		else
		{
			// Do not check again until the next session changed event (see OnSessionChanged).
			AvailabilityCheck.Disable();
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::OnSessionChanged()
{
	bIsAvailable = false;
	if (FInsightsManager::Get()->GetSession().IsValid())
	{
		AvailabilityCheck.Enable(0.5);
	}
	else
	{
		AvailabilityCheck.Disable();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::RegisterTimingProfilerLayoutExtensions(FInsightsMajorTabExtender& InOutExtender)
{
	TimingTabManager = InOutExtender.GetTabManager();

	FMinorTabConfig& MinorTabConfig = InOutExtender.AddMinorTabConfig();
	MinorTabConfig.TabId = FTaskGraphProfilerTabs::TaskTableTreeViewTabID;
	MinorTabConfig.TabLabel = LOCTEXT("TaskTableTreeViewTabTitle", "Tasks");
	MinorTabConfig.TabTooltip = LOCTEXT("TaskTableTreeViewTabTitleTooltip", "Opens the Task Table Tree View tab, that allows Task Graph profiling.");
	MinorTabConfig.TabIcon = FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TasksView");
	MinorTabConfig.OnSpawnTab = FOnSpawnTab::CreateRaw(this, &FTaskGraphProfilerManager::SpawnTab_TaskTableTreeView);
	MinorTabConfig.CanSpawnTab = FCanSpawnTab::CreateRaw(this, &FTaskGraphProfilerManager::CanSpawnTab_TaskTableTreeView);
	MinorTabConfig.WorkspaceGroup = InOutExtender.GetWorkspaceGroup();

	InOutExtender.GetLayoutExtender().ExtendLayout(FTimingProfilerTabs::StatsCountersID
		, ELayoutExtensionPosition::After
		, FTabManager::FTab(FTaskGraphProfilerTabs::TaskTableTreeViewTabID, ETabState::ClosedTab));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> FTaskGraphProfilerManager::SpawnTab_TaskTableTreeView(const FSpawnTabArgs& Args)
{
	TSharedRef<FTaskTable> TaskTable = MakeShared<FTaskTable>();
	TaskTable->Reset();
	TaskTable->SetDisplayName(LOCTEXT("TaskTableTreeViewTabTitle", "Tasks"));

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(TaskTableTreeView, STaskTableTreeView, TaskTable)
		];

	RegisterOnWindowClosedEventHandle();

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FTaskGraphProfilerManager::OnTaskTableTreeViewTabClosed));

	return DockTab;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTaskGraphProfilerManager::CanSpawnTab_TaskTableTreeView(const FSpawnTabArgs& Args)
{
	return bIsAvailable;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::OnTaskTableTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	if (TaskTableTreeView.IsValid())
	{
		TaskTableTreeView->OnClose();
	}

	TaskTableTreeView.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::ShowTaskRelations(const TraceServices::FTaskInfo* Task, const TraceServices::ITasksProvider* TasksProvider, const FThreadTrackEvent* InSelectedEvent)
{
	check(Task != nullptr);

	if (!GetShowAnyRelations())
	{
		return;
	}

	auto GetSingleTaskRelationsForAll = [this, TasksProvider, InSelectedEvent](const TArray< TraceServices::FTaskInfo::FRelationInfo>& Collection)
	{
		for (const TraceServices::FTaskInfo::FRelationInfo& Relation : Collection)
		{
			const TraceServices::FTaskInfo* Task = TasksProvider->TryGetTask(Relation.RelativeId);
			if (Task != nullptr)
			{
				GetSingleTaskTransitions(Task, TasksProvider, InSelectedEvent);
			}
		}
	};

	if (bShowCriticalPath)
	{
		GetRelationsOnCriticalPath(Task);
	}
	if (bShowTransitions)
	{
		GetSingleTaskTransitions(Task, TasksProvider, InSelectedEvent);
	}
	if (bShowConnections)
	{
		GetSingleTaskConnections(Task, TasksProvider, InSelectedEvent);
	}
	if (bShowPrerequisites)
	{
		GetSingleTaskRelationsForAll(Task->Prerequisites);
	}
	if (bShowSubsequents)
	{
		GetSingleTaskRelationsForAll(Task->Subsequents);
	}
	if (bShowParentTasks)
	{
		GetSingleTaskRelationsForAll(Task->ParentTasks);
	}
	if (bShowNestedTasks)
	{
		GetSingleTaskRelationsForAll(Task->NestedTasks);
	}

	OutputWarnings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::GetSingleTaskTransitions(const TraceServices::FTaskInfo* Task, const TraceServices::ITasksProvider* TasksProvider, const FThreadTrackEvent* InSelectedEvent)
{
	if (Task->CreatedTimestamp != Task->LaunchedTimestamp || Task->CreatedThreadId != Task->LaunchedThreadId)
	{
		AddRelation(InSelectedEvent, Task->CreatedTimestamp, Task->CreatedThreadId, Task->LaunchedTimestamp, Task->LaunchedThreadId, ETaskEventType::Created);
	}

	if (Task->LaunchedTimestamp != Task->ScheduledTimestamp || Task->LaunchedThreadId != Task->ScheduledThreadId)
	{
		AddRelation(InSelectedEvent, Task->LaunchedTimestamp, Task->LaunchedThreadId, Task->ScheduledTimestamp, Task->ScheduledThreadId, ETaskEventType::Launched);
	}

	int32 ExecutionStartedDepth = GetDepthOfTaskExecution(Task->StartedTimestamp, Task->FinishedTimestamp, Task->StartedThreadId);
	AddRelation(InSelectedEvent, Task->ScheduledTimestamp, Task->ScheduledThreadId, -1, Task->StartedTimestamp, Task->StartedThreadId, ExecutionStartedDepth, ETaskEventType::Scheduled);

	if (Task->FinishedTimestamp != Task->CompletedTimestamp || Task->CompletedThreadId != Task->StartedThreadId)
	{
		AddRelation(InSelectedEvent, Task->FinishedTimestamp, Task->StartedThreadId, ExecutionStartedDepth, Task->CompletedTimestamp, Task->CompletedThreadId, -1,  ETaskEventType::Finished);
	}

	if (Task->CompletedTimestamp != Task->DestroyedTimestamp || Task->CompletedThreadId != Task->DestroyedThreadId)
	{
		AddRelation(InSelectedEvent, Task->CompletedTimestamp, Task->CompletedThreadId, Task->DestroyedTimestamp, Task->DestroyedThreadId, ETaskEventType::Completed);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::GetSingleTaskConnections(const TraceServices::FTaskInfo* Task, const TraceServices::ITasksProvider* TasksProvider, const FThreadTrackEvent* InSelectedEvent)
{
	const int32 MaxTasksToShow = 30;

	int32 NumPrerequisitesToShow = FMath::Min(Task->Prerequisites.Num(), MaxTasksToShow);
	for (int32 i = 0; i != NumPrerequisitesToShow; ++i)
	{
		const TraceServices::FTaskInfo* Prerequisite = TasksProvider->TryGetTask(Task->Prerequisites[i].RelativeId);
		check(Prerequisite != nullptr);

		AddRelation(InSelectedEvent, Prerequisite->FinishedTimestamp, Prerequisite->StartedThreadId, Task->StartedTimestamp, Task->StartedThreadId, ETaskEventType::PrerequisiteStarted);
	}

	int32 NumSubsequentsToShow = FMath::Min(Task->Subsequents.Num(), MaxTasksToShow);
	for (int32 i = 0; i != NumSubsequentsToShow; ++i)
	{
		const TraceServices::FTaskInfo* Subsequent = TasksProvider->TryGetTask(Task->Subsequents[i].RelativeId);
		check(Subsequent != nullptr);

		AddRelation(InSelectedEvent, Task->CompletedTimestamp, Task->CompletedThreadId, Subsequent->StartedTimestamp, Subsequent->StartedThreadId, ETaskEventType::SubsequentStarted);
	}

	int32 NumParentsToShow = FMath::Min(Task->ParentTasks.Num(), MaxTasksToShow);
	for (int32 i = 0; i != NumParentsToShow; ++i)
	{
		const TraceServices::FTaskInfo* ParentTask = TasksProvider->TryGetTask(Task->ParentTasks[i].RelativeId);
		check(ParentTask != nullptr);

		AddRelation(InSelectedEvent, Task->CompletedTimestamp, Task->CompletedThreadId, ParentTask->CompletedTimestamp, ParentTask->CompletedThreadId, ETaskEventType::ParentStarted);
	}

	int32 NumNestedToShow = FMath::Min(Task->NestedTasks.Num(), MaxTasksToShow);
	for (int32 i = 0; i != NumNestedToShow; ++i)
	{
		const TraceServices::FTaskInfo::FRelationInfo& RelationInfo = Task->NestedTasks[i];
		const TraceServices::FTaskInfo* NestedTask = TasksProvider->TryGetTask(RelationInfo.RelativeId);
		check(NestedTask != nullptr);

		int32 NestedExecutionStartedDepth = GetDepthOfTaskExecution(NestedTask->StartedTimestamp, NestedTask->FinishedTimestamp, NestedTask->StartedThreadId);

		AddRelation(InSelectedEvent, RelationInfo.Timestamp, Task->StartedThreadId, -1, NestedTask->StartedTimestamp, NestedTask->StartedThreadId, NestedExecutionStartedDepth, ETaskEventType::NestedStarted);
		AddRelation(InSelectedEvent, NestedTask->FinishedTimestamp, NestedTask->StartedThreadId, NestedExecutionStartedDepth, Task->CompletedTimestamp, Task->CompletedThreadId, -1, ETaskEventType::NestedCompleted);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::GetRelationsOnCriticalPath(const TraceServices::FTaskInfo* Task)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	TArray<FTaskGraphRelation> Relations;
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::ITasksProvider* TasksProvider = TraceServices::ReadTasksProvider(*Session.Get());

		if (TasksProvider == nullptr)
		{
			return;
		}

		TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
		if (!TimingView.IsValid())
		{
			return;
		}

		GetRelationsOnCriticalPathAscendingRec(Task, TasksProvider, Relations);
		GetRelationsOnCriticalPathDescendingRec(Task, TasksProvider, Relations);
	}

	for (FTaskGraphRelation& Relation : Relations)
	{
		AddRelation(nullptr, Relation.GetSourceTime(), Relation.GetSourceThreadId(), Relation.GetSourceDepth(), Relation.GetTargetTime(), Relation.GetTargetThreadId(), Relation.GetTargetDepth(), ETaskEventType::CriticalPath);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::ShowTaskRelations(const FThreadTrackEvent* InSelectedEvent, uint32 ThreadId)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

	const TraceServices::ITasksProvider* TasksProvider = TraceServices::ReadTasksProvider(*Session.Get());

	if (TasksProvider == nullptr)
	{
		return;
	}

	const TraceServices::FTaskInfo* Task = TasksProvider->TryGetTask(ThreadId, InSelectedEvent->GetStartTime());
	ClearTaskRelations();

	if (Task != nullptr)
	{
		ShowTaskRelations(Task, TasksProvider, InSelectedEvent);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::ShowTaskRelations(TaskTrace::FId TaskId)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

	const TraceServices::ITasksProvider* TasksProvider = TraceServices::ReadTasksProvider(*Session.Get());

	if (TasksProvider == nullptr)
	{
		return;
	}

	const TraceServices::FTaskInfo* Task = TasksProvider->TryGetTask(TaskId);
	ClearTaskRelations();

	if (Task != nullptr)
	{
		ShowTaskRelations(Task, TasksProvider, nullptr);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::OnWindowClosedEvent()
{
	OnWindowClosedEventHandle.Reset();

	if (TaskTableTreeView.IsValid())
	{
		TaskTableTreeView->OnClose();
	}

	TSharedPtr<FTabManager> TimingTabManagerSharedPtr = TimingTabManager.Pin();

	if (TimingTabManagerSharedPtr.IsValid())
	{
		TSharedPtr<SDockTab> Tab = TimingTabManagerSharedPtr->FindExistingLiveTab(FTaskGraphProfilerTabs::TaskTableTreeViewTabID);
		if (Tab.IsValid())
		{
			Tab->RequestCloseTab();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::InitializeColorCode()
{
	auto ToLiniarColorNoAlpha = [](uint32 Value)
	{
		const float R = static_cast<float>((Value & 0xFF000000) >> 24) / 255.0f;
		const float G = static_cast<float>((Value & 0x00FF0000) >> 16) / 255.0f;
		const float B = static_cast<float>((Value & 0x0000FF00) >>  8) / 255.0f;
		return FLinearColor(R, G, B);
	};

	ColorCode[static_cast<uint32>(ETaskEventType::Created)]             = ToLiniarColorNoAlpha(0xFFDC1AFF); // Yellow
	ColorCode[static_cast<uint32>(ETaskEventType::Launched)]            = ToLiniarColorNoAlpha(0x8BC24AFF); // Green
	ColorCode[static_cast<uint32>(ETaskEventType::Scheduled)]           = ToLiniarColorNoAlpha(0x26BBFFFF); // Blue
	ColorCode[static_cast<uint32>(ETaskEventType::Started)]             = ToLiniarColorNoAlpha(0xFF0000FF); // Red
	ColorCode[static_cast<uint32>(ETaskEventType::Finished)]            = ToLiniarColorNoAlpha(0xFFDC1AFF); // Yellow
	ColorCode[static_cast<uint32>(ETaskEventType::Completed)]           = ToLiniarColorNoAlpha(0xFE9B07FF); // Orange
	ColorCode[static_cast<uint32>(ETaskEventType::PrerequisiteStarted)] = ToLiniarColorNoAlpha(0xFF729CFF); // Pink
	ColorCode[static_cast<uint32>(ETaskEventType::ParentStarted)]       = ToLiniarColorNoAlpha(0xB68F55FF); // Folder
	ColorCode[static_cast<uint32>(ETaskEventType::NestedStarted)]       = ToLiniarColorNoAlpha(0xB68F55FF); // Folder
	ColorCode[static_cast<uint32>(ETaskEventType::SubsequentStarted)]   = ToLiniarColorNoAlpha(0xFE9B07FF); // Orange
	ColorCode[static_cast<uint32>(ETaskEventType::NestedCompleted)]     = ToLiniarColorNoAlpha(0x804D39FF); // Brown
	ColorCode[static_cast<uint32>(ETaskEventType::CriticalPath)]        = ToLiniarColorNoAlpha(0xA139BFFF); // Purple
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FTaskGraphProfilerManager::GetColorForTaskEvent(ETaskEventType InEvent)
{
	check(InEvent < ETaskEventType::NumTaskEventTypes);
	return ColorCode[static_cast<uint32>(InEvent)];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FTaskGraphProfilerManager::GetColorForTaskEventAsPackedARGB(ETaskEventType InEvent)
{
	return GetColorForTaskEvent(InEvent).ToFColor(false).ToPackedARGB();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::AddRelation(const FThreadTrackEvent* InSelectedEvent, double SourceTimestamp, uint32 SourceThreadId, double TargetTimestamp, uint32 TargetThreadId, ETaskEventType Type)
{
	AddRelation(InSelectedEvent, SourceTimestamp, SourceThreadId, -1, TargetTimestamp, TargetThreadId, -1, Type);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::AddRelation(const FThreadTrackEvent* InSelectedEvent, double SourceTimestamp, uint32 SourceThreadId, int32 SourceDepth, double TargetTimestamp, uint32 TargetThreadId, int32 TargetDepth, ETaskEventType Type)
{
	if (SourceTimestamp == TraceServices::FTaskInfo::InvalidTimestamp || TargetTimestamp == TraceServices::FTaskInfo::InvalidTimestamp)
	{
		return;
	}

	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return;
	}

	TSharedPtr<TimingProfiler::FThreadTimingSharedState> ThreadSharedState = TimingView->GetThreadTimingSharedState();

	TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FTaskGraphRelation>(SourceTimestamp, SourceThreadId, TargetTimestamp, TargetThreadId, Type);
	FTaskGraphRelation* TaskRelationPtr = StaticCast<FTaskGraphRelation*>(Relation.Get());

	if (!TaskRelationPtr->GetSourceTrack().IsValid())
	{
		TSharedPtr<const TimingProfiler::FCpuTimingTrack> Track = ThreadSharedState->GetCpuTrack(TaskRelationPtr->GetSourceThreadId());
		if (Track.IsValid())
		{
			TaskRelationPtr->SetSourceTrack(Track);
			TaskRelationPtr->SetSourceDepth(GetRelationDisplayDepth(Track, TaskRelationPtr->GetSourceTime(), SourceDepth));
		}
	}

	if (!TaskRelationPtr->GetTargetTrack().IsValid())

	{
		TSharedPtr<const TimingProfiler::FCpuTimingTrack> Track = ThreadSharedState->GetCpuTrack(TaskRelationPtr->GetTargetThreadId());
		if (Track.IsValid())
		{
			TaskRelationPtr->SetTargetTrack(Track);
			TaskRelationPtr->SetTargetDepth(GetRelationDisplayDepth(Track, TaskRelationPtr->GetTargetTime(), TargetDepth));
		}
	}

	if (TaskRelationPtr->GetSourceTrack().IsValid() && !TaskRelationPtr->GetSourceTrack()->IsVisible())
	{
		HiddenTrackNames.Add(TaskRelationPtr->GetSourceTrack()->GetName());
	}

	if (TaskRelationPtr->GetTargetTrack().IsValid() && !TaskRelationPtr->GetTargetTrack()->IsVisible())
	{
		HiddenTrackNames.Add(TaskRelationPtr->GetTargetTrack()->GetName());
	}

	if (TaskRelationPtr->GetSourceTrack().IsValid() && TaskRelationPtr->GetTargetTrack().IsValid())
	{
		TimingView->AddRelation(Relation);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTaskGraphProfilerManager::GetRelationDisplayDepth(TSharedPtr<const TimingProfiler::FThreadTimingTrack> Track, double Time, int32 KnownDepth)
{
	if (KnownDepth >= 0)
	{
		return KnownDepth;
	}

	KnownDepth = Track->GetDepthAt(Time) - 1;

	// We return the depth 0 based, so the depth will be 0 when the event is on the first row or when there is no event on the track at the time.
	return FMath::Max(KnownDepth, 0);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::ClearTaskRelations()
{
	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return;
	}

	TArray<TUniquePtr<ITimingEventRelation>>& Relations = TimingView->EditCurrentRelations();
	Relations.RemoveAll([](TUniquePtr<ITimingEventRelation>& Relations)
		{
			return Relations->Is<FTaskGraphRelation>();
		});

	HiddenTrackNames.Empty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTaskGraphProfilerManager::GetDepthOfTaskExecution(double TaskStartedTime, double TaskFinishedTime, uint32 ThreadId)
{
	int32 Depth = -1;

	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return Depth;
	}

	TSharedPtr<TimingProfiler::FThreadTimingSharedState> ThreadSharedState = TimingView->GetThreadTimingSharedState();

	TSharedPtr<TimingProfiler::FCpuTimingTrack> Track = ThreadSharedState->GetCpuTrack(ThreadId);

	if (!Track.IsValid())
	{
		return Depth;
	}

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			TimingProfilerProvider->ReadTimeline(Track->GetTimelineIndex(),
				[TaskStartedTime, TaskFinishedTime, &Depth]
				(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
				{
					Timeline.EnumerateEventsDownSampled(TaskStartedTime, TaskFinishedTime, 0,
						[TaskStartedTime, &Depth]
						(bool IsEnter, double Time, const TraceServices::FTimingProfilerEvent& Event)
						{
							if (Time < TaskStartedTime)
							{
								check(IsEnter);
								++Depth;
								return TraceServices::EEventEnumerate::Continue;
							}

							if (IsEnter)
							{
								++Depth;
							}

							return TraceServices::EEventEnumerate::Stop;
						});
				});
		}
	}

	return Depth;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double FTaskGraphProfilerManager::GetRelationsOnCriticalPathAscendingRec(const TraceServices::FTaskInfo* Task, const TraceServices::ITasksProvider* TasksProvider, TArray<FTaskGraphRelation>& Relations)
{
	double MaxChainDuration = 0;
	int32 InitialRelationNum = Relations.Num();
	const TraceServices::FTaskInfo* NextTaskInChain = nullptr;

	if (Task->Prerequisites.Num() > 0)
	{
		TArray<FTaskGraphRelation> AscendingRelations;

		for (const TraceServices::FTaskInfo::FRelationInfo& PrerequisiteTaskRelation : Task->Prerequisites)
		{
			const TraceServices::FTaskInfo* PrerequisiteTaskInfo = TasksProvider->TryGetTask(PrerequisiteTaskRelation.RelativeId);
			if (PrerequisiteTaskInfo != nullptr)
			{
				double ChainDuration = GetRelationsOnCriticalPathAscendingRec(PrerequisiteTaskInfo, TasksProvider, AscendingRelations);
				if (ChainDuration > MaxChainDuration)
				{
					MaxChainDuration = ChainDuration;
					NextTaskInChain = PrerequisiteTaskInfo;
					// Remove the relations from the shorter branch.
					if (Relations.Num() > InitialRelationNum)
					{
						Relations.RemoveAt(InitialRelationNum, Relations.Num() - InitialRelationNum, EAllowShrinking::No);
					}

					Relations.Append(AscendingRelations);
				}
				AscendingRelations.Empty(AscendingRelations.Max());
			}
		}
	}

	if (NextTaskInChain)
	{
		int32 SourceDepth = GetDepthOfTaskExecution(NextTaskInChain->StartedTimestamp, NextTaskInChain->FinishedTimestamp, NextTaskInChain->StartedThreadId);
		int32 TargetDepth = GetDepthOfTaskExecution(Task->StartedTimestamp, Task->FinishedTimestamp, Task->StartedThreadId);
		FTaskGraphRelation& NewRelation = Relations.Emplace_GetRef(NextTaskInChain->FinishedTimestamp, NextTaskInChain->StartedThreadId, Task->StartedTimestamp, Task->StartedThreadId, ETaskEventType::CriticalPath);
		NewRelation.SetSourceDepth(SourceDepth);
		NewRelation.SetTargetDepth(TargetDepth);
	}
	else
	{
		int32 TargetDepth = GetDepthOfTaskExecution(Task->StartedTimestamp, Task->FinishedTimestamp, Task->StartedThreadId);
		FTaskGraphRelation& NewRelation = Relations.Emplace_GetRef(Task->CreatedTimestamp, Task->CreatedThreadId, Task->StartedTimestamp, Task->StartedThreadId, ETaskEventType::CriticalPath);
		NewRelation.SetTargetDepth(TargetDepth);
	}

	double TaskDuration = Task->FinishedTimestamp - Task->StartedTimestamp;
	return MaxChainDuration + TaskDuration;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double FTaskGraphProfilerManager::GetRelationsOnCriticalPathDescendingRec(const TraceServices::FTaskInfo* Task, const TraceServices::ITasksProvider* TasksProvider, TArray<FTaskGraphRelation>& Relations)
{
	double MaxChainDuration = 0;
	const TraceServices::FTaskInfo* NextTaskInChain = nullptr;
	int32 InitialRelationNum = Relations.Num();

	if (Task->Subsequents.Num() > 0)
	{
		TArray<FTaskGraphRelation> DescendingRelations;

		for (const TraceServices::FTaskInfo::FRelationInfo& SubsequentTaskRelation : Task->Subsequents)
		{
			const TraceServices::FTaskInfo* SubsequentTaskInfo = TasksProvider->TryGetTask(SubsequentTaskRelation.RelativeId);
			if (SubsequentTaskInfo != nullptr)
			{
				double ChainDuration = GetRelationsOnCriticalPathDescendingRec(SubsequentTaskInfo, TasksProvider, DescendingRelations);
				if (ChainDuration > MaxChainDuration)
				{
					MaxChainDuration = ChainDuration;
					NextTaskInChain = SubsequentTaskInfo;
					// Remove the relations from the shorter branch.
					if (Relations.Num() > InitialRelationNum)
					{
						Relations.RemoveAt(InitialRelationNum, Relations.Num() - InitialRelationNum, EAllowShrinking::No);
					}

					Relations.Append(DescendingRelations);
				}
				DescendingRelations.Empty(DescendingRelations.Max());
			}
		}
	}

	if (NextTaskInChain)
	{
		int32 SourceDepth = GetDepthOfTaskExecution(Task->StartedTimestamp, Task->FinishedTimestamp, Task->StartedThreadId);
		int32 TargetDepth = GetDepthOfTaskExecution(NextTaskInChain->StartedTimestamp, NextTaskInChain->FinishedTimestamp, NextTaskInChain->StartedThreadId);
		FTaskGraphRelation& NewRelation = Relations.Emplace_GetRef(FTaskGraphRelation(Task->FinishedTimestamp, Task->StartedThreadId, NextTaskInChain->StartedTimestamp, NextTaskInChain->StartedThreadId, ETaskEventType::CriticalPath));
		NewRelation.SetSourceDepth(SourceDepth);
		NewRelation.SetTargetDepth(TargetDepth);
	}

	double TaskDuration = Task->FinishedTimestamp - Task->StartedTimestamp;
	return MaxChainDuration + TaskDuration;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::SelectTaskInTaskTable(TaskTrace::FId InId)
{
	if (TaskTableTreeView.IsValid())
	{
		TaskTableTreeView->SelectTaskEntry(InId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::RegisterOnWindowClosedEventHandle()
{
	if (!OnWindowClosedEventHandle.IsValid())
	{
		using namespace UE::Insights::TimingProfiler;

		TSharedPtr<STimingProfilerWindow> Window = FTimingProfilerManager::Get()->GetProfilerWindow();
		if (!Window.IsValid())
		{
			return;
		}

		OnWindowClosedEventHandle = Window->GetWindowClosedEvent().AddSP(this, &FTaskGraphProfilerManager::OnWindowClosedEvent);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskGraphProfilerManager::OutputWarnings()
{
	if (HiddenTrackNames.Num() == 0)
	{
		return;
	}

	TStringBuilder<256> TrackList;

	constexpr int32 MaxListedTracks = 3;
	int32 NumTracks = 0;

	for (FString TrackName : HiddenTrackNames)
	{
		++NumTracks;
		TrackList.Append(TrackName);
		TrackList.Append(TEXT(","));

		if (NumTracks >= MaxListedTracks)
		{
			break;
		}
	}

	int32 RemainingTracksNum = HiddenTrackNames.Num() - NumTracks;
	FText WarningMessage;
	if (RemainingTracksNum > 0)
	{
		WarningMessage = FText::Format(LOCTEXT("RelationsOnMoreHiddenTracksWarningFmt", "Some task relations point to hidden tracks: {0} and {1} more."),
									   FText::FromString(TrackList.ToString()),
									   RemainingTracksNum);
	}
	else
	{
		TrackList.RemoveSuffix(1);
		WarningMessage = FText::Format(LOCTEXT("RelationsOnHiddenTracksWarningFmt", "Some task relations point to hidden tracks: {0}."),
									   FText::FromString(TrackList.ToString()));
	}

	using namespace UE::Insights::TimingProfiler;
	FName LogListingName = FTimingProfilerManager::Get()->GetLogListingName();
	FMessageLog ReportMessageLog(LogListingName);
	ReportMessageLog.Warning(WarningMessage);
	ReportMessageLog.Notify();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<TimingProfiler::STimingView> FTaskGraphProfilerManager::GetTimingView()
{
	using namespace UE::Insights::TimingProfiler;
	TSharedPtr<STimingProfilerWindow> Window = FTimingProfilerManager::Get()->GetProfilerWindow();
	return Window.IsValid() ? Window->GetTimingView() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TaskGraphProfiler

#undef LOCTEXT_NAMESPACE
