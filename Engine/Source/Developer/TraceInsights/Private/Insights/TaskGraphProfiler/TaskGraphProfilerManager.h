// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Async/TaskTrace.h"
#include "Containers/Ticker.h"
#include "Framework/Commands/UICommandList.h"
#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Common/AvailabilityCheck.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"

namespace TraceServices
{
	struct FTaskInfo;
	class ITasksProvider;
}

class FThreadTrackEvent;

namespace UE::Insights::TimingProfiler
{
	class FThreadTimingTrack;
	class STimingView;
}

namespace UE::Insights::TaskGraphProfiler
{

class FTaskGraphRelation;
class FTaskTimingSharedState;
class STaskTableTreeView;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FTaskGraphProfilerTabs
{
	// Tab identifiers
	static const FName TaskTableTreeViewTabID;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class ETaskEventType : uint32
{
	Created,
	Launched,
	Scheduled,
	Started,
	Finished,
	Completed,
	PrerequisiteStarted,
	ParentStarted,
	NestedStarted,
	NestedCompleted,
	SubsequentStarted,
	CriticalPath,

	NumTaskEventTypes,
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * This class manages the Task Graph Profiler state and settings.
 */
class FTaskGraphProfilerManager : public TSharedFromThis<FTaskGraphProfilerManager>, public IInsightsComponent
{
public:
	typedef TFunctionRef<void(double /*SourceTimestamp*/, uint32 /*SourceThreadId*/, double /*TargetTimestamp*/, uint32 /*TargetThreadId*/, ETaskEventType /*Type*/)> FAddRelationCallback;

	/** Creates the Task Graph Profiler manager, only one instance can exist. */
	FTaskGraphProfilerManager(TSharedRef<FUICommandList> InCommandList);

	/** Virtual destructor. */
	virtual ~FTaskGraphProfilerManager();

	/** Creates an instance of the Task Graph Profiler manager. */
	static TSharedPtr<FTaskGraphProfilerManager> CreateInstance();

	/**
	 * @return the global instance of the Task Graph Profiler manager.
	 * This is an internal singleton and cannot be used outside TraceInsights.
	 * For external use:
	 *     IUnrealInsightsModule& Module = FModuleManager::Get().LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	 *     Module.GetTaskGraphProfilerManager();
	 */
	static TSharedPtr<FTaskGraphProfilerManager> Get();

	//////////////////////////////////////////////////
	// IInsightsComponent

	virtual void Initialize(IUnrealInsightsModule& InsightsModule) override;
	virtual void Shutdown() override;
	virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) override;
	virtual void UnregisterMajorTabs() override;
	virtual void OnWindowClosedEvent() override;

	TSharedRef<SDockTab> SpawnTab_TaskTableTreeView(const FSpawnTabArgs& Args);
	bool CanSpawnTab_TaskTableTreeView(const FSpawnTabArgs& Args);
	void OnTaskTableTreeViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	//////////////////////////////////////////////////
	bool GetIsAvailable() { return bIsAvailable; }

	void OnSessionChanged();

	void ShowTaskRelations(const FThreadTrackEvent* InSelectedEvent, uint32 ThreadId);
	void ShowTaskRelations(TaskTrace::FId TaskId);
	void AddRelation(const FThreadTrackEvent* InSelectedEvent, double SourceTimestamp, uint32 SourceThreadId, double TargetTimestamp, uint32 TargetThreadId, ETaskEventType Type);
	void AddRelation(const FThreadTrackEvent* InSelectedEvent, double SourceTimestamp, uint32 SourceThreadId, int32 SourceDepth, double TargetTimestamp, uint32 TargetThreadId, int32 TargetDepth, ETaskEventType Type);
	void ClearTaskRelations();
	FLinearColor GetColorForTaskEvent(ETaskEventType InEvent);
	uint32 GetColorForTaskEventAsPackedARGB(ETaskEventType InEvent);

	TSharedPtr<FTaskTimingSharedState> GetTaskTimingSharedState() { return TaskTimingSharedState;	}

	bool GetShowCriticalPath() const { return bShowCriticalPath; }
	void SetShowCriticalPath(bool bInValue) { bShowCriticalPath = bInValue; }

	bool GetShowTransitions() const { return bShowTransitions; }
	void SetShowTransitions(bool bInValue) { bShowTransitions = bInValue; }

	bool GetShowConnections() const { return bShowConnections; }
	void SetShowConnections(bool bInValue) { bShowConnections = bInValue; }

	bool GetShowPrerequisites() const { return bShowPrerequisites; }
	void SetShowPrerequisites(bool bInValue) { bShowPrerequisites = bInValue; }

	bool GetShowSubsequents() const { return bShowSubsequents; }
	void SetShowSubsequents(bool bInValue) { bShowSubsequents = bInValue; }

	bool GetShowParentTasks() const { return bShowParentTasks; }
	void SetShowParentTasks(bool bInValue) { bShowParentTasks = bInValue; }

	bool GetShowNestedTasks() const { return bShowNestedTasks; }
	void SetShowNestedTasks(bool bInValue) { bShowNestedTasks = bInValue; }

	bool GetShowAnyRelations() const { return GetShowCriticalPath() || GetShowTransitions() || GetShowConnections() || GetShowPrerequisites() || GetShowSubsequents() || GetShowParentTasks() || GetShowNestedTasks(); }

	int32 GetDepthOfTaskExecution(double TaskStartedTime, double TaskFinishedTime, uint32 ThreadId);

	void SelectTaskInTaskTable(TaskTrace::FId);

	void RegisterOnWindowClosedEventHandle();

private:
	/** Updates this manager, done through FCoreTicker. */
	bool Tick(float DeltaTime);

	void RegisterTimingProfilerLayoutExtensions(FInsightsMajorTabExtender& InOutExtender);

	void ShowTaskRelations(const TraceServices::FTaskInfo* Task, const TraceServices::ITasksProvider* TasksProvider, const FThreadTrackEvent* InSelectedEvent);
	void GetSingleTaskTransitions(const TraceServices::FTaskInfo* Task, const TraceServices::ITasksProvider* TasksProvider, const FThreadTrackEvent* InSelectedEvent);
	void GetSingleTaskConnections(const TraceServices::FTaskInfo* Task, const TraceServices::ITasksProvider* TasksProvider, const FThreadTrackEvent* InSelectedEvent);
	void GetRelationsOnCriticalPath(const TraceServices::FTaskInfo* Task);

	double GetRelationsOnCriticalPathAscendingRec(const TraceServices::FTaskInfo* Task, const TraceServices::ITasksProvider* TasksProvider, TArray<FTaskGraphRelation>& Relations);
	double GetRelationsOnCriticalPathDescendingRec(const TraceServices::FTaskInfo* Task, const TraceServices::ITasksProvider* TasksProvider, TArray<FTaskGraphRelation>& Relations);

	void InitializeColorCode();
	int32 GetRelationDisplayDepth(TSharedPtr<const TimingProfiler::FThreadTimingTrack> Track, double Time, int32 KnownDepth);

	void OutputWarnings();

	static TSharedPtr<UE::Insights::TimingProfiler::STimingView> GetTimingView();

private:
	bool bIsInitialized;
	bool bIsAvailable;
	FAvailabilityCheck AvailabilityCheck;

	/** The delegate to be invoked when this manager ticks. */
	FTickerDelegate OnTick;

	/** Handle to the registered OnTick. */
	FTSTicker::FDelegateHandle OnTickHandle;

	/** A shared pointer to the global instance of the Task Graph Profiler manager. */
	static TSharedPtr<FTaskGraphProfilerManager> Instance;

	/** Shared state for task tracks */
	TSharedPtr<FTaskTimingSharedState> TaskTimingSharedState;

	TWeakPtr<FTabManager> TimingTabManager;

	TSharedPtr<STaskTableTreeView> TaskTableTreeView;
	FLinearColor ColorCode[static_cast<uint32>(ETaskEventType::NumTaskEventTypes)];
	bool bShowCriticalPath = false;
	bool bShowTransitions = true;
	bool bShowConnections = true;
	bool bShowPrerequisites = false;
	bool bShowSubsequents = false;
	bool bShowParentTasks = false;
	bool bShowNestedTasks = false;

	FDelegateHandle OnWindowClosedEventHandle;

	TSet<FString> HiddenTrackNames;
};

} // namespace UE::Insights::TaskGraphProfiler
