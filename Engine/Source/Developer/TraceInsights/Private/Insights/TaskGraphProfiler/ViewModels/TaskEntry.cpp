// Copyright Epic Games, Inc. All Rights Reserved.

#include "TaskEntry.h"

// TraceServices
#include "TraceServices/Model/TasksProfiler.h"

namespace UE::Insights::TaskGraphProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FTaskEntry::FTaskEntry(const TraceServices::FTaskInfo& TaskInfo)
	: Id(TaskInfo.Id)
	, DebugName(TaskInfo.DebugName)
	, bTracked(TaskInfo.bTracked)
	, ThreadToExecuteOn(TaskInfo.ThreadToExecuteOn)
	, CreatedTimestamp(TaskInfo.CreatedTimestamp)
	, LaunchedTimestamp(TaskInfo.LaunchedTimestamp)
	, ScheduledTimestamp(TaskInfo.ScheduledTimestamp)
	, StartedTimestamp(TaskInfo.StartedTimestamp)
	, FinishedTimestamp(TaskInfo.FinishedTimestamp)
	, CompletedTimestamp(TaskInfo.CompletedTimestamp)
	, DestroyedTimestamp(TaskInfo.DestroyedTimestamp)
	, CreatedThreadId(TaskInfo.CreatedThreadId)
	, LaunchedThreadId(TaskInfo.LaunchedThreadId)
	, ScheduledThreadId(TaskInfo.ScheduledThreadId)
	, StartedThreadId(TaskInfo.StartedThreadId)
	, CompletedThreadId(TaskInfo.CompletedThreadId)
	, DestroyedThreadId(TaskInfo.DestroyedThreadId)
	, TaskSize(TaskInfo.TaskSize)
	, NumPrerequisites(TaskInfo.Prerequisites.Num())
	, NumSubsequents(TaskInfo.Subsequents.Num())
	, NumParents(TaskInfo.ParentTasks.Num())
	, NumNested(TaskInfo.NestedTasks.Num())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TaskGraphProfiler
