// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Async/TaskTrace.h"

namespace TraceServices { struct FTaskInfo; }

namespace UE::Insights::TaskGraphProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTaskEntry
{
public:
	FTaskEntry(const TraceServices::FTaskInfo& TaskInfo);
	~FTaskEntry() = default;

	TaskTrace::FId GetId() const { return Id; }

	const TCHAR* GetDebugName() const { return DebugName; }
	bool IsTracked() const { return bTracked; }
	int32 GetThreadToExecuteOn() const { return ThreadToExecuteOn; }

	double GetCreatedTimestamp() const { return CreatedTimestamp; }
	uint32 GetCreatedThreadId() const { return CreatedThreadId; }

	double GetLaunchedTimestamp() const { return LaunchedTimestamp; }
	uint32 GetLaunchedThreadId() const { return LaunchedThreadId; }

	double GetScheduledTimestamp() const { return ScheduledTimestamp; }
	uint32 GetScheduledThreadId() const { return ScheduledThreadId; }

	double GetStartedTimestamp() const { return StartedTimestamp; }
	uint32 GetStartedThreadId() const { return StartedThreadId; }

	double GetFinishedTimestamp() const { return FinishedTimestamp; }

	double GetCompletedTimestamp() const { return CompletedTimestamp; }
	uint32 GetCompletedThreadId() const { return CompletedThreadId; }

	double GetDestroyedTimestamp() const { return DestroyedTimestamp; }
	uint32 GetDestroyedThreadId() const { return DestroyedThreadId; }

	uint64 GetTaskSize() const { return TaskSize; }

	uint32 GetNumPrerequisites() const { return NumPrerequisites; }
	uint32 GetNumSubsequents() const { return NumSubsequents; }
	uint32 GetNumParents() const { return NumParents; }
	uint32 GetNumNested() const { return NumNested; }

private:
	TaskTrace::FId Id;

	const TCHAR* DebugName;
	bool bTracked;
	int32 ThreadToExecuteOn;

	double CreatedTimestamp;
	double LaunchedTimestamp;
	double ScheduledTimestamp;
	double StartedTimestamp;
	double FinishedTimestamp;
	double CompletedTimestamp;
	double DestroyedTimestamp;

	uint32 CreatedThreadId;
	uint32 LaunchedThreadId;
	uint32 ScheduledThreadId;
	uint32 StartedThreadId;
	uint32 CompletedThreadId;
	uint32 DestroyedThreadId;

	uint64 TaskSize;

	uint32 NumPrerequisites;
	uint32 NumSubsequents;
	uint32 NumParents;
	uint32 NumNested;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TaskGraphProfiler
