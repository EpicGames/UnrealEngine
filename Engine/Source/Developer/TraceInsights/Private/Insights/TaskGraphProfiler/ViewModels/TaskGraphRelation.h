// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Insights/ViewModels/ITimingEvent.h"

namespace UE::Insights::TaskGraphProfiler
{

enum class ETaskEventType : uint32;

class FTaskGraphRelation : public ITimingEventRelation
{
	INSIGHTS_DECLARE_RTTI(FTaskGraphRelation, ITimingEventRelation)

public:
	FTaskGraphRelation(double InSourceTime, int32 InSourceThreadId, double InTargetTime, int32 InTargetThreadId, ETaskEventType InType);

	virtual void Draw(const FDrawContext& DrawContext, const FTimingTrackViewport& Viewport, const ITimingViewDrawHelper& Helper, const ITimingEventRelation::EDrawFilter Filter) override;

	void SetSourceTrack(TSharedPtr<const FBaseTimingTrack> InSourceTrack) { SourceTrack = InSourceTrack; }
	TSharedPtr<const FBaseTimingTrack> GetSourceTrack() { return SourceTrack.Pin(); }

	void SetTargetTrack(TSharedPtr<const FBaseTimingTrack> InTargetTrack) { TargetTrack = InTargetTrack; }
	TSharedPtr<const FBaseTimingTrack> GetTargetTrack() { return TargetTrack.Pin(); }

	double GetSourceTime() { return SourceTime; }
	int32 GetSourceThreadId() { return SourceThreadId; }
	void SetSourceDepth(int32 InDepth) { SourceDepth = InDepth; }
	int32 GetSourceDepth() { return SourceDepth; }

	double GetTargetTime() { return TargetTime; }
	int32 GetTargetThreadId() { return TargetThreadId; }
	void SetTargetDepth(int32 InDepth) { TargetDepth = InDepth; }
	int32 GetTargetDepth() { return TargetDepth; }

	ETaskEventType GetType() { return Type; }

private:
	double SourceTime;
	int32 SourceThreadId;
	int32 SourceDepth = -1;
	double TargetTime;
	int32 TargetThreadId;
	int32 TargetDepth = -1;
	ETaskEventType Type;

	TWeakPtr<const FBaseTimingTrack> SourceTrack;
	TWeakPtr<const FBaseTimingTrack> TargetTrack;
};

} // namespace UE::Insights::TaskGraphProfiler
