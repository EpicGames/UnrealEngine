// Copyright Epic Games, Inc. All Rights Reserved.

#include "TaskNode.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TaskGraphProfiler::FTaskNode"

namespace UE::Insights::TaskGraphProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FTaskNode)

const FText FTaskNode::GetDisplayName() const
{
	return FText::FromString(FString::Printf(TEXT("task_%d"), GetRowIndex()));
}

} // namespace UE::Insights::TaskGraphProfiler

#undef LOCTEXT_NAMESPACE
