// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"

// TraceInsights
#include "Insights/TaskGraphProfiler/ViewModels/TaskTable.h"
#include "Insights/TaskGraphProfiler/ViewModels/TaskEntry.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::TaskGraphProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTaskNode;

/** Type definition for shared pointers to instances of FTaskNode. */
typedef TSharedPtr<class FTaskNode> FTaskNodePtr;

/** Type definition for shared references to instances of FTaskNode. */
typedef TSharedRef<class FTaskNode> FTaskNodeRef;

/** Type definition for shared references to const instances of FTaskNode. */
typedef TSharedRef<const class FTaskNode> FTaskNodeRefConst;

/** Type definition for weak references to instances of FTaskNode. */
typedef TWeakPtr<class FTaskNode> FTaskNodeWeak;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class used to store information about a task node (used in the STaskTreeView).
 */
class FTaskNode : public FTableTreeNode
{
	INSIGHTS_DECLARE_RTTI(FTaskNode, FTableTreeNode)

public:
	/** Initialization constructor for the Task node. */
	explicit FTaskNode(TWeakPtr<FTaskTable> InParentTable, int32 InRowIndex)
		: FTableTreeNode(InParentTable, InRowIndex)
	{
	}

	FTaskTable& GetTaskTableChecked() const
	{
		const TSharedPtr<FTable>& TablePin = GetParentTable().Pin();
		check(TablePin.IsValid());
		return *StaticCastSharedPtr<FTaskTable>(TablePin);
	}

	virtual const FText GetDisplayName() const override;

	bool IsValidTask() const { return GetTaskTableChecked().IsValidRowIndex(GetRowIndex()); }
	const FTaskEntry* GetTask() const { return GetTaskTableChecked().GetTask(GetRowIndex()); }
	const FTaskEntry& GetTaskChecked() const { return GetTaskTableChecked().GetTaskChecked(GetRowIndex()); }
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TaskGraphProfiler
