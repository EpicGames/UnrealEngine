// Copyright Epic Games, Inc. All Rights Reserved.

#include "TaskTable.h"

// TraceServices
#include "TraceServices/Model/TasksProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/TableCellValue.h"
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/TaskGraphProfiler/ViewModels/TaskNode.h"
#include "Insights/TaskGraphProfiler/ViewModels/TaskTable.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TaskGraphProfiler::FTaskTable"

namespace UE::Insights::TaskGraphProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Column identifiers

const FName FTaskTableColumns::DebugNameColumnId(TEXT("DebugName"));
const FName FTaskTableColumns::CreatedTimestampColumnId(TEXT("CreatedTimestamp"));
const FName FTaskTableColumns::CreatedThreadIdColumnId(TEXT("CreatedThreadId"));
const FName FTaskTableColumns::LaunchedTimestampColumnId(TEXT("LaunchedTimestamp"));
const FName FTaskTableColumns::LaunchedThreadIdColumnId(TEXT("LaunchedThreadId"));
const FName FTaskTableColumns::ScheduledTimestampColumnId(TEXT("ScheduledTimestamp"));
const FName FTaskTableColumns::ScheduledThreadIdColumnId(TEXT("ScheduledThreadId"));
const FName FTaskTableColumns::StartedTimestampColumnId(TEXT("StartedTimestamp"));
const FName FTaskTableColumns::StartedThreadIdColumnId(TEXT("StartedThreadId"));
const FName FTaskTableColumns::FinishedTimestampColumnId(TEXT("FinishedTimestamp"));
const FName FTaskTableColumns::CompletedTimestampColumnId(TEXT("CompletedTimestamp"));
const FName FTaskTableColumns::CompletedThreadIdColumnId(TEXT("CompletedThreadId"));
const FName FTaskTableColumns::DestroyedTimestampColumnId(TEXT("DestroyedTimestamp"));
const FName FTaskTableColumns::DestroyedThreadIdColumnId(TEXT("DestroyedThreadId"));
const FName FTaskTableColumns::TaskSizeColumnId(TEXT("TaskSize"));
const FName FTaskTableColumns::NumParentColumnId(TEXT("NumParents"));
const FName FTaskTableColumns::NumNestedColumnId(TEXT("NumNested"));
const FName FTaskTableColumns::NumSubsequentsColumnId(TEXT("NumSubsequents"));
const FName FTaskTableColumns::NumPrerequisitesColumnId(TEXT("NumPrerequisites"));

////////////////////////////////////////////////////////////////////////////////////////////////////

typedef FTableCellValue (*TaskFieldGetter) (const FTableColumn&, const FTaskEntry&);

template<TaskFieldGetter Getter>
class FTaskColumnValueGetter : public FTableCellValueGetter
{
public:
	virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
	{
		if (Node.IsGroup())
		{
			const FTableTreeNode& NodePtr = static_cast<const FTableTreeNode&>(Node);
			if (NodePtr.HasAggregatedValue(Column.GetId()))
			{
				return NodePtr.GetAggregatedValue(Column.GetId());
			}
		}
		else //if (Node->Is<FTaskNode>())
		{
			const FTaskNode& TaskNode = static_cast<const FTaskNode&>(Node);
			const FTaskEntry* Task = TaskNode.GetTask();
			if (Task)
			{
				return Getter(Column, *Task);
			}
		}

		return TOptional<FTableCellValue>();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

double GetRelativeValue(double ValueA, double ValueB)
{
	if (ValueA == TraceServices::FTaskInfo::InvalidTimestamp || ValueB == TraceServices::FTaskInfo::InvalidTimestamp)
	{
		return TraceServices::FTaskInfo::InvalidTimestamp;
	}

	return ValueA - ValueB;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

struct DefaultTaskFieldGetterFuncts
{
	static FTableCellValue GetDebugName(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(Task.GetDebugName()); }
	static FTableCellValue GetCreatedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(Task.GetCreatedTimestamp());	}
	static FTableCellValue GetCreatedThreadId(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue((int64)Task.GetCreatedThreadId());	}

	static FTableCellValue GetLaunchedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(Task.GetLaunchedTimestamp());	}
	static FTableCellValue GetLaunchedThreadId(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue((int64)Task.GetLaunchedThreadId());	}

	static FTableCellValue GetScheduledTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(Task.GetScheduledTimestamp()); }
	static FTableCellValue GetScheduledThreadId(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue((int64)Task.GetScheduledThreadId()); }

	static FTableCellValue GetStartedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(Task.GetStartedTimestamp()); }
	static FTableCellValue GetStartedThreadId(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue((int64)Task.GetStartedThreadId()); }

	static FTableCellValue GetFinishedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(Task.GetFinishedTimestamp()); }

	static FTableCellValue GetCompletedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(Task.GetCompletedTimestamp()); }
	static FTableCellValue GetCompletedThreadId(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue((int64)Task.GetCompletedThreadId()); }

	static FTableCellValue GetDestroyedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(Task.GetDestroyedTimestamp()); }
	static FTableCellValue GetDestroyedThreadId(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue((int64)Task.GetDestroyedThreadId()); }

	static FTableCellValue GetTaskSize(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue((int64)Task.GetTaskSize()); }

	static FTableCellValue GetNumParents(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue((int64)Task.GetNumParents()); }
	static FTableCellValue GetNumNested(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue((int64)Task.GetNumNested());	}
	static FTableCellValue GetNumSubsequents(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue((int64)Task.GetNumSubsequents()); }
	static FTableCellValue GetNumPrerequisites(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue((int64)Task.GetNumPrerequisites());	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct RelativeToPreviousTaskFieldGetterFuncts
{
	static FTableCellValue GetCreatedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(Task.GetCreatedTimestamp()); }
	static FTableCellValue GetLaunchedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(GetRelativeValue(Task.GetLaunchedTimestamp(), Task.GetCreatedTimestamp())); }
	static FTableCellValue GetScheduledTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(GetRelativeValue(Task.GetScheduledTimestamp(), Task.GetLaunchedTimestamp())); }
	static FTableCellValue GetStartedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(GetRelativeValue(Task.GetStartedTimestamp(), Task.GetScheduledTimestamp())); }
	static FTableCellValue GetFinishedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(GetRelativeValue(Task.GetFinishedTimestamp(), Task.GetStartedTimestamp())); }
	static FTableCellValue GetCompletedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(GetRelativeValue(Task.GetCompletedTimestamp(), Task.GetFinishedTimestamp())); }
	static FTableCellValue GetDestroyedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(GetRelativeValue(Task.GetDestroyedTimestamp(), Task.GetCompletedTimestamp())); }
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct RelativeToCreatedTaskFieldGetterFuncts
{
	static FTableCellValue GetCreatedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(Task.GetCreatedTimestamp()); }
	static FTableCellValue GetLaunchedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(GetRelativeValue(Task.GetLaunchedTimestamp(), Task.GetCreatedTimestamp())); }
	static FTableCellValue GetScheduledTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(GetRelativeValue(Task.GetScheduledTimestamp(), Task.GetCreatedTimestamp())); }
	static FTableCellValue GetStartedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(GetRelativeValue(Task.GetStartedTimestamp(), Task.GetCreatedTimestamp())); }
	static FTableCellValue GetFinishedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(GetRelativeValue(Task.GetFinishedTimestamp(), Task.GetCreatedTimestamp())); }
	static FTableCellValue GetCompletedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(GetRelativeValue(Task.GetCompletedTimestamp(), Task.GetCreatedTimestamp())); }
	static FTableCellValue GetDestroyedTimestamp(const FTableColumn& Column, const FTaskEntry& Task) { return FTableCellValue(GetRelativeValue(Task.GetDestroyedTimestamp(), Task.GetCreatedTimestamp())); }
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTaskDoubleValueFormatterAsTimeAuto : public FTableCellValueFormatter
{
public:
	virtual FText FormatValue(const TOptional<FTableCellValue>& InValue) const override
	{
		if (InValue.IsSet())
		{
			const double Value = InValue.GetValue().Double;
			if (Value != TraceServices::FTaskInfo::InvalidTimestamp)
			{
				return FText::FromString(FormatTimeAuto(Value));
			}
		}

		return FText::FromString(TEXT("N/A"));
	}

	virtual FText FormatValueForTooltip(const TOptional<FTableCellValue>& InValue) const override
	{
		if (InValue.IsSet())
		{
			const double Value = InValue.GetValue().Double;
			if (Value == 0.0)
			{
				return FText::FromString(TEXT("0"));
			}
			else if (Value != TraceServices::FTaskInfo::InvalidTimestamp)
			{
				return FText::FromString(FString::Printf(TEXT("%f (%s)"), Value, *FormatTimeAuto(Value)));
			}
		}

		return FText::FromString(TEXT("N/A"));
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTaskTable
////////////////////////////////////////////////////////////////////////////////////////////////////

FTaskTable::FTaskTable()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTaskTable::~FTaskTable()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTable::Reset()
{
	FTable::Reset();

	AddDefaultColumns();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTable::AddDefaultColumns()
{
	using namespace UE::Insights;

	//////////////////////////////////////////////////
	// Hierarchy Column
	{
		const int32 HierarchyColumnIndex = -1;
		const TCHAR* HierarchyColumnName = nullptr;
		AddHierarchyColumn(HierarchyColumnIndex, HierarchyColumnName);

		const TSharedRef<FTableColumn>& ColumnRef = GetColumns()[0];
		ColumnRef->SetInitialWidth(100.0f);
		ColumnRef->SetShortName(LOCTEXT("TaskColumnName", "Hierarchy"));
		ColumnRef->SetTitleName(LOCTEXT("TaskColumnTitle", "Task Hierarchy"));
		ColumnRef->SetDescription(LOCTEXT("TaskColumnDesc", "Hierarchy of the task's tree"));
	}

	int32 ColumnIndex = 0;

	//////////////////////////////////////////////////
	// DebugName Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::DebugNameColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("DebugNameColumnName", "DebugName"));
		Column.SetTitleName(LOCTEXT("DebugNameColumnTitle", "DebugName"));
		Column.SetDescription(LOCTEXT("DebugNameColumnDesc", "DebugName"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(200.0f);

		Column.SetDataType(ETableCellDataType::CString);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetDebugName>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}

	//////////////////////////////////////////////////
	// Created Timestamp Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::CreatedTimestampColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("CreatedTimestampColumnName", "Created"));
		Column.SetTitleName(LOCTEXT("CreatedTimestampColumnTitle", "Created"));
		Column.SetDescription(LOCTEXT("CreatedTimestampColumnDesc", "The time when the task was created."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetCreatedTimestamp>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTaskDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Min);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Created Thread Id Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::CreatedThreadIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("CreatedThreadIdColumnName", "Created Thread Id"));
		Column.SetTitleName(LOCTEXT("CreatedThreadIdColumnTitle", "Created Thread Id"));
		Column.SetDescription(LOCTEXT("CreatedThreadIdColumnDesc", "The thread the task was created on."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetCreatedThreadId>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}

	//////////////////////////////////////////////////
	// Launched Timestamp Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::LaunchedTimestampColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("LaunchedTimestampColumnName", "Launched"));
		Column.SetTitleName(LOCTEXT("LaunchedTimestampColumnTitle", "Launched"));
		Column.SetDescription(LOCTEXT("LaunchedTimestampColumnDesc", "The time when the task was Launched."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetLaunchedTimestamp>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTaskDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Min);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Launched Thread Id Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::LaunchedThreadIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("LaunchedThreadIdColumnName", "Launched Thread Id"));
		Column.SetTitleName(LOCTEXT("LaunchedThreadIdColumnTitle", "Launched Thread Id"));
		Column.SetDescription(LOCTEXT("LaunchedThreadIdColumnDesc", "The thread the task was Launched on."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetLaunchedThreadId>>();;
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}

	//////////////////////////////////////////////////
	// Scheduled Timestamp Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::ScheduledTimestampColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("ScheduledTimestampColumnName", "Scheduled"));
		Column.SetTitleName(LOCTEXT("ScheduledTimestampColumnTitle", "Scheduled"));
		Column.SetDescription(LOCTEXT("ScheduledTimestampColumnDesc", "The time when the task was Scheduled."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetScheduledTimestamp>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTaskDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Min);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Scheduled Thread Id Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::ScheduledThreadIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("ScheduledThreadIdColumnName", "Scheduled Thread Id"));
		Column.SetTitleName(LOCTEXT("ScheduledThreadIdColumnTitle", "Scheduled Thread Id"));
		Column.SetDescription(LOCTEXT("ScheduledThreadIdColumnDesc", "The thread the task was Scheduled on."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetScheduledThreadId>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Started Timestamp Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::StartedTimestampColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("StartedTimestampColumnName", "Started"));
		Column.SetTitleName(LOCTEXT("StartedTimestampColumnTitle", "Started"));
		Column.SetDescription(LOCTEXT("StartedTimestampColumnDesc", "The time when the task was Started."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetStartedTimestamp>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTaskDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Min);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Started Thread Id Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::StartedThreadIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("StartedThreadIdColumnName", "Started Thread Id"));
		Column.SetTitleName(LOCTEXT("StartedThreadIdColumnTitle", "Started Thread Id"));
		Column.SetDescription(LOCTEXT("StartedThreadIdColumnDesc", "The thread the task was Started on."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetStartedThreadId>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Finished Timestamp Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::FinishedTimestampColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("FinishedTimestampColumnName", "Finished"));
		Column.SetTitleName(LOCTEXT("FinishedTimestampColumnTitle", "Finished"));
		Column.SetDescription(LOCTEXT("FinishedTimestampColumnDesc", "The time when the task was Finished."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetFinishedTimestamp>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTaskDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Min);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Completed Timestamp Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::CompletedTimestampColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("CompletedTimestampColumnName", "Completed"));
		Column.SetTitleName(LOCTEXT("CompletedTimestampColumnTitle", "Completed"));
		Column.SetDescription(LOCTEXT("CompletedTimestampColumnDesc", "The time when the task was Completed."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetCompletedTimestamp>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTaskDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Min);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Completed Thread Id Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::CompletedThreadIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("CompletedThreadIdColumnName", "Completed Thread Id"));
		Column.SetTitleName(LOCTEXT("CompletedThreadIdColumnTitle", "Completed Thread Id"));
		Column.SetDescription(LOCTEXT("CompletedThreadIdColumnDesc", "The thread the task was Completed on."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetCompletedThreadId>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Destroyed Timestamp Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::DestroyedTimestampColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("DestroyedTimestampColumnName", "Destroyed"));
		Column.SetTitleName(LOCTEXT("DestroyedTimestampColumnTitle", "Destroyed"));
		Column.SetDescription(LOCTEXT("DestroyedTimestampColumnDesc", "The time when the task was Destroyed."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Double);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetDestroyedTimestamp>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTaskDoubleValueFormatterAsTimeAuto>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::Min);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Destroyed Thread Id Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::DestroyedThreadIdColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("DestroyedThreadIdColumnName", "Destroyed Thread Id"));
		Column.SetTitleName(LOCTEXT("DestroyedThreadIdColumnTitle", "Destroyed Thread Id"));
		Column.SetDescription(LOCTEXT("DestroyedThreadIdColumnDesc", "The thread the task was Destroyed on."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetDestroyedThreadId>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Task Size Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::TaskSizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TaskSizeColumnName", "Task Size"));
		Column.SetTitleName(LOCTEXT("TaskSizeColumnTitle", "Task Size"));
		Column.SetDescription(LOCTEXT("TaskSizeColumnDesc", "The size of the task including user-provided task body."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(60.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetTaskSize>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Num Parent Tasks Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::NumParentColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("ParentTasksColumnName", "Parent Tasks"));
		Column.SetTitleName(LOCTEXT("ParentTasksColumnTitle", "Parent Tasks"));
		Column.SetDescription(LOCTEXT("ParentTasksColumnDesc", "The number of parent tasks."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetNumParents>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Num Nested Tasks Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::NumNestedColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("NestedTasksColumnName", "Nested Tasks"));
		Column.SetTitleName(LOCTEXT("NestedTasksColumnTitle", "Nested Tasks"));
		Column.SetDescription(LOCTEXT("NestedTasksColumnDesc", "The number of nested tasks."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetNumNested>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Num Subsequents Tasks Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::NumSubsequentsColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("SubsequentTasksColumnName", "Subsequent Tasks"));
		Column.SetTitleName(LOCTEXT("SubsequentTasksColumnTitle", "Subsequent Tasks"));
		Column.SetDescription(LOCTEXT("SubsequentTasksColumnDesc", "The number of Subsequent tasks."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetNumSubsequents>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Num Prerequisites Tasks Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FTaskTableColumns::NumPrerequisitesColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("PrerequisiteTasksColumnName", "Prerequisite Tasks"));
		Column.SetTitleName(LOCTEXT("PrerequisiteTasksColumnTitle", "Prerequisite Tasks"));
		Column.SetDescription(LOCTEXT("PrerequisiteTasksColumnDesc", "The number of Prerequisite tasks."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetNumPrerequisites>>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);

		AddColumn(ColumnRef);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTable::SwitchToAbsoluteTimestamps()
{
	FindColumnChecked(FTaskTableColumns::LaunchedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetLaunchedTimestamp>>());
	FindColumnChecked(FTaskTableColumns::ScheduledTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetScheduledTimestamp>>());
	FindColumnChecked(FTaskTableColumns::StartedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetStartedTimestamp>>());
	FindColumnChecked(FTaskTableColumns::FinishedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetFinishedTimestamp>>());
	FindColumnChecked(FTaskTableColumns::CompletedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetCompletedTimestamp>>());
	FindColumnChecked(FTaskTableColumns::DestroyedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<DefaultTaskFieldGetterFuncts::GetDestroyedTimestamp>>());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTable::SwitchToRelativeToPreviousTimestamps()
{
	FindColumnChecked(FTaskTableColumns::LaunchedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<RelativeToPreviousTaskFieldGetterFuncts::GetLaunchedTimestamp>>());
	FindColumnChecked(FTaskTableColumns::ScheduledTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<RelativeToPreviousTaskFieldGetterFuncts::GetScheduledTimestamp>>());
	FindColumnChecked(FTaskTableColumns::StartedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<RelativeToPreviousTaskFieldGetterFuncts::GetStartedTimestamp>>());
	FindColumnChecked(FTaskTableColumns::FinishedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<RelativeToPreviousTaskFieldGetterFuncts::GetFinishedTimestamp>>());
	FindColumnChecked(FTaskTableColumns::CompletedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<RelativeToPreviousTaskFieldGetterFuncts::GetCompletedTimestamp>>());
	FindColumnChecked(FTaskTableColumns::DestroyedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<RelativeToPreviousTaskFieldGetterFuncts::GetDestroyedTimestamp>>());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTable::SwitchToRelativeToCreatedTimestamps()
{
	FindColumnChecked(FTaskTableColumns::LaunchedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<RelativeToCreatedTaskFieldGetterFuncts::GetLaunchedTimestamp>>());
	FindColumnChecked(FTaskTableColumns::ScheduledTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<RelativeToCreatedTaskFieldGetterFuncts::GetScheduledTimestamp>>());
	FindColumnChecked(FTaskTableColumns::StartedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<RelativeToCreatedTaskFieldGetterFuncts::GetStartedTimestamp>>());
	FindColumnChecked(FTaskTableColumns::FinishedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<RelativeToCreatedTaskFieldGetterFuncts::GetFinishedTimestamp>>());
	FindColumnChecked(FTaskTableColumns::CompletedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<RelativeToCreatedTaskFieldGetterFuncts::GetCompletedTimestamp>>());
	FindColumnChecked(FTaskTableColumns::DestroyedTimestampColumnId)->SetValueGetter(MakeShared<FTaskColumnValueGetter<RelativeToCreatedTaskFieldGetterFuncts::GetDestroyedTimestamp>>());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TaskGraphProfiler

#undef LOCTEXT_NAMESPACE
