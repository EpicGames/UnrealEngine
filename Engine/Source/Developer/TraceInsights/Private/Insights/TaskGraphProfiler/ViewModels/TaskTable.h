// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "UObject/NameTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/Table.h"

// TraceInsights
#include "Insights/TaskGraphProfiler/ViewModels/TaskEntry.h"

namespace UE::Insights { class FTableColumn; }

namespace UE::Insights::TaskGraphProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

// Column identifiers
struct FTaskTableColumns
{
	static const FName DebugNameColumnId;
	static const FName CreatedTimestampColumnId;
	static const FName CreatedThreadIdColumnId;
	static const FName LaunchedTimestampColumnId;
	static const FName LaunchedThreadIdColumnId;
	static const FName ScheduledTimestampColumnId;
	static const FName ScheduledThreadIdColumnId;
	static const FName StartedTimestampColumnId;
	static const FName StartedThreadIdColumnId;
	static const FName FinishedTimestampColumnId;
	static const FName CompletedTimestampColumnId;
	static const FName CompletedThreadIdColumnId;
	static const FName DestroyedTimestampColumnId;
	static const FName DestroyedThreadIdColumnId;
	static const FName TaskSizeColumnId;
	static const FName NumParentColumnId;
	static const FName NumNestedColumnId;
	static const FName NumSubsequentsColumnId;
	static const FName NumPrerequisitesColumnId;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTaskTable : public FTable
{
public:
	FTaskTable();
	virtual ~FTaskTable();

	virtual void Reset();

	TArray<FTaskEntry>& GetTaskEntries() { return TaskEntries; }
	const TArray<FTaskEntry>& GetTaskEntries() const { return TaskEntries; }

	bool IsValidRowIndex(int32 InIndex) const { return InIndex >= 0 && InIndex < TaskEntries.Num(); }
	const FTaskEntry* GetTask(int32 InIndex) const { return IsValidRowIndex(InIndex) ? &TaskEntries[InIndex] : nullptr; }
	const FTaskEntry& GetTaskChecked(int32 InIndex) const { check(IsValidRowIndex(InIndex)); return TaskEntries[InIndex]; }

	void SwitchToAbsoluteTimestamps();
	void SwitchToRelativeToPreviousTimestamps();
	void SwitchToRelativeToCreatedTimestamps();

private:
	void AddDefaultColumns();

private:
	TArray<FTaskEntry> TaskEntries;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TaskGraphProfiler
