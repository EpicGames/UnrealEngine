// Copyright Epic Games, Inc. All Rights Reserved.

#include "TaskTimingTrack.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

// TraceServices
#include "TraceServices/Model/TasksProfiler.h"
#include "TraceServices/Model/Threads.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/TaskGraphProfiler/TaskGraphProfilerManager.h"
#include "Insights/TaskGraphProfiler/ViewModels/TaskGraphRelation.h"
#include "Insights/TaskGraphProfiler/ViewModels/TaskTrackEvent.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Tracks/ThreadTimingTrack.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/ViewModels/ThreadTrackEvent.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingViewDrawHelper.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TaskGraphProfiler"

namespace UE::Insights::TaskGraphProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTaskTimingStateCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

class FTaskTimingStateCommands : public TCommands<FTaskTimingStateCommands>
{
public:
	FTaskTimingStateCommands()
	: TCommands<FTaskTimingStateCommands>(
		TEXT("TaskTimingStateCommands"),
		NSLOCTEXT("Contexts", "TaskTimingStateCommands", "Insights - Task Timing View"),
		NAME_None,
		FInsightsStyle::GetStyleSetName())
	{
	}

	virtual ~FTaskTimingStateCommands()
	{
	}

	// UI_COMMAND takes long for the compiler to optimize
	UE_DISABLE_OPTIMIZATION_SHIP
	virtual void RegisterCommands() override
	{
		UI_COMMAND(Command_ShowTaskTransitions,
			"Show Task Transitions",
			"Show/hide transitions between the stages of the current task (for a selected CPU timing event.)",
			EUserInterfaceActionType::ToggleButton,
			FInputChord(EKeys::T));

		UI_COMMAND(Command_ShowTaskConnections,
			"Show Task Connections",
			"Show/hide connections between:\nThe current task's prerequisites completed time and the current task's started time.\nThe current task's completed time and the current task's subsequents started time.\nThe current task's nested tasks added time and their started time.",
			EUserInterfaceActionType::ToggleButton, FInputChord());

		UI_COMMAND(Command_ShowTaskPrerequisites,
			"Show Transitions of Prerequisites",
			"Show/hide stage transitions for the current task's prerequisites.",
			EUserInterfaceActionType::ToggleButton,
			FInputChord(EKeys::P));

		UI_COMMAND(Command_ShowTaskSubsequents,
			"Show Transitions of Subsequents",
			"Show/hide stage transitions for the current task's subsequents.",
			EUserInterfaceActionType::ToggleButton,
			FInputChord(EKeys::S));

		UI_COMMAND(Command_ShowParentTasks,
			"Show Transitions of Parent Tasks",
			"Show/hide stage transitions for the current task's parent tasks.",
			EUserInterfaceActionType::ToggleButton,
			FInputChord(EKeys::R));

		UI_COMMAND(Command_ShowNestedTasks,
			"Show Transitions of Nested Tasks",
			"Show/hide stage transitions for the current task's nested tasks.",
			EUserInterfaceActionType::ToggleButton,
			FInputChord(EKeys::N));

		UI_COMMAND(Command_ShowCriticalPath,
			"Show Task Critical Path",
			"Show/hide relations representing the critical path containing the current task.",
			EUserInterfaceActionType::ToggleButton,
			FInputChord());

		UI_COMMAND(Command_ShowTaskTrack,
			"Show Task Overview Track",
			"Show/hide the Task Overview Track when a task is selected.",
			EUserInterfaceActionType::ToggleButton,
			FInputChord());

		UI_COMMAND(Command_ShowDetailedTaskTrackInfo,
			"Show Detailed Info on the Task Overview Track",
			"Show the current task's prerequisites/nested tasks/subsequents in the Task Overview Track.",
			EUserInterfaceActionType::ToggleButton,
			FInputChord());
	}
	UE_ENABLE_OPTIMIZATION_SHIP

	TSharedPtr<FUICommandInfo> Command_ShowTaskTransitions;
	TSharedPtr<FUICommandInfo> Command_ShowTaskConnections;
	TSharedPtr<FUICommandInfo> Command_ShowTaskPrerequisites;
	TSharedPtr<FUICommandInfo> Command_ShowTaskSubsequents;
	TSharedPtr<FUICommandInfo> Command_ShowParentTasks;
	TSharedPtr<FUICommandInfo> Command_ShowNestedTasks;
	TSharedPtr<FUICommandInfo> Command_ShowCriticalPath;
	TSharedPtr<FUICommandInfo> Command_ShowTaskTrack;
	TSharedPtr<FUICommandInfo> Command_ShowDetailedTaskTrackInfo;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTaskTimingSharedState
////////////////////////////////////////////////////////////////////////////////////////////////////

FTaskTimingSharedState::FTaskTimingSharedState(TimingProfiler::STimingView* InTimingView)
	: TimingViewSession(InTimingView)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::OnBeginSession(Timing::ITimingViewSession& InSession)
{
	if (&InSession != TimingViewSession)
	{
		if (InSession.GetName() == FInsightsManagerTabs::TimingProfilerTabId)
		{
			TimingViewSession = &InSession;
		}
		else
		{
			return;
		}
	}

	TaskTrack = nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::OnEndSession(Timing::ITimingViewSession& InSession)
{
	if (&InSession != TimingViewSession)
	{
		return;
	}

	FTaskGraphProfilerManager::Get()->ClearTaskRelations();
	TaskTrack = nullptr;
	TimingViewSession = nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::Tick(Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	if (&InSession != TimingViewSession)
	{
		return;
	}

	if (!TaskTrack.IsValid() && FTaskGraphProfilerManager::Get()->GetIsAvailable())
	{
		TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();

		InitCommandList(TimingView);

		FTaskGraphProfilerManager::Get()->RegisterOnWindowClosedEventHandle();

		TaskTrack = MakeShared<FTaskTimingTrack>(*this, TEXT("Task Overview"), 0);
		TaskTrack->SetVisibilityFlag(true);
		TaskTrack->SetOrder(FTimingTrackOrder::Task);

		TimingView->OnSelectedEventChanged().AddSP(TaskTrack.Get(), &FTaskTimingTrack::OnTimingEventSelected);

		InSession.AddTopDockedTrack(TaskTrack);
	}

	if (bResetOnNextTick)
	{
		bResetOnNextTick = false;

		TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
		if (TimingView.IsValid() &&
			!TimingView->GetSelectedEvent().IsValid() &&
			(!TimingView->GetSelectedTrack().IsValid() || TimingView->GetSelectedTrack().Get() != TaskTrack.Get()))
		{
			SetTaskId(TaskTrace::InvalidId);
			FTaskGraphProfilerManager::Get()->ClearTaskRelations();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<TimingProfiler::STimingView> FTaskTimingSharedState::GetTimingView()
{
	using namespace UE::Insights::TimingProfiler;
	TSharedPtr<STimingProfilerWindow> Window = FTimingProfilerManager::Get()->GetProfilerWindow();
	return Window.IsValid() ? Window->GetTimingView() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::ExtendFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InOutMenuBuilder)
{
	if (&InSession != TimingViewSession)
	{
		return;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::SetTaskId(TaskTrace::FId InTaskId)
{
	if (TaskTrack.IsValid())
	{
		TaskTrack->SetTaskId(InTaskId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::ExtendOtherTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder)
{
	if (&InSession != TimingViewSession)
	{
		return;
	}

	InMenuBuilder.BeginSection("TaskGraphInsightsOptions", LOCTEXT("OthersMenu_Section_Tasks", "Tasks"));

	InMenuBuilder.AddSubMenu
	(
		LOCTEXT("ContextMenu_Tasks_SubMenu", "Tasks"),
		LOCTEXT("ContextMenu_Tasks_SubMenu_Desc", "Task Graph Insights settings"),
		FNewMenuDelegate::CreateSP(this, &FTaskTimingSharedState::BuildTasksSubMenu)
	);

	InMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::BuildTasksSubMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry
	(
		FTaskTimingStateCommands::Get().Command_ShowCriticalPath,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ShowTaskCriticalPath")
	);

	MenuBuilder.AddMenuEntry
	(
		FTaskTimingStateCommands::Get().Command_ShowTaskTransitions,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ShowTaskTransitions")
	);

	MenuBuilder.AddMenuEntry
	(
		FTaskTimingStateCommands::Get().Command_ShowTaskConnections,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ShowTaskConnections")
	);

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry
	(
		FTaskTimingStateCommands::Get().Command_ShowTaskPrerequisites,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ShowTaskPrerequisites")
	);

	MenuBuilder.AddMenuEntry
	(
		FTaskTimingStateCommands::Get().Command_ShowTaskSubsequents,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ShowTaskSubsequents")
	);

	MenuBuilder.AddMenuEntry
	(
		FTaskTimingStateCommands::Get().Command_ShowParentTasks,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ShowParentTasks")
	);

	MenuBuilder.AddMenuEntry
	(
		FTaskTimingStateCommands::Get().Command_ShowNestedTasks,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ShowNestedTasks")
	);

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry
	(
		FTaskTimingStateCommands::Get().Command_ShowTaskTrack,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ShowTaskTrack")
	);

	MenuBuilder.AddMenuEntry
	(
		FTaskTimingStateCommands::Get().Command_ShowDetailedTaskTrackInfo,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ShowDetailedTaskTrackInfo")
	);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::InitCommandList(TSharedPtr<TimingProfiler::STimingView> TimingView)
{
	if (!ensure(TimingView.IsValid()))
	{
		return;
	}

	TSharedPtr<FUICommandList> CommandList = TimingView->GetCommandList();
	if (!ensure(CommandList.IsValid()))
	{
		return;
	}

	FTaskTimingStateCommands::Register();

	CommandList->MapAction(
		FTaskTimingStateCommands::Get().Command_ShowTaskTransitions,
		FExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskTransitions_Execute),
		FCanExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskTransitions_CanExecute),
		FIsActionChecked::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskTransitions_IsChecked));

	CommandList->MapAction(
		FTaskTimingStateCommands::Get().Command_ShowTaskConnections,
		FExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskConnections_Execute),
		FCanExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskConnections_CanExecute),
		FIsActionChecked::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskConnections_IsChecked));

	CommandList->MapAction(
		FTaskTimingStateCommands::Get().Command_ShowTaskPrerequisites,
		FExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskPrerequisites_Execute),
		FCanExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskPrerequisites_CanExecute),
		FIsActionChecked::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskPrerequisites_IsChecked));

	CommandList->MapAction(
		FTaskTimingStateCommands::Get().Command_ShowTaskSubsequents,
		FExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskSubsequents_Execute),
		FCanExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskSubsequents_CanExecute),
		FIsActionChecked::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskSubsequents_IsChecked));

	CommandList->MapAction(
		FTaskTimingStateCommands::Get().Command_ShowParentTasks,
		FExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowParentTasks_Execute),
		FCanExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowParentTasks_CanExecute),
		FIsActionChecked::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowParentTasks_IsChecked));

	CommandList->MapAction(
		FTaskTimingStateCommands::Get().Command_ShowNestedTasks,
		FExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowNestedTasks_Execute),
		FCanExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowNestedTasks_CanExecute),
		FIsActionChecked::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowNestedTasks_IsChecked));

	CommandList->MapAction(
		FTaskTimingStateCommands::Get().Command_ShowCriticalPath,
		FExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowCriticalPath_Execute),
		FCanExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowCriticalPath_CanExecute),
		FIsActionChecked::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowCriticalPath_IsChecked));

	CommandList->MapAction(
		FTaskTimingStateCommands::Get().Command_ShowTaskTrack,
		FExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskTrack_Execute),
		FCanExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskTrack_CanExecute),
		FIsActionChecked::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowTaskTrack_IsChecked));

	CommandList->MapAction(
		FTaskTimingStateCommands::Get().Command_ShowDetailedTaskTrackInfo,
		FExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowDetailedTaskTrackInfo_Execute),
		FCanExecuteAction::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowDetailedTaskTrackInfo_CanExecute),
		FIsActionChecked::CreateSP(this, &FTaskTimingSharedState::ContextMenu_ShowDetailedTaskTrackInfo_IsChecked));
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Transitions

void FTaskTimingSharedState::ContextMenu_ShowTaskTransitions_Execute()
{
	TSharedPtr<FTaskGraphProfilerManager> TaskGraphManager = FTaskGraphProfilerManager::Get();
	if (TaskGraphManager.IsValid() && TaskGraphManager->GetIsAvailable())
	{
		TaskGraphManager->SetShowTransitions(!TaskGraphManager->GetShowTransitions());
		OnTaskSettingsChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTaskTimingSharedState::ContextMenu_ShowTaskTransitions_CanExecute()
{
	return FTaskGraphProfilerManager::Get().IsValid() && FTaskGraphProfilerManager::Get()->GetIsAvailable();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTaskTimingSharedState::ContextMenu_ShowTaskTransitions_IsChecked()
{
	TSharedPtr<FTaskGraphProfilerManager> TaskGraphManager = FTaskGraphProfilerManager::Get();
	return TaskGraphManager.IsValid() && TaskGraphManager->GetIsAvailable() && TaskGraphManager->GetShowTransitions();
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Connections

void FTaskTimingSharedState::ContextMenu_ShowTaskConnections_Execute()
{
	TSharedPtr<FTaskGraphProfilerManager> TaskGraphManager = FTaskGraphProfilerManager::Get();
	if (TaskGraphManager.IsValid() && TaskGraphManager->GetIsAvailable())
	{
		TaskGraphManager->SetShowConnections(!TaskGraphManager->GetShowConnections());
		OnTaskSettingsChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTaskTimingSharedState::ContextMenu_ShowTaskConnections_CanExecute()
{
	return FTaskGraphProfilerManager::Get().IsValid() && FTaskGraphProfilerManager::Get()->GetIsAvailable();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTaskTimingSharedState::ContextMenu_ShowTaskConnections_IsChecked()
{
	TSharedPtr<FTaskGraphProfilerManager> TaskGraphManager = FTaskGraphProfilerManager::Get();
	return TaskGraphManager.IsValid() && TaskGraphManager->GetIsAvailable() && TaskGraphManager->GetShowConnections();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Prerequisites

bool FTaskTimingSharedState::ContextMenu_ShowTaskPrerequisites_CanExecute()
{
	return FTaskGraphProfilerManager::Get().IsValid() && FTaskGraphProfilerManager::Get()->GetIsAvailable();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTaskTimingSharedState::ContextMenu_ShowTaskPrerequisites_IsChecked()
{
	TSharedPtr<FTaskGraphProfilerManager> TaskGraphManager = FTaskGraphProfilerManager::Get();
	return TaskGraphManager.IsValid() && TaskGraphManager->GetIsAvailable() && TaskGraphManager->GetShowPrerequisites();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::ContextMenu_ShowTaskPrerequisites_Execute()
{
	TSharedPtr<FTaskGraphProfilerManager> TaskGraphManager = FTaskGraphProfilerManager::Get();
	if (TaskGraphManager.IsValid() && TaskGraphManager->GetIsAvailable())
	{
		TaskGraphManager->SetShowPrerequisites(!TaskGraphManager->GetShowPrerequisites());
		OnTaskSettingsChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Subsequents

bool FTaskTimingSharedState::ContextMenu_ShowTaskSubsequents_CanExecute()
{
	return FTaskGraphProfilerManager::Get().IsValid() && FTaskGraphProfilerManager::Get()->GetIsAvailable();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTaskTimingSharedState::ContextMenu_ShowTaskSubsequents_IsChecked()
{
	TSharedPtr<FTaskGraphProfilerManager> TaskGraphManager = FTaskGraphProfilerManager::Get();
	return TaskGraphManager.IsValid() && TaskGraphManager->GetIsAvailable() && TaskGraphManager->GetShowSubsequents();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::ContextMenu_ShowTaskSubsequents_Execute()
{
	TSharedPtr<FTaskGraphProfilerManager> TaskGraphManager = FTaskGraphProfilerManager::Get();
	if (TaskGraphManager.IsValid() && TaskGraphManager->GetIsAvailable())
	{
		TaskGraphManager->SetShowSubsequents(!TaskGraphManager->GetShowSubsequents());
		OnTaskSettingsChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ParentTasks

bool FTaskTimingSharedState::ContextMenu_ShowParentTasks_CanExecute()
{
	return FTaskGraphProfilerManager::Get().IsValid() && FTaskGraphProfilerManager::Get()->GetIsAvailable();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTaskTimingSharedState::ContextMenu_ShowParentTasks_IsChecked()
{
	TSharedPtr<FTaskGraphProfilerManager> TaskGraphManager = FTaskGraphProfilerManager::Get();
	return TaskGraphManager.IsValid() && TaskGraphManager->GetIsAvailable() && TaskGraphManager->GetShowParentTasks();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::ContextMenu_ShowParentTasks_Execute()
{
	TSharedPtr<FTaskGraphProfilerManager> TaskGraphManager = FTaskGraphProfilerManager::Get();
	if (TaskGraphManager.IsValid() && TaskGraphManager->GetIsAvailable())
	{
		TaskGraphManager->SetShowParentTasks(!TaskGraphManager->GetShowParentTasks());
		OnTaskSettingsChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// NestedTasks

bool FTaskTimingSharedState::ContextMenu_ShowNestedTasks_CanExecute()
{
	return FTaskGraphProfilerManager::Get().IsValid() && FTaskGraphProfilerManager::Get()->GetIsAvailable();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTaskTimingSharedState::ContextMenu_ShowNestedTasks_IsChecked()
{
	TSharedPtr<FTaskGraphProfilerManager> TaskGraphManager = FTaskGraphProfilerManager::Get();
	return TaskGraphManager.IsValid() && TaskGraphManager->GetIsAvailable() && TaskGraphManager->GetShowNestedTasks();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::ContextMenu_ShowNestedTasks_Execute()
{
	TSharedPtr<FTaskGraphProfilerManager> TaskGraphManager = FTaskGraphProfilerManager::Get();
	if (TaskGraphManager.IsValid() && TaskGraphManager->GetIsAvailable())
	{
		TaskGraphManager->SetShowNestedTasks(!TaskGraphManager->GetShowNestedTasks());
		OnTaskSettingsChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// CriticalPath

bool FTaskTimingSharedState::ContextMenu_ShowCriticalPath_CanExecute()
{
	return FTaskGraphProfilerManager::Get().IsValid() && FTaskGraphProfilerManager::Get()->GetIsAvailable();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTaskTimingSharedState::ContextMenu_ShowCriticalPath_IsChecked()
{
	TSharedPtr<FTaskGraphProfilerManager> TaskGraphManager = FTaskGraphProfilerManager::Get();
	return TaskGraphManager.IsValid() && TaskGraphManager->GetIsAvailable() && TaskGraphManager->GetShowCriticalPath();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::ContextMenu_ShowCriticalPath_Execute()
{
	TSharedPtr<FTaskGraphProfilerManager> TaskGraphManager = FTaskGraphProfilerManager::Get();
	if (TaskGraphManager.IsValid() && TaskGraphManager->GetIsAvailable())
	{
		TaskGraphManager->SetShowCriticalPath(!TaskGraphManager->GetShowCriticalPath());
		OnTaskSettingsChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::ContextMenu_ShowTaskTrack_Execute()
{
	if (TaskTrack.IsValid())
	{
		TaskTrack->ToggleVisibility();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTaskTimingSharedState::ContextMenu_ShowTaskTrack_CanExecute()
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTaskTimingSharedState::ContextMenu_ShowTaskTrack_IsChecked()
{
	if (TaskTrack.IsValid())
	{
		return TaskTrack->IsVisible();
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::ContextMenu_ShowDetailedTaskTrackInfo_Execute()
{
	if (TaskTrack.IsValid())
	{
		TaskTrack->SetShowDetailedInfoOnTaskTrack(!TaskTrack->GetShowDetailedInfoOnTaskTrack());
		TaskTrack->SetDirtyFlag();
	}
}
////////////////////////////////////////////////////////////////////////////////////////////////////


bool FTaskTimingSharedState::ContextMenu_ShowDetailedTaskTrackInfo_CanExecute()
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTaskTimingSharedState::ContextMenu_ShowDetailedTaskTrackInfo_IsChecked()
{
	if (TaskTrack.IsValid())
	{
		return TaskTrack->GetShowDetailedInfoOnTaskTrack();
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingSharedState::OnTaskSettingsChanged()
{
	if (!TaskTrack.IsValid())
	{
		return;
	}

	FTaskGraphProfilerManager::Get()->ClearTaskRelations();
	TaskTrack->SetTaskId(TaskTrace::InvalidId);

	TSharedPtr<TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView)
	{
		return;
	}

	TSharedPtr<const ITimingEvent> SelectedEvent = TimingView->GetSelectedEvent();
	if (SelectedEvent.IsValid() && SelectedEvent->Is<const FThreadTrackEvent>())
	{
		TaskTrack->OnTimingEventSelected(SelectedEvent);
		return;
	}

	if (TaskTrack->GetTaskId() != TaskTrace::InvalidId)
	{
		FTaskGraphProfilerManager::Get()->ShowTaskRelations(TaskTrack->GetTaskId());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTaskTimingTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FTaskTimingTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	if (TaskId == TaskTrace::InvalidId)
	{
		return;
	}

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

	const TraceServices::ITasksProvider* TasksProvider = TraceServices::ReadTasksProvider(*Session.Get());
	if (!TasksProvider)
	{
		return;
	}

	const TraceServices::FTaskInfo* Task = TasksProvider->TryGetTask(TaskId);
	if (!Task)
	{
		return;
	}

	Builder.AddEvent(Task->CreatedTimestamp, Task->LaunchedTimestamp, 0, TEXT("Created"), 0, FTaskGraphProfilerManager::Get()->GetColorForTaskEventAsPackedARGB(ETaskEventType::Created));
	Builder.AddEvent(Task->LaunchedTimestamp, Task->ScheduledTimestamp, 0, TEXT("Launched"), 0, FTaskGraphProfilerManager::Get()->GetColorForTaskEventAsPackedARGB(ETaskEventType::Launched));
	Builder.AddEvent(Task->ScheduledTimestamp, Task->StartedTimestamp, 0, TEXT("Scheduled"), 0,  FTaskGraphProfilerManager::Get()->GetColorForTaskEventAsPackedARGB(ETaskEventType::Scheduled));
	Builder.AddEvent(Task->StartedTimestamp, Task->FinishedTimestamp, 0, TEXT("Executing"), 0, FTaskGraphProfilerManager::Get()->GetColorForTaskEventAsPackedARGB(ETaskEventType::Started));
	Builder.AddEvent(Task->FinishedTimestamp, Task->CompletedTimestamp, 0, TEXT("Finished"), 0, FTaskGraphProfilerManager::Get()->GetColorForTaskEventAsPackedARGB(ETaskEventType::Finished));
	Builder.AddEvent(Task->CompletedTimestamp, Task->DestroyedTimestamp, 0, TEXT("Completed"), 0, FTaskGraphProfilerManager::Get()->GetColorForTaskEventAsPackedARGB(ETaskEventType::Completed));

	if (bShowDetailInfoOnTaskTrack)
	{
		const uint32 MaxEventDepth = TimingProfiler::FTimingProfilerManager::Get()->GetEventDepthLimit() - 1;

		uint32 Depth = 0;
		for (TraceServices::FTaskInfo::FRelationInfo Relation : Task->Prerequisites)
		{
			if (Depth >= MaxEventDepth)
			{
				break;
			}
			const TraceServices::FTaskInfo* TaskInfo = TasksProvider->TryGetTask(Relation.RelativeId);
			if (TaskInfo)
			{
				Builder.AddEvent(TaskInfo->StartedTimestamp, TaskInfo->FinishedTimestamp, ++Depth, *FString::Printf(TEXT("Prerequisite Task %" UINT64_FMT " Executing"), TaskInfo->Id), 0, FTaskGraphProfilerManager::Get()->GetColorForTaskEventAsPackedARGB(ETaskEventType::PrerequisiteStarted));
			}
		}

		// parent tasks can overlap with prerequisites, hence they are shown under them (`Depth` is not reset)
		for (TraceServices::FTaskInfo::FRelationInfo Relation : Task->ParentTasks)
		{
			if (Depth >= MaxEventDepth)
			{
				break;
			}
			const TraceServices::FTaskInfo* TaskInfo = TasksProvider->TryGetTask(Relation.RelativeId);
			if (TaskInfo)
			{
				Builder.AddEvent(TaskInfo->StartedTimestamp, TaskInfo->FinishedTimestamp, ++Depth, *FString::Printf(TEXT("Parent Task %" UINT64_FMT " Executing"), TaskInfo->Id), 0, FTaskGraphProfilerManager::Get()->GetColorForTaskEventAsPackedARGB(ETaskEventType::ParentStarted));
			}
		}

		Depth = 0;
		for (TraceServices::FTaskInfo::FRelationInfo Relation : Task->NestedTasks)
		{
			if (Depth >= MaxEventDepth)
			{
				break;
			}
			const TraceServices::FTaskInfo* TaskInfo = TasksProvider->TryGetTask(Relation.RelativeId);
			if (TaskInfo)
			{
				Builder.AddEvent(TaskInfo->StartedTimestamp, TaskInfo->FinishedTimestamp, ++Depth, *FString::Printf(TEXT("Nested Task %" UINT64_FMT " Executing"), TaskInfo->Id), 0, FTaskGraphProfilerManager::Get()->GetColorForTaskEventAsPackedARGB(ETaskEventType::NestedStarted));
			}
		}

		Depth = 0;
		for (TraceServices::FTaskInfo::FRelationInfo Relation : Task->Subsequents)
		{
			if (Depth >= MaxEventDepth)
			{
				break;
			}
			const TraceServices::FTaskInfo* TaskInfo = TasksProvider->TryGetTask(Relation.RelativeId);
			if (TaskInfo)
			{
				Builder.AddEvent(TaskInfo->StartedTimestamp, TaskInfo->FinishedTimestamp, ++Depth, *FString::Printf(TEXT("Subsequent Task %" UINT64_FMT " Executing"), TaskInfo->Id), 0, FTaskGraphProfilerManager::Get()->GetColorForTaskEventAsPackedARGB(ETaskEventType::SubsequentStarted));
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingTrack::BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	BuildDrawState(Builder, Context);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingTrack::PostDraw(const ITimingTrackDrawContext& Context) const
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingTrack::OnTimingEventSelected(TSharedPtr<const ITimingEvent> InSelectedEvent)
{
	if (!FTaskGraphProfilerManager::Get()->GetShowAnyRelations())
	{
		return;
	}

	if ((InSelectedEvent.IsValid() && InSelectedEvent->GetTrack()->Is<FTaskTimingTrack>())
		|| IsSelected())
	{
		// The user has selected a Task Event. Do nothing.
		return;
	}

	if (!InSelectedEvent.IsValid() || !InSelectedEvent->Is<FThreadTrackEvent>())
	{
		if (TaskId != TaskTrace::InvalidId)
		{
			TaskId = TaskTrace::InvalidId;
			FTaskGraphProfilerManager::Get()->ClearTaskRelations();
			SetDirtyFlag();
		}
		return;
	}

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

	const TraceServices::ITasksProvider* TasksProvider = TraceServices::ReadTasksProvider(*Session.Get());
	if (!TasksProvider)
	{
		return;
	}

	const FThreadTrackEvent &ThreadEvent = InSelectedEvent->As<FThreadTrackEvent>();
	GetEventRelations(ThreadEvent);

	uint32 ThreadId = StaticCastSharedRef<const TimingProfiler::FThreadTimingTrack>(ThreadEvent.GetTrack())->GetThreadId();

	const TraceServices::FTaskInfo* Task = TasksProvider->TryGetTask(ThreadId, ThreadEvent.GetStartTime());
	if (Task)
	{
		TaskId = Task->Id;
		FTaskGraphProfilerManager::Get()->SelectTaskInTaskTable(TaskId);
	}
	else
	{
		TaskId = TaskTrace::InvalidId;
	}

	SetDirtyFlag();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FTaskTimingTrack::GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const
{
	TSharedPtr<FTaskTrackEvent> TimingEvent;

	if (TaskId == TaskTrace::InvalidId)
	{
		return TimingEvent;
	}

	const FTimingViewLayout& Layout = Viewport.GetLayout();
	const float TopLaneY = GetPosY() + 1.0f + Layout.TimelineDY; // +1.0f is for horizontal line between timelines
	const float DY = InPosY - TopLaneY;

	// If mouse is not above first sub-track or below last sub-track...
	if (DY >= 0 && DY < GetHeight() - 1.0f - 2 * Layout.TimelineDY)
	{
		const double EventTime = Viewport.SlateUnitsToTime(InPosX);
		const int32 Depth = static_cast<int32>(DY / (Layout.EventH + Layout.EventDY));

		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (!Session.IsValid())
		{
			return TimingEvent;
		}

		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::ITasksProvider* TasksProvider = TraceServices::ReadTasksProvider(*Session.Get());
		if (!TasksProvider)
		{
			return TimingEvent;
		}

		const TraceServices::FTaskInfo* Task = TasksProvider->TryGetTask(TaskId);
		if (!Task)
		{
			return TimingEvent;
		}

		const double SecondsPerPixel = 1.0 / Viewport.GetScaleX();

		auto IsEventTimeBetween = [SecondsPerPixel, EventTime](double TimeA, double TimeB)
		{
			return EventTime >= TimeA - 2 * SecondsPerPixel && EventTime <= TimeB + 2 * SecondsPerPixel;
		};

		if (Depth == 0)
		{
			if (IsEventTimeBetween(Task->CreatedTimestamp, Task->LaunchedTimestamp))
			{
				TimingEvent = MakeShared<FTaskTrackEvent>(SharedThis(this), Task->CreatedTimestamp, Task->LaunchedTimestamp, 0, ETaskEventType::Created);
			}
			else if (IsEventTimeBetween(Task->LaunchedTimestamp, Task->ScheduledTimestamp))
			{
				TimingEvent = MakeShared<FTaskTrackEvent>(SharedThis(this), Task->LaunchedTimestamp, Task->ScheduledTimestamp, 0, ETaskEventType::Launched);
			}
			else if (IsEventTimeBetween(Task->ScheduledTimestamp, Task->StartedTimestamp))
			{
				TimingEvent = MakeShared<FTaskTrackEvent>(SharedThis(this), Task->ScheduledTimestamp, Task->StartedTimestamp, 0, ETaskEventType::Scheduled);
			}
			// the unusual order of `if`s here is intended to give priority to smaller events
			else if (IsEventTimeBetween(Task->FinishedTimestamp, Task->CompletedTimestamp))
			{
				TimingEvent = MakeShared<FTaskTrackEvent>(SharedThis(this), Task->FinishedTimestamp, Task->CompletedTimestamp, 0, ETaskEventType::Finished);
			}
			else if (IsEventTimeBetween(Task->CompletedTimestamp, Task->DestroyedTimestamp))
			{
				TimingEvent = MakeShared<FTaskTrackEvent>(SharedThis(this), Task->CompletedTimestamp >= Task->FinishedTimestamp ? Task->CompletedTimestamp : Task->FinishedTimestamp, Task->DestroyedTimestamp, 0, ETaskEventType::Completed);
			}
			else if (IsEventTimeBetween(Task->StartedTimestamp, Task->FinishedTimestamp))
			{
				TimingEvent = MakeShared<FTaskTrackEvent>(SharedThis(this), Task->StartedTimestamp, Task->FinishedTimestamp, 0, ETaskEventType::Started);
			}

			if (TimingEvent.IsValid())
			{
				TimingEvent->SetTaskId(Task->Id);
			}
		}
		else if (bShowDetailInfoOnTaskTrack)
		{
			auto GetEventFromRelations =
				[&TasksProvider, this, EventTime, Depth, SecondsPerPixel]
				(const TArray<TraceServices::FTaskInfo::FRelationInfo>& Relations, int32 RelationIndex, ETaskEventType EventType) -> TSharedPtr<FTaskTrackEvent>
				{
					if (RelationIndex < 0 || Relations.Num() <= RelationIndex)
					{
						return nullptr;
					}
					const TraceServices::FTaskInfo::FRelationInfo& Relation = Relations[RelationIndex];
					{
						const TraceServices::FTaskInfo* TaskInfo = TasksProvider->TryGetTask(Relation.RelativeId);
						if (TaskInfo &&
							EventTime >= TaskInfo->StartedTimestamp - 2 * SecondsPerPixel &&
							EventTime <= TaskInfo->FinishedTimestamp + 2 * SecondsPerPixel)
						{
							TSharedPtr<FTaskTrackEvent> NewTimingEvent = MakeShared<FTaskTrackEvent>(SharedThis(this), TaskInfo->StartedTimestamp, TaskInfo->FinishedTimestamp, Depth, EventType);
							NewTimingEvent->SetTaskId(TaskInfo->Id);
							return NewTimingEvent;
						}
					}

					return nullptr;
				};

			TimingEvent = GetEventFromRelations(Task->Prerequisites, Depth - 1, ETaskEventType::PrerequisiteStarted);

			if (!TimingEvent.IsValid())
			{
				TimingEvent = GetEventFromRelations(Task->ParentTasks, Depth - 1 - Task->Prerequisites.Num(), ETaskEventType::ParentStarted);
			}

			if (!TimingEvent.IsValid())
			{
				TimingEvent = GetEventFromRelations(Task->NestedTasks, Depth - 1, ETaskEventType::NestedStarted);
			}

			if (!TimingEvent.IsValid())
			{
				TimingEvent = GetEventFromRelations(Task->Subsequents, Depth - 1, ETaskEventType::SubsequentStarted);
			}
		}
	}

	return TimingEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	InOutTooltip.ResetContent();

	if (!InTooltipEvent.CheckTrack(this) || !InTooltipEvent.Is<FTaskTrackEvent>())
	{
		return;
	}

	const FTaskTrackEvent& TaskTrackEvent = InTooltipEvent.As<FTaskTrackEvent>();
	InOutTooltip.AddTitle(TaskTrackEvent.GetEventName());

	using namespace UE::Insights;
	InOutTooltip.AddNameValueTextLine(TaskTrackEvent.GetStartLabel(), FormatTimeAuto(TaskTrackEvent.GetStartTime(), 6));
	InOutTooltip.AddNameValueTextLine(TaskTrackEvent.GetEndLabel(), FormatTimeAuto(TaskTrackEvent.GetEndTime(), 6));
	InOutTooltip.AddNameValueTextLine(TEXT("Duration:"), FormatTimeAuto(TaskTrackEvent.GetEndTime() - TaskTrackEvent.GetStartTime()));

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

	const TraceServices::ITasksProvider* TasksProvider = TraceServices::ReadTasksProvider(*Session.Get());
	if (!TasksProvider)
	{
		return;
	}

	const TraceServices::FTaskInfo* Task = TasksProvider->TryGetTask(TaskTrackEvent.GetTaskId());
	if (!Task)
	{
		return;
	}

	InOutTooltip.AddNameValueTextLine(TEXT("Task Id:"), FString::Printf(TEXT("%" UINT64_FMT), Task->Id));

	switch (TaskTrackEvent.GetTaskEventType())
	{
	case ETaskEventType::Created:
		break;
	case ETaskEventType::PrerequisiteStarted:
	case ETaskEventType::ParentStarted:
	case ETaskEventType::NestedStarted:
	case ETaskEventType::SubsequentStarted:
	{
		const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(*Session.Get());
		const TCHAR* StartedThreadName = ThreadProvider.GetThreadName(Task->StartedThreadId);
		if (StartedThreadName)
		{
			InOutTooltip.AddNameValueTextLine(TEXT("Executed Thread Id:"), FString::Printf(TEXT("%u (%s)"), Task->StartedThreadId, StartedThreadName));
		}
		else
		{
			InOutTooltip.AddNameValueTextLine(TEXT("Executed Thread Id:"), FString::Printf(TEXT("%u (<unknown>)"), Task->StartedThreadId));
		}
		break;
	}
	case ETaskEventType::Launched:
	{
		InOutTooltip.AddNameValueTextLine(TEXT("Prerequisite tasks:"), FString::Printf(TEXT("%d"), Task->Prerequisites.Num()));
		break;
	}
	case ETaskEventType::Scheduled:
		break;
	case ETaskEventType::Started:
		InOutTooltip.AddNameValueTextLine(TEXT("Nested tasks:"), FString::Printf(TEXT("%d"), Task->NestedTasks.Num()));
		break;
	case ETaskEventType::Finished:
		break;
	case ETaskEventType::Completed:
		InOutTooltip.AddNameValueTextLine(TEXT("Parent tasks:"), FString::Printf(TEXT("%d"), Task->ParentTasks.Num()));
		InOutTooltip.AddNameValueTextLine(TEXT("Subsequent tasks:"), FString::Printf(TEXT("%d"), Task->Subsequents.Num()));
		break;
	default:
		checkf(false, TEXT("Unknown task event type"));
		break;
	}

	InOutTooltip.UpdateLayout();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTaskTimingTrack::GetEventRelations(const FThreadTrackEvent& InSelectedEvent)
{
	const int32 MaxTasksToShow = 30;
	double StartTime = InSelectedEvent.GetStartTime();
	double EndTime = InSelectedEvent.GetEndTime();

	TSharedRef<const TimingProfiler::FThreadTimingTrack> EventTrack = StaticCastSharedRef<const TimingProfiler::FThreadTimingTrack>(InSelectedEvent.GetTrack());
	uint32 ThreadId = EventTrack->GetThreadId();

	FTaskGraphProfilerManager::Get()->ShowTaskRelations(&InSelectedEvent, ThreadId);

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	const TraceServices::ITasksProvider* TasksProvider = TraceServices::ReadTasksProvider(*Session.Get());
	if (!TasksProvider)
	{
		return;
	}

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
	if (!TimingProfilerProvider)
	{
		return;
	}

	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();

		const FThreadTrackEvent& ThreadTrackEvent = *static_cast<const FThreadTrackEvent*>(&InSelectedEvent);
		const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(ThreadTrackEvent.GetTimerIndex());
		if (!Timer)
		{
			return;
		}

		// if it's an event waiting for tasks completeness, add relations to these tasks
		const TraceServices::FWaitingForTasks* Waiting = TasksProvider->TryGetWaiting(Timer->Name, ThreadId, StartTime);
		if (Waiting)
		{
			int32 NumWaitedTasksToShow = FMath::Min(Waiting->Tasks.Num(), MaxTasksToShow);
			for (int32 TaskIndex = 0; TaskIndex != NumWaitedTasksToShow; ++TaskIndex)
			{
				const TraceServices::FTaskInfo* WaitedTask = TasksProvider->TryGetTask(Waiting->Tasks[TaskIndex]);
				if (WaitedTask)
				{
					int32 WaitingTaskExecutionDepth = FTaskGraphProfilerManager::Get()->GetDepthOfTaskExecution(WaitedTask->StartedTimestamp, WaitedTask->FinishedTimestamp, WaitedTask->StartedThreadId);

					FTaskGraphProfilerManager::Get()->AddRelation(&InSelectedEvent, StartTime, ThreadId, -1, WaitedTask->StartedTimestamp, WaitedTask->StartedThreadId, WaitingTaskExecutionDepth, ETaskEventType::NestedStarted);
					FTaskGraphProfilerManager::Get()->AddRelation(&InSelectedEvent, WaitedTask->CompletedTimestamp, WaitedTask->CompletedThreadId, WaitedTask->CompletedTimestamp, ThreadId, ETaskEventType::NestedCompleted);
				}
			}
		}

		TArray<TaskTrace::FId> ParallelForTasks = TasksProvider->TryGetParallelForTasks(Timer->Name, ThreadId, StartTime, EndTime);
		int32 NumTasksToShow = FMath::Min(ParallelForTasks.Num(), MaxTasksToShow);
		for (int32 TaskIndex = 0; TaskIndex != NumTasksToShow; ++TaskIndex)
		{
			const TraceServices::FTaskInfo* Task = TasksProvider->TryGetTask(ParallelForTasks[TaskIndex]);
			if (Task)
			{
				int32 TaskExecutionDepth = FTaskGraphProfilerManager::Get()->GetDepthOfTaskExecution(Task->StartedTimestamp, Task->FinishedTimestamp, Task->StartedThreadId);
				FTaskGraphProfilerManager::Get()->AddRelation(&InSelectedEvent, Task->StartedTimestamp, ThreadId, -1, Task->StartedTimestamp, Task->StartedThreadId, TaskExecutionDepth, ETaskEventType::NestedStarted);
				FTaskGraphProfilerManager::Get()->AddRelation(&InSelectedEvent, Task->CompletedTimestamp, Task->CompletedThreadId, Task->CompletedTimestamp, ThreadId, ETaskEventType::NestedCompleted);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply FTaskTimingTrack::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		MousePositionOnButtonDown = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	}

	return FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply FTaskTimingTrack::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FVector2D MousePositionOnButtonUp = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && MousePositionOnButtonUp.Equals(MousePositionOnButtonDown, 2.0f))
	{
		SharedState.SetResetOnNextTick(true);
	}

	return FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TaskGraphProfiler

#undef LOCTEXT_NAMESPACE
