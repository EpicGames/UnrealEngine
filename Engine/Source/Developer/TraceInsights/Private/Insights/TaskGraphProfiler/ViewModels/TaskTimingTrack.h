// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Async/TaskTrace.h"

// TraceInsights
#include "Insights/ITimingViewExtender.h"
#include "Insights/ViewModels/TimingEventsTrack.h"

class FUICommandList;

class FThreadTrackEvent;
namespace UE::Insights::TimingProfiler { class STimingView; }

namespace UE::Insights::TaskGraphProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTaskTimingTrack;

class FTaskTimingSharedState : public Timing::ITimingViewExtender, public TSharedFromThis<FTaskTimingSharedState>
{

public:
	FTaskTimingSharedState(TimingProfiler::STimingView* InTimingView);
	virtual ~FTaskTimingSharedState() = default;

	TSharedPtr<FTaskTimingTrack> GetTaskTrack() { return TaskTrack; }

	bool IsTaskTrackVisible() const;

	//////////////////////////////////////////////////
	// ITimingViewExtender interface

	virtual void OnBeginSession(Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(Timing::ITimingViewSession& InSession) override;
	virtual void Tick(Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;
	virtual void ExtendFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;
	virtual void ExtendOtherTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;

	//////////////////////////////////////////////////

	bool IsTaskTrackToggleOn() const { return bShowHideTaskTrack; }
	void SetTaskTrackToggle(bool bOnOff) { bShowHideTaskTrack = bOnOff; }
	void ShowTaskTrack() { SetTaskTrackToggle(true); }
	void HideTaskTrack() { SetTaskTrackToggle(false); }

	void SetTaskId(TaskTrace::FId InTaskId);

	void SetResetOnNextTick(bool bInValue) { bResetOnNextTick = bInValue; }

	static TSharedPtr<TimingProfiler::STimingView> GetTimingView();

private:
	void InitCommandList(TSharedPtr<TimingProfiler::STimingView> TimingView);

	void BuildTasksSubMenu(FMenuBuilder& MenuBuilder);

	void ContextMenu_ShowTaskTransitions_Execute();
	bool ContextMenu_ShowTaskTransitions_CanExecute();
	bool ContextMenu_ShowTaskTransitions_IsChecked();

	void ContextMenu_ShowTaskConnections_Execute();
	bool ContextMenu_ShowTaskConnections_CanExecute();
	bool ContextMenu_ShowTaskConnections_IsChecked();

	void ContextMenu_ShowTaskPrerequisites_Execute();
	bool ContextMenu_ShowTaskPrerequisites_CanExecute();
	bool ContextMenu_ShowTaskPrerequisites_IsChecked();

	void ContextMenu_ShowTaskSubsequents_Execute();
	bool ContextMenu_ShowTaskSubsequents_CanExecute();
	bool ContextMenu_ShowTaskSubsequents_IsChecked();

	void ContextMenu_ShowParentTasks_Execute();
	bool ContextMenu_ShowParentTasks_CanExecute();
	bool ContextMenu_ShowParentTasks_IsChecked();

	void ContextMenu_ShowNestedTasks_Execute();
	bool ContextMenu_ShowNestedTasks_CanExecute();
	bool ContextMenu_ShowNestedTasks_IsChecked();
	
	void ContextMenu_ShowCriticalPath_Execute();
	bool ContextMenu_ShowCriticalPath_CanExecute();
	bool ContextMenu_ShowCriticalPath_IsChecked();

	void ContextMenu_ShowTaskTrack_Execute();
	bool ContextMenu_ShowTaskTrack_CanExecute();
	bool ContextMenu_ShowTaskTrack_IsChecked();

	void ContextMenu_ShowDetailedTaskTrackInfo_Execute();
	bool ContextMenu_ShowDetailedTaskTrackInfo_CanExecute();
	bool ContextMenu_ShowDetailedTaskTrackInfo_IsChecked();

	void OnTaskSettingsChanged();

private:
	Timing::ITimingViewSession* TimingViewSession;

	bool bShowHideTaskTrack;
	bool bResetOnNextTick = false;

	TSharedPtr<FTaskTimingTrack> TaskTrack;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTaskTimingTrack : public FTimingEventsTrack
{
	INSIGHTS_DECLARE_RTTI(FTaskTimingTrack, FTimingEventsTrack)

public:
	explicit FTaskTimingTrack(FTaskTimingSharedState& InSharedState, const FString& InName, uint32 InTimelineIndex)
		: FTimingEventsTrack(InName)
		, TimelineIndex(InTimelineIndex)
		, SharedState(InSharedState)
	{
	}

	virtual ~FTaskTimingTrack() = default;

	uint32 GetTimelineIndex() const { return TimelineIndex; }

	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual void PostDraw(const ITimingTrackDrawContext& Context) const override;

	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;

	virtual const TSharedPtr<const ITimingEvent> GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const override;

	void SetTaskId(TaskTrace::FId InTaskId) { TaskId = InTaskId; SetDirtyFlag(); }
	TaskTrace::FId GetTaskId() { return TaskId; }
	
	void OnTimingEventSelected(TSharedPtr<const ITimingEvent> InSelectedEvent);
	void GetEventRelations(const FThreadTrackEvent& InSelectedEvent);

	void SetShowDetailedInfoOnTaskTrack(bool InValue) { bShowDetailInfoOnTaskTrack = InValue; }
	bool GetShowDetailedInfoOnTaskTrack() { return bShowDetailInfoOnTaskTrack; }

private:
	uint32 TimelineIndex;

	FTaskTimingSharedState& SharedState;

	TaskTrace::FId TaskId = TaskTrace::InvalidId;

	FVector2D MousePositionOnButtonDown;

	bool bShowDetailInfoOnTaskTrack = true;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TaskGraphProfiler
