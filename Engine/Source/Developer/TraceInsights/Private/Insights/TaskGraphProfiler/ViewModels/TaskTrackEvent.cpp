// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/TaskGraphProfiler/ViewModels/TaskTrackEvent.h"

namespace UE::Insights::TaskGraphProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FTaskTrackEvent)

////////////////////////////////////////////////////////////////////////////////////////////////////

FTaskTrackEvent::FTaskTrackEvent(const TSharedRef<const FBaseTimingTrack> InTrack, double InStartTime, double InEndTime, uint32 InDepth, ETaskEventType InType)
	: FTimingEvent(InTrack, InStartTime, InEndTime, InDepth)
	, TaskEventType(InType)
{}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FTaskTrackEvent::GetStartLabel() const
{
	switch (TaskEventType)
	{
	case ETaskEventType::Created:
		return TEXT("Created Time:");
	case ETaskEventType::Launched:
		return TEXT("Launched Time");
	case ETaskEventType::Scheduled:
		return TEXT("Scheduled Time:");
	case ETaskEventType::Started:
	case ETaskEventType::PrerequisiteStarted:
	case ETaskEventType::ParentStarted:
	case ETaskEventType::NestedStarted:
	case ETaskEventType::SubsequentStarted:
		return TEXT("Started Time:");
	case ETaskEventType::Finished:
		return TEXT("Finished Time:");
	case ETaskEventType::Completed:
		return TEXT("Completed Time:");
	default:
		checkf(false, TEXT("Unknown task event type"));
		break;
	}

	return TEXT("");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FTaskTrackEvent::GetEndLabel() const
{
	switch (TaskEventType)
	{
	case ETaskEventType::Created:
		return TEXT("Launched Time:");
	case ETaskEventType::Launched:
		return TEXT("Scheduled Time");
	case ETaskEventType::Scheduled:
		return TEXT("Started Time:");
	case ETaskEventType::Started:
	case ETaskEventType::PrerequisiteStarted:
	case ETaskEventType::ParentStarted:
	case ETaskEventType::NestedStarted:
	case ETaskEventType::SubsequentStarted:
		return TEXT("Finished Time:");
	case ETaskEventType::Finished:
		return TEXT("Completed Time:");
	case ETaskEventType::Completed:
		return TEXT("Destroyed Time:");
	default:
		checkf(false, TEXT("Unknown task event type"));
		break;
	}

	return TEXT("");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FTaskTrackEvent::GetEventName() const
{
	switch (TaskEventType)
	{
	case ETaskEventType::Created:
		return TEXT("Created");
	case ETaskEventType::Launched:
		return TEXT("Launched");
	case ETaskEventType::Scheduled:
		return TEXT("Scheduled");
	case ETaskEventType::Started:
		return TEXT("Executing");
	case ETaskEventType::Finished:
		return TEXT("Finished");
	case ETaskEventType::Completed:
		return TEXT("Completed");
	case ETaskEventType::PrerequisiteStarted:
		return FString::Printf(TEXT("Prerequisite Task %" UINT64_FMT " Executing"), GetTaskId());	
	case ETaskEventType::ParentStarted:
		return FString::Printf(TEXT("Parent Task %" UINT64_FMT " Executing"), GetTaskId());
	case ETaskEventType::NestedStarted:
		return FString::Printf(TEXT("Nested Task %" UINT64_FMT " Executing"), GetTaskId());
	case ETaskEventType::SubsequentStarted:
		return FString::Printf(TEXT("Subsequent Task %" UINT64_FMT " Executing"), GetTaskId());

	default:
		checkf(false, TEXT("Unknown task event type"));
		break;
	}

	return TEXT("");
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TaskGraphProfiler
