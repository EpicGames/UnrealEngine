// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Async/TaskTrace.h"

// TraceInsights
#include "Insights/TaskGraphProfiler/TaskGraphProfilerManager.h"
#include "Insights/ViewModels/TimingEvent.h"

namespace UE::Insights::TaskGraphProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTaskTrackEvent : public FTimingEvent
{
	INSIGHTS_DECLARE_RTTI(FTaskTrackEvent, FTimingEvent)

public:
	FTaskTrackEvent(const TSharedRef<const FBaseTimingTrack> InTrack, double InStartTime, double InEndTime, uint32 InDepth, ETaskEventType InTaskEventType);
	virtual ~FTaskTrackEvent() = default;

	FString GetStartLabel() const;
	FString GetEndLabel() const;

	FString GetEventName() const;

	ETaskEventType GetTaskEventType() const { return TaskEventType; }

	TaskTrace::FId GetTaskId() const { return TaskId; }
	void SetTaskId(TaskTrace::FId InTaskId) { TaskId = InTaskId; }

private:
	ETaskEventType TaskEventType;
	TaskTrace::FId TaskId = TaskTrace::InvalidId;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TaskGraphProfiler
