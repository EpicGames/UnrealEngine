// Copyright Epic Games, Inc. All Rights Reserved.

#include "STaskTableTreeView.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SComboBox.h"

// TraceServices
#include "TraceServices/Model/TasksProfiler.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/TaskGraphProfiler/TaskGraphProfilerManager.h"
#include "Insights/TaskGraphProfiler/ViewModels/TaskEntry.h"
#include "Insights/TaskGraphProfiler/ViewModels/TaskNode.h"
#include "Insights/TaskGraphProfiler/ViewModels/TaskTimingTrack.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Tracks/CpuTimingTrack.h"
#include "Insights/TimingProfiler/ViewModels/ThreadTimingSharedState.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TaskGraphProfiler::STaskTableTreeView"

namespace UE::Insights::TaskGraphProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTaskTableTreeViewCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

class FTaskTableTreeViewCommands : public TCommands<FTaskTableTreeViewCommands>
{
public:
	FTaskTableTreeViewCommands()
	: TCommands<FTaskTableTreeViewCommands>(
		TEXT("TaskTableTreeViewCommands"),
		NSLOCTEXT("Contexts", "TaskTableTreeViewCommands", "Insights - Task Table Tree View"),
		NAME_None,
		FInsightsStyle::GetStyleSetName())
	{
	}

	virtual ~FTaskTableTreeViewCommands()
	{
	}

	// UI_COMMAND takes long for the compiler to optimize
	UE_DISABLE_OPTIMIZATION_SHIP
	virtual void RegisterCommands() override
	{
		UI_COMMAND(Command_GoToTask,
			"Go To Task",
			"Pans and zooms to the task in the Timing View.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_OpenInIDE,
			"Open in IDE",
			"Opens the source location where the selected task was launched, in IDE.",
			EUserInterfaceActionType::Button,
			FInputChord());
	}
	UE_ENABLE_OPTIMIZATION_SHIP

	TSharedPtr<FUICommandInfo> Command_GoToTask;
	TSharedPtr<FUICommandInfo> Command_OpenInIDE;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

STaskTableTreeView::STaskTableTreeView()
{
	bRunInAsyncMode = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

STaskTableTreeView::~STaskTableTreeView()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::Construct(const FArguments& InArgs, TSharedPtr<FTaskTable> InTablePtr)
{
	ConstructWidget(InTablePtr);

	AddCommmands();

	// Make sure the default value is applied.
	TimestampOptions_OnSelectionChanged(SelectedTimestampOption);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::ExtendMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Task", LOCTEXT("ContextMenu_Section_Task", "Task"));

	{
		MenuBuilder.AddMenuEntry(
			FTaskTableTreeViewCommands::Get().Command_GoToTask,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.GoToTask"));
	}

	{
		FString File;
		uint32 Line = 0;
		bool bIsValidSource = GetSourceFileAndLineForSelectedTask(File, Line);
		FString UsablePath;
		const bool bPathExists = FInsightsManager::Get()->GetSourceFilePathHelper().GetUsableFilePath(File, UsablePath);

		ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
		ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();

		FText ItemLabel;
		FText ItemToolTip;

		if (SourceCodeAccessor.CanAccessSourceCode())
		{
			ItemLabel = FText::Format(LOCTEXT("ContextMenu_OpenSource", "Open Source in {0}"), SourceCodeAccessor.GetNameText());
			if (bIsValidSource)
			{
				ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSource_Desc1", "Opens the source location where the selected task was launched, in {0}.\n{1} ({2})"),
					SourceCodeAccessor.GetNameText(),
					FText::FromString(UsablePath),
					FText::AsNumber(Line, &FNumberFormattingOptions::DefaultNoGrouping()));
			}
			else
			{
				ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSource_Desc2", "Opens the source location where the selected task was launched, in {0}."),
					SourceCodeAccessor.GetNameText());
			}
		}
		else
		{
			ItemLabel = LOCTEXT("ContextMenu_OpenSourceNA", "Open Source");
			if (bIsValidSource)
			{
				ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSourceNA_Desc1", "{0} ({1})\nSource Code Accessor is not available."),
					FText::FromString(UsablePath),
					FText::AsNumber(Line, &FNumberFormattingOptions::DefaultNoGrouping()));
			}
			else
			{
				ItemToolTip = LOCTEXT("ContextMenu_OpenSourceNA_Desc2", "Source Code Accessor is not available.");
			}
		}

		MenuBuilder.AddMenuEntry(
			FTaskTableTreeViewCommands::Get().Command_OpenInIDE,
			NAME_None,
			ItemLabel,
			ItemToolTip,
			FSlateIcon(SourceCodeAccessor.GetStyleSet(), SourceCodeAccessor.GetOpenIconName()));
	}

	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::AddCommmands()
{
	FTaskTableTreeViewCommands::Register();

	CommandList->MapAction(FTaskTableTreeViewCommands::Get().Command_GoToTask, FExecuteAction::CreateSP(this, &STaskTableTreeView::ContextMenu_GoToTask_Execute), FCanExecuteAction::CreateSP(this, &STaskTableTreeView::ContextMenu_GoToTask_CanExecute));
	CommandList->MapAction(FTaskTableTreeViewCommands::Get().Command_OpenInIDE, FExecuteAction::CreateSP(this, &STaskTableTreeView::ContextMenu_OpenInIDE_Execute), FCanExecuteAction::CreateSP(this, &STaskTableTreeView::ContextMenu_OpenInIDE_CanExecute));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::Reset()
{
	STableTreeView::Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	STableTreeView::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (!bIsUpdateRunning)
	{
		RebuildTree(false);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::RebuildTree(bool bResync)
{
	using namespace UE::Insights::TimingProfiler;

	double NewQueryStartTime = FTimingProfilerManager::Get()->GetSelectionStartTime();
	double NewQueryEndTime = FTimingProfilerManager::Get()->GetSelectionEndTime();

	if (NewQueryStartTime >= NewQueryEndTime)
	{
		return;
	}

	if (bResync || NewQueryStartTime != QueryStartTime || NewQueryEndTime != QueryEndTime)
	{
		StopAllTableDataTasks();

		QueryStartTime = NewQueryStartTime;
		QueryEndTime = NewQueryEndTime;
		TSharedPtr<FTaskTable> TaskTable = GetTaskTable();
		TArray<FTaskEntry>& Tasks = TaskTable->GetTaskEntries();
		Tasks.Empty();
		TableRowNodes.Empty();

		if (QueryStartTime < QueryEndTime && Session.IsValid())
		{
			const TraceServices::ITasksProvider* TasksProvider = TraceServices::ReadTasksProvider(*Session.Get());
			if (TasksProvider)
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

				TArray<FTableTreeNodePtr>* Nodes = &TableRowNodes;
				TasksProvider->EnumerateTasks(QueryStartTime, QueryEndTime, SelectedTasksSelectionOption,
					[&Tasks, &TaskTable, Nodes]
					(const TraceServices::FTaskInfo& TaskInfo)
					{
						Tasks.Emplace(TaskInfo);
						int32 Index = Tasks.Num() - 1;
						FTaskNodePtr NodePtr = MakeShared<FTaskNode>(TaskTable, Index);
						Nodes->Add(NodePtr);
						return TraceServices::ETaskEnumerationResult::Continue;
					});
			}
		}

		UpdateTree();
		TreeView->RebuildList();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STaskTableTreeView::IsRunning() const
{
	return STableTreeView::IsRunning();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double STaskTableTreeView::GetAllOperationsDuration()
{
	return STableTreeView::GetAllOperationsDuration();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STaskTableTreeView::GetCurrentOperationName() const
{
	return STableTreeView::GetCurrentOperationName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STaskTableTreeView::ConstructHeaderArea(TSharedRef<SVerticalBox> InWidgetContent)
{
	InWidgetContent->AddSlot()
	.VAlign(VAlign_Center)
	.AutoHeight()
	.Padding(2.0f)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			ConstructSearchBox()
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			ConstructFilterConfiguratorButton()
		]
	];

	InWidgetContent->AddSlot()
	.VAlign(VAlign_Center)
	.AutoHeight()
	.Padding(2.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Tasks", "Tasks"))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.Padding(4.0f, 0.0f, 10.0f, 0.0f)
		[
			SNew(SBox)
			.MinDesiredWidth(160.0f)
			[
				SNew(SComboBox<TSharedPtr<TraceServices::ETaskEnumerationOption>>)
				.OptionsSource(GetAvailableTasksSelectionOptions())
				.OnSelectionChanged(this, &STaskTableTreeView::TasksSelectionOptions_OnSelectionChanged)
				.OnGenerateWidget(this, &STaskTableTreeView::TasksSelectionOptions_OnGenerateWidget)
				.IsEnabled(this, &STaskTableTreeView::TasksSelectionOptions_IsEnabled)
				[
					SNew(STextBlock)
					.Text(this, &STaskTableTreeView::TasksSelectionOptions_GetSelectionText)
				]
			]
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Timestamps", "Timestamps"))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SBox)
				.MinDesiredWidth(160.0f)
				[
					SNew(SComboBox<TSharedPtr<ETimestampOptions>>)
					.OptionsSource(GetAvailableTimestampOptions())
					.OnSelectionChanged(this, &STaskTableTreeView::TimestampOptions_OnSelectionChanged)
					.OnGenerateWidget(this, &STaskTableTreeView::TimestampOptions_OnGenerateWidget)
					.IsEnabled(this, &STaskTableTreeView::TimestampOptions_IsEnabled)
					[
						SNew(STextBlock)
						.Text(this, &STaskTableTreeView::TimestampOptions_GetSelectionText)
					]
				]
			]
		]
	];

	InWidgetContent->AddSlot()
	.VAlign(VAlign_Center)
	.AutoHeight()
	.Padding(2.0f)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			ConstructHierarchyBreadcrumbTrail()
		]
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> STaskTableTreeView::ConstructFooter()
{
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STaskTableTreeView::TimestampOptions_OnGenerateWidget(TSharedPtr<ETimestampOptions> InOption)
{
	auto GetTooltipText = [](ETimestampOptions InOption)
	{
		switch (InOption)
		{
			case ETimestampOptions::Absolute:
			{
				return LOCTEXT("AbsoluteValueTooltip", "The timestamps for all columns will show absolute values.");
			}
			case ETimestampOptions::RelativeToPrevious:
			{
				return LOCTEXT("RelativeToPreviousTooltip", "The timestamps for all columns will show values relative to the previous stage. Ex: Scheduled will be relative to Launched.");
			}
			case ETimestampOptions::RelativeToCreated:
			{
				return LOCTEXT("RelativeToCreatedTooltip", "The timestamps for all columns will show values relative to the created time.");
			}
		}

		return FText();
	};

	TSharedRef<SHorizontalBox> Widget = SNew(SHorizontalBox);
	Widget->AddSlot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(TimestampOptions_GetText(*InOption))
			.ToolTipText(GetTooltipText(*InOption))
			.Margin(2.0f)
		];

	return Widget;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STaskTableTreeView::TasksSelectionOptions_OnGenerateWidget(TSharedPtr<TraceServices::ETaskEnumerationOption> InOption)
{
	using namespace TraceServices;

	auto GetTooltipText = [](ETaskEnumerationOption InOption)
	{
		switch (InOption)
		{
		case ETaskEnumerationOption::Alive:
		{
			return LOCTEXT("AliveTooltip", "Tasks that were created before the selection and destroyed after.");
		}
		case ETaskEnumerationOption::Launched:
		{
			return LOCTEXT("LaunchedTooltip", "Tasks that were launched during the selection.");
		}
		case ETaskEnumerationOption::Active:
		{
			return LOCTEXT("ActiveTooltip", "Tasks that were active (being executed) at any moment of the selection.");
		}
		case ETaskEnumerationOption::WaitingForPrerequisites:
		{
			return LOCTEXT("WaitingForPrerequisitesTooltip", "Tasks that are blocked by prerequisites for the entire duration of the selection.");
		}
		case ETaskEnumerationOption::Queued:
		{
			return LOCTEXT("QueuedTooltip", "Tasks that for the entire duration of the selection are waiting in the scheduler queue.");
		}
		case ETaskEnumerationOption::Executing:
		{
			return LOCTEXT("ExecutingTooltip", "Tasks that for the entire duration of the selection have been executed.");
		}
		case ETaskEnumerationOption::WaitingForNested:
		{
			return LOCTEXT("WaitingForNestedTooltip", "Tasks that for the entire duration of the selection have been waiting for nested tasks.");
		}
		case ETaskEnumerationOption::Completed:
		{
			return LOCTEXT("CompletedTooltip", "Tasks that for the entire duration of the selection have been already completed but not destroyed.");
		}
		}

		return FText();
	};

	TSharedRef<SHorizontalBox> Widget = SNew(SHorizontalBox);
	Widget->AddSlot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(TasksSelectionOptions_GetText(*InOption))
			.ToolTipText(GetTooltipText(*InOption))
			.Margin(2.0f)
		];

	return Widget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::InternalCreateGroupings()
{
	STableTreeView::InternalCreateGroupings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<TSharedPtr<STaskTableTreeView::ETimestampOptions>>* STaskTableTreeView::GetAvailableTimestampOptions()
{
	if (AvailableTimestampOptions.Num() == 0)
	{
		AvailableTimestampOptions.Add(MakeShared<ETimestampOptions>(ETimestampOptions::Absolute));
		AvailableTimestampOptions.Add(MakeShared<ETimestampOptions>(ETimestampOptions::RelativeToPrevious));
		AvailableTimestampOptions.Add(MakeShared<ETimestampOptions>(ETimestampOptions::RelativeToCreated));
	}

	return &AvailableTimestampOptions;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<TSharedPtr<TraceServices::ETaskEnumerationOption>>* STaskTableTreeView::GetAvailableTasksSelectionOptions()
{
	if (AvailableTasksSelectionOptions.Num() == 0)
	{
		using namespace TraceServices;
		AvailableTasksSelectionOptions.Add(MakeShared<ETaskEnumerationOption>(ETaskEnumerationOption::Alive));
		AvailableTasksSelectionOptions.Add(MakeShared<ETaskEnumerationOption>(ETaskEnumerationOption::Launched));
		AvailableTasksSelectionOptions.Add(MakeShared<ETaskEnumerationOption>(ETaskEnumerationOption::Active));
		AvailableTasksSelectionOptions.Add(MakeShared<ETaskEnumerationOption>(ETaskEnumerationOption::WaitingForPrerequisites));
		AvailableTasksSelectionOptions.Add(MakeShared<ETaskEnumerationOption>(ETaskEnumerationOption::Queued));
		AvailableTasksSelectionOptions.Add(MakeShared<ETaskEnumerationOption>(ETaskEnumerationOption::Executing));
		AvailableTasksSelectionOptions.Add(MakeShared<ETaskEnumerationOption>(ETaskEnumerationOption::WaitingForNested));
		AvailableTasksSelectionOptions.Add(MakeShared<ETaskEnumerationOption>(ETaskEnumerationOption::Completed));
	}

	return &AvailableTasksSelectionOptions;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::TimestampOptions_OnSelectionChanged(TSharedPtr<ETimestampOptions> InOption, ESelectInfo::Type SelectInfo)
{
	TimestampOptions_OnSelectionChanged(*InOption);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::TimestampOptions_OnSelectionChanged(ETimestampOptions InOption)
{
	SelectedTimestampOption = InOption;

	switch (SelectedTimestampOption)
	{
	case ETimestampOptions::Absolute:
	{
		GetTaskTable()->SwitchToAbsoluteTimestamps();
		break;
	}
	case ETimestampOptions::RelativeToPrevious:
	{
		GetTaskTable()->SwitchToRelativeToPreviousTimestamps();
		break;
	}
	case ETimestampOptions::RelativeToCreated:
	{
		GetTaskTable()->SwitchToRelativeToCreatedTimestamps();
		break;
	}
	}

	UpdateTree();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::TasksSelectionOptions_OnSelectionChanged(TSharedPtr<TraceServices::ETaskEnumerationOption> InOption, ESelectInfo::Type SelectInfo)
{
	TasksSelectionOptions_OnSelectionChanged(*InOption);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::TasksSelectionOptions_OnSelectionChanged(TraceServices::ETaskEnumerationOption InOption)
{
	SelectedTasksSelectionOption = InOption;
	RebuildTree(true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STaskTableTreeView::TimestampOptions_GetSelectionText() const
{
	return TimestampOptions_GetText(SelectedTimestampOption);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STaskTableTreeView::TimestampOptions_GetText(ETimestampOptions InOption) const
{
	switch (InOption)
	{
		case ETimestampOptions::Absolute:
		{
			return LOCTEXT("Absolute", "Absolute");
		}
		case ETimestampOptions::RelativeToPrevious:
		{
			return LOCTEXT("RelativeToPrevious", "Relative To Previous");
		}
		case ETimestampOptions::RelativeToCreated:
		{
			return LOCTEXT("RelativeToCreated", "Relative To Created");
		}
	}

	return FText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STaskTableTreeView::TasksSelectionOptions_GetSelectionText() const
{
	return TasksSelectionOptions_GetText(SelectedTasksSelectionOption);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STaskTableTreeView::TasksSelectionOptions_GetText(TraceServices::ETaskEnumerationOption InOption) const
{
	using namespace TraceServices;

	switch (InOption)
	{
	case ETaskEnumerationOption::Alive:
	{
		return LOCTEXT("Alive", "Alive");
	}
	case ETaskEnumerationOption::Launched:
	{
		return LOCTEXT("Launched", "Launched");
	}
	case ETaskEnumerationOption::Active:
	{
		return LOCTEXT("Active", "Active");
	}
	case ETaskEnumerationOption::WaitingForPrerequisites:
	{
		return LOCTEXT("WaitingForPrerequisites", "Waiting for prerequisites");
	}
	case ETaskEnumerationOption::Queued:
	{
		return LOCTEXT("Queued", "Queued");
	}
	case ETaskEnumerationOption::Executing:
	{
		return LOCTEXT("Executing", "Executing");
	}
	case ETaskEnumerationOption::WaitingForNested:
	{
		return LOCTEXT("WaitingForNested", "Waiting for nested");
	}
	case ETaskEnumerationOption::Completed:
	{
		return LOCTEXT("Completed", "Completed");
	}
	}

	return FText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STaskTableTreeView::TimestampOptions_IsEnabled() const
{
	return !bIsUpdateRunning;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STaskTableTreeView::TasksSelectionOptions_IsEnabled() const
{
	return !bIsUpdateRunning;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STaskTableTreeView::ContextMenu_GoToTask_CanExecute() const
{
	TArray<FTableTreeNodePtr> SelectedItems;
	TreeView->GetSelectedItems(SelectedItems);

	if (SelectedItems.Num() != 1)
	{
		return false;
	}

	FTaskNodePtr SelectedTask = StaticCastSharedPtr<FTaskNode>(SelectedItems[0]);

	if (!SelectedTask.IsValid() || SelectedTask->IsGroup())
	{
		return false;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::ContextMenu_GoToTask_Execute()
{
	TArray<FTableTreeNodePtr> SelectedItems;
	TreeView->GetSelectedItems(SelectedItems);

	if (SelectedItems.Num() != 1)
	{
		return;
	}

	FTaskNodePtr SelectedTask = StaticCastSharedPtr<FTaskNode>(SelectedItems[0]);
	const FTaskEntry* TaskEntry = SelectedTask.IsValid() ? SelectedTask->GetTask() : nullptr;

	if (TaskEntry == nullptr)
	{
		return;
	}

	using namespace UE::Insights::TimingProfiler;

	TSharedPtr<STimingProfilerWindow> TimingWindow = FTimingProfilerManager::Get()->GetProfilerWindow();
	if (!TimingWindow.IsValid())
	{
		return;
	}

	TSharedPtr<STimingView> TimingView = TimingWindow->GetTimingView();
	if (!TimingView.IsValid())
	{
		return;
	}

	double CreatedTimestamp = TaskEntry->GetCreatedTimestamp();
	if (CreatedTimestamp == TraceServices::FTaskInfo::InvalidTimestamp)
	{
		CreatedTimestamp = TimingView->GetViewport().GetMinValidTime();
	}
	double FinishedTimestamp = TaskEntry->GetFinishedTimestamp();
	if (FinishedTimestamp == TraceServices::FTaskInfo::InvalidTimestamp)
	{
		FinishedTimestamp = TimingView->GetViewport().GetMaxValidTime();
	}
	const double Duration = FinishedTimestamp - CreatedTimestamp;
	TimingView->ZoomOnTimeInterval(CreatedTimestamp - Duration * 0.15, Duration * 1.30);

	if (TSharedPtr<FTaskTimingSharedState> TaskSharedState = FTaskGraphProfilerManager::Get()->GetTaskTimingSharedState())
	{
		if (FTaskGraphProfilerManager::Get()->GetShowAnyRelations())
		{
			TaskSharedState->SetTaskId(TaskEntry->GetId());
		}
	}

	if (TSharedPtr<FThreadTimingSharedState> ThreadTimingState = TimingView->GetThreadTimingSharedState())
	{
		if (TSharedPtr<FCpuTimingTrack> Track = ThreadTimingState->GetCpuTrack(TaskEntry->GetStartedThreadId()))
		{
			TimingView->SelectTimingTrack(nullptr, false); // to force bring the track into view, if already selected
			TimingView->SelectTimingTrack(Track, true);
		}
	}

	FTaskGraphProfilerManager::Get()->ShowTaskRelations(TaskEntry->GetId());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STaskTableTreeView::ContextMenu_OpenInIDE_CanExecute() const
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();

	if (!SourceCodeAccessor.CanAccessSourceCode())
	{
		return false;
	}

	FString File;
	uint32 Line = 0;
	return GetSourceFileAndLineForSelectedTask(File, Line);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STaskTableTreeView::GetSourceFileAndLineForSelectedTask(FString& OutFile, uint32& OutLine) const
{
	TArray<FTableTreeNodePtr> SelectedItems;
	TreeView->GetSelectedItems(SelectedItems);

	if (SelectedItems.Num() != 1)
	{
		return false;
	}

	FTableTreeNodePtr SelectedTreeNode = SelectedItems[0];
	if (!SelectedTreeNode.IsValid() || !SelectedTreeNode->Is<FTaskNode>())
	{
		return false;
	}

	FTaskNodePtr SelectedTask = StaticCastSharedPtr<FTaskNode>(SelectedTreeNode);

	const FTaskEntry* TaskEntry = SelectedTask->GetTask();
	if (TaskEntry == nullptr || TaskEntry->GetDebugName() == nullptr)
	{
		return false;
	}

	const FString DebugName = TaskEntry->GetDebugName();
	if (DebugName.Len() < 4 ||
		DebugName[DebugName.Len() - 1] != TEXT(')'))
	{
		return false;
	}

	int32 OpenBracketPos;
	if (!DebugName.FindChar(TEXT('('), OpenBracketPos) ||
		OpenBracketPos > DebugName.Len() - 3)
	{
		return false;
	}

	OutFile = DebugName.Left(OpenBracketPos);

	FString LineStr = DebugName.Mid(OpenBracketPos + 1, DebugName.Len() - OpenBracketPos - 2);
	OutLine = FCString::Atoi(*LineStr);

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::ContextMenu_OpenInIDE_Execute()
{
	FString File;
	uint32 Line = 0;
	if (!GetSourceFileAndLineForSelectedTask(File, Line))
	{
		return;
	}

	FString UsablePath;
	const bool bPathExists = FInsightsManager::Get()->GetSourceFilePathHelper().GetUsableFilePath(File, UsablePath);
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");

	if (bPathExists)
	{
		ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();
		SourceCodeAccessor.OpenFileAtLine(UsablePath, Line);
	}
	else
	{
		SourceCodeAccessModule.OnOpenFileFailed().Broadcast(UsablePath);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::TreeView_OnMouseButtonDoubleClick(FTableTreeNodePtr TreeNode)
{
	if (!TreeNode->IsGroup())
	{
		ContextMenu_GoToTask_Execute();
	}

	STableTreeView::TreeView_OnMouseButtonDoubleClick(TreeNode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::SelectTaskEntry(TaskTrace::FId InId)
{
	TaskIdToSelect = InId;
	StartTableDataTask<FSearchForItemToSelectTask>(SharedThis(this));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STaskTableTreeView::SearchForItem(TSharedPtr<FTableTaskCancellationToken> CancellationToken)
{
	TSharedPtr<FTaskTable> TaskTable = GetTaskTable();
	TArray<FTaskEntry>& Tasks = TaskTable->GetTaskEntries();

	uint32 NumEntries = (uint32)Tasks.Num();
	for (uint32 Index = 0; Index < NumEntries; ++Index)
	{
		if (CancellationToken->ShouldCancel())
		{
			break;
		}

		if (Tasks[Index].GetId() == TaskIdToSelect)
		{
			TGraphTask<FSelectNodeByTableRowIndexTask>::CreateTask().ConstructAndDispatchWhenReady(CancellationToken, SharedThis(this), Index);
			break;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TaskGraphProfiler

#undef LOCTEXT_NAMESPACE
