// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Async/TaskTrace.h"

// TraceServices
#include "TraceServices/Model/TasksProfiler.h"

// TraceInsights
#include "Insights/Table/Widgets/SSessionTableTreeView.h"
#include "Insights/TaskGraphProfiler/ViewModels/TaskTable.h"

class FMenuBuilder;

namespace UE::Insights::TaskGraphProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class STaskTableTreeView : public SSessionTableTreeView
{
private:
	struct FColumnConfig
	{
		const FName& ColumnId;
		bool bIsVisible;
		float Width;
	};

	enum class ETimestampOptions : uint32
	{
		Absolute,
		RelativeToPrevious,
		RelativeToCreated,
	};

public:
	/** Default constructor. */
	STaskTableTreeView();

	/** Virtual destructor. */
	virtual ~STaskTableTreeView();

	SLATE_BEGIN_ARGS(STaskTableTreeView) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<FTaskTable> InTablePtr);

	void ConstructHeaderArea(TSharedRef<SVerticalBox> InWidgetContent) override;

	virtual TSharedPtr<SWidget> ConstructFooter() override;

	TSharedPtr<FTaskTable> GetTaskTable() { return StaticCastSharedPtr<FTaskTable>(GetTable()); }
	const TSharedPtr<FTaskTable> GetTaskTable() const { return StaticCastSharedPtr<FTaskTable>(GetTable()); }

	//void UpdateSourceTable(TSharedPtr<TraceServices::ITaskTable> SourceTable);

	virtual void Reset();

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/**
	 * Rebuilds the tree (if necessary).
	 * @param bResync - If true, it forces a resync even if the list did not changed since last sync.
	 */
	virtual void RebuildTree(bool bResync);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// IAsyncOperationStatusProvider implementation

	virtual bool IsRunning() const override;
	virtual double GetAllOperationsDuration() override;
	virtual FText GetCurrentOperationName() const override;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	virtual void TreeView_OnMouseButtonDoubleClick(FTableTreeNodePtr TreeNode) override;

	void SelectTaskEntry(TaskTrace::FId InId);

protected:
	virtual void InternalCreateGroupings() override;

	virtual void ExtendMenu(FMenuBuilder& MenuBuilder) override;

	virtual void SearchForItem(TSharedPtr<FTableTaskCancellationToken> CancellationToken) override;

private:
	void AddCommmands();

	const TArray<TSharedPtr<ETimestampOptions>>* GetAvailableTimestampOptions();
	TSharedRef<SWidget> TimestampOptions_OnGenerateWidget(TSharedPtr<ETimestampOptions> InOption);
	void TimestampOptions_OnSelectionChanged(TSharedPtr<ETimestampOptions> InOption, ESelectInfo::Type SelectInfo);
	void TimestampOptions_OnSelectionChanged(ETimestampOptions InOption);
	FText TimestampOptions_GetSelectionText() const;
	FText TimestampOptions_GetText(ETimestampOptions InOption) const;
	bool TimestampOptions_IsEnabled() const;

	const TArray<TSharedPtr<TraceServices::ETaskEnumerationOption>>* GetAvailableTasksSelectionOptions();
	TSharedRef<SWidget> TasksSelectionOptions_OnGenerateWidget(TSharedPtr<TraceServices::ETaskEnumerationOption> InOption);
	void TasksSelectionOptions_OnSelectionChanged(TSharedPtr<TraceServices::ETaskEnumerationOption> InOption, ESelectInfo::Type SelectInfo);
	void TasksSelectionOptions_OnSelectionChanged(TraceServices::ETaskEnumerationOption InOption);
	FText TasksSelectionOptions_GetSelectionText() const;
	FText TasksSelectionOptions_GetText(TraceServices::ETaskEnumerationOption InOption) const;
	bool TasksSelectionOptions_IsEnabled() const;

	bool ContextMenu_GoToTask_CanExecute() const;
	void ContextMenu_GoToTask_Execute();

	bool ContextMenu_OpenInIDE_CanExecute() const;
	void ContextMenu_OpenInIDE_Execute();

	bool GetSourceFileAndLineForSelectedTask(FString& OutFile, uint32& OutLine) const;

private:
	double QueryStartTime = 0.0f;
	double QueryEndTime = 0.0f;

	ETimestampOptions SelectedTimestampOption = ETimestampOptions::RelativeToPrevious;
	TArray<TSharedPtr<ETimestampOptions>> AvailableTimestampOptions;

	TraceServices::ETaskEnumerationOption SelectedTasksSelectionOption = TraceServices::ETaskEnumerationOption::Alive;
	TArray<TSharedPtr<TraceServices::ETaskEnumerationOption>> AvailableTasksSelectionOptions;

	TaskTrace::FId TaskIdToSelect;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TaskGraphProfiler
