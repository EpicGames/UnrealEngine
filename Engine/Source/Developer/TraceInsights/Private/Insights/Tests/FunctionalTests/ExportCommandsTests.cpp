// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformFileManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"

// TraceInsightsCore
#include "InsightsCore/Common/MiscUtils.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/Tests/InsightsTestUtils.h"

DEFINE_LATENT_AUTOMATION_COMMAND_FIVE_PARAMETER(FVerifyExportedLinesCommand, const FString, ExportReportPath, const FString, CmdLogPath, const FString, Elements, FAutomationTestBase*, Test, double, Timeout);

bool FVerifyExportedLinesCommand::Update()
{
	if (FPlatformTime::Seconds() - StartTime >= Timeout)
	{
		Test->AddError(TEXT("The FVerifyExportedLinesCommand timed out"));
		return true;
	}

	FString FileContent;
	if (FFileHelper::LoadFileToString(FileContent, *ExportReportPath))
	{
		TArray<FString> Lines;
		FString ExpectedResult;
		FInsightsTestUtils Utils(Test);
		FileContent.ParseIntoArrayLines(Lines);
		ExpectedResult = FString::Printf(TEXT("Exported %d %s to file"), Lines.Num() - 1, *Elements);
		if (Utils.FileContainsString(CmdLogPath, ExpectedResult, 1.0f))
		{
			return true;
		}
	}

	return false;
}

DEFINE_LATENT_AUTOMATION_COMMAND_FOUR_PARAMETER(FDeleteDirectoryCommand, IPlatformFile&, PlatformFile, const FString, DirectoryPath, FAutomationTestBase*, Test, double, Timeout);

bool FDeleteDirectoryCommand::Update()
{
	if (FPlatformTime::Seconds() - StartTime >= Timeout)
	{
		Test->AddError(TEXT("The FDeleteDirectoryCommand timed out"));
		return true;
	}

	IFileManager::Get().DeleteDirectory(*DirectoryPath, false, true);

	if (!PlatformFile.DirectoryExists(*DirectoryPath))
	{
		return true;
	}

	return false;
}

#if WITH_AUTOMATION_TESTS

// The goal of this test is to verify that threads data can be exported from a trace
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCommandsExportThreadsDataTest, "System.Insights.Trace.Analysis.CommandsExport.ThreadsData", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool FCommandsExportThreadsDataTest::RunTest(const FString& Parameters)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	const FString SourceTracePath = FPaths::RootDir() / TEXT("EngineTest/SourceAssets/Utrace/CommandsExportTest_5.4.utrace");
	const FString StoreTracePath = TEXT("CommandsExportTest_5.4.utrace");
	const FString TestResultsDirPath = TEXT("TestResults");
	const FString CmdThreadLogPath = TestResultsDirPath / TEXT("Logs/cmd_treads.log");
	const FString ExportThreadsReportPath = TestResultsDirPath / TEXT("SingleCommand/Threads.csv");
	const FString ExportThreadsTask = TEXT("TimingInsights.ExportThreads ") + ExportThreadsReportPath;
	double Timeout = 30.0;

	UTEST_TRUE(TEXT("Trace in project exists"), PlatformFile.FileExists(*SourceTracePath));
	PlatformFile.CopyFile(*StoreTracePath, *SourceTracePath);
	UTEST_TRUE(TEXT("Trace in store should exists after copy"), PlatformFile.FileExists(*StoreTracePath));

	if (PlatformFile.DirectoryExists(*TestResultsDirPath))
	{
		AddInfo(TEXT("The TestResults directory already exists. Deleting to avoid undefined behavior"));
		ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirectoryCommand(PlatformFile, TestResultsDirPath, this, Timeout));
	}

	FString InsightsParameters = FString::Printf(TEXT("-OpenTraceFile=\"%s\" -ABSLOG=\"%s\" -AutoQuit -NoUI -ExecOnAnalysisCompleteCmd=\"%s\" -log"), *StoreTracePath, *CmdThreadLogPath, *ExportThreadsTask);
	UE::Insights::FMiscUtils::OpenUnrealInsights(*InsightsParameters);
	ADD_LATENT_AUTOMATION_COMMAND(FVerifyExportedLinesCommand(ExportThreadsReportPath, CmdThreadLogPath, TEXT("threads"), this, Timeout));
	ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirectoryCommand(PlatformFile, TestResultsDirPath, this, Timeout));

	return true;
}

// The goal of this test is to verify that timers data can be exported from a trace
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCommandsExportTimersDataTest, "System.Insights.Trace.Analysis.CommandsExport.TimersData", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool FCommandsExportTimersDataTest::RunTest(const FString& Parameters)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	const FString SourceTracePath = FPaths::RootDir() / TEXT("EngineTest/SourceAssets/Utrace/CommandsExportTest_5.4.utrace");
	const FString StoreTracePath = TEXT("CommandsExportTest_5.4.utrace");
	const FString TestResultsDirPath = TEXT("TestResults");
	const FString CmdTimersLogPath = TestResultsDirPath / TEXT("Logs/cmd_timers.log");
	const FString ExportTimersReportPath = TestResultsDirPath / TEXT("SingleCommand/Timers.csv");
	const FString ExportTimersTask = TEXT("TimingInsights.ExportTimers ") + ExportTimersReportPath;
	double Timeout = 30.0;

	UTEST_TRUE(TEXT("Trace in project exists"), PlatformFile.FileExists(*SourceTracePath));
	PlatformFile.CopyFile(*StoreTracePath, *SourceTracePath);
	UTEST_TRUE(TEXT("Trace in store should exists after copy"), PlatformFile.FileExists(*StoreTracePath));

	if (PlatformFile.DirectoryExists(*TestResultsDirPath))
	{
		AddInfo(TEXT("The TestResults directory already exists. Deleting to avoid undefined behavior"));
		ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirectoryCommand(PlatformFile, TestResultsDirPath, this, Timeout));
	}

	FString InsightsParameters = FString::Printf(TEXT("-OpenTraceFile=\"%s\" -ABSLOG=\"%s\" -AutoQuit -NoUI -ExecOnAnalysisCompleteCmd=\"%s\" -log"), *StoreTracePath, *CmdTimersLogPath, *ExportTimersTask);
	UE::Insights::FMiscUtils::OpenUnrealInsights(*InsightsParameters);
	ADD_LATENT_AUTOMATION_COMMAND(FVerifyExportedLinesCommand(ExportTimersReportPath, CmdTimersLogPath, TEXT("timers"), this, Timeout));
	ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirectoryCommand(PlatformFile, TestResultsDirPath, this, Timeout));

	return true;
}

// The goal of this test is to verify that timing events data can be exported from a trace
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCommandsExportTimingEventsDataTest, "System.Insights.Trace.Analysis.CommandsExport.TimingEventsData", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool FCommandsExportTimingEventsDataTest::RunTest(const FString& Parameters)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	const FString SourceTracePath = FPaths::RootDir() / TEXT("EngineTest/SourceAssets/Utrace/CommandsExportTest_5.4.utrace");
	const FString StoreTracePath = TEXT("CommandsExportTest_5.4.utrace");
	const FString TestResultsDirPath = TEXT("TestResults");
	const FString CmdTimingEventsLogPath = TestResultsDirPath / TEXT("Logs/cmd_timing_events.log");
	const FString ExportTimingEventsReportPath = TestResultsDirPath / TEXT("SingleCommand/TimingEvents.txt");
	const FString ExportTimingEventsTask = TEXT("TimingInsights.ExportTimingEvents " + ExportTimingEventsReportPath);
	double Timeout = 30.0;

	UTEST_TRUE(TEXT("Trace in project exists"), PlatformFile.FileExists(*SourceTracePath));
	PlatformFile.CopyFile(*StoreTracePath, *SourceTracePath);
	UTEST_TRUE(TEXT("Trace in store should exists after copy"), PlatformFile.FileExists(*StoreTracePath));

	if (PlatformFile.DirectoryExists(*TestResultsDirPath))
	{
		AddInfo(TEXT("The TestResults directory already exists. Deleting to avoid undefined behavior"));
		ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirectoryCommand(PlatformFile, TestResultsDirPath, this, Timeout));
	}

	FString InsightsParameters = FString::Printf(TEXT("-OpenTraceFile=\"%s\" -ABSLOG=\"%s\" -AutoQuit -NoUI -ExecOnAnalysisCompleteCmd=\"%s\" -log"), *StoreTracePath, *CmdTimingEventsLogPath, *ExportTimingEventsTask);
	UE::Insights::FMiscUtils::OpenUnrealInsights(*InsightsParameters);
	ADD_LATENT_AUTOMATION_COMMAND(FVerifyExportedLinesCommand(ExportTimingEventsReportPath, CmdTimingEventsLogPath, TEXT("timing events"), this, Timeout));
	ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirectoryCommand(PlatformFile, TestResultsDirPath, this, Timeout));

	return true;
}

// The goal of this test is to verify that a filtered list of timing events data can be exported from a trace
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCommandsExportFilteredTimingEventsDataTest, "System.Insights.Trace.Analysis.CommandsExport.FilteredTimingEventsData", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool FCommandsExportFilteredTimingEventsDataTest::RunTest(const FString& Parameters)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	const FString SourceTracePath = FPaths::RootDir() / TEXT("EngineTest/SourceAssets/Utrace/CommandsExportTest_5.4.utrace");
	const FString StoreTracePath = TEXT("CommandsExportTest_5.4.utrace");
	const FString TestResultsDirPath = TEXT("TestResults");
	const FString CmdTimingEventsNonDefaultLogPath = TestResultsDirPath / TEXT("Logs/cmd_timing_non_default.log");
	const FString ExportTimingEventsNonDefaultReportPath = TestResultsDirPath / TEXT("SingleCommand/TimingEventsNonDefault.csv");
	const FString ExportTimingEventsBorderedTask = TEXT("TimingInsights.ExportTimingEvents " + ExportTimingEventsNonDefaultReportPath + " -columns=ThreadId,ThreadName,TimerId,TimerName,StartTime,EndTime,Duration,Depth -threads=GameThread -timers=* -startTime=10 -endTime=20");
	double Timeout = 30.0;

	UTEST_TRUE(TEXT("Trace in project exists"), PlatformFile.FileExists(*SourceTracePath));
	PlatformFile.CopyFile(*StoreTracePath, *SourceTracePath);
	UTEST_TRUE(TEXT("Trace in store should exists after copy"), PlatformFile.FileExists(*StoreTracePath));

	if (PlatformFile.DirectoryExists(*TestResultsDirPath))
	{
		AddInfo(TEXT("The TestResults directory already exists. Deleting to avoid undefined behavior"));
		ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirectoryCommand(PlatformFile, TestResultsDirPath, this, Timeout));
	}

	FString InsightsParameters = FString::Printf(TEXT("-OpenTraceFile=\"%s\" -ABSLOG=\"%s\" -AutoQuit -NoUI -ExecOnAnalysisCompleteCmd=\"%s\" -log"), *StoreTracePath, *CmdTimingEventsNonDefaultLogPath, *ExportTimingEventsBorderedTask);
	UE::Insights::FMiscUtils::OpenUnrealInsights(*InsightsParameters);

	const TArray<FStringView> ExpectedTimingEventsBorderedElements =
	{
		TEXTVIEW("ThreadId"),
		TEXTVIEW("ThreadName"),
		TEXTVIEW("TimerId"),
		TEXTVIEW("TimerName"),
		TEXTVIEW("StartTime"),
		TEXTVIEW("EndTime"),
		TEXTVIEW("Duration"),
		TEXTVIEW("Depth"),
	};

	bool bLineFound = false;
	FInsightsTestUtils Utils(this);
	for (int i = 0; i < ExpectedTimingEventsBorderedElements.Num(); i++)
	{
		bLineFound = Utils.FileContainsString(ExportTimingEventsNonDefaultReportPath, ExpectedTimingEventsBorderedElements[i].GetData(), Timeout);
		UTEST_TRUE(FString::Printf(TEXT("Line '%s' should exists in file: '%s'"), ExpectedTimingEventsBorderedElements[i].GetData(), *ExportTimingEventsNonDefaultReportPath), bLineFound);
	}

	ADD_LATENT_AUTOMATION_COMMAND(FVerifyExportedLinesCommand(ExportTimingEventsNonDefaultReportPath, CmdTimingEventsNonDefaultLogPath, TEXT("timing events"), this, Timeout));
	ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirectoryCommand(PlatformFile, TestResultsDirPath, this, Timeout));

	return true;
}

// The goal of this test is to verify that files with data are generated when executing multiple export commands using a response file
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCommandsExportMultipleExportCommandsTest, "System.Insights.Trace.Analysis.CommandsExport.MultipleExportCommands", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool FCommandsExportMultipleExportCommandsTest::RunTest(const FString& Parameters)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	const FString SourceTracePath = FPaths::RootDir() / TEXT("EngineTest/SourceAssets/Utrace/CommandsExportTest_5.4.utrace");
	const FString SourceExportPath = FPaths::RootDir() / TEXT("EngineTest/SourceAssets/Rsp/export.rsp");
	const FString StoreTracePath = TEXT("CommandsExportTest_5.4.utrace");
	const FString LogResultExportPath = TEXT("export.rsp");
	const FString TestResultsDirPath = TEXT("TestResults");
	const FString CmdExportLogPath = TestResultsDirPath / TEXT("Logs/cmd_export.log");
	double Timeout = 30.0;

	UTEST_TRUE(TEXT("Trace in project exists"), PlatformFile.FileExists(*SourceTracePath));
	UTEST_TRUE(TEXT("Export in project exists"), PlatformFile.FileExists(*SourceExportPath));
	PlatformFile.CopyFile(*LogResultExportPath, *SourceExportPath);
	PlatformFile.CopyFile(*StoreTracePath, *SourceTracePath);
	UTEST_TRUE(TEXT("Rsp in log directory should exists after copy"), PlatformFile.FileExists(*LogResultExportPath));
	UTEST_TRUE(TEXT("Trace in store should exists after copy"), PlatformFile.FileExists(*StoreTracePath));

	if (PlatformFile.DirectoryExists(*TestResultsDirPath))
	{
		AddInfo(TEXT("The TestResults directory already exists. Deleting to avoid undefined behavior"));
		ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirectoryCommand(PlatformFile, TestResultsDirPath, this, Timeout));
	}

	FString InsightsParameters = FString::Printf(TEXT("-OpenTraceFile=\"%s\" -ABSLOG=\"%s\" -AutoQuit -NoUI -ExecOnAnalysisCompleteCmd=\"@=%s\" -log"), *StoreTracePath, *CmdExportLogPath, *LogResultExportPath);
	UE::Insights::FMiscUtils::OpenUnrealInsights(*InsightsParameters);

	const TArray<FStringView> ExpectedThreadsElementsRsp =
	{
		TEXTVIEW("/TestResults/RSPtest/CSV/Threads_rsp.csv"),
		TEXTVIEW("/TestResults/RSPtest/TSV/Threads_rsp.tsv"),
		TEXTVIEW("/TestResults/RSPtest/TXT/Threads_rsp.txt"),
	};
	const TArray<FStringView> ExpectedTimersElementsRsp =
	{
		TEXTVIEW("/TestResults/RSPtest/CSV/Timers_rsp.csv"),
		TEXTVIEW("/TestResults/RSPtest/TSV/Timers_rsp.tsv"),
		TEXTVIEW("/TestResults/RSPtest/TXT/Timers_rsp.txt"),
	};
	const TArray<FStringView> ExpectedTimingEventsRsp =
	{
		TEXTVIEW("/TestResults/RSPtest/CSV/TimingEvents_rsp.csv"),
		TEXTVIEW("/TestResults/RSPtest/TSV/TimingEvents_rsp.tsv"),
		TEXTVIEW("/TestResults/RSPtest/TXT/TimingEvents_rsp.txt"),
	};
	const TMap<FStringView, TArray<FStringView>> ExpectedElementsRspMap =
	{
		{TEXTVIEW("threads"), ExpectedThreadsElementsRsp},
		{TEXTVIEW("timers"), ExpectedTimersElementsRsp},
		{TEXTVIEW("timing events"), ExpectedTimingEventsRsp},
	};

	bool bLineFound = false;
	FInsightsTestUtils Utils(this);
	for (const auto& [Element, ExpectedElementsRsp] : ExpectedElementsRspMap)
	{
		for (int i = 0; i < ExpectedElementsRsp.Num(); i++)
		{
			bLineFound = Utils.FileContainsString(CmdExportLogPath, ExpectedElementsRsp[i].GetData(), Timeout);
			UTEST_TRUE(FString::Printf(TEXT("Line '%s' should exists in file: '%s'"), ExpectedElementsRsp[i].GetData(), *CmdExportLogPath), bLineFound);
			ADD_LATENT_AUTOMATION_COMMAND(FVerifyExportedLinesCommand(ExpectedElementsRsp[i].GetData(), CmdExportLogPath, Element.GetData(), this, Timeout));
		}
	}
	ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirectoryCommand(PlatformFile, TestResultsDirPath, this, Timeout));

	return true;
}

#endif //WITH_AUTOMATION_TESTS