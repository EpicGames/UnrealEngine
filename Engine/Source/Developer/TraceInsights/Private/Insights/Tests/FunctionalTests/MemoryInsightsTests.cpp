// Copyright Epic Games, Inc. All Rights Reserved.

#include "Algo/Find.h"
#include "AutomationDriverCommon.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"

// TraceServices
#include "TraceServices/Model/AllocationsProvider.h"
#include "TraceServices/Model/Callstack.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"
#include "InsightsCore/Table/Widgets/STableTreeView.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/MemoryProfiler/MemoryProfilerManager.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocGroupingByCallstack.h"
#include "Insights/MemoryProfiler/ViewModels/MemAllocNode.h"
#include "Insights/MemoryProfiler/ViewModels/MemoryAlloc.h"
#include "Insights/MemoryProfiler/ViewModels/MemorySharedState.h"
#include "Insights/MemoryProfiler/Widgets/SMemAllocTableTreeView.h"
#include "Insights/MemoryProfiler/Widgets/SMemoryProfilerWindow.h"
#include "Insights/Tests/InsightsTestUtils.h"
#include "Insights/Widgets/STimingView.h"

DECLARE_LOG_CATEGORY_EXTERN(MemoryInsightsTests, Log, All);

#if WITH_AUTOMATION_TESTS

class FMemoryInsightsTestBase : public FAutomationTestBase
{
public:
	FMemoryInsightsTestBase(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
	{
	}

	virtual bool CanRunInEnvironment(const FString& TestParams, FString* OutReason, bool* OutWarn) const override
	{
		using namespace UE::Insights::MemoryProfiler;

		FMemorySharedState* SharedState = FMemoryProfilerManager::Get()->GetSharedState();
		if (!SharedState)
		{
			if (OutReason)
			{
				*OutReason = TEXT("ProfilerWindow should be valid. Please, run this test through Insights Session automation tab.");
			}

			if (OutWarn)
			{
				*OutWarn = true;
			}

			return false;
		}

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FMemoryInsightsUploadLLMXMLReportsTraceTest, FMemoryInsightsTestBase, "System.Insights.Trace.Analysis.MemoryInsights.UploadMemoryInsightsLLMXMLReportsTrace", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool FMemoryInsightsUploadLLMXMLReportsTraceTest::RunTest(const FString& Parameters)
{
	using namespace UE::Insights::MemoryProfiler;

	const FString ReportGraphsXMLPath = FPaths::RootDir() / TEXT("EngineTest/SourceAssets/Utrace/ReportGraphs.xml");
	const FString LLMReportTypesXMLPath = FPaths::RootDir() / TEXT("EngineTest/SourceAssets/Utrace/LLMReportTypes.xml");

	FMemorySharedState* SharedState = FMemoryProfilerManager::Get()->GetSharedState();
	SharedState->RemoveAllMemTagGraphTracks();
	const int DefaultTracksAmount = SharedState->GetTimingView()->GetAllTracks().Num();
	SharedState->RemoveAllMemTagGraphTracks();
	AddExpectedError("Failed to load Report");
	SharedState->CreateTracksFromReport(ReportGraphsXMLPath);
	SharedState = FMemoryProfilerManager::Get()->GetSharedState();
	const int AfterReportGraphsUploadTrackAmount = SharedState->GetTimingView()->GetAllTracks().Num();
	TestTrue("Tracks amount should be default ", DefaultTracksAmount == AfterReportGraphsUploadTrackAmount);

	SharedState->RemoveAllMemTagGraphTracks();
	SharedState->CreateTracksFromReport(LLMReportTypesXMLPath);
	SharedState = FMemoryProfilerManager::Get()->GetSharedState();
	const int AfterLLMReportTypesUploadTrackAmount = SharedState->GetTimingView()->GetAllTracks().Num();
	TestTrue("Tracks should not be default", DefaultTracksAmount != AfterLLMReportTypesUploadTrackAmount);

	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FWaitForRunningQuieryFinishedCommand, TSharedPtr<UE::Insights::MemoryProfiler::SMemAllocTableTreeView>, MemAllocTableTreeView, double, Timeout, FAutomationTestBase*, Test);
bool FWaitForRunningQuieryFinishedCommand::Update()
{
	if (!MemAllocTableTreeView->IsRunning())
	{
		return true;
	}

	if (FPlatformTime::Seconds() - StartTime >= Timeout)
	{
		Test->AddError(TEXT("FWaitForRunningQuieryFinishedCommand timed out"));
		return true;
	}

	return false;
}

DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FChangeGroupingCommand, TSharedPtr<UE::Insights::MemoryProfiler::SMemAllocTableTreeView>, MemAllocTableTreeView, FAutomationTestBase*, Test);
bool FChangeGroupingCommand::Update()
{
	TArray<TSharedPtr<UE::Insights::FTreeNodeGrouping>> CurrentGroupings;
	for (const auto& Grouping : MemAllocTableTreeView->GetAvailableGroupings())
	{
		if (Grouping->GetTitleName().ToString().Contains(TEXT("By Free Callstack")))
		{
			CurrentGroupings.Add(Grouping);
		}
	}

	Test->TestTrue(TEXT("CurrentGroupings should not be empty"), !CurrentGroupings.IsEmpty());
	MemAllocTableTreeView->SetCurrentGroupings(CurrentGroupings);

	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FVerifyHierarchyCallStackCommand, TSharedPtr<UE::Insights::MemoryProfiler::SMemAllocTableTreeView>, MemAllocTableTreeView, double, Timeout, FAutomationTestBase*, Test);
bool FVerifyHierarchyCallStackCommand::Update()
{
	using namespace UE::Insights::MemoryProfiler;
	FInsightsTestUtils InsightsTestUtils(Test);

	if (!MemAllocTableTreeView->IsRunningAsyncUpdate())
	{
		for (const TSharedPtr<UE::Insights::FTableTreeNode>& Node : MemAllocTableTreeView->GetTableRowNodes())
		{
			const FMemAllocNode& MemAllocNode = static_cast<const FMemAllocNode&>(*Node);
			const FMemoryAlloc Alloc = MemAllocNode.GetMemAllocChecked();
			if (!(!Alloc.GetAllocCallstack() || Alloc.GetAllocCallstack()->Num() == 0 || (Alloc.GetAllocCallstack()->Num() != 0 && Alloc.GetAllocCallstack()->Num() < 256)))
			{
				Test->AddError(TEXT("Resolved alloc callstack should be valid"));
			}
			if (!(!Alloc.GetFreeCallstack() || Alloc.GetFreeCallstack()->Num() == 0 || (Alloc.GetFreeCallstack()->Num() != 0 && Alloc.GetFreeCallstack()->Num() < 256)))
			{
				Test->AddError(TEXT("Resolved free callstack should be valid"));
			}
		}
		return true;
	}

	if (FPlatformTime::Seconds() - StartTime >= Timeout)
	{
		Test->AddError(TEXT("FVerifyHierarchyCallStackCommand timed out"));
		return true;
	}

	return false;
}

const TMap<TraceServices::IAllocationsProvider::EQueryRule, UE::Insights::MemoryProfiler::SMemAllocTableTreeView::FQueryParams> AllocsTimeMarkerStandaloneGameGetterMap
{
	{TraceServices::IAllocationsProvider::EQueryRule::aAf, {nullptr, {5.0, 0.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::afA, {nullptr, {10.0, 0.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::Aaf, {nullptr, {10.0, 0.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::aAfB, {nullptr, {50.0, 51.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AaBf, {nullptr, {50.0, 51.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::aAfaBf, {nullptr, {50.0, 51.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AfB, {nullptr, {50.0, 51.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AaBCf, {nullptr, {50.0, 51.0, 52.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AaBfC, {nullptr, {50.0, 51.0, 52.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::aABfC, {nullptr, {50.0, 51.0, 52.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AaBCfD, {nullptr, {50.0, 51.0, 52.0, 53.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::aABf, {nullptr, {50.0, 51.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AafB, {nullptr, {50.0, 51.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AaB, {nullptr, {50.0, 51.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AoB, {nullptr, {10.0, 20.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AiB, {nullptr, {10.0, 20.0, 0.0, 0.0}}},
};

const TMap<TraceServices::IAllocationsProvider::EQueryRule, UE::Insights::MemoryProfiler::SMemAllocTableTreeView::FQueryParams> AllocsTimeMarkerEditorPackageGetterMap
{
	{TraceServices::IAllocationsProvider::EQueryRule::aAf, {nullptr, {5.0, 0.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::afA, {nullptr, {10.0, 0.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::Aaf, {nullptr, {10.0, 0.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::aAfB, {nullptr, {2.0, 3.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AaBf, {nullptr, {2.0, 3.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::aAfaBf, {nullptr, {2.0, 3.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AfB, {nullptr, {2.0, 3.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AaBCf, {nullptr, {1.0, 2.0, 3.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AaBfC, {nullptr, {1.0, 2.0, 3.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::aABfC, {nullptr, {1.0, 2.0, 3.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AaBCfD, {nullptr, {1.0, 2.0, 3.0, 4.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AafB, {nullptr, {2.0, 3.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AaB, {nullptr, {2.0, 3.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::aABf, {nullptr, {2.0, 3.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AoB, {nullptr, {10.0, 20.0, 0.0, 0.0}}},
	{TraceServices::IAllocationsProvider::EQueryRule::AiB, {nullptr, {10.0, 20.0, 0.0, 0.0}}},
};

const TSet<TraceServices::IAllocationsProvider::EQueryRule> TemporaryExcludedRules
{
	TraceServices::IAllocationsProvider::EQueryRule::AoB,
	TraceServices::IAllocationsProvider::EQueryRule::AiB
};

bool MemoryInsightsAllocationsQueryTableTest(const FString& Parameters, const TMap <TraceServices::IAllocationsProvider::EQueryRule, UE::Insights::MemoryProfiler::SMemAllocTableTreeView::FQueryParams> AllocsTimeMarkerGetterMap, FAutomationTestBase* Test)
{
	using namespace UE::Insights::MemoryProfiler;

	double Timeout = 30.0;
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = FMemoryProfilerManager::Get()->GetProfilerWindow();
	TSharedPtr<UE::Insights::FInsightsManager> InsightsManager = UE::Insights::FInsightsManager::Get();
	FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();

	TSharedPtr<FMemoryRuleSpec> MemoryRule = *Algo::FindByPredicate(SharedState.GetMemoryRules(),
		[&Parameters](const TSharedPtr<FMemoryRuleSpec>& Rule)
		{
			return Rule->GetShortName().ToString().Contains(Parameters);
		});

	if (!MemoryRule.IsValid())
	{
		Test->AddError(TEXT("MemoryRule should not be null"));
		return false;
	}

	TSharedPtr<SMemAllocTableTreeView> MemAllocTableTreeView = ProfilerWindow->ShowMemAllocTableTreeViewTab();

	SMemAllocTableTreeView::FQueryParams QueryParams = AllocsTimeMarkerGetterMap.FindChecked(MemoryRule->GetValue());
	if (MemoryRule->GetValue() == TraceServices::IAllocationsProvider::EQueryRule::Aaf)
	{
		QueryParams.TimeMarkers[0] = InsightsManager->GetSessionDuration() - 10.0;
	}
	QueryParams.Rule = MemoryRule;
	MemAllocTableTreeView->SetQueryParams(QueryParams);

	ADD_LATENT_AUTOMATION_COMMAND(FWaitForRunningQuieryFinishedCommand(MemAllocTableTreeView, Timeout, Test));
	ADD_LATENT_AUTOMATION_COMMAND(FChangeGroupingCommand(MemAllocTableTreeView, Test));
	ADD_LATENT_AUTOMATION_COMMAND(FWaitForRunningQuieryFinishedCommand(MemAllocTableTreeView, Timeout, Test));
	ADD_LATENT_AUTOMATION_COMMAND(FVerifyHierarchyCallStackCommand(MemAllocTableTreeView, Timeout, Test));

	return true;
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FMemoryInsightsAllocationsQueryTableEditorPackageTest, "System.Insights.Trace.Analysis.MemoryInsights.AllocationsQueryTable.Editor.Package", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool FMemoryInsightsAllocationsQueryTableEditorPackageTest::RunTest(const FString& Parameters)
{
	bool bSuccess = MemoryInsightsAllocationsQueryTableTest(Parameters, AllocsTimeMarkerEditorPackageGetterMap, this);
	return bSuccess;
}

void FMemoryInsightsAllocationsQueryTableEditorPackageTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const
{
	using namespace UE::Insights::MemoryProfiler;

	if (!FMemoryProfilerManager::Get().IsValid())
	{
		return;
	}
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = FMemoryProfilerManager::Get()->GetProfilerWindow();
	if (!ProfilerWindow.IsValid())
	{
		return;
	}

	FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();

	for (const auto& MemoryRule : SharedState.GetMemoryRules())
	{
		if (!TemporaryExcludedRules.Contains(MemoryRule->GetValue()))
		{
			const FString& MemoryRuleName = MemoryRule->GetShortName().ToString();

			OutBeautifiedNames.Add(FString::Printf(TEXT("%s"), *MemoryRuleName));
			OutTestCommands.Add(MemoryRuleName);
		}
	}
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FMemoryInsightsAllocationsQueryTableStandaloneTest, "System.Insights.Trace.Analysis.MemoryInsights.AllocationsQueryTable.Standalone", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool FMemoryInsightsAllocationsQueryTableStandaloneTest::RunTest(const FString& Parameters)
{
	bool bSuccess = MemoryInsightsAllocationsQueryTableTest(Parameters, AllocsTimeMarkerStandaloneGameGetterMap, this);
	return bSuccess;
}

void FMemoryInsightsAllocationsQueryTableStandaloneTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const
{
	using namespace UE::Insights::MemoryProfiler;

	if (!FMemoryProfilerManager::Get().IsValid())
	{
		return;
	}
	TSharedPtr<SMemoryProfilerWindow> ProfilerWindow = FMemoryProfilerManager::Get()->GetProfilerWindow();
	if (!ProfilerWindow.IsValid())
	{
		return;
	}

	FMemorySharedState& SharedState = ProfilerWindow->GetSharedState();

	for (const auto& MemoryRule : SharedState.GetMemoryRules())
	{
		if (!TemporaryExcludedRules.Contains(MemoryRule->GetValue()))
		{
			const FString& MemoryRuleName = MemoryRule->GetShortName().ToString();

			OutBeautifiedNames.Add(FString::Printf(TEXT("%s"), *MemoryRuleName));
			OutTestCommands.Add(MemoryRuleName);
		}
	}
}

#endif // WITH_AUTOMATION_TESTS
