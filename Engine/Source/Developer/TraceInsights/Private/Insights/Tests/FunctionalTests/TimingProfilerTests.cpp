// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/Tests/TimingProfilerTests.h"

#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"

// TraceServices
#include "TraceServices/Model/Threads.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/Tests/InsightsTestUtils.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY(TimingProfilerTests);

////////////////////////////////////////////////////////////////////////////////////////////////////

#if WITH_AUTOMATION_TESTS

bool AnalyzeTrace(const FString& RelativePath, FAutomationTestBase* Test)
{
	FInsightsTestUtils Utils(Test);
	FString AbsolutePath = FPaths::RootDir() / RelativePath;
	return Utils.AnalyzeTrace(*AbsolutePath);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEnumerateTest, "System.Insights.Analysis.TimingInsights.Enumerate", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool FEnumerateTest::RunTest(const FString& Parameters)
{
	if (!AnalyzeTrace(TEXT("EngineTest/SourceAssets/Utrace/r424_win64_game_11590231.utrace"), this))
	{
		return !HasAnyErrors();
	}

	FTimingProfilerTests::FEnumerateTestParams Params;
	Params.Interval = 0.01;
	Params.NumEnumerations = 10000;

	FTimingProfilerTests::FCheckValues CheckValues;
	FTimingProfilerTests::RunEnumerateBenchmark(Params, CheckValues);

	TestEqual(TEXT("SessionDuration"), CheckValues.SessionDuration, 307.0172116, 1.e-6);
	TestEqual(TEXT("TotalEventDuration"), CheckValues.TotalEventDuration, 680.943945, 1.e-6);
	TestEqual(TEXT("EventCount"), CheckValues.EventCount, 10836057ull);
	TestEqual(TEXT("SumDepth"), (uint64)CheckValues.SumDepth, 80030008ull);
	TestEqual(TEXT("SumTimerIndex"), (uint64)CheckValues.SumTimerIndex, 4137608268ull);

	AddInfo(FString::Printf(TEXT("Enumeration Duration: %f seconds."), CheckValues.EnumerationDuration));

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEnumeratePerformanceTest, "System.Insights.Analysis.TimingInsights.EnumeratePerformance", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool FEnumeratePerformanceTest::RunTest(const FString& Parameters)
{
	if (!AnalyzeTrace(TEXT("EngineTest/SourceAssets/Utrace/r425_win64_game_13649855.utrace"), this))
	{
		return !HasAnyErrors();
	}

	FTimingProfilerTests::FEnumerateTestParams Params;
	Params.Interval = 0.01;
	Params.NumEnumerations = 100000;

	FTimingProfilerTests::FCheckValues CheckValues;
	FTimingProfilerTests::RunEnumerateBenchmark(Params, CheckValues);

	TestEqual(TEXT("SessionDuration"), CheckValues.SessionDuration, 341.073285, 1.e-6);
	TestEqual(TEXT("TotalEventDuration"), CheckValues.TotalEventDuration, 10912.775537, 1.e-6);
	TestEqual(TEXT("EventCount"), CheckValues.EventCount, 137000700ull);
	TestEqual(TEXT("SumDepth"), (uint64)CheckValues.SumDepth, 1134384338ull);
	TestEqual(TEXT("SumTimerIndex"), (uint64)CheckValues.SumTimerIndex, 3636619455ull);

	const double BenchmarkBaseline = 16.0;
	AddInfo(FString::Printf(TEXT("Enumeration Duration: %f seconds."), CheckValues.EnumerationDuration));

	if (CheckValues.EnumerationDuration > 1.5 * BenchmarkBaseline)
	{
		AddWarning(FString::Printf(TEXT("Enumeration duration (%f seconds) exceeded baseline by %.2f%%."), CheckValues.EnumerationDuration, CheckValues.EnumerationDuration / BenchmarkBaseline * 100.0));
	}
	else if (CheckValues.EnumerationDuration > 1.25 * BenchmarkBaseline)
	{
		AddInfo(FString::Printf(TEXT("Enumeration duration (%f seconds) exceeded baseline by %.2f%%."), CheckValues.EnumerationDuration, CheckValues.EnumerationDuration / BenchmarkBaseline * 100.0));
	}

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEnumerateFastTest, "System.Insights.Analysis.TimingInsights.EnumerateFast", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool FEnumerateFastTest::RunTest(const FString& Parameters)
{
	if (!AnalyzeTrace(TEXT("EngineTest/SourceAssets/Utrace/r423_win64_game_10478456.utrace"), this))
	{
		return !HasAnyErrors();
	}

	FTimingProfilerTests::FEnumerateTestParams Params;
	Params.Interval = 0.01;
	Params.NumEnumerations = 10000;

	FTimingProfilerTests::FCheckValues CheckValues;
	FTimingProfilerTests::RunEnumerateBenchmark(Params, CheckValues);

	TestEqual(TEXT("SessionDuration"), CheckValues.SessionDuration, 305.232584, 1.e-6);
	TestEqual(TEXT("TotalEventDuration"), CheckValues.TotalEventDuration, 1647.693886, 1.e-6);
	TestEqual(TEXT("EventCount"), CheckValues.EventCount, 1759740ull);
	TestEqual(TEXT("SumDepth"), (uint64)CheckValues.SumDepth, 15189227ull);
	TestEqual(TEXT("SumTimerIndex"), (uint64)CheckValues.SumTimerIndex, 1241561258ull);

	AddInfo(FString::Printf(TEXT("Enumeration Duration: %f seconds."), CheckValues.EnumerationDuration));

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(EnumerateEventsToFile, "System.Insights.Trace.Analysis.TimingInsights.EnumerateEventsToFile", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool EnumerateEventsToFile::RunTest(const FString& Parameters)
{
	if (!AnalyzeTrace(TEXT("EngineTest/SourceAssets/Utrace/r423_win64_game_10478456.utrace"), this))
	{
		return !HasAnyErrors();
	}

	double SessionTime = 0.0;
	uint32 TimelineIndex = (uint32)-1;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = UE::Insights::FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			TimelineIndex = FTimingProfilerTests::GetTimelineIndex(TEXT("GameThread"));

			AddErrorIfFalse(TimelineIndex != (uint32)-1, TEXT("Failed to get track named GameThread"));

			const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();

			FString FilePath = FPaths::ProjectSavedDir() + TEXT("/EnumerateEventsToFile.txt");
			TUniquePtr<FArchive> ArchiveWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*FilePath));
			TimingProfilerProvider->ReadTimeline(TimelineIndex,
				[Session, &ArchiveWriter, &TimerReader]
				(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
				{
					Timeline.EnumerateEvents(-1.0, Session->GetDurationSeconds() + 1.0,
						[&ArchiveWriter, &TimerReader]
						(double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event)
						{
							const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(Event.TimerIndex);

							(*ArchiveWriter) << FString::Printf(TEXT("%s %f %f %d\n"), Timer->Name, EventStartTime, EventEndTime, EventDepth).GetCharArray();
							return TraceServices::EEventEnumerate::Continue;
						});
				});

			ArchiveWriter->Close();
		}
	}

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(EnumerateScopesToFile, "System.Insights.Trace.Analysis.TimingInsights.EnumerateScopesToFile", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool EnumerateScopesToFile::RunTest(const FString& Parameters)
{
	double SessionTime = 0.0;
	uint32 TimelineIndex = (uint32)-1;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = UE::Insights::FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			TimelineIndex = FTimingProfilerTests::GetTimelineIndex(TEXT("GameThread"));

			AddErrorIfFalse(TimelineIndex != (uint32)-1, TEXT("Failed to get track named GameThread"));

			const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();

			FString FilePath = FPaths::ProjectSavedDir() + TEXT("/EnumerateScopesToFile.txt");
			TUniquePtr<FArchive> ArchiveWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*FilePath));
			TimingProfilerProvider->ReadTimeline(TimelineIndex,
				[Session, &ArchiveWriter, &TimerReader]
				(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
				{
					Timeline.EnumerateEvents(-1.0, Session->GetDurationSeconds() + 1.0,
						[&ArchiveWriter, &TimerReader]
						(bool bStart, double Time, const TraceServices::FTimingProfilerEvent& Event)
						{
							const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(Event.TimerIndex);

							(*ArchiveWriter) << FString::Printf(TEXT("%s %d %f\n"), Timer->Name, (int32)bStart, Time).GetCharArray();
							return TraceServices::EEventEnumerate::Continue;
						});
				});

			ArchiveWriter->Close();
		}
	}

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(InsightsEnumerate10K, "System.Insights.Trace.Analysis.TimingInsights.Enumerate10K", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool InsightsEnumerate10K::RunTest(const FString& Parameters)
{
	FTimingProfilerTests::FEnumerateTestParams Params;
	Params.Interval = 0.01;
	Params.NumEnumerations = 10000;

	FTimingProfilerTests::FCheckValues CheckValues;
	FTimingProfilerTests::RunEnumerateBenchmark(Params, CheckValues);

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(InsightsEnumerate100K, "System.Insights.Trace.Analysis.TimingInsights.Enumerate100K", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool InsightsEnumerate100K::RunTest(const FString& Parameters)
{
	FTimingProfilerTests::FEnumerateTestParams Params;
	Params.Interval = 0.01;
	Params.NumEnumerations = 100000;

	FTimingProfilerTests::FCheckValues CheckValues;
	FTimingProfilerTests::RunEnumerateBenchmark(Params, CheckValues);

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(EnumerateByEndTimeAsyncAllTracks, "System.Insights.Trace.Analysis.TimingInsights.EnumerateByEndTimeAsyncAllTracks", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool EnumerateByEndTimeAsyncAllTracks::RunTest(const FString& Parameters)
{
	FTimingProfilerTests::FEnumerateTestParams Params;
	Params.Interval = 24.0 * 3600; //A day - Should be big enough to contain any valid session in [0, 0 + interval]
	Params.NumEnumerations = 1;
	Params.SortOrder = TraceServices::EEventSortOrder::ByEndTime;

	FTimingProfilerTests::RunEnumerateSyncAsyncComparisonTest(*this, Params, false);

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(EnumerateByEndTimeAsyncGameThreadTrack, "System.Insights.Trace.Analysis.TimingInsights.EnumerateByEndTimeAsyncGameThreadTrack", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool EnumerateByEndTimeAsyncGameThreadTrack::RunTest(const FString& Parameters)
{
	FTimingProfilerTests::FEnumerateTestParams Params;
	Params.Interval = 24.0 * 3600; //A day - Should be big enough to contain any valid session in [0, 0 + interval]
	Params.NumEnumerations = 1;
	Params.SortOrder = TraceServices::EEventSortOrder::ByEndTime;

	FTimingProfilerTests::RunEnumerateSyncAsyncComparisonTest(*this, Params, true);

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(EnumerateByEndTimeAllTracks10sIntervals, "System.Insights.Trace.Analysis.TimingInsights.EnumerateByEndTimeAllTracks10sIntervals", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool EnumerateByEndTimeAllTracks10sIntervals::RunTest(const FString& Parameters)
{
	FTimingProfilerTests::FEnumerateTestParams Params;
	Params.Interval = 10.0;
	Params.NumEnumerations = 100;
	Params.SortOrder = TraceServices::EEventSortOrder::ByEndTime;

	FTimingProfilerTests::RunEnumerateSyncAsyncComparisonTest(*this, Params, false);

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(EnumerateByEndTimeAllTracks5sIntervals, "System.Insights.Trace.Analysis.TimingInsights.EnumerateByEndTimeAllTracks5sIntervals", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool EnumerateByEndTimeAllTracks5sIntervals::RunTest(const FString& Parameters)
{
	FTimingProfilerTests::FEnumerateTestParams Params;
	Params.Interval = 5.0;
	Params.NumEnumerations = 200;
	Params.SortOrder = TraceServices::EEventSortOrder::ByEndTime;

	FTimingProfilerTests::RunEnumerateSyncAsyncComparisonTest(*this, Params, false);

	return !HasAnyErrors();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////Enumerate Async Ordered by Start Time Tests/////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

IMPLEMENT_SIMPLE_AUTOMATION_TEST(EnumerateByStartTimeAsyncAllTracks, "System.Insights.Trace.Analysis.TimingInsights.EnumerateByStartTimeAsyncAllTracks", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool EnumerateByStartTimeAsyncAllTracks::RunTest(const FString& Parameters)
{
	FTimingProfilerTests::FEnumerateTestParams Params;
	Params.Interval = 24.0 * 3600; //A day - Should be big enough to contain any valid session in [0, 0 + interval]
	Params.NumEnumerations = 1;
	Params.SortOrder = TraceServices::EEventSortOrder::ByStartTime;

	FTimingProfilerTests::RunEnumerateSyncAsyncComparisonTest(*this, Params, false);

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(EnumerateByStartTimeAsyncGameThreadTrack, "System.Insights.Trace.Analysis.TimingInsights.EnumerateByStartTimeAsyncGameThreadTrack", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool EnumerateByStartTimeAsyncGameThreadTrack::RunTest(const FString& Parameters)
{
	FTimingProfilerTests::FEnumerateTestParams Params;
	Params.Interval = 24.0 * 3600; //A day - Should be big enough to contain any valid session in [0, 0 + interval]
	Params.NumEnumerations = 1;
	Params.SortOrder = TraceServices::EEventSortOrder::ByStartTime;

	FTimingProfilerTests::RunEnumerateSyncAsyncComparisonTest(*this, Params, true);

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(EnumerateByStartTimeAllTracks10sIntervals, "System.Insights.Trace.Analysis.TimingInsights.EnumerateByStartTimeAllTracks10sIntervals", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool EnumerateByStartTimeAllTracks10sIntervals::RunTest(const FString& Parameters)
{
	FTimingProfilerTests::FEnumerateTestParams Params;
	Params.Interval = 10.0;
	Params.NumEnumerations = 100;
	Params.SortOrder = TraceServices::EEventSortOrder::ByStartTime;

	FTimingProfilerTests::RunEnumerateSyncAsyncComparisonTest(*this, Params, false);

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(EnumerateByStartTimeAllTracks5sIntervals, "System.Insights.Trace.Analysis.TimingInsights.EnumerateByStartTimeAllTracks5sIntervals", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool EnumerateByStartTimeAllTracks5sIntervals::RunTest(const FString& Parameters)
{
	FTimingProfilerTests::FEnumerateTestParams Params;
	Params.Interval = 5.0;
	Params.NumEnumerations = 200;
	Params.SortOrder = TraceServices::EEventSortOrder::ByStartTime;

	FTimingProfilerTests::RunEnumerateSyncAsyncComparisonTest(*this, Params, false);

	return !HasAnyErrors();
}
#endif //WITH_AUTOMATION_TESTS

void FTimingProfilerTests::RunEnumerateBenchmark(const FEnumerateTestParams& InParams, FCheckValues& OutCheckValues)
{
	UE_LOGF(TimingProfilerTests, Log, "RUNNING BENCHMARK...");

	UE::Insights::FStopwatch Stopwatch;
	Stopwatch.Start();

	double SessionTime = 0.0;
	uint32 TimelineIndex = (uint32)-1;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = UE::Insights::FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			SessionTime = Session->GetDurationSeconds();
			OutCheckValues.SessionDuration = SessionTime;

			const double TimeIncrement = SessionTime / static_cast<double>(InParams.NumEnumerations);

			TimelineIndex = GetTimelineIndex(TEXT("GameThread"));

			TimingProfilerProvider->ReadTimeline(TimelineIndex,
				[&OutCheckValues, &InParams, TimeIncrement]
				(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
				{
					double Time = 0.0;
					for (int32 Index = 0; Index < InParams.NumEnumerations; ++Index)
					{
						Timeline.EnumerateEvents(Time, Time + InParams.Interval,
							[&OutCheckValues]
							(double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event)
							{
								EventEndTime = FMath::Min(EventEndTime, OutCheckValues.SessionDuration);
								OutCheckValues.TotalEventDuration += EventEndTime - EventStartTime;
								++OutCheckValues.EventCount;
								OutCheckValues.SumDepth += EventDepth;
								OutCheckValues.SumTimerIndex += Event.TimerIndex;
								return TraceServices::EEventEnumerate::Continue;
							});

						Time += TimeIncrement;
					}
				});
		}
	}

	Stopwatch.Stop();
	OutCheckValues.EnumerationDuration = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TimingProfilerTests, Log, "BENCHMARK RESULT: %f seconds", OutCheckValues.EnumerationDuration);
	UE_LOGF(TimingProfilerTests, Log, "SessionTime: %f seconds", SessionTime);
	UE_LOGF(TimingProfilerTests, Log, "TimelineIndex: %u", TimelineIndex);
	UE_LOGF(TimingProfilerTests, Log, "Check Values: %f %llu %u %u", OutCheckValues.TotalEventDuration, OutCheckValues.EventCount, OutCheckValues.SumDepth, OutCheckValues.SumTimerIndex);
}

void FTimingProfilerTests::RunEnumerateAsyncBenchmark(const FEnumerateTestParams& InParams, FCheckValues& OutCheckValues)
{
	UE_LOGF(TimingProfilerTests, Log, "RUNNING ASYNC ENUMERATE BENCHMARK...");

	UE::Insights::FStopwatch Stopwatch;
	Stopwatch.Start();

	double SessionTime = 0.0;
	uint32 TimelineIndex = (uint32)-1;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = UE::Insights::FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			SessionTime = Session->GetDurationSeconds();
			OutCheckValues.SessionDuration = SessionTime;

			const double TimeIncrement = SessionTime / static_cast<double>(InParams.NumEnumerations);

			TimelineIndex = GetTimelineIndex(TEXT("GameThread"));

			TArray<FCheckValues> TaskCheckValues;

			TimingProfilerProvider->ReadTimeline(TimelineIndex,
				[SessionTime, &InParams, TimeIncrement, &TaskCheckValues]
				(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
				{
					double Time = 0.0;
					for (int32 Index = 0; Index < InParams.NumEnumerations; ++Index)
					{
						TraceServices::ITimeline<TraceServices::FTimingProfilerEvent>::EnumerateAsyncParams Params;
						Params.IntervalStart = Time;
						Params.IntervalEnd = Time + InParams.Interval;
						Params.Resolution = 0.0;
						Params.SortOrder = InParams.SortOrder;
						Params.SetupCallback =
							[&TaskCheckValues](uint32 NumTasks)
							{
								TaskCheckValues.AddDefaulted(NumTasks);
							};
						Params.EventRangeCallback =
							[&TaskCheckValues, SessionTime]
							(double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event, uint32 TaskIndex)
							{
								EventEndTime = FMath::Min(EventEndTime, SessionTime);
								TaskCheckValues[TaskIndex].TotalEventDuration += EventEndTime - EventStartTime;
								++TaskCheckValues[TaskIndex].EventCount;
								TaskCheckValues[TaskIndex].SumDepth += EventDepth;
								TaskCheckValues[TaskIndex].SumTimerIndex += Event.TimerIndex;
								return TraceServices::EEventEnumerate::Continue;
							};

						Timeline.EnumerateEventsDownSampledAsync(Params);

						Time += TimeIncrement;
					}
				});

			for (auto& CheckValues : TaskCheckValues)
			{
				OutCheckValues.TotalEventDuration += CheckValues.TotalEventDuration;
				OutCheckValues.EventCount += CheckValues.EventCount;
				OutCheckValues.SumDepth += CheckValues.SumDepth;
				OutCheckValues.SumTimerIndex += CheckValues.SumTimerIndex;
			}
		}
	}

	Stopwatch.Stop();
	OutCheckValues.EnumerationDuration = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TimingProfilerTests, Log, "ASYNC ENUMERATE BENCHMARK RESULT: %f seconds", OutCheckValues.EnumerationDuration);
	UE_LOGF(TimingProfilerTests, Log, "SessionTime: %f seconds", SessionTime);
	UE_LOGF(TimingProfilerTests, Log, "TimelineIndex: %u", TimelineIndex);
	UE_LOGF(TimingProfilerTests, Log, "Check Values: %f %llu %u %u", OutCheckValues.TotalEventDuration, OutCheckValues.EventCount, OutCheckValues.SumDepth, OutCheckValues.SumTimerIndex);
}

void FTimingProfilerTests::RunEnumerateAllTracksBenchmark(const FEnumerateTestParams& InParams, FCheckValues& OutCheckValues)
{
	UE_LOGF(TimingProfilerTests, Log, "RUNNING ENUMERATE ALL TRACKS BENCHMARK...");

	UE::Insights::FStopwatch Stopwatch;
	Stopwatch.Start();

	double SessionTime = 0.0;
	uint32 TimelineIndex = (uint32)-1;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = UE::Insights::FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			SessionTime = Session->GetDurationSeconds();
			OutCheckValues.SessionDuration = SessionTime;

			const double TimeIncrement = SessionTime / static_cast<double>(InParams.NumEnumerations);

			const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(*Session.Get());
			ThreadProvider.EnumerateThreads(
				[&TimelineIndex, TimingProfilerProvider, &OutCheckValues, &InParams, TimeIncrement]
				(const TraceServices::FThreadInfo& ThreadInfo)
				{
					TimingProfilerProvider->GetCpuThreadTimelineIndex(ThreadInfo.Id, TimelineIndex);

					TimingProfilerProvider->ReadTimeline(TimelineIndex,
						[&OutCheckValues, &InParams, TimeIncrement]
						(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
						{
							double Time = 0.0;
							for (int32 Index = 0; Index < InParams.NumEnumerations; ++Index)
							{
								Timeline.EnumerateEvents(Time, Time + InParams.Interval,
									[&OutCheckValues]
									(double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event)
									{
										EventEndTime = FMath::Min(EventEndTime, OutCheckValues.SessionDuration);
										OutCheckValues.TotalEventDuration += EventEndTime - EventStartTime;
										++OutCheckValues.EventCount;
										OutCheckValues.SumDepth += EventDepth;
										OutCheckValues.SumTimerIndex += Event.TimerIndex;
										return TraceServices::EEventEnumerate::Continue;
									});

								Time += TimeIncrement;
							}
						});
				});
		}
	}

	Stopwatch.Stop();
	OutCheckValues.EnumerationDuration = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TimingProfilerTests, Log, "ENUMERATE ALL TRACKS BENCHMARK RESULT: %f seconds", OutCheckValues.EnumerationDuration);
	UE_LOGF(TimingProfilerTests, Log, "SessionTime: %f seconds", SessionTime);
	UE_LOGF(TimingProfilerTests, Log, "TimelineIndex: %u", TimelineIndex);
	UE_LOGF(TimingProfilerTests, Log, "Check Values: %f %llu %u %u", OutCheckValues.TotalEventDuration, OutCheckValues.EventCount, OutCheckValues.SumDepth, OutCheckValues.SumTimerIndex);
}

void FTimingProfilerTests::RunEnumerateAsyncAllTracksBenchmark(const FEnumerateTestParams& InParams, FCheckValues& OutCheckValues)
{
	UE_LOGF(TimingProfilerTests, Log, "RUNNING ASYNC ENUMERATE ALL TRACKS BENCHMARK...");

	UE::Insights::FStopwatch Stopwatch;
	Stopwatch.Start();

	double SessionTime = 0.0;
	uint32 TimelineIndex = (uint32)-1;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = UE::Insights::FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			SessionTime = Session->GetDurationSeconds();
			OutCheckValues.SessionDuration = SessionTime;

			const double TimeIncrement = SessionTime / static_cast<double>(InParams.NumEnumerations);

			TArray<FCheckValues> TaskCheckValues;

			const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(*Session.Get());
			ThreadProvider.EnumerateThreads(
				[&TimelineIndex, TimingProfilerProvider, &OutCheckValues, &InParams, TimeIncrement, &TaskCheckValues]
				(const TraceServices::FThreadInfo& ThreadInfo)
				{
					TimingProfilerProvider->GetCpuThreadTimelineIndex(ThreadInfo.Id, TimelineIndex);

					TimingProfilerProvider->ReadTimeline(TimelineIndex,
						[&OutCheckValues, &InParams, TimeIncrement, &TaskCheckValues]
						(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
						{
							double Time = 0.0;
							for (int32 Index = 0; Index < InParams.NumEnumerations; ++Index)
							{
								TraceServices::ITimeline<TraceServices::FTimingProfilerEvent>::EnumerateAsyncParams Params;
								Params.IntervalStart = Time;
								Params.IntervalEnd = Time + InParams.Interval;
								Params.Resolution = 0.0;
								Params.SortOrder = InParams.SortOrder;
								Params.SetupCallback =
									[&TaskCheckValues](uint32 NumTasks)
									{
										TaskCheckValues.AddDefaulted(NumTasks);
									};
								Params.EventRangeCallback =
									[&OutCheckValues, &TaskCheckValues]
									(double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event, uint32 TaskIndex)
									{
										EventEndTime = FMath::Min(EventEndTime, OutCheckValues.SessionDuration);
										TaskCheckValues[TaskIndex].TotalEventDuration += EventEndTime - EventStartTime;
										++TaskCheckValues[TaskIndex].EventCount;
										TaskCheckValues[TaskIndex].SumDepth += EventDepth;
										TaskCheckValues[TaskIndex].SumTimerIndex += Event.TimerIndex;
										return TraceServices::EEventEnumerate::Continue;
									};

								Timeline.EnumerateEventsDownSampledAsync(Params);

								for (auto& CheckValues : TaskCheckValues)
								{
									OutCheckValues.TotalEventDuration += CheckValues.TotalEventDuration;
									OutCheckValues.EventCount += CheckValues.EventCount;
									OutCheckValues.SumDepth += CheckValues.SumDepth;
									OutCheckValues.SumTimerIndex += CheckValues.SumTimerIndex;
								}

								TaskCheckValues.Empty();
								Time += TimeIncrement;
							}
						});
				});
		}
	}

	Stopwatch.Stop();
	OutCheckValues.EnumerationDuration = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TimingProfilerTests, Log, "ASYNC ENUMERATE ALL TRACKS BENCHMARK RESULT: %f seconds", OutCheckValues.EnumerationDuration);
	UE_LOGF(TimingProfilerTests, Log, "SessionTime: %f seconds", SessionTime);
	UE_LOGF(TimingProfilerTests, Log, "TimelineIndex: %u", TimelineIndex);
	UE_LOGF(TimingProfilerTests, Log, "Check Values: %f %llu %u %u", OutCheckValues.TotalEventDuration, OutCheckValues.EventCount, OutCheckValues.SumDepth, OutCheckValues.SumTimerIndex);
}

uint32 FTimingProfilerTests::GetTimelineIndex(const TCHAR* InName)
{
	uint32 TimelineIndex = (uint32)-1;

	// Assumes this is called under FAnalysisSessionReadScope.
	TSharedPtr<const TraceServices::IAnalysisSession> Session = UE::Insights::FInsightsManager::Get()->GetSession();
	check(Session.IsValid());

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
	if (TimingProfilerProvider)
	{
		const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(*Session.Get());
		ThreadProvider.EnumerateThreads(
			[&TimelineIndex, TimingProfilerProvider, InName]
			(const TraceServices::FThreadInfo& ThreadInfo)
			{
				if (!FCString::Strcmp(ThreadInfo.Name, InName))
				{
					TimingProfilerProvider->GetCpuThreadTimelineIndex(ThreadInfo.Id, TimelineIndex);
				}
			});
	}

	return TimelineIndex;
}

bool FTimingProfilerTests::RunEnumerateSyncAsyncComparisonTest(FAutomationTestBase& Test, const FEnumerateTestParams& InParams, bool bGameThreadOnly)
{
	FTimingProfilerTests::FCheckValues CheckValues;
	FTimingProfilerTests::FCheckValues CheckValuesAsync;

	if (bGameThreadOnly)
	{
		FTimingProfilerTests::RunEnumerateBenchmark(InParams, CheckValues);
		FTimingProfilerTests::RunEnumerateAsyncBenchmark(InParams, CheckValuesAsync);
	}
	else
	{
		FTimingProfilerTests::RunEnumerateAllTracksBenchmark(InParams, CheckValues);
		FTimingProfilerTests::RunEnumerateAsyncAllTracksBenchmark(InParams, CheckValuesAsync);
	}

	FTimingProfilerTests::VerifyCheckValues(Test, CheckValues, CheckValuesAsync);

	Test.AddInfo(FString::Printf(TEXT("Enumeration Duration: %f seconds."), CheckValues.EnumerationDuration));
	Test.AddInfo(FString::Printf(TEXT("Async Enumeration Duration: %f seconds."), CheckValuesAsync.EnumerationDuration));

	return !Test.HasAnyErrors();
}

void FTimingProfilerTests::VerifyCheckValues(FAutomationTestBase& Test, FCheckValues First, FCheckValues Second)
{
	Test.TestEqual(TEXT("SessionDuration"), First.SessionDuration, Second.SessionDuration, 1.e-6);
	Test.TestEqual(TEXT("TotalEventDuration"), First.TotalEventDuration, Second.TotalEventDuration, 1.e-3);
	Test.TestEqual(TEXT("EventCount"), First.EventCount, Second.EventCount);
	Test.TestEqual(TEXT("SumDepth"), (uint64)First.SumDepth, (uint64)Second.SumDepth);
	Test.TestEqual(TEXT("SumTimerIndex"), (uint64)First.SumTimerIndex, (uint64)Second.SumTimerIndex);
}