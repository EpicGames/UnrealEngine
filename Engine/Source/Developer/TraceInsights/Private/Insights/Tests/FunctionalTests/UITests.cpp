// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#include "Logging/LogMacros.h"
#include "Misc/AutomationTest.h"

// TraceServices
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Filter/ViewModels/TimeFilterValueConverter.h"

// TraceInsights
#include "Insights/MemoryProfiler/ViewModels/MemAllocFilterValueConverter.h"
#include "Insights/TaskGraphProfiler/ViewModels/TaskTimingTrack.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Tracks/TimeRulerTrack.h"
#include "Insights/Widgets/STimingView.h"

DECLARE_LOG_CATEGORY_EXTERN(UITests, Log, All);

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHideAndShowAllTimingViewTabs, "System.Insights.Analysis.TimingInsights.HideAndShowAllTimingViewTabs", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool FHideAndShowAllTimingViewTabs::RunTest(const FString& Parameters)
{
	using namespace UE::Insights::TimingProfiler;
	TSharedPtr<FTimingProfilerManager> TimingProfilerManager = FTimingProfilerManager::Get();

	TimingProfilerManager->ShowHideTimingView(false);
	TimingProfilerManager->ShowHideCalleesTreeView(false);
	TimingProfilerManager->ShowHideCallersTreeView(false);
	TimingProfilerManager->ShowHideFramesTrack(false);
	TimingProfilerManager->ShowHideLogView(false);
	TimingProfilerManager->ShowHideTimersView(false);

	TimingProfilerManager->ShowHideTimingView(true);
	TimingProfilerManager->ShowHideCalleesTreeView(true);
	TimingProfilerManager->ShowHideCallersTreeView(true);
	TimingProfilerManager->ShowHideFramesTrack(true);
	TimingProfilerManager->ShowHideLogView(true);
	TimingProfilerManager->ShowHideTimersView(true);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMemoryFilterValueConverterTest, "System.Insights.Analysis.MemoryFilterValueConverter", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FMemoryFilterValueConverterTest::RunTest(const FString& Parameters)
{
	using namespace UE::Insights::MemoryProfiler;
	FMemoryFilterValueConverter Converter;

	FText Error;
	int64 Value;

	Converter.Convert(TEXT("152485"), Value, Error);
	TestEqual(TEXT("BasicValue"), 152485LL, Value);

	Converter.Convert(TEXT("125.56"), Value, Error);
	TestEqual(TEXT("DoubleValue"), 125LL, Value);

	Converter.Convert(TEXT("3 KiB"), Value, Error);
	TestEqual(TEXT("Kib"), 3072LL, Value);

	Converter.Convert(TEXT("7.14 KiB"), Value, Error);
	TestEqual(TEXT("KibDouble"), 7311LL, Value);

	Converter.Convert(TEXT("5 MiB"), Value, Error);
	TestEqual(TEXT("Mib"), 5242880LL, Value);

	Converter.Convert(TEXT("1 EiB"), Value, Error);
	TestEqual(TEXT("Eib"), 1152921504606846976LL, Value);

	Converter.Convert(TEXT("2 kib"), Value, Error);
	TestEqual(TEXT("CaseInsesitive"), 2048LL, Value);

	TestFalse(TEXT("Fail1"), Converter.Convert(TEXT("23test"), Value, Error));
	TestFalse(TEXT("Fail2"), Converter.Convert(TEXT("43 kOb"), Value, Error));
	TestFalse(TEXT("FailInvalidChar"), Converter.Convert(TEXT("45,"), Value, Error));

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTimeFilterValueConverterTest, "System.Insights.Analysis.TimeFilterValueConverter", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FTimeFilterValueConverterTest::RunTest(const FString& Parameters)
{
	UE::Insights::FTimeFilterValueConverter Converter;

	FText Error;
	double Value;

	Converter.Convert(TEXT("15.3"), Value, Error);
	TestEqual(TEXT("ValueInSeconds"), 15.3, Value);

	Converter.Convert(TEXT("0.2"), Value, Error);
	TestEqual(TEXT("ValueInSeconds2"), 0.2, Value);

	Converter.Convert(TEXT("0.4s"), Value, Error);
	TestEqual(TEXT("ValueInSeconds3"), 0.4, Value);

	Converter.Convert(TEXT("125.56ms"), Value, Error);
	TestEqual(TEXT("ValueInMiliseconds"), 0.12556, Value);

	Converter.Convert(TEXT("14.2 µs"), Value, Error);
	TestEqual(TEXT("ValueInMicroseconds"), 14.2 * 1.e-6, Value);

	Converter.Convert(TEXT("0.72us"), Value, Error);
	TestEqual(TEXT("ValueInMicroseconds2"), 0.72 * 1.e-6, Value); 

	Converter.Convert(TEXT("3ns"), Value, Error);
	TestEqual(TEXT("ValueInNanoseconds"), 3 * 1.e-9, Value);

	Converter.Convert(TEXT("17ns"), Value, Error);
	TestEqual(TEXT("ValueInPicoseconds"), 17 * 1.e-12, Value);

	Converter.Convert(TEXT("0.5m"), Value, Error);
	TestEqual(TEXT("ValueInMinutes"), 30.0, Value);

	Converter.Convert(TEXT("1.1h"), Value, Error);
	TestEqual(TEXT("ValueInHours"), 3960.0, Value);

	Converter.Convert(TEXT("2d"), Value, Error);
	TestEqual(TEXT("ValueInHours"), 2 * 60 * 60 * 24.0, Value);

	TestFalse(TEXT("Fail1"), Converter.Convert(TEXT("1.23ss"), Value, Error));
	TestFalse(TEXT("Fail2"), Converter.Convert(TEXT("abc"), Value, Error));
	TestFalse(TEXT("FailInvalidChar"), Converter.Convert(TEXT("0,2"), Value, Error));

	return !HasAnyErrors();
}