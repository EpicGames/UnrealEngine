// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsTestRunner.h"

#if !UE_BUILD_SHIPPING && !WITH_EDITOR

#include "IAutomationControllerModule.h"
#include "IAutomationWindowModule.h"
#include "IAutomationWorkerModule.h"
#include "ISessionManager.h"
#include "ISessionServicesModule.h"
#include "Misc/App.h"
#include "Misc/AutomationTest.h"
#include "Misc/CoreMisc.h"
#include "Widgets/Docking/SDockTab.h"

// TraceInsightsCore
#include "InsightsCore/Common/InsightsCoreStyle.h"

// TraceInsights
#include "Insights/Common/InsightsMenuBuilder.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY(InsightsTestRunner);

#define LOCTEXT_NAMESPACE "InsightsTestRunner"

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FInsightsTestRunner> FInsightsTestRunner::Instance;

const TCHAR* FInsightsTestRunner::AutoQuitMsgOnComplete = TEXT("Application is closing because it was started with the AutoQuit parameter and session analysis is complete and all scheduled tests have completed.");

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsTestRunner::ScheduleCommand(const FString& InCmd)
{
	FString ActualCmd = InCmd.Replace(TEXT("\""), TEXT(""));
	if (!ActualCmd.StartsWith(TEXT("Automation RunTests")))
	{
		UE_LOGF(InsightsTestRunner, Warning, "[InsightsTestRunner] Command %ls does not start with Automation RunTests. Command will be ignored.", *InCmd);
		return;
	}

	CommandToExecute = ActualCmd;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsTestRunner::Initialize(IUnrealInsightsModule& InsightsModule)
{
	if (bInitAutomationModules)
	{
		FApp::SetSessionName(TEXT("UnrealInsights"));
		ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");
		auto SessionService = SessionServicesModule.GetSessionService();
		SessionService->Start();

		// Create Session Manager.
		SessionServicesModule.GetSessionManager();

		IAutomationControllerModule& AutomationControllerModule = FModuleManager::LoadModuleChecked<IAutomationControllerModule>(TEXT("AutomationController"));
		AutomationControllerModule.Init();

		// Initialize the target platform manager as it is needed by Automation Window.
		GetTargetPlatformManager();
		FModuleManager::Get().LoadModule("AutomationWindow");
		FModuleManager::Get().LoadModule("AutomationWorker");
	}

	using namespace UE::Insights;
	SessionAnalysisCompletedHandle = FInsightsManager::Get()->GetSessionAnalysisCompletedEvent().AddSP(this, &FInsightsTestRunner::OnSessionAnalysisCompleted);

	// Register tick functions.
	OnTick = FTickerDelegate::CreateSP(this, &FInsightsTestRunner::Tick);
	OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(OnTick, 1.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsTestRunner::Shutdown()
{
	FTSTicker::RemoveTicker(OnTickHandle);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsTestRunner::RegisterMajorTabs(IUnrealInsightsModule& InsightsModule)
{
	using namespace UE::Insights;

	if (bInitAutomationModules)
	{
		const FInsightsMajorTabConfig& AutomationConfig = InsightsModule.FindMajorTabConfig(FInsightsManagerTabs::AutomationWindowTabId);
		if (AutomationConfig.bIsAvailable)
		{
			// Register tab spawner for the Automation window.
			FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(FInsightsManagerTabs::AutomationWindowTabId,
				FOnSpawnTab::CreateRaw(this, &FInsightsTestRunner::SpawnAutomationWindowTab))
				.SetDisplayName(AutomationConfig.TabLabel.IsSet() ? AutomationConfig.TabLabel.GetValue() : LOCTEXT("AutomationTab", "Automation"))
				.SetTooltipText(AutomationConfig.TabTooltip.IsSet() ? AutomationConfig.TabTooltip.GetValue() : LOCTEXT("AutomationTooltipText", "Opens the automation tab."))
				.SetIcon(AutomationConfig.TabIcon.IsSet() ? AutomationConfig.TabIcon.GetValue() : FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.TestAutomation"));

			TSharedRef<FWorkspaceItem> Group = AutomationConfig.WorkspaceGroup.IsValid() ? AutomationConfig.WorkspaceGroup.ToSharedRef() : FInsightsManager::Get()->GetInsightsMenuBuilder()->GetWindowsGroup();
			TabSpawnerEntry.SetGroup(Group);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsTestRunner::UnregisterMajorTabs()
{
	using namespace UE::Insights;
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(FInsightsManagerTabs::AutomationWindowTabId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FInsightsTestRunner> FInsightsTestRunner::CreateInstance()
{
	ensure(!FInsightsTestRunner::Instance.IsValid());
	if (FInsightsTestRunner::Instance.IsValid())
	{
		FInsightsTestRunner::Instance.Reset();
	}

	FInsightsTestRunner::Instance = MakeShared<FInsightsTestRunner>();

	return FInsightsTestRunner::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FInsightsTestRunner> FInsightsTestRunner::Get()
{
	return FInsightsTestRunner::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsTestRunner::RunTests()
{
	if (CommandToExecute.IsEmpty())
	{
		return;
	}

	IAutomationControllerModule& AutomationControllerModule = FModuleManager::LoadModuleChecked<IAutomationControllerModule>(TEXT("AutomationController"));
	IAutomationControllerManagerRef AutomationControllerManager = AutomationControllerModule.GetAutomationController();

	bool& bIsRunningTestsLocal = bIsRunningTests;
	AutomationControllerManager->OnTestsComplete().AddLambda([&bIsRunningTestsLocal]()
		{
			bIsRunningTestsLocal = false;
		});

	bIsRunningTests = true;
	StaticExec(NULL, *CommandToExecute);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsTestRunner::~FInsightsTestRunner()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsTestRunner::OnSessionAnalysisCompleted()
{
	RunTests();

	bIsAnalysisComplete = true;

	using namespace UE::Insights;
	FInsightsManager::Get()->GetSessionAnalysisCompletedEvent().Remove(SessionAnalysisCompletedHandle);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> FInsightsTestRunner::SpawnAutomationWindowTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	IAutomationWindowModule& AutomationWindowModule = FModuleManager::LoadModuleChecked<IAutomationWindowModule>("AutomationWindow");
	IAutomationControllerModule& AutomationControllerModule = FModuleManager::LoadModuleChecked<IAutomationControllerModule>(TEXT("AutomationController"));
	IAutomationControllerManagerPtr AutomationController = AutomationControllerModule.GetAutomationController();
	ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");

	auto SessionManager = SessionServicesModule.GetSessionManager();

	auto TabWidget = AutomationWindowModule.CreateAutomationWindow(
		AutomationController.ToSharedRef(),
		SessionManager.ToSharedRef()
	);

	DockTab->SetContent(TabWidget);

	TArray<TSharedPtr<ISessionInfo>> OutSessions;
	SessionManager->GetSessions(OutSessions);
	for (auto Session : OutSessions)
	{
		if (Session->GetSessionName() == TEXT("UnrealInsights"))
		{
			SessionManager->SelectSession(OutSessions[0]);
			TArray<TSharedPtr<ISessionInstanceInfo>> OutSessionsInstances;
			OutSessions[0]->GetInstances(OutSessionsInstances);

			SessionManager->SetInstanceSelected(OutSessionsInstances[0].ToSharedRef(), true);
		}
	}
	return DockTab;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsTestRunner::Tick(float DeltaTime)
{
	if (bAutoQuit && bIsAnalysisComplete && !bIsRunningTests)
	{
		RequestEngineExit(AutoQuitMsgOnComplete);
	}

	IAutomationWorkerModule& AutomationWorkerModule = FModuleManager::LoadModuleChecked<IAutomationWorkerModule>("AutomationWorker");
	IAutomationControllerModule& AutomationControllerModule = FModuleManager::LoadModuleChecked<IAutomationControllerModule>(TEXT("AutomationController"));

	AutomationControllerModule.Tick();
	AutomationWorkerModule.Tick();

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

#endif //UE_BUILD_SHIPPING && !WITH_EDITOR
