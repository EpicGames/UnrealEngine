// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Containers/Ticker.h"

#if !UE_BUILD_SHIPPING && !WITH_EDITOR

#include "Insights/IUnrealInsightsModule.h"

DECLARE_LOG_CATEGORY_EXTERN(InsightsTestRunner, Log, All);

class FInsightsTestRunner : public TSharedFromThis<FInsightsTestRunner>, public IInsightsComponent
{
public:
	TRACEINSIGHTS_API virtual ~FInsightsTestRunner();

	TRACEINSIGHTS_API void ScheduleCommand(const FString& InCmd);

	TRACEINSIGHTS_API virtual void Initialize(IUnrealInsightsModule& InsightsModule) override;
	TRACEINSIGHTS_API virtual void Shutdown() override;
	TRACEINSIGHTS_API virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) override;
	TRACEINSIGHTS_API virtual void UnregisterMajorTabs() override;

	static TRACEINSIGHTS_API TSharedPtr<FInsightsTestRunner> CreateInstance();
	static TRACEINSIGHTS_API TSharedPtr<FInsightsTestRunner> Get();

	TRACEINSIGHTS_API bool Tick(float DeltaTime);

	void SetAutoQuit(bool InAutoQuit) { bAutoQuit = InAutoQuit; }
	bool GetAutoQuit() const { return bAutoQuit; }

	void SetInitAutomationModules(bool InInitAutomationModules) { bInitAutomationModules = InInitAutomationModules; }
	bool GetInitAutomationModules() const { return bInitAutomationModules; }
	TRACEINSIGHTS_API void RunTests();

private:
	TRACEINSIGHTS_API TSharedRef<SDockTab> SpawnAutomationWindowTab(const FSpawnTabArgs& Args);
	TRACEINSIGHTS_API void OnSessionAnalysisCompleted();

private:
	/** The delegate to be invoked when this manager ticks. */
	FTickerDelegate OnTick;

	/** Handle to the registered OnTick. */
	FTSTicker::FDelegateHandle OnTickHandle;

	FDelegateHandle SessionAnalysisCompletedHandle;

	FString CommandToExecute;

	bool bAutoQuit = false;
	bool bInitAutomationModules = false;
	bool bIsRunningTests = false;
	bool bIsAnalysisComplete = false;

	static TRACEINSIGHTS_API const TCHAR* AutoQuitMsgOnComplete;

	static TRACEINSIGHTS_API TSharedPtr<FInsightsTestRunner> Instance;
};

#endif //UE_BUILD_SHIPPING && !WITH_EDITOR
