// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/Tests/InsightsTestUtils.h"

#include "HAL/FileManagerGeneric.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"

// TraceAnalysis
#include "Trace/StoreClient.h"

// TraceServices
#include "TraceServices/AnalysisService.h"
#include "TraceServices/ITraceServicesModule.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Diagnostics.h"
#include "TraceServices/ModuleService.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/IUnrealInsightsModule.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsTestUtils::FInsightsTestUtils(FAutomationTestBase* InTest) :
	Test(InTest)
{
#if WITH_EDITOR
	ITraceServicesModule& TraceServicesModule = FModuleManager::LoadModuleChecked<ITraceServicesModule>("TraceServices");
	TSharedPtr<TraceServices::IModuleService> ModuleService = TraceServicesModule.GetModuleService();
	ModuleService->SetModuleEnabled(FName("TraceModule_LoadTimeProfiler"), true);
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsTestUtils::AnalyzeTrace(const TCHAR* Path) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = GetSession(Path);
	if (!Session)
	{
		Test->AddError(TEXT("Session analysis failed to start."));
		return false;
	}

	return AnalyzeTrace(Session);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsTestUtils::AnalyzeTrace(TSharedPtr<const TraceServices::IAnalysisSession> Session) const
{
	if (!Session)
	{
		Test->AddError(TEXT("Session analysis failed to start."));
		return false;
	}

	UE::Insights::FStopwatch StopWatch;
	StopWatch.Start();

	double Duration = 0.0f;
	constexpr double MaxDuration = 75.0f;
	while (!Session->IsAnalysisComplete())
	{
		FPlatformProcess::Sleep(0.033f);

		if (Duration > MaxDuration)
		{
			Test->AddError(FString::Format(TEXT("Session analysis took longer than the maximum allowed time of {0} seconds. Aborting test."), { MaxDuration }));
			return false;
		}

		StopWatch.Update();
		Duration = StopWatch.GetAccumulatedTime();
	}

	StopWatch.Stop();
	Duration = StopWatch.GetAccumulatedTime();

	Test->AddInfo(FString::Format(TEXT("Session analysis took {0} seconds."), { Duration }));

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<const TraceServices::IAnalysisSession> FInsightsTestUtils::GetSession(const TCHAR* TraceFilePath) const
{
	IUnrealInsightsModule& TraceInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

	if (!FPaths::FileExists(TraceFilePath))
	{
		Test->AddError(FString::Printf(TEXT("File does not exist: %s."), TraceFilePath));
		return nullptr;
	}

	TraceInsightsModule.StartAnalysisForTraceFile(TraceFilePath);

	return TraceInsightsModule.GetAnalysisSession();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsTestUtils::FileContainsString(const FString& PathToFile, const FString& ExpectedString, double Timeout) const
{
	double StartTime = FPlatformTime::Seconds();
	while (FPlatformTime::Seconds() - StartTime < Timeout)
	{
		if (!FPaths::FileExists(PathToFile))
		{
			Test->AddInfo("Unable to find EngineTest.log at " + PathToFile);
			FPlatformProcess::Sleep(0.1f);
		}
		else
		{
			FString LogFileContents;
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			TUniquePtr<IFileHandle> FileHandle(PlatformFile.OpenRead(*PathToFile, true)); // Open the file with shared read access
			if (FileHandle)
			{
				TArray<uint8> FileData;
				FileData.SetNumUninitialized(static_cast<int32>(FileHandle->Size()));
				FileHandle->Read(FileData.GetData(), FileData.Num());
				FFileHelper::BufferToString(LogFileContents, FileData.GetData(), FileData.Num());

				if (LogFileContents.Contains(ExpectedString))
				{
					return true;
				}
			}
			FPlatformProcess::Sleep(0.1f);
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsTestUtils::IsUnrealTraceServerReady(const TCHAR* Host, int32 Port) const
{
	using namespace UE::Trace;
	TUniquePtr<FStoreClient> StoreClient = FStoreClient::Create(Host, Port);
	if (!StoreClient.IsValid())
	{
		Test->AddInfo(TEXT("Cannot connect to UTS"));
		return false;
	}
	const UE::Trace::FStoreClient::FVersion* Version = StoreClient->GetVersion();
	if (!Version)
	{
		Test->AddError(TEXT("Cannot get version of UTS"));
		return false;
	}
	uint32 MajorVersion = Version->GetMajorVersion();
	uint32 MinorVersion = Version->GetMinorVersion();
	Test->AddInfo(FString::Printf(TEXT("Connected to UTS version %u.%u"), MajorVersion, MinorVersion));
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsTestUtils::StartTracing(FTraceAuxiliary::EConnectionType ConnectionType, double Timeout) const
{
	bool bStarted = false;

	double TraceVerifyStartTime = FPlatformTime::Seconds();
	while (FPlatformTime::Seconds() - TraceVerifyStartTime < Timeout)
	{
		if (FTraceAuxiliary::IsConnected())
		{
			break;
		}

		switch (ConnectionType)
		{
		case FTraceAuxiliary::EConnectionType::Network:
			Test->AddInfo(TEXT("Connection type is Network"));
			bStarted = FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::Network, TEXT("localhost"), nullptr);
			break;
		case FTraceAuxiliary::EConnectionType::File:
			Test->AddInfo(TEXT("Connection type is File"));
			bStarted = FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::File, nullptr, nullptr);
			break;
		default:
			Test->AddWarning(TEXT("Not expected connection type"));
		}

		FPlatformProcess::Sleep(0.4f);
	}

	return bStarted;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsTestUtils::SetupUTS(double Timeout, bool bUseFork) const
{
	const FString UnrealTraceServerName = TEXT("UnrealTraceServer");

	if (FPlatformProcess::IsApplicationRunning(*UnrealTraceServerName))
	{
		Test->AddInfo(TEXT("UTS is already running"));
		return true;
	}

	const FString UTSPath = GetUTSPath();
	if (!FPaths::FileExists(UTSPath))
	{
		Test->AddError(FString::Printf(TEXT("UTS executable can't be found at '%s'"), *UTSPath));
		return false;
	}

	FString UTSParameters;
	if (bUseFork)
	{
		UTSParameters = TEXT("fork");
	}
	else
	{
		UTSParameters = TEXT("daemon");
	}
	UTSParameters += FString::Printf(TEXT(" --sponsor %d"), FPlatformProcess::GetCurrentProcessId());
	constexpr bool bLaunchDetached = true;
	constexpr bool bLaunchHidden = false;
	constexpr bool bLaunchReallyHidden = false;
	uint32 ProcessID = 0;
	const int32 PriorityModifier = 0;
	const TCHAR* OptionalWorkingDirectory = nullptr;
	void* PipeWriteChild = nullptr;
	void* PipeOutput = nullptr;
	verify(FPlatformProcess::CreatePipe(PipeOutput, PipeWriteChild));
	FProcHandle UTSHandle = FPlatformProcess::CreateProc(*UTSPath, *UTSParameters, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, &ProcessID, PriorityModifier, OptionalWorkingDirectory, PipeWriteChild, nullptr);
	if (!UTSHandle.IsValid())
	{
		Test->AddError(TEXT("The UTSHandle should be valid"));
		return false;
	}

	double StartTime = FPlatformTime::Seconds();
	while (FPlatformTime::Seconds() - StartTime < Timeout && FPlatformProcess::IsProcRunning(UTSHandle))
	{
		FPlatformProcess::Sleep(0.1f);
		if (!FPlatformProcess::IsApplicationRunning(*UnrealTraceServerName))
		{
			Test->AddInfo(TEXT("UTS not started yet"));
			continue;
		}
		if (!IsUnrealTraceServerReady())
		{
			Test->AddInfo(TEXT("UTS not ready yet"));
			continue;
		}
		Test->AddInfo(TEXT("UTS is ready"));
		return true;
	}

	Test->AddError(TEXT("UTS failed to start"));
	if (!FPlatformProcess::IsProcRunning(UTSHandle))
	{
		FString StringOutput = FPlatformProcess::ReadPipe(PipeOutput);
		int32 ExitCode = 0;
		FPlatformProcess::GetProcReturnCode(UTSHandle, &ExitCode);
		Test->AddError(FString::Printf(TEXT("UTS exitcode=%d stdout:\n%s"), ExitCode, *StringOutput));
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsTestUtils::KillUTS(double Timeout) const
{
	const FString UnrealTraceServerName = TEXT("UnrealTraceServer");

	const FString UTSPath = GetUTSPath();
	FString UTSParameters = TEXT("kill");
	constexpr bool bLaunchDetached = true;
	constexpr bool bLaunchHidden = false;
	constexpr bool bLaunchReallyHidden = false;
	uint32 ProcessID = 0;
	const int32 PriorityModifier = 0;
	const TCHAR* OptionalWorkingDirectory = nullptr;
	void* PipeWriteChild = nullptr;
	void* PipeReadChild = nullptr;
	FProcHandle UTSHandle = FPlatformProcess::CreateProc(*UTSPath, *UTSParameters, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, &ProcessID, PriorityModifier, OptionalWorkingDirectory, PipeWriteChild, PipeReadChild);
	if (!UTSHandle.IsValid())
	{
		Test->AddError(TEXT("The UTSHandle should be valid"));
		return false;
	}

	double StartTime = FPlatformTime::Seconds();
	while (FPlatformTime::Seconds() - StartTime < Timeout)
	{
		FPlatformProcess::Sleep(0.1f);
		if (!FPlatformProcess::IsApplicationRunning(*UnrealTraceServerName))
		{
			Test->AddInfo(TEXT("The UTS successfully killed"));
			return true;
		}
	}

	Test->AddError(TEXT("UTS failed to kill"));
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsTestUtils::ResetSession() const
{
	using namespace UE::Insights;
	TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();
	if (InsightsManager.IsValid())
	{
		InsightsManager->ResetSession();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<UE::Trace::FStoreClient> FInsightsTestUtils::CreateStoreClient(const TCHAR* Host, int32 Port) const
{
	TUniquePtr<UE::Trace::FStoreClient> StoreClient = UE::Trace::FStoreClient::Create(Host, Port);
	TSharedPtr<UE::Trace::FStoreClient> SharedStoreClient = TSharedPtr<UE::Trace::FStoreClient>(StoreClient.Release());
	if (!SharedStoreClient.IsValid())
	{
		Test->AddInfo(TEXT("The StoreClient shouldn't be null"));
	}
	
	return SharedStoreClient;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FInsightsTestUtils::GetValidSessionCount(TSharedPtr<UE::Trace::FStoreClient> StoreClient) const
{
	uint32 SessionCount = StoreClient->GetSessionCount();
	Test->AddInfo(FString::Printf(TEXT("The SessionCount is %d"), SessionCount));
	return SessionCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const UE::Trace::FStoreClient::FTraceInfo* FInsightsTestUtils::FindTraceInfoByName(TSharedPtr<UE::Trace::FStoreClient> StoreClient, const FString& TraceName) const
{
	uint32 SessionCount = GetValidSessionCount(StoreClient);

	for (uint32 Index = 0; Index < SessionCount; ++Index)
	{
		const UE::Trace::FStoreClient::FSessionInfo* SessionInfo = StoreClient->GetSessionInfo(Index);
		if (!SessionInfo)
		{
			continue;
		}

		uint32 TraceId = SessionInfo->GetTraceId();
		const UE::Trace::FStoreClient::FTraceInfo* TraceInfo = StoreClient->GetTraceInfoById(TraceId);
		if (TraceInfo)
		{
			const FUtf8StringView Utf8TraceNameView = TraceInfo->GetName();
			FString ActualTraceName(Utf8TraceNameView);
			if (TraceName.Contains(ActualTraceName))
			{
				Test->AddInfo(FString::Printf(TEXT("The live trace is %s"), *ActualTraceName));
				return TraceInfo;
			}
		}
	}

	Test->AddInfo(FString::Printf(TEXT("The trace with name %s does not have live status"), *TraceName));
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsTestUtils::GetMetadata(const FString& TraceFilePath, TMap<FString, FString>& OutMetadata) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = GetSession(*TraceFilePath);
	if (!Session)
	{
		Test->AddError(TEXT("Unable to retrieve metadata from an invalid session."));
		return false;
	}

	AnalyzeTrace(Session);

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
	const TraceServices::IDiagnosticsProvider* DiagnosticsProvider = TraceServices::ReadDiagnosticsProvider(*Session.Get());
	if (DiagnosticsProvider && DiagnosticsProvider->IsSessionInfoAvailable())
	{
		TraceServices::FSessionInfo SessionInfo = DiagnosticsProvider->GetSessionInfo();

		OutMetadata.Add("Platform", SessionInfo.Platform);
		OutMetadata.Add("AppName", SessionInfo.AppName);
		OutMetadata.Add("ProjectName", SessionInfo.ProjectName);
		OutMetadata.Add("Branch", SessionInfo.Branch);
		OutMetadata.Add("BuildVersion", SessionInfo.BuildVersion);
		OutMetadata.Add("EngineVersion", SessionInfo.EngineVersion);
		OutMetadata.Add("Changelist", FString::FromInt(SessionInfo.Changelist));
		OutMetadata.Add("BuildConfigurationType", LexToString(SessionInfo.ConfigurationType));
		OutMetadata.Add("BuildTargetType", LexToString(SessionInfo.TargetType));
		OutMetadata.Add("CommandLine", SessionInfo.CommandLine);
		OutMetadata.Add("VFSPaths", SessionInfo.VFSPaths);

		return true;
	}

	Test->AddError(TEXT("Metadata map is empty"));
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsTestUtils::IsTraceHasLiveStatus(const FString& TraceName, const TCHAR* Host, int32 Port) const
{
	TSharedPtr<UE::Trace::FStoreClient> StoreClient = CreateStoreClient(Host, Port);
	if (!StoreClient.IsValid())
	{
		return false;
	}

	const UE::Trace::FStoreClient::FTraceInfo* TraceInfo = FindTraceInfoByName(StoreClient, TraceName);
	if (TraceInfo)
	{
		Test->AddInfo(TEXT("Trace is live"));
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FInsightsTestUtils::GetLiveTrace(const TCHAR* Host, int32 Port) const
{
	TSharedPtr<UE::Trace::FStoreClient> StoreClient = CreateStoreClient(Host, Port);
	if (!StoreClient.IsValid())
	{
		return TEXT("");
	}

	const uint32 SessionCount = GetValidSessionCount(StoreClient);
	for (uint32 Index = 0; Index < SessionCount; ++Index)
	{
		const UE::Trace::FStoreClient::FSessionInfo* SessionInfo = StoreClient->GetSessionInfo(Index);
		if (!SessionInfo)
		{
			continue;
		}

		uint32 TraceId = SessionInfo->GetTraceId();
		const UE::Trace::FStoreClient::FTraceInfo* Info = StoreClient->GetTraceInfoById(TraceId);
		if (Info)
		{
			FString LiveTraceName(static_cast<FString>(Info->GetName()));
			LiveTraceName = LiveTraceName + TEXT(".utrace");
			return LiveTraceName;
		}
	}

	Test->AddInfo(TEXT("There isn't any live trace"));
	return TEXT("");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FInsightsTestUtils::FInsightsTestUtils::GetUTSPath() const
{
#if PLATFORM_WINDOWS
	const FString EnginePathToUTS = FString(TEXT("Engine/Binaries/Win64/UnrealTraceServer.exe"));
#endif

#if PLATFORM_MAC
	const FString EnginePathToUTS = FString(TEXT("Engine/Binaries/Mac/UnrealTraceServer"));
#endif

#if PLATFORM_LINUX
	const FString EnginePathToUTS = FString(TEXT("Engine/Binaries/Linux/UnrealTraceServer"));
#endif

	FString EntireUTSPath;
	FString RootDirectory = FPaths::RootDir();
	while (FPaths::DirectoryExists(RootDirectory))
	{
		EntireUTSPath = FPaths::Combine(*RootDirectory, EnginePathToUTS);
		if (FPaths::FileExists(EntireUTSPath))
		{
			return EntireUTSPath;
		}
		RootDirectory = FPaths::GetPath(RootDirectory);
	}

	Test->AddError(TEXT("Coudln't find UTS file"));
	return {};
}