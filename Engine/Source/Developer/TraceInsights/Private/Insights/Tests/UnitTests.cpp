// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AutomationTest.h"
#include "Insights/Tests/InsightsTestUtils.h"

using namespace UE::Insights::Automation::TestNames;

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTraceInsightsUnitTest, UnitTests.GetData(), EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

bool FTraceInsightsUnitTest::RunTest(const FString& Parameters)
{
	return true;
}
