// Copyright Epic Games, Inc. All Rights Reserved.

#include "CounterGraphSeries.h"

// TraceServices
#include "TraceServices/Model/Counters.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/TimingProfiler/GraphTracks/TimingGraphTrack.h"
#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "Insights/ViewModels/TimingTrackViewport.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FTimingGraphSeries"

namespace UE::Insights::TimingProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FCounterGraphSeries)

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCounterGraphSeries::InitFromProvider()
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = UE::Insights::FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::ICounterProvider& CountersProvider = TraceServices::ReadCounterProvider(*Session.Get());
		if (CounterId < CountersProvider.GetCounterCount())
		{
			CountersProvider.ReadCounter(CounterId,
				[this](const TraceServices::ICounter& Counter)
				{
					const TCHAR* CounterName = Counter.GetName();
					if (CounterName)
					{
						SetName(CounterName);
					}
					switch(Counter.GetDisplayHint())
					{
					case TraceServices::CounterDisplayHint_Memory: 
						Type = EGraphType::Memory; break;
					case TraceServices::CounterDisplayHint_Bandwidth: 
						Type = EGraphType::Bandwidth; break;
					case TraceServices::CounterDisplayHint_Percent: 
						Type = EGraphType::Percent; break;
					// case TraceServices::CounterDisplayHint_Time: 
					// 	Type = EGraphType::Time; break;
					default:
						Type = EGraphType::Dimensionless;
					}
					bIsFloatingPoint = Counter.IsFloatingPoint();
				});
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FCounterGraphSeries::FormatValue(double Value) const
{
	switch(Type)
	{
		case EGraphType::Time:
			return FormatTimeAuto(Value);

		case EGraphType::Memory:
			{
				const int64 MemValue = static_cast<int64>(Value);
				if (MemValue == 0)
				{
					return TEXT("0");
				}
				if (MemValue < 1024)
				{
					return FString::Printf(TEXT("%s bytes"), *FText::AsNumber(MemValue).ToString());
				}
				else
				{
					FNumberFormattingOptions FormattingOptions;
					FormattingOptions.MaximumFractionalDigits = 2;
					return FString::Printf(TEXT("%s (%s bytes)"), *FText::AsMemory(MemValue, &FormattingOptions).ToString(), *FText::AsNumber(MemValue).ToString());
				}
			}

		case EGraphType::Bandwidth:
			{
				if (Value > 1024.0)
				{
					FNumberFormattingOptions FormattingOptions;
					FormattingOptions.MaximumFractionalDigits = 2;
					return FString::Printf(TEXT("%s/s"), *FText::AsMemory(static_cast<int64>(Value), &FormattingOptions).ToString());
				}
				else
				{
					return FString::Printf(TEXT("%s bytes/s"), *FText::AsNumber(Value).ToString());
				}
			}

		case EGraphType::Percent:
			{
				FNumberFormattingOptions FormattingOptions;
				FormattingOptions.MaximumFractionalDigits = 2;
				return FText::AsPercent(Value).ToString();
			}

		default:
			if (bIsFloatingPoint)
			{
				return FString::Printf(TEXT("%g"), Value);
			}
			else
			{
				const int64 Int64Value = static_cast<int64>(Value);
				return FText::AsNumber(Int64Value).ToString();
			}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCounterGraphSeries::Update(FTimingGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport)
{
	FGraphTrackBuilder Builder(GraphTrack, *this, Viewport);

	TSharedPtr<const TraceServices::IAnalysisSession> Session = UE::Insights::FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

	const TraceServices::ICounterProvider& CounterProvider = TraceServices::ReadCounterProvider(*Session.Get());
	CounterProvider.ReadCounter(CounterId,
		[this, &Viewport, &Builder, &GraphTrack]
		(const TraceServices::ICounter& Counter)
		{
			const float TopY = 4.0f;
			const float BottomY = GraphTrack.GetHeight() - 4.0f;

			if (IsAutoZoomEnabled() && TopY < BottomY)
			{
				double MinValue =  std::numeric_limits<double>::infinity();
				double MaxValue = -std::numeric_limits<double>::infinity();

				if (Counter.IsFloatingPoint())
				{
					Counter.EnumerateFloatValues(Viewport.GetStartTime(), Viewport.GetEndTime(), true,
						[&Builder, &MinValue, &MaxValue]
						(double Time, double Value)
						{
							if (Value < MinValue)
							{
								MinValue = Value;
							}
							if (Value > MaxValue)
							{
								MaxValue = Value;
							}
						});
				}
				else
				{
					Counter.EnumerateValues(Viewport.GetStartTime(), Viewport.GetEndTime(), true,
						[&Builder, &MinValue, &MaxValue]
						(double Time, int64 IntValue)
						{
							const double Value = static_cast<double>(IntValue);
							if (Value < MinValue)
							{
								MinValue = Value;
							}
							if (Value > MaxValue)
							{
								MaxValue = Value;
							}
						});
				}

				UpdateAutoZoom(TopY, BottomY, MinValue, MaxValue);
			}

			if (Counter.IsFloatingPoint())
			{
				Counter.EnumerateFloatValues(Viewport.GetStartTime(), Viewport.GetEndTime(), true,
					[&Builder]
					(double Time, double Value)
					{
						//TODO: add a "value unit converter"
						Builder.AddEvent(Time, 0.0, Value);
					});
			}
			else
			{
				Counter.EnumerateValues(Viewport.GetStartTime(), Viewport.GetEndTime(), true,
					[&Builder]
					(double Time, int64 IntValue)
					{
						//TODO: add a "value unit converter"
						Builder.AddEvent(Time, 0.0, static_cast<double>(IntValue));
					});
			}
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
