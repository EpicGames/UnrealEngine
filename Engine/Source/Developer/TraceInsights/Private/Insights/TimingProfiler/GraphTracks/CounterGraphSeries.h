// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/TimingProfiler/GraphTracks/TimingGraphSeries.h"

namespace UE::Insights::TimingProfiler
{

class FCounterGraphSeries : public FTimingGraphSeries
{
	INSIGHTS_DECLARE_RTTI(FCounterGraphSeries, FTimingGraphSeries)

public:
	FCounterGraphSeries(uint32 InCounterId) : CounterId(InCounterId) {}
	virtual ~FCounterGraphSeries() = default;

	void InitFromProvider();
	virtual FString FormatValue(double Value) const override;
	uint32 GetCounterId() const { return CounterId; }
	virtual bool IsTimeUnit() const { return Type == EGraphType::Time; }
	virtual void Update(FTimingGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport) override;

private:
	enum class EGraphType : uint8
	{
		Dimensionless,
		Time,
		Memory,
		Percent,
		Bandwidth,
	};
	
	uint32 CounterId = 0;
	EGraphType Type = EGraphType::Dimensionless;
	bool bIsFloatingPoint = false; // for stats counters
};

} // namespace UE::Insights::TimingProfiler
