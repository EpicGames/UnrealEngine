// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameGraphSeries.h"

// TraceServices
#include "TraceServices/Model/Frames.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/TimingProfiler/GraphTracks/TimingGraphTrack.h"
#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "Insights/ViewModels/TimingTrackViewport.h"

//#include <limits>

#define LOCTEXT_NAMESPACE "UE::Insights::FTimingGraphSeries"

namespace UE::Insights::TimingProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FFrameGraphSeries)

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FFrameGraphSeries::FormatValue(double Value) const
{
	if (Value != 0.0)
	{
		return FString::Printf(TEXT("%s (%g fps)"), *FormatTimeAuto(Value), 1.0 / Value);
	}
	else
	{
		return FString(TEXT("0"));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FFrameGraphSeries> CreateGameFrameGraphSeries(const FGraphValueViewport& SharedValueViewport)
{
	TSharedRef<FFrameGraphSeries> NewSeries = MakeShared<FFrameGraphSeries>(TraceFrameType_Game);
	FFrameGraphSeries& Series = *NewSeries;

	Series.SetName(TEXT("Game Frames"));
	Series.SetDescription(TEXT("Duration of Game frames"));
	Series.SetColor(FLinearColor(0.3f, 0.3f, 1.0f, 1.0f), FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
	Series.SetBaselineY(SharedValueViewport.GetBaselineY());
	Series.SetScaleY(SharedValueViewport.GetScaleY());
	Series.EnableSharedViewport();

	return NewSeries;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FFrameGraphSeries> CreateRenderingFrameGraphSeries(const FGraphValueViewport& SharedValueViewport)
{
	TSharedRef<FFrameGraphSeries> NewSeries = MakeShared<FFrameGraphSeries>(TraceFrameType_Rendering);
	FFrameGraphSeries& Series = *NewSeries;

	Series.SetName(TEXT("Rendering Frames"));
	Series.SetDescription(TEXT("Duration of Rendering frames"));
	Series.SetColor(FLinearColor(1.0f, 0.3f, 0.3f, 1.0f), FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
	Series.SetBaselineY(SharedValueViewport.GetBaselineY());
	Series.SetScaleY(SharedValueViewport.GetScaleY());
	Series.EnableSharedViewport();

	return NewSeries;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameGraphSeries::Update(FTimingGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport)
{
	FGraphTrackBuilder Builder(GraphTrack, *this, Viewport);

	TSharedPtr<const TraceServices::IAnalysisSession> Session = UE::Insights::FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

	const TraceServices::IFrameProvider& FramesProvider = ReadFrameProvider(*Session.Get());

	const TArray64<double>& FrameStartTimes = FramesProvider.GetFrameStartTimes(FrameType);

	const int64 StartLowerBound = Algo::LowerBound(FrameStartTimes, Viewport.GetStartTime());
	const uint64 StartIndex = (StartLowerBound > 1) ? StartLowerBound - 2 : 0;

	const int64 EndLowerBound = Algo::LowerBound(FrameStartTimes, Viewport.GetEndTime());
	const uint64 EndIndex = EndLowerBound + 1;

	FramesProvider.EnumerateFrames(FrameType, StartIndex, EndIndex,
		[&Builder]
		(const TraceServices::FFrame& Frame)
		{
			//TODO: add a "frame converter" (i.e. to fps, milliseconds or seconds)
			const double Duration = Frame.EndTime - Frame.StartTime;
			Builder.AddEvent(Frame.StartTime, Duration, Duration);
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
