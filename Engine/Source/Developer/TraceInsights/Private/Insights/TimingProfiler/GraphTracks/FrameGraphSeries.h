// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "ProfilingDebugging/MiscTrace.h"

// TraceInsights
#include "Insights/TimingProfiler/GraphTracks/TimingGraphSeries.h"

namespace UE::Insights::TimingProfiler
{

class FTimingGraphTrack;

class FFrameGraphSeries : public FTimingGraphSeries
{
	INSIGHTS_DECLARE_RTTI(FFrameGraphSeries, FTimingGraphSeries)

public:
	FFrameGraphSeries(ETraceFrameType InFrameType) : FrameType(InFrameType) {}
	virtual ~FFrameGraphSeries() = default;

	ETraceFrameType GetFrameType() const { return FrameType; }
	virtual bool IsTimeUnit() const override { return true; }

	virtual FString FormatValue(double Value) const override;
	virtual void Update(FTimingGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport) override;

private:
	ETraceFrameType FrameType = ETraceFrameType::TraceFrameType_Game;
};

TSharedRef<FFrameGraphSeries> CreateGameFrameGraphSeries(const FGraphValueViewport& SharedValueViewport);
TSharedRef<FFrameGraphSeries> CreateRenderingFrameGraphSeries(const FGraphValueViewport& SharedValueViewport);

} // namespace UE::Insights::TimingProfiler
