// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameStatsTimerGraphSeries.h"

#include "Algo/BinarySearch.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/Frames.h"
#include "TraceServices/Model/StackSamples.h"
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/TimingProfiler/GraphTracks/TimingGraphTrack.h"
#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "Insights/ViewModels/TimingTrackViewport.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FTimingGraphSeries"

namespace UE::Insights::TimingProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FFrameStatsTimerGraphSeries)

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FFrameStatsTimerGraphSeries::FormatValue(double Value) const
{
	return FormatTimeAuto(Value);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameStatsTimerGraphSeries::Update(FTimingGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport)
{
	FGraphTrackBuilder Builder(GraphTrack, *this, Viewport);

	TSharedPtr<const TraceServices::IAnalysisSession> Session = UE::Insights::FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	TSet<uint32> VisibleTimelines;
	GraphTrack.GetVisibleTimelineIndexes(VisibleTimelines);

	TSet<uint32> VisibleCpuSamplingThreads;
	GraphTrack.GetVisibleCpuSamplingThreads(VisibleCpuSamplingThreads);

	double SessionDuration = 0.0;
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		SessionDuration = Session->GetDurationSeconds();
	}

	if (CachedSessionDuration != SessionDuration ||
		CachedTimelineCount != VisibleTimelines.Num() ||
		CachedCpuSamplingTimelineCount != VisibleCpuSamplingThreads.Num())
	{
		CachedSessionDuration = SessionDuration;
		CachedTimelineCount = VisibleTimelines.Num();
		CachedCpuSamplingTimelineCount = VisibleCpuSamplingThreads.Num();
		FrameStatsCachedEvents.Empty();

		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::IFrameProvider& FramesProvider = ReadFrameProvider(*Session.Get());

		uint64 FrameCount = FramesProvider.GetFrameCount(FrameType);
		if (FrameCount == 0)
		{
			return;
		}

		FramesProvider.EnumerateFrames(FrameType, 0ull, FrameCount,
			[this]
			(const TraceServices::FFrame& Frame)
			{
				FFrameStatsCachedEvent Event;
				Event.FrameStartTime = Frame.StartTime;
				Event.FrameEndTime = Frame.EndTime;
				Event.Duration.store(0.0f);
				FrameStatsCachedEvents.Add(Event);
			});

		// CPU & GPU timelines
		if (VisibleTimelines.Num() > 0)
		{
			FFrameStatsHelper::ComputeFrameStatsForTimer(FrameStatsCachedEvents, TimerId, VisibleTimelines);
		}

		// CPU Sampling timelines
		if (VisibleCpuSamplingThreads.Num() > 0)
		{
			const TraceServices::IStackSamplesProvider* StackSamplesProvider = TraceServices::ReadStackSamplesProvider(*Session.Get());
			if (StackSamplesProvider)
			{
				for (uint32 ThreadId : VisibleCpuSamplingThreads)
				{
					TraceServices::FProviderReadScopeLock _(*StackSamplesProvider);
					const TraceServices::ITimingProfilerProvider::Timeline& Timeline = *StackSamplesProvider->GetTimeline(ThreadId);
					FFrameStatsHelper::ComputeFrameStatsForTimer(FrameStatsCachedEvents, TimerId, Timeline);
				}
			}
		}
	}

	if (FrameStatsCachedEvents.Num() > 0)
	{
		int32 StartIndex = Algo::UpperBoundBy(FrameStatsCachedEvents, Viewport.GetStartTime(), &FFrameStatsCachedEvent::FrameStartTime);
		if (StartIndex > 0)
		{
			StartIndex--;
		}
		int32 EndIndex = Algo::UpperBoundBy(FrameStatsCachedEvents, Viewport.GetEndTime(), &FFrameStatsCachedEvent::FrameStartTime);
		if (EndIndex < FrameStatsCachedEvents.Num())
		{
			EndIndex++;
		}
		for (int32 Index = StartIndex; Index < EndIndex; ++Index)
		{
			const FFrameStatsCachedEvent& Entry = FrameStatsCachedEvents[Index];
			Builder.AddEvent(Entry.FrameStartTime, Entry.Duration.load(), Entry.Duration.load());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
