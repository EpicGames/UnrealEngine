// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/TimingProfiler/GraphTracks/TimingGraphSeries.h"
#include "Insights/TimingProfiler/ViewModels/FrameStatsHelper.h"

namespace UE::Insights::TimingProfiler
{

class FFrameStatsTimerGraphSeries : public FTimingGraphSeries
{
	INSIGHTS_DECLARE_RTTI(FFrameStatsTimerGraphSeries, FTimingGraphSeries)

public:
	FFrameStatsTimerGraphSeries(uint32 InTimerId, ETraceFrameType InFrameType) : TimerId(InTimerId), FrameType(InFrameType) {}
	virtual ~FFrameStatsTimerGraphSeries() = default;

	uint32 GetTimerId() const { return TimerId; }
	ETraceFrameType GetFrameType() const { return FrameType; }
	virtual bool IsTimer(uint32 InTimerId) const override { return TimerId == InTimerId; }
	virtual bool IsTimeUnit() const override { return true; }

	virtual FString FormatValue(double Value) const override;
	virtual void Update(FTimingGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport) override;

private:
	uint32 TimerId = 0;
	ETraceFrameType FrameType = ETraceFrameType::TraceFrameType_Game;
	double CachedSessionDuration = 0.0;
	uint32 CachedTimelineCount = 0;
	uint32 CachedCpuSamplingTimelineCount = 0;
	TArray<FFrameStatsCachedEvent> FrameStatsCachedEvents;
};

} // namespace UE::Insights::TimingProfiler
