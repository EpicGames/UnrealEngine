// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimerGraphSeries.h"

#include "Algo/BinarySearch.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/StackSamples.h"
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/TimingProfiler/GraphTracks/TimingGraphTrack.h"
#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "Insights/ViewModels/TimingTrackViewport.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FTimingGraphSeries"

namespace UE::Insights::TimingProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FTimerGraphSeries)

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FTimerGraphSeries::FormatValue(double Value) const
{
	return FormatTimeAuto(Value);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimerGraphSeries::Update(FTimingGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport)
{
	FGraphTrackBuilder Builder(GraphTrack, *this, Viewport);

	TSharedPtr<const TraceServices::IAnalysisSession> Session = UE::Insights::FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	TSet<uint32> VisibleTimelines;
	GraphTrack.GetVisibleTimelineIndexes(VisibleTimelines);

	TSet<uint32> VisibleCpuSamplingThreads;
	GraphTrack.GetVisibleCpuSamplingThreads(VisibleCpuSamplingThreads);

	double SessionDuration = 0.0;
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		SessionDuration = Session->GetDurationSeconds();
	}

	if (CachedSessionDuration != SessionDuration ||
		CachedTimelineCount != VisibleTimelines.Num() ||
		CachedCpuSamplingTimelineCount != VisibleCpuSamplingThreads.Num())
	{
		CachedSessionDuration = SessionDuration;
		CachedTimelineCount = VisibleTimelines.Num();
		CachedCpuSamplingTimelineCount = VisibleCpuSamplingThreads.Num();
		CachedEvents.Empty();

		uint32 NumTimelinesContainingEvent = 0;

		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();

			auto TimelineCallback =
				[this, SessionDuration, &GraphTrack, &TimerReader, &Viewport, &NumTimelinesContainingEvent]
				(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
				{
					TArray<TArray<FSimpleTimingEvent>> Events;

					TraceServices::ITimingProfilerTimeline::EnumerateAsyncParams Params;
					Params.IntervalStart = 0;
					Params.IntervalEnd = SessionDuration;
					Params.Resolution = 0.0;
					Params.SetupCallback =
						[&Events]
						(uint32 NumTasks)
						{
							Events.AddDefaulted(NumTasks);
						};
					Params.EventRangeCallback =
						[this, &Events, &TimerReader]
						(double StartTime, double EndTime, uint32 Depth, const TraceServices::FTimingProfilerEvent& Event, uint32 TaskIndex)
						{
							const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(Event.TimerIndex);
							if (ensure(Timer != nullptr))
							{
								if (Timer->Id == TimerId)
								{
									const double Duration = EndTime - StartTime;
									Events[TaskIndex].Add({ StartTime, Duration });
								}
							}
							return TraceServices::EEventEnumerate::Continue;
						};
					Timeline.EnumerateEventsDownSampledAsync(Params);

					int32 NumOfCachedEvents = CachedEvents.Num();
					for (auto& Array : Events) //-V1078
					{
						for (auto& Event : Array)
						{
							CachedEvents.Add(Event);
						}
						Array.Empty();
					}

					if (NumOfCachedEvents != CachedEvents.Num())
					{
						++NumTimelinesContainingEvent;
					}
				};

			// CPU & GPU timelines
			for (uint32 TimelineIndex : VisibleTimelines)
			{
				TimingProfilerProvider->ReadTimeline(TimelineIndex, TimelineCallback);
			}

			// CPU Sampling timelines
			if (VisibleCpuSamplingThreads.Num() > 0)
			{
				const TraceServices::IStackSamplesProvider* StackSamplesProvider = TraceServices::ReadStackSamplesProvider(*Session.Get());
				if (StackSamplesProvider)
				{
					for (uint32 ThreadId : VisibleCpuSamplingThreads)
					{
						TraceServices::FProviderReadScopeLock _(*StackSamplesProvider);
						const TraceServices::ITimingProfilerProvider::Timeline& Timeline = *StackSamplesProvider->GetTimeline(ThreadId);
						TimelineCallback(Timeline);
					}
				}
			}
		}

		// If events come from multiple timelines, we have to sort the whole thing.
		// If they come from a single timeline, they are already sorted.
		if (NumTimelinesContainingEvent > 1)
		{
			CachedEvents.Sort(&CompareEventsByStartTime);
		}
	}

	if (CachedEvents.Num() > 0)
	{
		int32 StartIndex = Algo::UpperBoundBy(CachedEvents, Viewport.GetStartTime(), &FSimpleTimingEvent::StartTime);
		if (StartIndex > 0)
		{
			StartIndex--;
		}
		int32 EndIndex = Algo::UpperBoundBy(CachedEvents, Viewport.GetEndTime(), &FSimpleTimingEvent::StartTime);
		if (EndIndex < CachedEvents.Num())
		{
			EndIndex++;
		}
		for (int32 Index = StartIndex; Index < EndIndex; ++Index)
		{
			const FSimpleTimingEvent& Event = CachedEvents[Index];
			Builder.AddEvent(Event.StartTime, Event.Duration, Event.Duration);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
