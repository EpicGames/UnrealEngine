// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/TimingProfiler/GraphTracks/TimingGraphSeries.h"

namespace UE::Insights::TimingProfiler
{

class FTimerGraphSeries : public FTimingGraphSeries
{
	INSIGHTS_DECLARE_RTTI(FTimerGraphSeries, FTimingGraphSeries)

private:
	struct FSimpleTimingEvent
	{
		double StartTime;
		double Duration;
	};

public:
	FTimerGraphSeries(uint32 InTimerId) : TimerId(InTimerId) {}
	virtual ~FTimerGraphSeries() = default;

	uint32 GetTimerId() const { return TimerId; }
	virtual bool IsTimer(uint32 InTimerId) const override { return TimerId == InTimerId; }
	virtual bool IsTimeUnit() const override { return true; }

	virtual FString FormatValue(double Value) const override;
	virtual void Update(FTimingGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport) override;

private:
	static bool CompareEventsByStartTime(const FSimpleTimingEvent& EventA, const FSimpleTimingEvent& EventB)
	{
		return EventA.StartTime < EventB.StartTime;
	}

private:
	uint32 TimerId = 0;
	double CachedSessionDuration = 0.0;
	uint32 CachedTimelineCount = 0;
	uint32 CachedCpuSamplingTimelineCount = 0;
	TArray<FSimpleTimingEvent> CachedEvents;
};

} // namespace UE::Insights::TimingProfiler
