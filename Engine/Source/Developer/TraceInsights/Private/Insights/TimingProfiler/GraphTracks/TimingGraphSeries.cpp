// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimingGraphSeries.h"

#define LOCTEXT_NAMESPACE "UE::Insights::FTimingGraphSeries"

namespace UE::Insights::TimingProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FTimingGraphSeries)

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingGraphSeries::SetVisibility(bool bOnOff)
{
	FGraphSeries::SetVisibility(bOnOff);

	VisibilityChangedDelegate.Broadcast(bOnOff);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
