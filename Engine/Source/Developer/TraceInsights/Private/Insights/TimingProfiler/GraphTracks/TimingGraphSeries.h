// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/DelegateCombinations.h"

// TraceInsights
#include "Insights/ViewModels/GraphSeries.h"

class FTimingTrackViewport;

namespace UE::Insights::TimingProfiler
{

class FTimingGraphTrack;

/** The delegate to be invoked when a series visibility is changed. */
DECLARE_MULTICAST_DELEGATE_OneParam(FSeriesVisibilityChangedDelegate, bool bOnOff);

class FTimingGraphSeries : public FGraphSeries
{
	INSIGHTS_DECLARE_RTTI(FTimingGraphSeries, FGraphSeries)

public:
	FTimingGraphSeries() {}
	virtual ~FTimingGraphSeries() = default;

	virtual bool IsTimer(uint32 TimerId) const { return false; }
	virtual bool IsTimeUnit() const { return false; }

	virtual void SetVisibility(bool bOnOff) override;
	//virtual FString FormatValue(double Value) const override;
	virtual void Update(FTimingGraphTrack& GraphTrack, const FTimingTrackViewport& Viewport) = 0;

public:
	FSeriesVisibilityChangedDelegate VisibilityChangedDelegate;
};

} // namespace UE::Insights::TimingProfiler
