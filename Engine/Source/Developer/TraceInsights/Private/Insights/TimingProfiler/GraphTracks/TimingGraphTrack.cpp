// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimingGraphTrack.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/TimingProfiler/GraphTracks/CounterGraphSeries.h"
#include "Insights/TimingProfiler/GraphTracks/FrameGraphSeries.h"
#include "Insights/TimingProfiler/GraphTracks/FrameStatsTimerGraphSeries.h"
#include "Insights/TimingProfiler/GraphTracks/TimerGraphSeries.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Tracks/ThreadTimingTrack.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/ViewModels/AxisViewportDouble.h"
#include "Insights/Widgets/STimingView.h"

#include <limits>

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::FTimingGraphTrack"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimingGraphTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FTimingGraphTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingGraphTrack::FTimingGraphTrack(TSharedPtr<STimingView> InTimingView)
	: FGraphTrack()
	, TimingView(InTimingView)
{
	LoadDefaultSettings();

	// Add non editable options.
	EnabledOptions |=
		EGraphOptions::ShowBaseline |
		EGraphOptions::ShowThresholds |
		EGraphOptions::ShowVerticalAxisGrid |
		EGraphOptions::ShowHeader;

	bNotifyTimersOnDestruction = InTimingView->GetName() == FInsightsManagerTabs::TimingProfilerTabId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingGraphTrack::~FTimingGraphTrack()
{
	if (OnTrackVisibilityChangedHandle.IsValid())
	{
		UnregisterTimingViewCallbacks();
	}

	if (GameFrameSeriesVisibilityHandle.IsValid())
	{
		TSharedPtr<FTimingGraphSeries> GameFramesSeries = GetFrameSeries(ETraceFrameType::TraceFrameType_Game);
		if (GameFramesSeries.IsValid())
		{
			GameFramesSeries->VisibilityChangedDelegate.Remove(GameFrameSeriesVisibilityHandle);
		}
	}

	if (RenderingFrameSeriesVisibilityHandle.IsValid())
	{
		TSharedPtr<FTimingGraphSeries> RenderingFramesSeries = GetFrameSeries(ETraceFrameType::TraceFrameType_Rendering);
		if (RenderingFramesSeries.IsValid())
		{
			RenderingFramesSeries->VisibilityChangedDelegate.Remove(RenderingFrameSeriesVisibilityHandle);
		}
	}

	if (bNotifyTimersOnDestruction)
	{
		TSharedPtr<STimingProfilerWindow> ProfilerWindow = FTimingProfilerManager::Get()->GetProfilerWindow();
		if (ProfilerWindow.IsValid())
		{
			TArray<uint32> TimerIds;
			for (TSharedPtr<FGraphSeries>& Series : AllSeries)
			{
				if (Series->Is<FTimerGraphSeries>())
				{
					TimerIds.Add(Series->As<FTimerGraphSeries>().GetTimerId());
				}
				else if (Series->Is<FFrameStatsTimerGraphSeries>())
				{
					TimerIds.Add(Series->As<FFrameStatsTimerGraphSeries>().GetTimerId());
				}
			}
			AllSeries.Reset();
			for (uint32 TimerId : TimerIds)
			{
				ProfilerWindow->OnTimerAddedToGraphsChanged(TimerId);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingGraphTrack::RegisterTimingViewCallbacks()
{
	TSharedPtr<STimingView> TimingViewPtr = TimingView.Pin();
	if (TimingViewPtr.IsValid())
	{
		auto OnTrackAddedRemovedLambda =
			[this](const TSharedPtr<const FBaseTimingTrack> Track)
			{
				if (Track->Is<FThreadTimingTrack>())
				{
					// If there are more series than the default frame series.
					if (AllSeries.Num() > ETraceFrameType::TraceFrameType_Count)
					{
						SetDirtyFlag();
					}
				}
			};

		OnTrackAddedHandle = TimingViewPtr->OnTrackAdded().AddLambda(OnTrackAddedRemovedLambda);
		OnTrackRemovedHandle = TimingViewPtr->OnTrackRemoved().AddLambda(OnTrackAddedRemovedLambda);

		OnTrackVisibilityChangedHandle = TimingViewPtr->OnTrackVisibilityChanged().AddLambda(
			[this]()
			{
				// If there are more series than the default frame series.
				if (AllSeries.Num() > ETraceFrameType::TraceFrameType_Count)
				{
					SetDirtyFlag();
				}
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingGraphTrack::UnregisterTimingViewCallbacks()
{
	TSharedPtr<STimingView> TimingViewPtr = TimingView.Pin();
	if (TimingView.IsValid())
	{
		TimingViewPtr->OnTrackAdded().Remove(OnTrackAddedHandle);
		TimingViewPtr->OnTrackRemoved().Remove(OnTrackRemovedHandle);
		TimingViewPtr->OnTrackVisibilityChanged().Remove(OnTrackVisibilityChangedHandle);
	}

	OnTrackAddedHandle.Reset();
	OnTrackRemovedHandle.Reset();
	OnTrackVisibilityChangedHandle.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingGraphTrack::Update(const ITimingTrackUpdateContext& Context)
{
	FGraphTrack::Update(Context);

	if (!OnTrackVisibilityChangedHandle.IsValid())
	{
		RegisterTimingViewCallbacks();
	}

	const bool bIsEntireGraphTrackDirty = IsDirty() || Context.GetViewport().IsHorizontalViewportDirty();
	bool bNeedsUpdate = bIsEntireGraphTrackDirty;

	if (!bNeedsUpdate)
	{
		for (TSharedPtr<FGraphSeries>& Series : AllSeries)
		{
			if (Series->IsVisible() && Series->IsDirty())
			{
				// At least one series is dirty.
				bNeedsUpdate = true;
				break;
			}
		}
	}

	if (bNeedsUpdate)
	{
		ClearDirtyFlag();

		NumAddedEvents = 0;

		const FTimingTrackViewport& Viewport = Context.GetViewport();

		for (TSharedPtr<FGraphSeries>& Series : AllSeries)
		{
			if (Series->IsVisible() &&
				(bIsEntireGraphTrackDirty || Series->IsDirty()) &&
				Series->Is<FTimingGraphSeries>())
			{
				// Clear the flag before updating, because the update itself may further need to set the series as dirty.
				Series->ClearDirtyFlag();

				Series->As<FTimingGraphSeries>().Update(*this, Viewport);
			}
		}

		UpdateStats();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Frame Series
////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingGraphTrack::AddDefaultFrameSeries()
{
	TSharedPtr<STimingView> TimingViewPtr = TimingView.Pin();

	TSharedRef<FFrameGraphSeries> GameFramesSeries = CreateGameFrameGraphSeries(SharedValueViewport);
	TSharedRef<FFrameGraphSeries> RenderingFramesSeries = CreateRenderingFrameGraphSeries(SharedValueViewport);

	if (TimingViewPtr.IsValid() &&
		TimingViewPtr->GetName() == FInsightsManagerTabs::TimingProfilerTabId)
	{
		const FInsightsSettings& Settings = UE::Insights::FInsightsManager::Get()->GetSettings();

		GameFramesSeries->SetVisibility(Settings.GetTimingViewMainGraphShowGameFrames());
		GameFrameSeriesVisibilityHandle = GameFramesSeries->VisibilityChangedDelegate.AddLambda(
			[](bool bOnOff)
			{
				FInsightsSettings& Settings = UE::Insights::FInsightsManager::Get()->GetSettings();
				Settings.SetAndSaveTimingViewMainGraphShowGameFrames(bOnOff);
			});

		RenderingFramesSeries->SetVisibility(Settings.GetTimingViewMainGraphShowRenderingFrames());
		RenderingFrameSeriesVisibilityHandle = RenderingFramesSeries->VisibilityChangedDelegate.AddLambda(
			[](bool bOnOff)
			{
				FInsightsSettings& Settings = UE::Insights::FInsightsManager::Get()->GetSettings();
				Settings.SetAndSaveTimingViewMainGraphShowRenderingFrames(bOnOff);
			});
	}

	AllSeries.Add(GameFramesSeries.ToSharedPtr());
	AllSeries.Add(RenderingFramesSeries.ToSharedPtr());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTimingGraphSeries> FTimingGraphTrack::GetFrameSeries(ETraceFrameType FrameType)
{
	TSharedPtr<FGraphSeries>* Ptr = AllSeries.FindByPredicate(
		[FrameType]
		(const TSharedPtr<FGraphSeries>& Series)
		{
			return Series->Is<FFrameGraphSeries>() &&
				   Series->As<FFrameGraphSeries>().GetFrameType() == FrameType;
		});
	return (Ptr != nullptr) ? StaticCastSharedPtr<FTimingGraphSeries>(*Ptr) : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Timer Series
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTimingGraphSeries> FTimingGraphTrack::GetTimerSeries(uint32 TimerId)
{
	TSharedPtr<FGraphSeries>* Ptr = AllSeries.FindByPredicate(
		[TimerId]
		(const TSharedPtr<FGraphSeries>& Series)
		{
			return Series->Is<FTimerGraphSeries>() &&
				   Series->As<FTimerGraphSeries>().GetTimerId() == TimerId;
		});
	return (Ptr != nullptr) ? StaticCastSharedPtr<FTimingGraphSeries>(*Ptr) : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTimingGraphSeries> FTimingGraphTrack::AddTimerSeries(uint32 TimerId, FLinearColor Color)
{
	TSharedRef<FTimerGraphSeries> NewSeries = MakeShared<FTimerGraphSeries>(TimerId);
	FTimerGraphSeries& Series = *NewSeries;

	Series.SetName(TEXT("<Timer>"));
	Series.SetDescription(TEXT("Timer series"));

	const FLinearColor BorderColor(Color.R + 0.4f, Color.G + 0.4f, Color.B + 0.4f, 1.0f);
	Series.SetColor(Color, BorderColor);

	// Use shared viewport.
	Series.SetBaselineY(SharedValueViewport.GetBaselineY());
	Series.SetScaleY(SharedValueViewport.GetScaleY());
	Series.EnableSharedViewport();

	AllSeries.Add(NewSeries);
	return NewSeries;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingGraphTrack::RemoveTimerSeries(uint32 TimerId)
{
	AllSeries.RemoveAll(
		[TimerId]
		(const TSharedPtr<FGraphSeries>& Series)
		{
			return Series->Is<FTimerGraphSeries>() &&
				   Series->As<FTimerGraphSeries>().GetTimerId() == TimerId;
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Frame Stats Timer Series
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTimingGraphSeries> FTimingGraphTrack::GetFrameStatsTimerSeries(uint32 TimerId, ETraceFrameType FrameType)
{
	TSharedPtr<FGraphSeries>* Ptr = AllSeries.FindByPredicate(
		[TimerId, FrameType]
		(const TSharedPtr<FGraphSeries>& Series)
		{
			return Series->Is<FFrameStatsTimerGraphSeries>() &&
				   Series->As<FFrameStatsTimerGraphSeries>().GetTimerId() == TimerId &&
				   Series->As<FFrameStatsTimerGraphSeries>().GetFrameType() == FrameType;
		});
	return (Ptr != nullptr) ? StaticCastSharedPtr<FTimingGraphSeries>(*Ptr) : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTimingGraphSeries> FTimingGraphTrack::AddFrameStatsTimerSeries(uint32 TimerId, ETraceFrameType FrameType, FLinearColor Color)
{
	TSharedRef<FFrameStatsTimerGraphSeries> NewSeries = MakeShared<FFrameStatsTimerGraphSeries>(TimerId, FrameType);
	FFrameStatsTimerGraphSeries& Series = *NewSeries;

	Series.SetName(TEXT("<Frame Stats Timer>"));
	Series.SetDescription(TEXT("Frame Stats Timer series"));

	const FLinearColor BorderColor(Color.R + 0.4f, Color.G + 0.4f, Color.B + 0.4f, 1.0f);
	Series.SetColor(Color, BorderColor);

	// Use shared viewport.
	Series.SetBaselineY(SharedValueViewport.GetBaselineY());
	Series.SetScaleY(SharedValueViewport.GetScaleY());
	Series.EnableSharedViewport();

	AllSeries.Add(NewSeries);
	return NewSeries;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingGraphTrack::RemoveFrameStatsTimerSeries(uint32 TimerId, ETraceFrameType FrameType)
{
	AllSeries.RemoveAll(
		[TimerId, FrameType]
		(const TSharedPtr<FGraphSeries>& Series)
		{
			return Series->Is<FFrameStatsTimerGraphSeries>() &&
				   Series->As<FFrameStatsTimerGraphSeries>().GetTimerId() == TimerId &&
				   Series->As<FFrameStatsTimerGraphSeries>().GetFrameType() == FrameType;
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Counter Series
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTimingGraphSeries> FTimingGraphTrack::GetStatsCounterSeries(uint32 CounterId)
{
	TSharedPtr<FGraphSeries>* Ptr = AllSeries.FindByPredicate(
		[CounterId]
		(const TSharedPtr<FGraphSeries>& Series)
		{
			return Series->Is<FCounterGraphSeries>() &&
				   Series->As<FCounterGraphSeries>().GetCounterId() == CounterId;
		});
	return (Ptr != nullptr) ? StaticCastSharedPtr<FTimingGraphSeries>(*Ptr) : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTimingGraphSeries> FTimingGraphTrack::AddStatsCounterSeries(uint32 CounterId, FLinearColor Color)
{
	TSharedRef<FCounterGraphSeries> NewSeries = MakeShared<FCounterGraphSeries>(CounterId);
	FCounterGraphSeries& Series = *NewSeries;

	Series.SetName(TEXT("<StatsCounter>"));
	Series.SetDescription(TEXT("Stats counter series"));

	FLinearColor BorderColor(Color.R + 0.4f, Color.G + 0.4f, Color.B + 0.4f, 1.0f);
	Series.SetColor(Color, BorderColor);

	Series.SetBaselineY(GetHeight() - 1.0f);
	Series.SetScaleY(1.0);
	Series.EnableAutoZoom();

	Series.InitFromProvider();

	AllSeries.Add(NewSeries);
	return NewSeries;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingGraphTrack::RemoveStatsCounterSeries(uint32 CounterId)
{
	AllSeries.RemoveAll(
		[CounterId]
		(const TSharedPtr<FGraphSeries>& Series)
		{
			return Series->Is<FCounterGraphSeries>() &&
				   Series->As<FCounterGraphSeries>().GetCounterId() == CounterId;
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FThreadTimingSharedState> FTimingGraphTrack::GetThreadTimingSharedState() const
{
	TSharedPtr<STimingView> TimingViewPtr = TimingView.Pin();
	if (!TimingViewPtr.IsValid())
	{
		return nullptr;
	}

	return TimingViewPtr->GetThreadTimingSharedState();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingGraphTrack::GetVisibleTimelineIndexes(TSet<uint32>& TimelineIndexes)
{
	TSharedPtr<FThreadTimingSharedState> ThreadSharedState = GetThreadTimingSharedState();
	if (ThreadSharedState)
	{
		ThreadSharedState->GetVisibleTimelineIndexes(TimelineIndexes);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingGraphTrack::GetVisibleCpuSamplingThreads(TSet<uint32>& Threads)
{
	TSharedPtr<FThreadTimingSharedState> ThreadSharedState = GetThreadTimingSharedState();
	if (ThreadSharedState)
	{
		ThreadSharedState->GetVisibleCpuSamplingThreads(Threads);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingGraphTrack::ContextMenu_ToggleOption_Execute(EGraphOptions Option)
{
	FGraphTrack::ContextMenu_ToggleOption_Execute(Option);

	TSharedPtr<STimingView> TimingViewPtr = TimingView.Pin();
	if (!TimingViewPtr.IsValid())
	{
		return;
	}
	if (TimingViewPtr->GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	FInsightsSettings& Settings = UE::Insights::FInsightsManager::Get()->GetSettings();
	if (EnumHasAnyFlags(Option, EGraphOptions::ShowPoints))
	{
		Settings.SetAndSaveTimingViewMainGraphShowPoints(EnumHasAnyFlags(EnabledOptions, EGraphOptions::ShowPoints));
	}
	if (EnumHasAnyFlags(Option, EGraphOptions::ShowPointsWithBorder))
	{
		Settings.SetAndSaveTimingViewMainGraphShowPointsWithBorder(EnumHasAnyFlags(EnabledOptions, EGraphOptions::ShowPointsWithBorder));
	}
	if (EnumHasAnyFlags(Option, EGraphOptions::ShowLines))
	{
		Settings.SetAndSaveTimingViewMainGraphShowConnectedLines(EnumHasAnyFlags(EnabledOptions, EGraphOptions::ShowLines));
	}
	if (EnumHasAnyFlags(Option, EGraphOptions::ShowPolygon))
	{
		Settings.SetAndTimingViewMainGraphShowPolygons(EnumHasAnyFlags(EnabledOptions, EGraphOptions::ShowPolygon));
	}
	if (EnumHasAnyFlags(Option, EGraphOptions::UseEventDuration))
	{
		Settings.SetAndSaveTimingViewMainGraphShowEventDuration(EnumHasAnyFlags(EnabledOptions, EGraphOptions::UseEventDuration));
	}
	if (EnumHasAnyFlags(Option, EGraphOptions::ShowBars))
	{
		Settings.SetAndSaveTimingViewMainGraphShowBars(EnumHasAnyFlags(EnabledOptions, EGraphOptions::ShowBars));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingGraphTrack::LoadDefaultSettings()
{
	TSharedPtr<STimingView> TimingViewPtr = TimingView.Pin();
	if (TimingViewPtr.IsValid() && TimingViewPtr->GetName() == FInsightsManagerTabs::TimingProfilerTabId)
	{
		const FInsightsSettings& Settings = UE::Insights::FInsightsManager::Get()->GetSettings();
		if (Settings.GetTimingViewMainGraphShowPoints())
		{
			EnabledOptions |= EGraphOptions::ShowPoints;
		}
		if (Settings.GetTimingViewMainGraphShowPointsWithBorder())
		{
			EnabledOptions |= EGraphOptions::ShowPointsWithBorder;
		}
		if (Settings.GetTimingViewMainGraphShowConnectedLines())
		{
			EnabledOptions |= EGraphOptions::ShowLines;
		}
		if (Settings.GetTimingViewMainGraphShowPolygons())
		{
			EnabledOptions |= EGraphOptions::ShowPolygon;
		}
		if (Settings.GetTimingViewMainGraphShowEventDuration())
		{
			EnabledOptions |= EGraphOptions::UseEventDuration;
		}
		if (Settings.GetTimingViewMainGraphShowBars())
		{
			EnabledOptions |= EGraphOptions::ShowBars;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingGraphTrack::DrawVerticalAxisGrid(const ITimingTrackDrawContext& Context) const
{
	FTimingGraphSeries* FirstTimeUnitSeries = nullptr;
	for (const TSharedPtr<FGraphSeries>& Series : AllSeries)
	{
		if (Series->IsVisible() &&
			Series->Is<FTimingGraphSeries>() &&
			Series->As<FTimingGraphSeries>().IsTimeUnit())
		{
			FirstTimeUnitSeries = &Series->As<FTimingGraphSeries>();
			break;
		}
	}
	if (!FirstTimeUnitSeries)
	{
		return;
	}

	FAxisViewportDouble ViewportY;
	ViewportY.SetSize(GetHeight());
	ViewportY.SetScaleLimits(std::numeric_limits<double>::min(), std::numeric_limits<double>::max());
	ViewportY.SetScale(SharedValueViewport.GetScaleY());
	ViewportY.ScrollAtPos(static_cast<float>(SharedValueViewport.GetBaselineY()) - GetHeight());

	const float ViewWidth = Context.GetViewport().GetWidth();
	const float RoundedViewHeight = FMath::RoundToFloat(GetHeight());

	const float X0 = ViewWidth - 12.0f; // let some space for the vertical scrollbar
	const float Y0 = GetPosY();

	constexpr float MinDY = 32.0f; // min vertical distance between horizontal grid lines
	constexpr float TextH = 14.0f; // label height

	UE::Insights::FDrawContext& DrawContext = Context.GetDrawContext();
	const FSlateBrush* Brush = Context.GetHelper().GetWhiteBrush();
	//const FSlateFontInfo& Font = Context.GetHelper().GetEventFont();
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const float FontScale = DrawContext.Geometry.Scale;

	const double TopValue = ViewportY.GetValueAtOffset(RoundedViewHeight);
	const double GridValue = ViewportY.GetValueAtOffset(MinDY);
	const double BottomValue = ViewportY.GetValueAtOffset(0.0f);
	const double Delta = GridValue - BottomValue;

	if (Delta > 0.0)
	{
		const double Thresholds[] =
		{
			1.0e-9,	// 1ns
			1.0e-8,	// 10ns
			1.0e-7,	// 100ns
			1.0e-6,	// 1us
			1.0e-5,	// 10us
			0.0001,	// 100us
			0.001,	// 1ms
			0.01,	// 10ms
			0.1,	// 100ms
			1.0,	// 1s
			10.0,	// 10s
			60.0,	// 1m
			600.0,	// 10m
			3600.0,	// 1h
			36000.0,// 10h
			86400.0	// 1d
		};
		constexpr int32 NumThresholds = sizeof(Thresholds) / sizeof(double);
		int32 Index = static_cast<int32>(Algo::LowerBound(Thresholds, Delta));
		if (Index > 0)
		{
			Index--;
		}
		double TickUnit = Thresholds[Index];
		int64 DeltaTicks = static_cast<int64>(FMath::CeilToDouble(Delta / TickUnit));
		if (Index < NumThresholds - 1)
		{
			const double NextTickUnit = Thresholds[Index + 1];
			if (NextTickUnit <= static_cast<double>(DeltaTicks + 1) * TickUnit)
			{
				TickUnit = NextTickUnit;
				DeltaTicks = 1;
			}
			else if (DeltaTicks != 1 && DeltaTicks != 5 && DeltaTicks % 2 == 1) // prefer even grid values
			{
				DeltaTicks++;
			}
		}
		const double Grid = static_cast<double>(DeltaTicks) * TickUnit;
		const double StartValue = FMath::GridSnap(BottomValue, Grid);

		const FLinearColor GridColor(0.0f, 0.0f, 0.0f, 0.1f);
		const FLinearColor TextBgColor(0.05f, 0.05f, 0.05f, 1.0f);
		const FLinearColor TextColor = FirstTimeUnitSeries->GetColor().CopyWithNewOpacity(1.0f);

		for (double Value = StartValue; Value < TopValue; Value += Grid)
		{
			const float Y = Y0 + RoundedViewHeight - FMath::RoundToFloat(ViewportY.GetOffsetForValue(Value));

			const FString LabelText = UE::Insights::FormatTimeAuto(Value);

			// Draw horizontal grid line.
			DrawContext.DrawBox(0, Y, ViewWidth, 1, Brush, GridColor);

			const FVector2D LabelTextSize = FontMeasureService->Measure(LabelText, Font, FontScale) / FontScale;
			const float LabelX = X0 - static_cast<float>(LabelTextSize.X) - 4.0f;
			const float LabelY = FMath::Min(Y0 + GetHeight() - TextH, FMath::Max(Y0, Y - TextH / 2));

			// Draw background for value text.
			DrawContext.DrawBox(LabelX, LabelY, static_cast<float>(LabelTextSize.X) + 4.0f, TextH, Brush, TextBgColor);

			// Draw value text.
			DrawContext.DrawText(LabelX + 2.0f, LabelY + 1.0f, LabelText, Font, TextColor);
		}

		DrawContext.LayerId++;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingGraphTrack::HasAnySeriesForTimer(uint32 TimerId) const
{
	for (const TSharedPtr<FGraphSeries>& Series : AllSeries)
	{
		if (Series->Is<FTimingGraphSeries>() &&
			Series->As<FTimingGraphSeries>().IsTimer(TimerId))
		{
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FTimingGraphTrack::GetNumSeriesForTimer(uint32 TimerId) const
{
	uint32 NumSeries = 0;
	for (const TSharedPtr<FGraphSeries>& Series : AllSeries)
	{
		if (Series->Is<FTimingGraphSeries>() &&
			Series->As<FTimingGraphSeries>().IsTimer(TimerId))
		{
			++NumSeries;
		}
	}
	return NumSeries;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
