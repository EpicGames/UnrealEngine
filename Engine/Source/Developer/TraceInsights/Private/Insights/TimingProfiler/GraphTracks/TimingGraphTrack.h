// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/ThreadTimingSharedState.h"
#include "Insights/ViewModels/GraphSeries.h"
#include "Insights/ViewModels/GraphTrack.h"

namespace UE::Insights::TimingProfiler
{

struct FFrameStatsCachedEvent;
class STimingView;

class FTimingGraphSeries;

class FTimingGraphTrack : public FGraphTrack
{
	INSIGHTS_DECLARE_RTTI(FTimingGraphTrack, FGraphTrack)

public:
	FTimingGraphTrack(TSharedPtr<STimingView> InTimingView);
	virtual ~FTimingGraphTrack();

	virtual void Update(const ITimingTrackUpdateContext& Context) override;

	void AddDefaultFrameSeries();
	TSharedPtr<FTimingGraphSeries> GetFrameSeries(ETraceFrameType FrameType);

	TSharedPtr<FTimingGraphSeries> GetTimerSeries(uint32 TimerId);
	TSharedPtr<FTimingGraphSeries> AddTimerSeries(uint32 TimerId, FLinearColor Color);
	void RemoveTimerSeries(uint32 TimerId);

	TSharedPtr<FTimingGraphSeries> GetFrameStatsTimerSeries(uint32 TimerId, ETraceFrameType FrameType);
	TSharedPtr<FTimingGraphSeries> AddFrameStatsTimerSeries(uint32 TimerId, ETraceFrameType FrameType, FLinearColor Color);
	void RemoveFrameStatsTimerSeries(uint32 TimerId, ETraceFrameType FrameType);

	TSharedPtr<FTimingGraphSeries> GetStatsCounterSeries(uint32 CounterId);
	TSharedPtr<FTimingGraphSeries> AddStatsCounterSeries(uint32 CounterId, FLinearColor Color);
	void RemoveStatsCounterSeries(uint32 CounterId);

	bool HasAnySeriesForTimer(uint32 TimerId) const;
	uint32 GetNumSeriesForTimer(uint32 TimerId) const;

	void GetVisibleTimelineIndexes(TSet<uint32>& TimelineIndexes);
	void GetVisibleCpuSamplingThreads(TSet<uint32>& Threads);

private:
	void RegisterTimingViewCallbacks();
	void UnregisterTimingViewCallbacks();

	TSharedPtr<FThreadTimingSharedState> GetThreadTimingSharedState() const;

	virtual void DrawVerticalAxisGrid(const ITimingTrackDrawContext& Context) const override;

	void LoadDefaultSettings();

private:
	virtual void ContextMenu_ToggleOption_Execute(EGraphOptions Option);

private:
	FDelegateHandle OnTrackVisibilityChangedHandle;
	FDelegateHandle OnTrackAddedHandle;
	FDelegateHandle OnTrackRemovedHandle;

	FDelegateHandle GameFrameSeriesVisibilityHandle;
	FDelegateHandle RenderingFrameSeriesVisibilityHandle;

	TWeakPtr<STimingView> TimingView;
	bool bNotifyTimersOnDestruction;
};

} // namespace UE::Insights::TimingProfiler
