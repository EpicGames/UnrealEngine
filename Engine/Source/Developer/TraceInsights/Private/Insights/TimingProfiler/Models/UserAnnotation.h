// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Math/Color.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * FUserAnnotation
 *
 * Represents a single user-created annotation on the Insights timeline.
 * These are serialized to the .utrace.ini sidecar file under the [Annotations] section.
 */
struct FUserAnnotation
{
	/** Unique identifier for this annotation. */
	FGuid Id;

	/** Position in trace time (seconds), in the same coordinate system as other Insights timing events. */
	double Time = 0.0;

	/** End time for range annotations (seconds). 0 = point annotation (backward compatible). */
	double EndTime = 0.0;

	/** Returns true if this annotation represents a time range rather than a point in time. */
	bool IsRange() const { return EndTime > Time; }

	/** Game frame number at the annotation time. 0 if unavailable. */
	uint32 GameFrameNumber = 0;

	/** Rendering frame number at the annotation time. 0 if unavailable. */
	uint32 RenderFrameNumber = 0;

	/** Game frame number at EndTime (range annotations only). 0 if unavailable or point annotation. */
	uint32 GameFrameNumberEnd = 0;

	/** Rendering frame number at EndTime (range annotations only). 0 if unavailable or point annotation. */
	uint32 RenderFrameNumberEnd = 0;

	/**
	 * Name of the track the annotation was placed from. For point / range annotations on
	 * a CPU thread timing track this is the thread name; for event annotations this is
	 * whatever track hosted the anchored event (thread track, load-time track, allocation
	 * track, etc.). Empty if the annotation has no associated track.
	 */
	FString ThreadName;

	/** Timer/event name for event-anchored annotations. Empty if not anchored to a specific event. */
	FString TimerName;

	/** Exact start time of the anchored timing event (seconds). 0 if not event-anchored. */
	double EventStartTime = 0.0;

	/** Exact end time of the anchored timing event (seconds). 0 if not event-anchored. */
	double EventEndTime = 0.0;

	/** Depth of the anchored timing event in the track. 0 if not event-anchored. */
	uint32 EventDepth = 0;

	/** Returns true if this annotation is anchored to a specific timing event (function call). */
	bool HasEventAnchor() const { return !TimerName.IsEmpty() && EventEndTime > EventStartTime; }

	/** User-provided text for the annotation. */
	FString Text;

	/** Optional longer description for the annotation. Shown as tooltip in the panel. */
	FString Description;

	/** Visual color for the annotation marker. Default: gold/amber. */
	FLinearColor Color = FLinearColor(1.0f, 0.85f, 0.0f, 1.0f);

	/** Whether this annotation is visible in the timing view. Hidden annotations are still listed in the panel. */
	bool bVisible = true;

	/** Author of the annotation. Set by the caller (typically to FPlatformProcess::UserName()). */
	FString Author;

	/**
	 * Which Insights window the annotation was created from. One of "Timing", "Memory",
	 * "Asset Loading", or empty if unknown. Shown in the panel's "Source" column so users
	 * can discover where an annotation originated when the same store is shared across
	 * multiple open Insights windows on the same trace.
	 */
	FString Source;

	/** When the annotation was created. */
	FDateTime CreatedAt;

	/** When the annotation was last modified. */
	FDateTime ModifiedAt;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
