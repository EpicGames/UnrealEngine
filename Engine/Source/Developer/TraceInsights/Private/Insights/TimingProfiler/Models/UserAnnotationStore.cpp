// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/TimingProfiler/Models/UserAnnotationStore.h"

#include "Misc/DateTime.h"
#include "Misc/Guid.h"

#include "Insights/Common/TraceMetadataFile.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

static constexpr const TCHAR* AnnotationsSectionName = TEXT("Annotations");

////////////////////////////////////////////////////////////////////////////////////////////////////

// INI values are single-line, so any user-supplied string field must escape backslashes and
// newlines before writing and reverse on read. Without this, a value containing \r\n truncates
// the stored value at the first newline and leaks the rest into the next INI line.
static FString EscapeIniString(const FString& In)
{
	FString Out = In;
	Out.ReplaceInline(TEXT("\\"), TEXT("\\\\"));
	Out.ReplaceInline(TEXT("\n"), TEXT("\\n"));
	Out.ReplaceInline(TEXT("\r"), TEXT("\\r"));
	return Out;
}

static void UnescapeIniStringInline(FString& InOut)
{
	// Route escaped backslashes through a sentinel so "\\n" (literal backslash + n)
	// isn't misread as a newline after the \n substitution.
	InOut.ReplaceInline(TEXT("\\\\"), TEXT("\x01"));
	InOut.ReplaceInline(TEXT("\\n"), TEXT("\n"));
	InOut.ReplaceInline(TEXT("\\r"), TEXT("\r"));
	InOut.ReplaceInline(TEXT("\x01"), TEXT("\\"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FUserAnnotationStore::FUserAnnotationStore(TSharedPtr<FTraceMetadataFile> InMetadataFile)
	: MetadataFile(InMetadataFile)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationStore::LoadFromMetadata()
{
	Annotations.Empty();

	if (!MetadataFile.IsValid() || !MetadataFile->IsLoaded())
	{
		return;
	}

	int32 Count = 0;
	MetadataFile->GetInt(AnnotationsSectionName, TEXT("AnnotationCount"), Count);

	for (int32 i = 0; i < Count; ++i)
	{
		FUserAnnotation Annotation;
		if (ReadAnnotationFromMetadata(i, Annotation))
		{
			Annotations.Add(MoveTemp(Annotation));
		}
	}

	++ChangeNumber; // observers (panel) refresh on first load too
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationStore::SyncSectionFromAnnotations()
{
	if (!MetadataFile.IsValid())
	{
		return;
	}

	MetadataFile->ClearSection(AnnotationsSectionName);
	MetadataFile->SetInt(AnnotationsSectionName, TEXT("AnnotationCount"), Annotations.Num());

	for (int32 i = 0; i < Annotations.Num(); ++i)
	{
		WriteAnnotationToMetadata(i, Annotations[i]);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationStore::SetAnnotationsVisible(bool bInVisible)
{
	// Deliberately doesn't bump ChangeNumber — observers that need live visibility re-read
	// AreAnnotationsVisible() per-frame, and bumping would force the panel to rebuild rows
	// on every toggle, resetting selection and scroll.
	bAnnotationsVisible = bInVisible;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FUserAnnotationStore::SaveToMetadata()
{
	if (!MetadataFile.IsValid())
	{
		return false;
	}

	SyncSectionFromAnnotations();
	return MetadataFile->Save();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FUserAnnotationStore::AddAnnotation(const FUserAnnotation& Annotation)
{
	Annotations.Add(Annotation);
	++ChangeNumber;

	if (!SaveToMetadata())
	{
		Annotations.Pop();
		--ChangeNumber;
		SyncSectionFromAnnotations();
		return false;
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FUserAnnotationStore::UpdateAnnotation(const FUserAnnotation& Annotation)
{
	for (int32 i = 0; i < Annotations.Num(); ++i)
	{
		if (Annotations[i].Id == Annotation.Id)
		{
			FUserAnnotation Previous = Annotations[i];
			Annotations[i] = Annotation;
			++ChangeNumber;

			if (!SaveToMetadata())
			{
				Annotations[i] = MoveTemp(Previous);
				--ChangeNumber;
				SyncSectionFromAnnotations();
				return false;
			}
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FUserAnnotationStore::RemoveAnnotation(const FGuid& Id)
{
	for (int32 i = 0; i < Annotations.Num(); ++i)
	{
		if (Annotations[i].Id == Id)
		{
			FUserAnnotation Removed = MoveTemp(Annotations[i]);
			Annotations.RemoveAt(i);
			++ChangeNumber;

			if (!SaveToMetadata())
			{
				Annotations.Insert(MoveTemp(Removed), i);
				--ChangeNumber;
				SyncSectionFromAnnotations();
				return false;
			}
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FUserAnnotation* FUserAnnotationStore::FindAnnotation(const FGuid& Id) const
{
	for (const FUserAnnotation& Annotation : Annotations)
	{
		if (Annotation.Id == Id)
		{
			return &Annotation;
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationStore::EnumerateAnnotations(double StartTime, double EndTime, TFunctionRef<void(const FUserAnnotation&)> Callback) const
{
	for (const FUserAnnotation& Annotation : Annotations)
	{
		if (Annotation.IsRange())
		{
			// Range annotation: visible if the range overlaps the view window
			if (Annotation.EndTime >= StartTime && Annotation.Time <= EndTime)
			{
				Callback(Annotation);
			}
		}
		else
		{
			// Point annotation: visible if the point is within the view window
			if (Annotation.Time >= StartTime && Annotation.Time <= EndTime)
			{
				Callback(Annotation);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationStore::WriteAnnotationToMetadata(int32 Index, const FUserAnnotation& Annotation)
{
	const FString Prefix = FString::Printf(TEXT("Annotation%d"), Index);

	MetadataFile->SetString(AnnotationsSectionName,
		*(Prefix + TEXT(".Id")), Annotation.Id.ToString());
	MetadataFile->SetDouble(AnnotationsSectionName,
		*(Prefix + TEXT(".Time")), Annotation.Time);
	MetadataFile->SetInt(AnnotationsSectionName,
		*(Prefix + TEXT(".GameFrame")), static_cast<int32>(Annotation.GameFrameNumber));
	MetadataFile->SetInt(AnnotationsSectionName,
		*(Prefix + TEXT(".RenderFrame")), static_cast<int32>(Annotation.RenderFrameNumber));
	if (Annotation.IsRange())
	{
		MetadataFile->SetDouble(AnnotationsSectionName,
			*(Prefix + TEXT(".EndTime")), Annotation.EndTime);
		MetadataFile->SetInt(AnnotationsSectionName,
			*(Prefix + TEXT(".GameFrameEnd")), static_cast<int32>(Annotation.GameFrameNumberEnd));
		MetadataFile->SetInt(AnnotationsSectionName,
			*(Prefix + TEXT(".RenderFrameEnd")), static_cast<int32>(Annotation.RenderFrameNumberEnd));
	}
	if (!Annotation.ThreadName.IsEmpty())
	{
		MetadataFile->SetString(AnnotationsSectionName,
			*(Prefix + TEXT(".ThreadName")), EscapeIniString(Annotation.ThreadName));
	}
	if (Annotation.HasEventAnchor())
	{
		MetadataFile->SetString(AnnotationsSectionName,
			*(Prefix + TEXT(".TimerName")), EscapeIniString(Annotation.TimerName));
		MetadataFile->SetDouble(AnnotationsSectionName,
			*(Prefix + TEXT(".EventStartTime")), Annotation.EventStartTime);
		MetadataFile->SetDouble(AnnotationsSectionName,
			*(Prefix + TEXT(".EventEndTime")), Annotation.EventEndTime);
		MetadataFile->SetInt(AnnotationsSectionName,
			*(Prefix + TEXT(".EventDepth")), static_cast<int32>(Annotation.EventDepth));
	}
	MetadataFile->SetString(AnnotationsSectionName,
		*(Prefix + TEXT(".Text")), EscapeIniString(Annotation.Text));
	if (!Annotation.Description.IsEmpty())
	{
		MetadataFile->SetString(AnnotationsSectionName,
			*(Prefix + TEXT(".Description")), EscapeIniString(Annotation.Description));
	}
	MetadataFile->SetString(AnnotationsSectionName,
		*(Prefix + TEXT(".Color")),
		FString::Printf(TEXT("(R=%f,G=%f,B=%f,A=%f)"),
			Annotation.Color.R, Annotation.Color.G, Annotation.Color.B, Annotation.Color.A));
	if (!Annotation.bVisible)
	{
		MetadataFile->SetInt(AnnotationsSectionName,
			*(Prefix + TEXT(".bVisible")), Annotation.bVisible ? 1 : 0);
	}
	MetadataFile->SetString(AnnotationsSectionName,
		*(Prefix + TEXT(".Author")), EscapeIniString(Annotation.Author));
	if (!Annotation.Source.IsEmpty())
	{
		MetadataFile->SetString(AnnotationsSectionName,
			*(Prefix + TEXT(".Source")), EscapeIniString(Annotation.Source));
	}
	MetadataFile->SetString(AnnotationsSectionName,
		*(Prefix + TEXT(".CreatedAt")), Annotation.CreatedAt.ToIso8601());
	MetadataFile->SetString(AnnotationsSectionName,
		*(Prefix + TEXT(".ModifiedAt")), Annotation.ModifiedAt.ToIso8601());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FUserAnnotationStore::ReadAnnotationFromMetadata(int32 Index, FUserAnnotation& OutAnnotation)
{
	const FString Prefix = FString::Printf(TEXT("Annotation%d"), Index);

	FString IdStr;
	if (!MetadataFile->GetString(AnnotationsSectionName, *(Prefix + TEXT(".Id")), IdStr))
	{
		return false;
	}
	if (!FGuid::Parse(IdStr, OutAnnotation.Id))
	{
		return false;
	}

	MetadataFile->GetDouble(AnnotationsSectionName, *(Prefix + TEXT(".Time")), OutAnnotation.Time);
	MetadataFile->GetDouble(AnnotationsSectionName, *(Prefix + TEXT(".EndTime")), OutAnnotation.EndTime);

	int32 GameFrame = 0;
	if (MetadataFile->GetInt(AnnotationsSectionName, *(Prefix + TEXT(".GameFrame")), GameFrame))
	{
		OutAnnotation.GameFrameNumber = static_cast<uint32>(GameFrame);
	}
	int32 RenderFrame = 0;
	if (MetadataFile->GetInt(AnnotationsSectionName, *(Prefix + TEXT(".RenderFrame")), RenderFrame))
	{
		OutAnnotation.RenderFrameNumber = static_cast<uint32>(RenderFrame);
	}
	int32 GameFrameEnd = 0;
	if (MetadataFile->GetInt(AnnotationsSectionName, *(Prefix + TEXT(".GameFrameEnd")), GameFrameEnd))
	{
		OutAnnotation.GameFrameNumberEnd = static_cast<uint32>(GameFrameEnd);
	}
	int32 RenderFrameEnd = 0;
	if (MetadataFile->GetInt(AnnotationsSectionName, *(Prefix + TEXT(".RenderFrameEnd")), RenderFrameEnd))
	{
		OutAnnotation.RenderFrameNumberEnd = static_cast<uint32>(RenderFrameEnd);
	}
	if (MetadataFile->GetString(AnnotationsSectionName, *(Prefix + TEXT(".ThreadName")), OutAnnotation.ThreadName))
	{
		UnescapeIniStringInline(OutAnnotation.ThreadName);
	}

	if (MetadataFile->GetString(AnnotationsSectionName, *(Prefix + TEXT(".TimerName")), OutAnnotation.TimerName))
	{
		UnescapeIniStringInline(OutAnnotation.TimerName);
	}
	MetadataFile->GetDouble(AnnotationsSectionName, *(Prefix + TEXT(".EventStartTime")), OutAnnotation.EventStartTime);
	MetadataFile->GetDouble(AnnotationsSectionName, *(Prefix + TEXT(".EventEndTime")), OutAnnotation.EventEndTime);
	int32 EventDepth = 0;
	if (MetadataFile->GetInt(AnnotationsSectionName, *(Prefix + TEXT(".EventDepth")), EventDepth))
	{
		OutAnnotation.EventDepth = static_cast<uint32>(EventDepth);
	}

	if (MetadataFile->GetString(AnnotationsSectionName, *(Prefix + TEXT(".Text")), OutAnnotation.Text))
	{
		UnescapeIniStringInline(OutAnnotation.Text);
	}
	if (MetadataFile->GetString(AnnotationsSectionName, *(Prefix + TEXT(".Description")), OutAnnotation.Description))
	{
		UnescapeIniStringInline(OutAnnotation.Description);
	}

	FString ColorStr;
	if (MetadataFile->GetString(AnnotationsSectionName, *(Prefix + TEXT(".Color")), ColorStr))
	{
		OutAnnotation.Color.InitFromString(ColorStr);
	}

	{
		int32 VisibleInt = 1; // Default true for backward compat
		if (MetadataFile->GetInt(AnnotationsSectionName, *(Prefix + TEXT(".bVisible")), VisibleInt))
		{
			OutAnnotation.bVisible = (VisibleInt != 0);
		}
	}
	if (MetadataFile->GetString(AnnotationsSectionName, *(Prefix + TEXT(".Author")), OutAnnotation.Author))
	{
		UnescapeIniStringInline(OutAnnotation.Author);
	}
	if (MetadataFile->GetString(AnnotationsSectionName, *(Prefix + TEXT(".Source")), OutAnnotation.Source))
	{
		UnescapeIniStringInline(OutAnnotation.Source);
	}

	FString CreatedAtStr;
	if (MetadataFile->GetString(AnnotationsSectionName, *(Prefix + TEXT(".CreatedAt")), CreatedAtStr))
	{
		FDateTime::ParseIso8601(*CreatedAtStr, OutAnnotation.CreatedAt);
	}

	FString ModifiedAtStr;
	if (MetadataFile->GetString(AnnotationsSectionName, *(Prefix + TEXT(".ModifiedAt")), ModifiedAtStr))
	{
		FDateTime::ParseIso8601(*ModifiedAtStr, OutAnnotation.ModifiedAt);
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
