// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

#include "Insights/TimingProfiler/Models/UserAnnotation.h"

namespace UE::Insights { class FTraceMetadataFile; }

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * In-memory store for user annotations, backed by an FTraceMetadataFile.
 * Serializes to the [Annotations] section of the .utrace.ini sidecar.
 * Each mutation auto-saves to disk and increments ChangeNumber.
 */
class FUserAnnotationStore
{
public:
	explicit FUserAnnotationStore(TSharedPtr<FTraceMetadataFile> InMetadataFile);

	void LoadFromMetadata();
	bool SaveToMetadata();

	bool AddAnnotation(const FUserAnnotation& Annotation);
	bool UpdateAnnotation(const FUserAnnotation& Annotation);
	bool RemoveAnnotation(const FGuid& Id);
	const FUserAnnotation* FindAnnotation(const FGuid& Id) const;

	void EnumerateAnnotations(double StartTime, double EndTime, TFunctionRef<void(const FUserAnnotation&)> Callback) const;

	const TArray<FUserAnnotation>& GetAllAnnotations() const { return Annotations; }
	uint64 GetChangeNumber() const { return ChangeNumber; }

	/** Runtime mirror of "show annotations" toggle. Owned by the extender; not persisted here. */
	bool AreAnnotationsVisible() const { return bAnnotationsVisible; }
	void SetAnnotationsVisible(bool bInVisible);

private:
	/** Rewrites the [Annotations] section in MetadataFile from the in-memory Annotations array. */
	void SyncSectionFromAnnotations();
	void WriteAnnotationToMetadata(int32 Index, const FUserAnnotation& Annotation);
	bool ReadAnnotationFromMetadata(int32 Index, FUserAnnotation& OutAnnotation);

	TSharedPtr<FTraceMetadataFile> MetadataFile;
	TArray<FUserAnnotation> Annotations;
	uint64 ChangeNumber = 0;
	bool bAnnotationsVisible = true;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
