// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/TimingProfiler/TimingProfilerCommands.h"

#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimingProfilerCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingProfilerCommands::FTimingProfilerCommands()
: TCommands<FTimingProfilerCommands>(
	TEXT("TimingProfilerCommands"),
	NSLOCTEXT("Contexts", "TimingProfilerCommands", "Insights - Timing Insights"),
	NAME_None,
	FInsightsStyle::GetStyleSetName())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// UI_COMMAND takes long for the compiler to optimize
UE_DISABLE_OPTIMIZATION_SHIP
void FTimingProfilerCommands::RegisterCommands()
{
	UI_COMMAND(ToggleFramesTrackVisibility,
		"Frames",
		"Toggles the visibility of the Frames track.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleTimingViewVisibility,
		"Timing",
		"Toggles the visibility of the main Timing view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleTimersViewVisibility,
		"Timers",
		"Toggles the visibility of the Timers view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleCallersTreeViewVisibility,
		"Callers",
		"Toggles the visibility of the Callers tree view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleCalleesTreeViewVisibility,
		"Callees",
		"Toggles the visibility of the Callees tree view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleStatsCountersViewVisibility,
		"Counters",
		"Toggles the visibility of the Counters view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleLogViewVisibility,
		"Log",
		"Toggles the visibility of the Log view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleModulesViewVisibility,
		"Modules",
		"Toggles the visibility of the Modules view.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleUserAnnotationsViewVisibility,
		"Annotations",
		"Toggles the visibility of the Annotations panel",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimingViewCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingViewCommands::FTimingViewCommands()
: TCommands<FTimingViewCommands>(
	TEXT("TimingViewCommands"),
	NSLOCTEXT("Contexts", "TimingViewCommands", "Insights - Timing View"),
	NAME_None,
	FInsightsStyle::GetStyleSetName())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// UI_COMMAND takes long for the compiler to optimize
UE_DISABLE_OPTIMIZATION_SHIP
void FTimingViewCommands::RegisterCommands()
{
	UI_COMMAND(AutoHideEmptyTracks,
		"Auto Hide Empty Tracks",
		"Auto hide empty tracks (ex.: ones without timing events in the current viewport).\nThis option is persistent to the UnrealInsightsSettings.ini file.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::V));

	UI_COMMAND(PanningOnScreenEdges,
		"Allow Panning on Screen Edges",
		"If enabled, the panning is allowed to continue when the mouse cursor reaches the edges of the screen.\nThis option is persistent to the UnrealInsightsSettings.ini file.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleCompactMode,
		"Compact Mode",
		"Toggle compact mode for supporting tracks.\n(ex.: the timing tracks will be displayed with reduced height)",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::C));

	UI_COMMAND(ShowMainGraphTrack,
		"Graph Track",
		"Shows/hides the main Graph track.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::G));

	UI_COMMAND(QuickFind,
		"Quick Find...",
		"Quick find or filter events in the timing view.",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Control, EKeys::F));

	UI_COMMAND(GoToFrame,
		"Go to Frame...",
		"Navigate the timing view to a specific game or render frame by number.",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Control, EKeys::G));

	UI_COMMAND(GoToTime,
		"Go to Time...",
		"Navigate the timing view to a specific timestamp.",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Control, EKeys::T));

	UI_COMMAND(TrimAndSaveAs,
		"Trim and Save As...",
		"Trim the current trace using the selected time range and save it as new trace file.",
		EUserInterfaceActionType::Button,
		FInputChord());

	UI_COMMAND(NextAnnotation,
		"Next Annotation",
		"Navigate to the next user annotation",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Control, EKeys::N));

	UI_COMMAND(PrevAnnotation,
		"Previous Annotation",
		"Navigate to the previous user annotation",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::N));

	UI_COMMAND(ToggleFloatingAnnotations,
		"Toggle Floating Annotations",
		"Toggle floating annotation callouts",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ToggleAllAnnotationVisibility,
		"Toggle All Annotations",
		"Show or hide all user annotations",
		EUserInterfaceActionType::Button,
		FInputChord());

	UI_COMMAND(AddPointAnnotation,
		"Add Time Annotation",
		"Add a time annotation at the current position.\nRequires the Annotations track to be visible (toggle via Other menu).",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Control, EKeys::B));

	UI_COMMAND(AddRangeAnnotation,
		"Add Range Annotation",
		"Add a range annotation over the current selection.\nRequires the Annotations track to be visible (toggle via Other menu).",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::B));

	UI_COMMAND(AddEventAnnotation,
		"Add Event Annotation",
		"Add an annotation on the hovered event.\nRequires the Annotations track to be visible (toggle via Other menu).",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Control | EModifierKey::Alt, EKeys::B));

	UI_COMMAND(FindPreviousInstance,
		"Previous Instance",
		"Navigates to and selects the previous timing event instance for the selected event's timer.",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Shift, EKeys::Comma));

	UI_COMMAND(FindNextInstance,
		"Next Instance",
		"Navigates to and selects the next timing event instance for the selected event's timer.",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Shift, EKeys::Period));

	UI_COMMAND(FindMaxInstance,
		"Maximum Duration Instance",
		"Navigates to and selects the timing event instance with the maximum duration, for the selected event's timer.",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Shift, EKeys::Slash));

	UI_COMMAND(FindPreviousInstanceInSelection,
		"Previous Instance in Selection",
		"Navigates to and selects the previous timing event instance for the selected event's timer, in the selected time range.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Comma));

	UI_COMMAND(FindNextInstanceInSelection,
		"Next Instance in Selection",
		"Navigates to and selects the next timing event instance for the selected event's timer, in the selected time range.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Period));

	UI_COMMAND(FindMaxInstanceInSelection,
		"Maximum Duration Instance in Selection",
		"Navigates to and selects the timing event instance with the maximum duration, for the selected event's timer, in the selected time range.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Slash));
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////
// Toggle Commands
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FTimingProfilerActionManager, This, ToggleFramesTrackVisibility, IsFramesTrackVisible, ShowHideFramesTrack)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FTimingProfilerActionManager, This, ToggleTimingViewVisibility, IsTimingViewVisible, ShowHideTimingView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FTimingProfilerActionManager, This, ToggleTimersViewVisibility, IsTimersViewVisible, ShowHideTimersView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FTimingProfilerActionManager, This, ToggleCallersTreeViewVisibility, IsCallersTreeViewVisible, ShowHideCallersTreeView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FTimingProfilerActionManager, This, ToggleCalleesTreeViewVisibility, IsCalleesTreeViewVisible, ShowHideCalleesTreeView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FTimingProfilerActionManager, This, ToggleStatsCountersViewVisibility, IsStatsCountersViewVisible, ShowHideStatsCountersView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FTimingProfilerActionManager, This, ToggleLogViewVisibility, IsLogViewVisible, ShowHideLogView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FTimingProfilerActionManager, This, ToggleModulesViewVisibility, IsModulesViewVisible, ShowHideModulesView)
INSIGHTS_IMPLEMENT_TOGGLE_COMMAND(FTimingProfilerActionManager, This, ToggleUserAnnotationsViewVisibility, IsUserAnnotationsViewVisible, ShowHideUserAnnotationsView)

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
