// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/Commands.h"

// TraceInsights
#include "Insights/Common/CommandUtils.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class that holds all profiler commands.
 */
class FTimingProfilerCommands : public TCommands<FTimingProfilerCommands>
{
public:
	FTimingProfilerCommands();
	virtual ~FTimingProfilerCommands() = default;
	virtual void RegisterCommands() override;

public:
	/** Toggles visibility for the Frames Track. */
	TSharedPtr<FUICommandInfo> ToggleFramesTrackVisibility;

	/** Toggles visibility for the Timing View. */
	TSharedPtr<FUICommandInfo> ToggleTimingViewVisibility;

	/** Toggles visibility for the Timers View. */
	TSharedPtr<FUICommandInfo> ToggleTimersViewVisibility;

	/** Toggles visibility for the Callers Tree View. */
	TSharedPtr<FUICommandInfo> ToggleCallersTreeViewVisibility;

	/** Toggles visibility for the Callees Tree View. */
	TSharedPtr<FUICommandInfo> ToggleCalleesTreeViewVisibility;

	/** Toggles visibility for the Stats Counters View. */
	TSharedPtr<FUICommandInfo> ToggleStatsCountersViewVisibility;

	/** Toggles visibility for the Log View. */
	TSharedPtr<FUICommandInfo> ToggleLogViewVisibility;

	/** Toggles visibility for the Modules view. */
	TSharedPtr<FUICommandInfo> ToggleModulesViewVisibility;

	/** Toggles visibility for the User Annotations view. */
	TSharedPtr<FUICommandInfo> ToggleUserAnnotationsViewVisibility;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingViewCommands : public TCommands<FTimingViewCommands>
{
public:
	FTimingViewCommands();
	virtual ~FTimingViewCommands() = default;
	virtual void RegisterCommands() override;

public:
	/** Toggles visibility of empty tracks. */
	TSharedPtr<FUICommandInfo> AutoHideEmptyTracks;

	/** Toggles "panning on screen edges". */
	TSharedPtr<FUICommandInfo> PanningOnScreenEdges;

	/** Toggles 'compact mode' for timing tracks. */
	TSharedPtr<FUICommandInfo> ToggleCompactMode;

	/** Toggles visibility for Main Graph track. */
	TSharedPtr<FUICommandInfo> ShowMainGraphTrack;

	/** Opens the Quick Find widget. */
	TSharedPtr<FUICommandInfo> QuickFind;

	/** Opens the Go to Frame dialog to navigate to a specific game or render frame. */
	TSharedPtr<FUICommandInfo> GoToFrame;

	/** Opens the Go to Time dialog to navigate to a specific timestamp. */
	TSharedPtr<FUICommandInfo> GoToTime;

	/** Trim the current trace using the selected time range and save it as new trace file. */
	TSharedPtr<FUICommandInfo> TrimAndSaveAs;

	/** Navigate to the next user annotation. */
	TSharedPtr<FUICommandInfo> NextAnnotation;

	/** Navigate to the previous user annotation. */
	TSharedPtr<FUICommandInfo> PrevAnnotation;

	/** Toggle floating annotation callouts. */
	TSharedPtr<FUICommandInfo> ToggleFloatingAnnotations;

	/** Toggle all annotation visibility. */
	TSharedPtr<FUICommandInfo> ToggleAllAnnotationVisibility;

	/** Add a point annotation at the current time. */
	TSharedPtr<FUICommandInfo> AddPointAnnotation;

	/** Add a range annotation over the current selection. */
	TSharedPtr<FUICommandInfo> AddRangeAnnotation;

	/** Add an event annotation on the hovered event. */
	TSharedPtr<FUICommandInfo> AddEventAnnotation;

	/** Navigates to the previous timing event instance for the selected event's timer. */
	TSharedPtr<FUICommandInfo> FindPreviousInstance;

	/** Navigates to the next timing event instance for the selected event's timer. */
	TSharedPtr<FUICommandInfo> FindNextInstance;

	/** Navigates to the maximum duration timing event instance for the selected event's timer. */
	TSharedPtr<FUICommandInfo> FindMaxInstance;

	/** Navigates to the previous timing event instance for the selected event's timer, within the selected time range. */
	TSharedPtr<FUICommandInfo> FindPreviousInstanceInSelection;

	/** Navigates to the next timing event instance for the selected event's timer, within the selected time range. */
	TSharedPtr<FUICommandInfo> FindNextInstanceInSelection;

	/** Navigates to the maximum duration timing event instance for the selected event's timer, within the selected time range. */
	TSharedPtr<FUICommandInfo> FindMaxInstanceInSelection;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class that provides helper functions for the commands to avoid cluttering profiler manager with many small functions.
 * Can't contain any variables. Directly operates on the profiler manager instance.
 */
class FTimingProfilerActionManager
{
	friend class FTimingProfilerManager;

private:
	/** Private constructor. */
	FTimingProfilerActionManager(class FTimingProfilerManager* Instance)
		: This(Instance)
	{}

	//////////////////////////////////////////////////
	// Toggle Commands

	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleFramesTrackVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleTimingViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleTimersViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleCallersTreeViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleCalleesTreeViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleStatsCountersViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleLogViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleModulesViewVisibility)
	INSIGHTS_DECLARE_TOGGLE_COMMAND(ToggleUserAnnotationsViewVisibility)

	//////////////////////////////////////////////////

protected:
	/** Reference to the global instance of the profiler manager. */
	class FTimingProfilerManager* This;
};

} // namespace UE::Insights::TimingProfiler
