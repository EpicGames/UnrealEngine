// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimingProfilerManager.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"

// TraceServices
#include "TraceServices/Model/Counters.h"

// TraceInsights
#include "Insights/Common/InsightsMenuBuilder.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsSettings.h"
#include "Insights/InsightsStyle.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/TimingProfiler/UserAnnotationsTimingViewExtender.h"
#include "Insights/TraceInsightsModule.h"
#include "Insights/TimingProfiler/ViewModels/TimerButterflyAggregator.h"
#include "Insights/TimingProfiler/ViewModels/TimerNode.h"
#include "Insights/TimingProfiler/ViewModels/TimingExporter.h"
#include "Insights/TimingProfiler/Widgets/SFrameTrack.h"
#include "Insights/TimingProfiler/Widgets/SStatsView.h"
#include "Insights/TimingProfiler/Widgets/STimersView.h"
#include "Insights/TimingProfiler/Widgets/STimerTreeView.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/TimingProfilerCommon.h"
#include "Insights/ViewModels/LogFilter.h"
#include "Insights/Widgets/SLogView.h"
#include "Insights/Widgets/STimingView.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY(LogTimingProfiler);

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler"

// For Timing Insights in editor preview.
static bool GEnableTimingInsightsInEditor = true;
static FAutoConsoleVariableRef CVAREnableTimingInsights(
	TEXT("Trace.EnableTimingInsights"),
	GEnableTimingInsightsInEditor,
	TEXT("Enables the Timing Insights feature in the Editor."),
	ECVF_Default
);

namespace UE::Insights::TimingProfiler
{

TSharedPtr<FTimingProfilerManager> FTimingProfilerManager::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTimingProfilerManager> FTimingProfilerManager::Get()
{
	return FTimingProfilerManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTimingProfilerManager> FTimingProfilerManager::CreateInstance()
{
	ensure(!FTimingProfilerManager::Instance.IsValid());
	if (FTimingProfilerManager::Instance.IsValid())
	{
		FTimingProfilerManager::Instance.Reset();
	}

	FTimingProfilerManager::Instance = MakeShared<FTimingProfilerManager>(FInsightsManager::Get()->GetCommandList());

	return FTimingProfilerManager::Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingProfilerManager::FTimingProfilerManager(TSharedRef<FUICommandList> InCommandList)
	: CommandList(InCommandList)
	, ActionManager(this)
	, Filter(MakeShared<FLogFilter>())
	, TimerButterflyAggregator(MakeShared<FTimerButterflyAggregator>())
	, MajorTabId(FInsightsManagerTabs::TimingProfilerTabId)
	, LogListingName("TimingInsights")
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::Initialize(IUnrealInsightsModule& InsightsModule)
{
	ensure(!bIsInitialized);
	if (bIsInitialized)
	{
		return;
	}
	bIsInitialized = true;

	UE_LOGF(LogTimingProfiler, Log, "Initialize");

	// Register tick functions.
	OnTick = FTickerDelegate::CreateSP(this, &FTimingProfilerManager::Tick);
	OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(OnTick, 0.0f);

	const FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
	EventDepthLimitMin = Settings.GetTimingTrackDepthLimitMin();
	EventDepthLimitMax = Settings.GetTimingTrackDepthLimitMax();

	FTimingProfilerCommands::Register();
	BindCommands();

	InsightsModule.OnRegisterMajorTabExtension(MajorTabId);

	FInsightsManager::Get()->GetSessionChangedEvent().AddSP(this, &FTimingProfilerManager::OnSessionChanged);
	OnSessionChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::Shutdown()
{
	if (!bIsInitialized)
	{
		return;
	}
	bIsInitialized = false;

	// If the MessageLog module was already unloaded as part of the global Shutdown process, do not load it again.
	if (FModuleManager::Get().IsModuleLoaded("MessageLog"))
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		if (MessageLogModule.IsRegisteredLogListing(GetLogListingName()))
		{
			MessageLogModule.UnregisterLogListing(GetLogListingName());
		}
	}

	FInsightsManager::Get()->GetSessionChangedEvent().RemoveAll(this);

	FTimingProfilerCommands::Unregister();

	Filter->Reset();

	// Unregister tick function.
	FTSTicker::RemoveTicker(OnTickHandle);

	FTimingProfilerManager::Instance.Reset();

	UE_LOGF(LogTimingProfiler, Log, "Shutdown");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingProfilerManager::~FTimingProfilerManager()
{
	ensure(!bIsInitialized);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::BindCommands()
{
	ActionManager.Map_ToggleFramesTrackVisibility();
	ActionManager.Map_ToggleTimingViewVisibility();
	ActionManager.Map_ToggleTimersViewVisibility();
	ActionManager.Map_ToggleCallersTreeViewVisibility();
	ActionManager.Map_ToggleCalleesTreeViewVisibility();
	ActionManager.Map_ToggleStatsCountersViewVisibility();
	ActionManager.Map_ToggleLogViewVisibility();
	ActionManager.Map_ToggleModulesViewVisibility();
	ActionManager.Map_ToggleUserAnnotationsViewVisibility();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::RegisterMajorTabs(IUnrealInsightsModule& InsightsModule)
{
	bool bShouldRegister = true;

#if WITH_EDITOR
	bShouldRegister = GEnableTimingInsightsInEditor;
#endif

	if (!bIsTabRegistered && bShouldRegister)
	{
		bIsTabRegistered = true;
		const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(MajorTabId);
		if (Config.bIsAvailable)
		{
			// Register tab spawner for the Timing Insights.
			FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(MajorTabId,
					FOnSpawnTab::CreateRaw(this, &FTimingProfilerManager::SpawnTab),
					FCanSpawnTab::CreateRaw(this, &FTimingProfilerManager::CanSpawnTab))
				.SetDisplayName(Config.TabLabel.IsSet() ? Config.TabLabel.GetValue() : LOCTEXT("TimingProfilerTabTitle", "Timing Insights"))
				.SetTooltipText(Config.TabTooltip.IsSet() ? Config.TabTooltip.GetValue() : LOCTEXT("TimingProfilerTooltipText", "Open the Timing Insights tab."))
				.SetIcon(Config.TabIcon.IsSet() ? Config.TabIcon.GetValue() : FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TimingProfiler"));

			TSharedRef<FWorkspaceItem> Group = Config.WorkspaceGroup.IsValid() ? Config.WorkspaceGroup.ToSharedRef() : FInsightsManager::Get()->GetInsightsMenuBuilder()->GetInsightsToolsGroup();
			TabSpawnerEntry.SetGroup(Group);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::UnregisterMajorTabs()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(MajorTabId);
	bIsTabRegistered = false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> FTimingProfilerManager::SpawnTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	// Register OnTabClosed to handle Timing profiler manager shutdown.
	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FTimingProfilerManager::OnTabClosed));

	// Create the STimingProfilerWindow widget.
	TSharedRef<STimingProfilerWindow> Window = SNew(STimingProfilerWindow, DockTab, Args.GetOwnerWindow());
	DockTab->SetContent(Window);

	AssignProfilerWindow(Window);

	return DockTab;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingProfilerManager::CanSpawnTab(const FSpawnTabArgs& Args) const
{
#if WITH_EDITOR
	return true;
#else
	return bIsAvailable;
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::OnTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	OnWindowClosedEvent();

	RemoveProfilerWindow();

	// Disable TabClosed delegate.
	TabBeingClosed->SetOnTabClosed(SDockTab::FOnTabClosedCallback());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedRef<FUICommandList> FTimingProfilerManager::GetCommandList() const
{
	return CommandList;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FTimingProfilerCommands& FTimingProfilerManager::GetCommands()
{
	return FTimingProfilerCommands::Get();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingProfilerActionManager& FTimingProfilerManager::GetActionManager()
{
	return FTimingProfilerManager::Instance->ActionManager;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingProfilerManager::Tick(float DeltaTime)
{
#if WITH_EDITOR
	if (!bIsTabRegistered && GEnableTimingInsightsInEditor)
	{
		IUnrealInsightsModule& TraceInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		RegisterMajorTabs(TraceInsightsModule);
	}
#endif

	TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();
	check(InsightsManager.IsValid());

	TSharedPtr<const TraceServices::IAnalysisSession> Session = InsightsManager->GetSession();
	if (Session.IsValid())
	{
		// Check if session has Timing events (to spawn the tab), but not too often.
		if (!bIsAvailable && AvailabilityCheck.Tick())
		{
			bIsAvailable = true;

			FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
			MessageLogModule.RegisterLogListing(GetLogListingName(), LOCTEXT("TimingInsights", "Timing Insights"));
			MessageLogModule.EnableMessageLogDisplay(true);

#if !WITH_EDITOR
			if (FGlobalTabmanager::Get()->HasTabSpawner(MajorTabId))
			{
				UE_LOGF(LogTimingProfiler, Log, "Opening the \"Timing Insights\" tab...");
				FGlobalTabmanager::Get()->TryInvokeTab(MajorTabId);
			}
#endif
		}
		else
		{
			// Do not check again until the next session changed event (see OnSessionChanged).
			AvailabilityCheck.Disable();
		}

		TimerButterflyAggregator->Tick(Session, 0.0f, DeltaTime, [this]() { FinishTimerButterflyAggregation(); });
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::FinishTimerButterflyAggregation()
{
	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		TSharedPtr<STimerTreeView> CallersTreeView = Wnd->GetCallersTreeView();
		if (CallersTreeView)
		{
			TraceServices::ITimingProfilerButterfly* TimingProfilerButterfly = TimerButterflyAggregator->GetResultButterfly();
			const TraceServices::FTimingProfilerButterflyNode& Callers = TimingProfilerButterfly->GenerateCallersTree(SelectedTimerId);
			CallersTreeView->SetTree(Callers);
		}

		TSharedPtr<STimerTreeView> CalleesTreeView = Wnd->GetCalleesTreeView();
		if (CalleesTreeView)
		{
			TraceServices::ITimingProfilerButterfly* TimingProfilerButterfly = TimerButterflyAggregator->GetResultButterfly();
			const TraceServices::FTimingProfilerButterflyNode& Callees = TimingProfilerButterfly->GenerateCalleesTree(SelectedTimerId);
			CalleesTreeView->SetTree(Callees);
		}
	}

	TimerButterflyAggregator->ResetResults();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::ActivateMajorTab()
{
	// Ensure Timing Insights / Timing View is the active tab / view.
	TSharedPtr<SDockTab> MajorTab = FGlobalTabmanager::Get()->FindExistingLiveTab(MajorTabId);
	if (MajorTab.IsValid())
	{
		MajorTab->ActivateInParent(ETabActivationCause::SetDirectly);
		ActivateView();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::ActivateView()
{
	// Ensure Timing View is the active view.
	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		TSharedPtr<FTabManager> TabManager = Wnd->GetTabManager();
		TSharedPtr<SDockTab> MinorTab = TabManager->FindExistingLiveTab(FTimingProfilerTabs::TimingViewID);
		if (MinorTab.IsValid())
		{
			MinorTab->ActivateInParent(ETabActivationCause::SetDirectly);
			FSlateApplication::Get().SetKeyboardFocus(MinorTab->GetContent());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::OnSessionChanged()
{
	UE_LOGF(LogTimingProfiler, Log, "OnSessionChanged");

	bIsAvailable = false;
	if (FInsightsManager::Get()->GetSession().IsValid())
	{
		AvailabilityCheck.Enable(0.0);
	}
	else
	{
		AvailabilityCheck.Disable();
	}

	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->Reset();
	}

	SelectionStartTime = 0.0;
	SelectionEndTime = 0.0;
	SelectedTimerId = InvalidTimerId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::ShowHideFramesTrack(const bool bIsVisible)
{
	bIsFramesTrackVisible = bIsVisible;

	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FTimingProfilerTabs::FramesTrackID, bIsFramesTrackVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::ShowHideTimingView(const bool bIsVisible)
{
	bIsTimingViewVisible = bIsVisible;

	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FTimingProfilerTabs::TimingViewID, bIsVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::ShowHideTimersView(const bool bIsVisible)
{
	bIsTimersViewVisible = bIsVisible;

	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FTimingProfilerTabs::TimersID, bIsVisible);

		if (bIsTimersViewVisible)
		{
			UpdateAggregatedTimerStats();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::ShowHideCallersTreeView(const bool bIsVisible)
{
	bIsCallersTreeViewVisible = bIsVisible;

	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FTimingProfilerTabs::CallersID, bIsVisible);

		if (bIsCallersTreeViewVisible)
		{
			UpdateCallersAndCallees();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::ShowHideCalleesTreeView(const bool bIsVisible)
{
	bIsCalleesTreeViewVisible = bIsVisible;

	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FTimingProfilerTabs::CalleesID, bIsVisible);

		if (bIsCalleesTreeViewVisible)
		{
			UpdateCallersAndCallees();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::ShowHideStatsCountersView(const bool bIsVisible)
{
	bIsStatsCountersViewVisible = bIsVisible;

	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FTimingProfilerTabs::StatsCountersID, bIsVisible);

		if (bIsStatsCountersViewVisible)
		{
			UpdateAggregatedCounterStats();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::ShowHideLogView(const bool bIsVisible)
{
	bIsLogViewVisible = bIsVisible;

	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FTimingProfilerTabs::LogViewID, bIsVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::ShowHideModulesView(const bool bIsVisible)
{
	bIsModulesViewVisible = bIsVisible;

	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FTimingProfilerTabs::ModulesViewID, bIsVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::ShowHideUserAnnotationsView(const bool bIsVisible)
{
	bIsUserAnnotationsViewVisible = bIsVisible;

	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		Wnd->ShowHideTab(FTimingProfilerTabs::UserAnnotationsID, bIsVisible);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::SetSelectedTimeRange(double InStartTime, double InEndTime)
{
	if (InStartTime != SelectionStartTime ||
		InEndTime != SelectionEndTime)
	{
		SelectionStartTime = InStartTime;
		SelectionEndTime = InEndTime;

		UpdateCallersAndCallees();
		UpdateAggregatedTimerStats();
		UpdateAggregatedCounterStats();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNodePtr FTimingProfilerManager::GetTimerNode(uint32 InTimerId) const
{
	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		TSharedPtr<STimersView> TimersView = Wnd->GetTimersView();
		if (TimersView)
		{
			FTimerNodePtr TimerNodePtr = TimersView->GetTimerNode(InTimerId);

			if (TimerNodePtr == nullptr)
			{
				// List of timers in TimersView not up to date?
				// Refresh and try again.
				TimersView->RebuildTree(false);
				TimerNodePtr = TimersView->GetTimerNode(InTimerId);
			}

			return TimerNodePtr;
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::SetSelectedTimer(uint32 InTimerId)
{
	if (InTimerId != SelectedTimerId)
	{
		SelectedTimerId = InTimerId;

		if (SelectedTimerId != InvalidTimerId)
		{
			UpdateCallersAndCallees();

			TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
			if (Wnd)
			{
				TSharedPtr<STimersView> TimersView = Wnd->GetTimersView();
				if (TimersView)
				{
					TimersView->SelectTimerNode(InTimerId);
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::ToggleTimingViewMainGraphEventSeries(uint32 InTimerId)
{
	FTimerNodePtr NodePtr = GetTimerNode(InTimerId);
	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd && NodePtr)
	{
		TSharedPtr<STimersView> TimersView = Wnd->GetTimersView();
		if (TimersView)
		{
			TimersView->ToggleTimingViewMainGraphEventSeries(NodePtr);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::OnThreadFilterChanged()
{
	UpdateCallersAndCallees();
	UpdateAggregatedCounterStats();

	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		TSharedPtr<STimersView> TimersView = Wnd->GetTimersView();
		if (TimersView)
		{
			TimersView->OnTimingViewTrackListChanged();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::ResetCallersAndCallees()
{
	TimerButterflyAggregator->Cancel();
	TimerButterflyAggregator->SetTimeInterval(0.0, 0.0);

	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		TSharedPtr<STimerTreeView> CallersTreeView = Wnd->GetCallersTreeView();
		TSharedPtr<STimerTreeView> CalleesTreeView = Wnd->GetCalleesTreeView();

		if (CallersTreeView)
		{
			CallersTreeView->Reset();
		}

		if (CalleesTreeView)
		{
			CalleesTreeView->Reset();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::UpdateCallersAndCallees()
{
	if (SelectionStartTime < SelectionEndTime && SelectedTimerId != InvalidTimerId)
	{
		TimerButterflyAggregator->Cancel();
		TimerButterflyAggregator->SetTimeInterval(SelectionStartTime, SelectionEndTime);

		TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
		if (Wnd)
		{
			TSharedPtr<STimerTreeView> CallersTreeView = Wnd->GetCallersTreeView();
			TSharedPtr<STimerTreeView> CalleesTreeView = Wnd->GetCalleesTreeView();

			if (CallersTreeView)
			{
				CallersTreeView->Reset();
			}

			if (CalleesTreeView)
			{
				CalleesTreeView->Reset();
			}

			if (CallersTreeView || CalleesTreeView)
			{
				TimerButterflyAggregator->Start();
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::UpdateAggregatedTimerStats()
{
	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		TSharedPtr<STimersView> TimersView = Wnd->GetTimersView();
		if (TimersView)
		{
			TimersView->UpdateStats(SelectionStartTime, SelectionEndTime);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::UpdateAggregatedCounterStats()
{
	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		TSharedPtr<SStatsView> StatsView = Wnd->GetStatsView();
		if (StatsView)
		{
			StatsView->UpdateStats(SelectionStartTime, SelectionEndTime);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::OnWindowClosedEvent()
{
	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	if (Wnd)
	{
		TSharedPtr<STimingView> TimingView = Wnd->GetTimingView();
		if (TimingView)
		{
			TimingView->CloseQuickFindTab();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingProfilerManager::Exec(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("TimingInsights.ExportThreads")))
	{
		Ar.Logf(TEXT("TimingInsights.ExportThreads %s"), Cmd);
		check(FInsightsManager::Get().IsValid() && FInsightsManager::Get()->GetSession().IsValid());
		FTimingExporter Exporter(*FInsightsManager::Get()->GetSession().Get());
		FTimingExporter::FExportThreadsParams Params; // default

		const bool bUseEscape = true;
		FString Filename = FParse::Token(Cmd, bUseEscape);
		Filename.TrimQuotesInline();
		Ar.Logf(TEXT("  Filename: \"%s\""), *Filename);

		Exporter.ExportThreadsAsText(Filename, Params);
		return true;
	}

	if (FParse::Command(&Cmd, TEXT("TimingInsights.ExportTimers")))
	{
		Ar.Logf(TEXT("TimingInsights.ExportTimers %s"), Cmd);
		check(FInsightsManager::Get().IsValid() && FInsightsManager::Get()->GetSession().IsValid());
		FTimingExporter Exporter(*FInsightsManager::Get()->GetSession().Get());
		FTimingExporter::FExportTimersParams Params; // default

		const bool bUseEscape = true;
		FString Filename = FParse::Token(Cmd, bUseEscape);
		Filename.TrimQuotesInline();
		Ar.Logf(TEXT("  Filename: \"%s\""), *Filename);

		while (Cmd && Cmd[0] != TEXT('\0'))
		{
			FString Token;
			if (FParse::Token(Cmd, Token, bUseEscape))
			{
				Ar.Logf(TEXT("  Token: %s"), *Token);

				static constexpr TCHAR ShowFormatInTimerNamesToken[] = TEXT("-ShowFormatInTimerNames=");
				if (Token.StartsWith(ShowFormatInTimerNamesToken))
				{
					Token.RightChopInline(UE_ARRAY_COUNT(ShowFormatInTimerNamesToken) - 1);
					Token.TrimQuotesInline();
					Params.bShowFormatInTimerNames = Token.ToBool();
				}
			}
		}

		Exporter.ExportTimersAsText(Filename, Params);
		return true;
	}

	if (FParse::Command(&Cmd, TEXT("TimingInsights.ExportTimingEvents")))
	{
		Ar.Logf(TEXT("TimingInsights.ExportTimingEvents %s"), Cmd);

		check(FInsightsManager::Get().IsValid() && FInsightsManager::Get()->GetSession().IsValid());
		FTimingExporter Exporter(*FInsightsManager::Get()->GetSession().Get());
		FTimingExporter::FExportTimingEventsParams Params; // default (all timing events)

		// These variables needs to be in the same scope with the call to Exporter.ExportTimingEventsAsText().
		TArray<FName> Columns; // referenced by Params.Columns
		TSet<uint32> IncludedThreads; // referenced in Params.ThreadFilter lambda function
		TSet<uint32> IncludedTimers; // referenced in Params.TimingEventFilter lambda function

		//////////////////////////////////////////////////

		const bool bUseEscape = true;
		FString Filename = FParse::Token(Cmd, bUseEscape);
		Filename.TrimQuotesInline();
		Ar.Logf(TEXT("  Filename: \"%s\""), *Filename);
		// '{region}' in Filename (if any) will be replaced with the resolved name of the region.

		while (Cmd && Cmd[0] != TEXT('\0'))
		{
			FString Token;
			if (FParse::Token(Cmd, Token, bUseEscape))
			{
				Ar.Logf(TEXT("  Token: %s"), *Token);

				static constexpr TCHAR ColumnsToken[] = TEXT("-columns=");
				static constexpr TCHAR ThreadsToken[] = TEXT("-threads=");
				static constexpr TCHAR TimersToken[] = TEXT("-timers=");
				static constexpr TCHAR StartTimeToken[] = TEXT("-startTime=");
				static constexpr TCHAR EndTimeToken[] = TEXT("-endTime=");
				static constexpr TCHAR RegionToken[] = TEXT("-region=");

				if (Token.StartsWith(ColumnsToken))
				{
					// Comma-delimited list of column names. Supports *?-type wildcard.
					// Default: -columns="ThreadId,TimerId,StartTime,EndTime,Depth"
					// Example: -columns="*" -columns="TimerName,Duration"
					Token.RightChopInline(UE_ARRAY_COUNT(ColumnsToken) - 1);
					Token.TrimQuotesInline();
					Exporter.MakeExportTimingEventsColumnList(Token, Columns);
					Params.Columns = &Columns;
				}
				else if (Token.StartsWith(ThreadsToken))
				{
					// Comma-delimited list of thread names. Supports *?-type wildcard.
					// Default: -threads="*" (all threads; no filter)
					// Example: -threads="GameThread" -threads="GPU1,GPU2,GameThread,Render*"
					Token.RightChopInline(UE_ARRAY_COUNT(ThreadsToken) - 1);
					Token.TrimQuotesInline();
					Params.ThreadFilter = Exporter.MakeThreadFilterInclusive(Token, IncludedThreads);
				}
				else if (Token.StartsWith(TimersToken))
				{
					// Comma-delimited list of timer names. Supports *?-type wildcard.
					// Default: -timers="*" (all timers; no filter)
					// Example: -timers="A,B,*z"
					Token.RightChopInline(UE_ARRAY_COUNT(TimersToken) - 1);
					Token.TrimQuotesInline();
					Params.TimingEventFilter = Exporter.MakeTimingEventFilterByTimersInclusive(Token, IncludedTimers);
				}
				else if (Token.StartsWith(StartTimeToken))
				{
					// Default: -startTime=-infinite
					// Example: -startTime=10.0
					Token.RightChopInline(UE_ARRAY_COUNT(StartTimeToken) - 1);
					Params.IntervalStartTime = FCString::Atof(*Token);
				}
				else if (Token.StartsWith(EndTimeToken))
				{
					// Default: -endTime=+infinite
					// Example: -endTime=20.0
					Token.RightChopInline(UE_ARRAY_COUNT(EndTimeToken) - 1);
					Params.IntervalEndTime = FCString::Atof(*Token);
				}
				else if (Token.StartsWith(RegionToken))
				{
					// Comma-delimited list of region names. Supports *?-type wildcard.
					// Each region is exported to a separate file.
					// '{region}' in Filename (if any) will be replaced with the resolved name of the region.
					// Default: -Region=
					// Example: -Region="RegionName,Game_*,OtherRegion"
					Token.RightChopInline(UE_ARRAY_COUNT(RegionToken) - 1);
					Token.TrimQuotesInline();
					Params.Region = TCHAR_TO_ANSI(*Token);
				}
				else
				{
					Ar.Logf(ELogVerbosity::Warning, TEXT("Unknown Cmd Param: %s"), *Token);
				}
			}
		}

		//////////////////////////////////////////////////

		Exporter.ExportTimingEventsAsText(Filename, Params);
		return true;
	}

	if (FParse::Command(&Cmd, TEXT("TimingInsights.ExportTimerStatistics")))
	{
		Ar.Logf(TEXT("TimingInsights.ExportTimerStatistics %s"), Cmd);

		check(FInsightsManager::Get().IsValid() && FInsightsManager::Get()->GetSession().IsValid());
		FTimingExporter Exporter(*FInsightsManager::Get()->GetSession().Get());
		FTimingExporter::FExportTimerStatisticsParams Params; // default (all timing events)

		// These variables needs to be in the same scope with the call to Exporter.ExportTimerStatisticsAsText().
		TArray<FName> Columns; // referenced by Params.Columns
		TSet<uint32> IncludedThreads; // referenced in Params.ThreadFilter lambda function
		TSet<uint32> IncludedTimers; // referenced in Params.TimingEventFilter lambda function

		//////////////////////////////////////////////////

		const bool bUseEscape = true;
		FString Filename = FParse::Token(Cmd, bUseEscape);
		Filename.TrimQuotesInline();
		Ar.Logf(TEXT("  Filename: \"%s\""), *Filename);
		// '{region}' in Filename (if any) will be replaced with the resolved name of the region.

		while (Cmd && Cmd[0] != TEXT('\0'))
		{
			FString Token;
			if (FParse::Token(Cmd, Token, bUseEscape))
			{
				Ar.Logf(TEXT("  Token: %s"), *Token);

				static constexpr TCHAR ColumnsToken[] = TEXT("-columns=");
				static constexpr TCHAR ThreadsToken[] = TEXT("-threads=");
				static constexpr TCHAR TimersToken[] = TEXT("-timers=");
				static constexpr TCHAR StartTimeToken[] = TEXT("-startTime=");
				static constexpr TCHAR EndTimeToken[] = TEXT("-endTime=");
				static constexpr TCHAR RegionToken[] = TEXT("-region=");
				static constexpr TCHAR MaxTimerCountToken[] = TEXT("-maxTimerCount=");
				static constexpr TCHAR SortByToken[] = TEXT("-sortBy=");
				static constexpr TCHAR SortOrderToken[] = TEXT("-sortOrder=");

				if (Token.StartsWith(ColumnsToken))
				{
					// Comma-delimited list of column names. Supports *?-type wildcard.
					// Default: -columns="ThreadId,TimerId,StartTime,EndTime,Depth"
					// Example: -columns="*" -columns="TimerName,Duration"
					Token.RightChopInline(UE_ARRAY_COUNT(ColumnsToken) - 1);
					Token.TrimQuotesInline();
					Exporter.MakeExportTimingEventsColumnList(Token, Columns);
					Params.Columns = &Columns;
				}
				else if (Token.StartsWith(ThreadsToken))
				{
					// Comma-delimited list of thread names. Supports *?-type wildcard.
					// Default: -threads="*" (all threads; no filter)
					// Example: -threads="GameThread" -threads="GPU1,GPU2,GameThread,Render*"
					Token.RightChopInline(UE_ARRAY_COUNT(ThreadsToken) - 1);
					Token.TrimQuotesInline();
					Params.ThreadFilter = Exporter.MakeThreadFilterInclusive(Token, IncludedThreads);
				}
				else if (Token.StartsWith(TimersToken))
				{
					// Comma-delimited list of timer names. Supports *?-type wildcard.
					// Default: -timers="*" (all timers; no filter)
					// Example: -timers="A,B,*z"
					Token.RightChopInline(UE_ARRAY_COUNT(TimersToken) - 1);
					Token.TrimQuotesInline();
					Params.TimingEventFilter = Exporter.MakeTimingEventFilterByTimersInclusive(Token, IncludedTimers);
				}
				else if (Token.StartsWith(StartTimeToken))
				{
					// Default: -startTime=-infinite
					// Example: -startTime=10.0
					Token.RightChopInline(UE_ARRAY_COUNT(StartTimeToken) - 1);
					Params.IntervalStartTime = FCString::Atof(*Token);
				}
				else if (Token.StartsWith(EndTimeToken))
				{
					// Default: -endTime=+infinite
					// Example: -endTime=20.0
					Token.RightChopInline(UE_ARRAY_COUNT(EndTimeToken) - 1);
					Params.IntervalEndTime = FCString::Atof(*Token);
				}
				else if (Token.StartsWith(RegionToken))
				{
					// Comma-delimited list of region names. Supports *?-type wildcard.
					// Each region is exported to a separate file.
					// '{region}' in Filename (if any) will be replaced with the resolved name of the region.
					// Default: -Region=
					// Example: -Region="RegionName,Game_*,OtherRegion"
					Token.RightChopInline(UE_ARRAY_COUNT(RegionToken) - 1);
					Token.TrimQuotesInline();
					Params.Region = TCHAR_TO_ANSI(*Token);
				}
				else if (Token.StartsWith(MaxTimerCountToken))
				{
					// Integer limiting the number of timers to export
					// Example: -MaxTimerCount=100
					Token.RightChopInline(UE_ARRAY_COUNT(MaxTimerCountToken) - 1);
					Params.MaxExportedEvents = FCString::Atoi(*Token);
				}
				else if (Token.StartsWith(SortByToken))
				{
					static constexpr TCHAR TotalInclusiveTimeToken[] = TEXT("TotalInclusiveTime");

					// Choice of field to sort the exported timers by
					// Example: -sortBy=TotalInclusiveTime
					Token.RightChopInline(UE_ARRAY_COUNT(SortByToken) - 1);
					Token.TrimQuotesInline();
					if (Token.Equals(TotalInclusiveTimeToken, ESearchCase::IgnoreCase))
					{
						Params.SortBy = FTimingExporter::FExportTimerStatisticsParams::ESortBy::TotalInclusiveTime;
						// default to descending order to avoid needing to pass
						if (Params.SortOrder == FTimingExporter::FExportTimerStatisticsParams::ESortOrder::DontSort)
						{
							Params.SortOrder = FTimingExporter::FExportTimerStatisticsParams::ESortOrder::Descending;
						}
					}
					else
					{
						Ar.Logf(ELogVerbosity::Warning, TEXT("Unsupported sortBy value: %s"), *Token);
					}
				}
				else if (Token.StartsWith(SortOrderToken))
				{
					static constexpr TCHAR DescendingToken[] = TEXT("Descending");
					static constexpr TCHAR AscendingToken[] = TEXT("Ascending");

					// Sorting order for the exported timers.
					// Example: -sortOrder=Descending
					Token.RightChopInline(UE_ARRAY_COUNT(SortOrderToken) - 1);
					Token.TrimQuotesInline();
					if (Token.Equals(DescendingToken, ESearchCase::IgnoreCase))
					{
						Params.SortOrder = FTimingExporter::FExportTimerStatisticsParams::ESortOrder::Descending;
					}
					else if (Token.Equals(AscendingToken, ESearchCase::IgnoreCase))
					{
						Params.SortOrder = FTimingExporter::FExportTimerStatisticsParams::ESortOrder::Ascending;
					}
					else
					{
						Ar.Logf(ELogVerbosity::Warning, TEXT("Unsupported sortOrder value: %s"), *Token);
					}
				}
				else
				{
					Ar.Logf(ELogVerbosity::Warning, TEXT("Unknown Cmd Param: %s"), *Token);
				}
			}
		}

		//////////////////////////////////////////////////

		if (Params.ThreadFilter == nullptr)
		{
			Params.ThreadFilter = [](unsigned int){ return true; };
		}

		Exporter.ExportTimerStatisticsAsText(Filename, Params);
		return true;
	}

	if (FParse::Command(&Cmd, TEXT("TimingInsights.ExportTimerCallees")))
	{
		Ar.Logf(TEXT("TimingInsights.ExportTimerCallees %s"), Cmd);

		check(FInsightsManager::Get().IsValid() && FInsightsManager::Get()->GetSession().IsValid());
		FTimingExporter Exporter(*FInsightsManager::Get()->GetSession().Get());
		FTimingExporter::FExportTimerCalleesParams Params;

		// These variables needs to be in the same scope with the call to Exporter.ExportTimerStatisticsAsText().
		TSet<uint32> IncludedThreads; // referenced in Params.ThreadFilter lambda function

		//////////////////////////////////////////////////

		const bool bUseEscape = true;
		FString Filename = FParse::Token(Cmd, bUseEscape);
		Filename.TrimQuotesInline();
		Ar.Logf(TEXT("  Filename: \"%s\""), *Filename);
		// '{region}' in Filename (if any) will be replaced with the resolved name of the region.

		while (Cmd && Cmd[0] != TEXT('\0'))
		{
			FString Token;
			if (FParse::Token(Cmd, Token, bUseEscape))
			{
				Ar.Logf(TEXT("  Token: %s"), *Token);

				static constexpr TCHAR ThreadsToken[] = TEXT("-threads=");
				static constexpr TCHAR TimersToken[] = TEXT("-timers=");
				static constexpr TCHAR StartTimeToken[] = TEXT("-startTime=");
				static constexpr TCHAR EndTimeToken[] = TEXT("-endTime=");
				static constexpr TCHAR RegionToken[] = TEXT("-region=");
				static constexpr TCHAR ShowFormatInTimerNamesToken[] = TEXT("-ShowFormatInTimerNames=");

				if (Token.StartsWith(ThreadsToken))
				{
					// Comma-delimited list of thread names. Supports *?-type wildcard.
					// Default: -threads="*" (all threads; no filter)
					// Example: -threads="GameThread" -threads="GPU1,GPU2,GameThread,Render*"
					Token.RightChopInline(UE_ARRAY_COUNT(ThreadsToken) - 1);
					Token.TrimQuotesInline();
					Params.ThreadFilter = Exporter.MakeThreadFilterInclusive(Token, IncludedThreads);
				}
				else if (Token.StartsWith(TimersToken))
				{
					// Comma-delimited list of timer names. Supports *?-type wildcard.
					// Default: -timers="*" (all timers; no filter)
					// Example: -timers="A,B,*z"
					Token.RightChopInline(UE_ARRAY_COUNT(TimersToken) - 1);
					Token.TrimQuotesInline();
					Exporter.MakeTimingEventFilterByTimersInclusive(Token, Params.TimerIds);
				}
				else if (Token.StartsWith(StartTimeToken))
				{
					// Default: -startTime=-infinite
					// Example: -startTime=10.0
					Token.RightChopInline(UE_ARRAY_COUNT(StartTimeToken) - 1);
					Params.IntervalStartTime = FCString::Atof(*Token);
				}
				else if (Token.StartsWith(EndTimeToken))
				{
					// Default: -endTime=+infinite
					// Example: -endTime=20.0
					Token.RightChopInline(UE_ARRAY_COUNT(EndTimeToken) - 1);
					Params.IntervalEndTime = FCString::Atof(*Token);
				}
				else if (Token.StartsWith(RegionToken))
				{
					// Comma-delimited list of region names. Supports *?-type wildcard.
					// Each region is exported to a separate file.
					// '{region}' in Filename (if any) will be replaced with the resolved name of the region.
					// Default: -Region=
					// Example: -Region="RegionName,Game_*,OtherRegion"
					Token.RightChopInline(UE_ARRAY_COUNT(RegionToken) - 1);
					Token.TrimQuotesInline();
					Params.Region = TCHAR_TO_ANSI(*Token);
				}
				else if (Token.StartsWith(ShowFormatInTimerNamesToken))
				{
					// Default: -ShowFormatInTimerNames=
					// Example: -ShowFormatInTimerNames="False"
					Token.RightChopInline(UE_ARRAY_COUNT(ShowFormatInTimerNamesToken) - 1);
					Token.TrimQuotesInline();
					Params.bShowFormatInTimerNames = Token.ToBool();
				}
				else
				{
					Ar.Logf(ELogVerbosity::Warning, TEXT("Unknown Cmd Param: %s"), *Token);
				}
			}
		}

		//////////////////////////////////////////////////

		if (Params.ThreadFilter == nullptr)
		{
			Params.ThreadFilter = [](unsigned int) { return true; };
		}

		Exporter.ExportTimerCalleesAsText(Filename, Params);
		return true;
	}

	if (FParse::Command(&Cmd, TEXT("TimingInsights.ExportCounters")))
	{
		Ar.Logf(TEXT("TimingInsights.ExportCounters %s"), Cmd);
		check(FInsightsManager::Get().IsValid() && FInsightsManager::Get()->GetSession().IsValid());
		FTimingExporter Exporter(*FInsightsManager::Get()->GetSession().Get());
		FTimingExporter::FExportCountersParams Params; // default
		Exporter.ExportCountersAsText(Cmd, Params);
		return true;
	}

	if (FParse::Command(&Cmd, TEXT("TimingInsights.ExportCounterValues")))
	{
		Ar.Logf(TEXT("TimingInsights.ExportCounterValues %s"), Cmd);
		check(FInsightsManager::Get().IsValid() && FInsightsManager::Get()->GetSession().IsValid());
		FTimingExporter Exporter(*FInsightsManager::Get()->GetSession().Get());
		FTimingExporter::FExportCounterParams Params; // default

		TArray<FString> Counters; // list of counters to export

		// These variables needs to be in the same scope with the call to Exporter.ExportCounterAsText().
		TArray<FName> Columns; // referenced by Params.Columns

		//////////////////////////////////////////////////

		const bool bUseEscape = true;
		FString Filename = FParse::Token(Cmd, bUseEscape);
		Filename.TrimQuotesInline();
		Ar.Logf(TEXT("  Filename: \"%s\""), *Filename);
		// '{counter}' in Filename (if any) will be replaced with the name of the counter.
		// '{region}' in Filename (if any) will be replaced with the resolved name of the region.

		while (Cmd && Cmd[0] != TEXT('\0'))
		{
			FString Token;
			if (FParse::Token(Cmd, Token, bUseEscape))
			{
				static constexpr TCHAR CounterToken[] = TEXT("-counter=");
				static constexpr TCHAR ColumnsToken[] = TEXT("-columns=");
				static constexpr TCHAR StartTimeToken[] = TEXT("-startTime=");
				static constexpr TCHAR EndTimeToken[] = TEXT("-endTime=");
				static constexpr TCHAR RegionToken[] = TEXT("-region=");

				if (Token.StartsWith(CounterToken))
				{
					// Comma-delimited list of counter names. Supports *?-type wildcard.
					// Default: -counter=
					// Example: -counter="PC / *"
					Token.RightChopInline(UE_ARRAY_COUNT(CounterToken) - 1);
					Token.TrimQuotesInline();
					Token.ParseIntoArray(Counters, TEXT(","), true);
				}
				else if (Token.StartsWith(ColumnsToken))
				{
					// Comma-delimited list of column names. Supports *?-type wildcard.
					// Default: -columns="Time,Value"
					// Example: -columns="*" -columns="Value"
					Token.RightChopInline(UE_ARRAY_COUNT(ColumnsToken) - 1);
					Token.TrimQuotesInline();
					Exporter.MakeExportTimingEventsColumnList(Token, Columns);
					Params.Columns = &Columns;
				}
				else if (Token.StartsWith(StartTimeToken))
				{
					// Default: -startTime=-infinite
					// Example: -startTime=10.0
					Token.RightChopInline(UE_ARRAY_COUNT(StartTimeToken) - 1);
					Params.IntervalStartTime = FCString::Atof(*Token);
				}
				else if (Token.StartsWith(EndTimeToken))
				{
					// Default: -endTime=+infinite
					// Example: -endTime=20.0
					Token.RightChopInline(UE_ARRAY_COUNT(EndTimeToken) - 1);
					Params.IntervalEndTime = FCString::Atof(*Token);
				}
				else if (Token.StartsWith(RegionToken))
				{
					// Comma-delimited list of region names. Supports *?-type wildcard.
					// Each region is exported to a separate file.
					// '{region}' in Filename (if any) will be replaced with the resolved name of the region.
					// Default: -Region=
					// Example: -Region="RegionName,Game_*,OtherRegion"
					Token.RightChopInline(UE_ARRAY_COUNT(RegionToken) - 1);
					Token.TrimQuotesInline();
					Params.Region = TCHAR_TO_ANSI(*Token);
				}
				else
				{
					Ar.Logf(ELogVerbosity::Warning, TEXT("Unknown Cmd Param: %s"), *Token);
				}
			}
		}

		struct FExportCounterInfo
		{
			uint32 Id;
			FString Name;
		};
		TArray<FExportCounterInfo> CountersToExport;

		if (Counters.Num() > 0)
		{
			for (const FString& CounterWildcard : Counters)
			{
				Ar.Logf(TEXT("  Searching counters with name: \"%s\""), *CounterWildcard);
			}
			const TraceServices::IAnalysisSession& Session = *FInsightsManager::Get()->GetSession().Get();
			TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);
			const TraceServices::ICounterProvider& CounterProvider = TraceServices::ReadCounterProvider(Session);
			CounterProvider.EnumerateCounters([&Ar, &Counters, &CountersToExport](uint32 CounterId, const TraceServices::ICounter& Counter)
			{
				FString CounterName(Counter.GetName());
				for (const FString& CounterWildcard : Counters)
				{
					if (CounterName.MatchesWildcard(CounterWildcard))
					{
						CountersToExport.Add({ CounterId, CounterName });
						break;
					}
				}
			});
		}

		Ar.Logf(TEXT("  Exporting values for %d counters..."), CountersToExport.Num());
		for (const FExportCounterInfo& Counter : CountersToExport)
		{
			Ar.Logf(TEXT("  Exporting counter: \"%s\" (id=%d)"), *Counter.Name, Counter.Id);
			Exporter.ExportCounterAsText(Filename, Counter.Id, Params);
		}

		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::SetEventDepthLimitMin(uint32 InEventDepthLimitMin)
{
	EventDepthLimitMin = FMath::Min<uint32>(InEventDepthLimitMin, MaxEventDepthLimitMin);

	// Persistent option. Save it to the config file.
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
	Settings.SetAndSaveTimingTrackDepthLimitMin(static_cast<int32>(EventDepthLimitMin));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingProfilerManager::SetEventDepthLimitMax(uint32 InEventDepthLimitMax)
{
	EventDepthLimitMax = FMath::Clamp<uint32>(InEventDepthLimitMax, 1u, UnlimitedEventDepth);

	// Persistent option. Save it to the config file.
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
	Settings.SetAndSaveTimingTrackDepthLimitMax(static_cast<int32>(EventDepthLimitMax));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FUserAnnotationsTimingViewExtender* FTimingProfilerManager::GetUserAnnotationsExtender()
{
	if (!CachedUserAnnotationsExtender)
	{
		FTraceInsightsModule* Module = static_cast<FTraceInsightsModule*>(
			FModuleManager::GetModulePtr<IUnrealInsightsModule>("TraceInsights"));
		if (Module)
		{
			CachedUserAnnotationsExtender = &Module->GetUserAnnotationsExtender();
		}
	}
	return CachedUserAnnotationsExtender;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
