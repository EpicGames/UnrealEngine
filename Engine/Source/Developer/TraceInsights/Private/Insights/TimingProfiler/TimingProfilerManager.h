// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Ticker.h"
#include "Framework/Commands/UICommandList.h"
#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Common/AvailabilityCheck.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/TimingProfiler/TimingProfilerCommands.h"
#include "Insights/TimingProfilerCommon.h"

namespace UE::Insights
{
	enum class ETimingEventsColoringMode : uint32
	{
		ByTimerName,
		ByTimerNameAndMetadata,
		ByTimerId,
		BySourceFile,
		ByDuration,

		Count
	};

	class FLogFilter;
}

namespace UE::Insights::TimingProfiler
{

class FTimerNode;
class FTimerButterflyAggregator;
class FUserAnnotationsTimingViewExtender;
class STimingProfilerWindow;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * This class manages the Timing Profiler (Timing Insights) state and settings.
 */
class FTimingProfilerManager : public TSharedFromThis<FTimingProfilerManager>, public IInsightsComponent
{
	friend class FTimingProfilerActionManager;

public:
	inline static constexpr uint32 MaxEventDepthLimitMin = 100; // maximum value for the EventDepthLimitMin
	inline static constexpr uint32 UnlimitedEventDepth = 1000;

public:
	/** Creates the Timing Profiler manager, only one instance can exist. */
	FTimingProfilerManager(TSharedRef<FUICommandList> InCommandList);

	/** Virtual destructor. */
	virtual ~FTimingProfilerManager();

	/** Creates an instance of the Timing Profiler manager. */
	static TSharedPtr<FTimingProfilerManager> CreateInstance();

	/**
	 * @return the global instance of the Timing Profiler manager.
	 * This is an internal singleton and cannot be used outside TraceInsights.
	 * For external use:
	 *     IUnrealInsightsModule& Module = FModuleManager::Get().LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	 *     Module.GetTimingProfiler();
	 */
	static TSharedPtr<FTimingProfilerManager> Get();

	bool IsAvailable() const { return bIsAvailable; }

	//////////////////////////////////////////////////
	// IInsightsComponent

	virtual void Initialize(IUnrealInsightsModule& InsightsModule) override;
	virtual void Shutdown() override;
	virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) override;
	virtual void UnregisterMajorTabs() override;
	virtual void OnWindowClosedEvent() override;
	virtual bool Exec(const TCHAR* Cmd, FOutputDevice& Ar) override;

	//////////////////////////////////////////////////

	/** @return UI command list for the Timing Profiler manager. */
	const TSharedRef<FUICommandList> GetCommandList() const;

	/** @return an instance of the Timing Profiler commands. */
	static const FTimingProfilerCommands& GetCommands();

	/** @return an instance of the Timing Profiler action manager. */
	static FTimingProfilerActionManager& GetActionManager();

	/**
	 * Converts profiler window weak pointer to a shared pointer and returns it.
	 * Make sure the returned pointer is valid before trying to dereference it.
	 */
	TSharedPtr<STimingProfilerWindow> GetProfilerWindow() const
	{
		return ProfilerWindowWeakPtr.Pin();
	}

	const FName& GetMajorTabId() const { return MajorTabId; }
	const FName& GetLogListingName() const { return LogListingName; }

	const TSharedRef<FLogFilter>& GetLogFilter() const { return Filter; }

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Getters and setters used by Toggle Commands.

	/** @return true, if the Frames track/view is visible */
	const bool IsFramesTrackVisible() const { return bIsFramesTrackVisible; }
	void SetFramesTrackVisible(const bool bIsVisible) { bIsFramesTrackVisible = bIsVisible; }
	void ShowHideFramesTrack(const bool bIsVisible);

	/** @return true, if the Timing view is visible */
	const bool IsTimingViewVisible() const { return bIsTimingViewVisible; }
	void SetTimingViewVisible(const bool bIsVisible) { bIsTimingViewVisible = bIsVisible; }
	void ShowHideTimingView(const bool bIsVisible);

	/** @return true, if the Timers view is visible */
	const bool IsTimersViewVisible() const { return bIsTimersViewVisible; }
	void SetTimersViewVisible(const bool bIsVisible) { bIsTimersViewVisible = bIsVisible; }
	void ShowHideTimersView(const bool bIsVisible);

	/** @return true, if the Callers tree view is visible */
	const bool IsCallersTreeViewVisible() const { return bIsCallersTreeViewVisible; }
	void SetCallersTreeViewVisible(const bool bIsVisible) { bIsCallersTreeViewVisible = bIsVisible; }
	void ShowHideCallersTreeView(const bool bIsVisible);

	/** @return true, if the Callees tree view is visible */
	const bool IsCalleesTreeViewVisible() const { return bIsCalleesTreeViewVisible; }
	void SetCalleesTreeViewVisible(const bool bIsVisible) { bIsCalleesTreeViewVisible = bIsVisible; }
	void ShowHideCalleesTreeView(const bool bIsVisible);

	/** @return true, if the Counters view is visible */
	const bool IsStatsCountersViewVisible() const { return bIsStatsCountersViewVisible; }
	void SetStatsCountersViewVisible(const bool bIsVisible) { bIsStatsCountersViewVisible = bIsVisible; }
	void ShowHideStatsCountersView(const bool bIsVisible);

	/** @return true, if the Log view is visible */
	const bool IsLogViewVisible() const { return bIsLogViewVisible; }
	void SetLogViewVisible(const bool bIsVisible) { bIsLogViewVisible = bIsVisible; }
	void ShowHideLogView(const bool bIsVisible);

	/** @return true, if the Modules view is visible */
	const bool IsModulesViewVisible() const { return bIsModulesViewVisible; }
	void SetModulesViewVisible(const bool bIsVisible) { bIsModulesViewVisible = bIsVisible; }
	void ShowHideModulesView(const bool bIsVisible);

	/** @return true, if the User Annotations view is visible */
	const bool IsUserAnnotationsViewVisible() const { return bIsUserAnnotationsViewVisible; }
	void SetUserAnnotationsViewVisible(const bool bIsVisible) { bIsUserAnnotationsViewVisible = bIsVisible; }
	void ShowHideUserAnnotationsView(const bool bIsVisible);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void ActivateMajorTab();
	void ActivateView();

	void OnSessionChanged();

	bool IsValidTimeSelection() const { return SelectionStartTime < SelectionEndTime; }
	double GetSelectionStartTime() const { return SelectionStartTime; }
	double GetSelectionEndTime() const { return SelectionEndTime; }
	void SetSelectedTimeRange(double StartTime, double EndTime);

	TSharedPtr<FTimerNode> GetTimerNode(uint32 TimerId) const;
	static constexpr uint32 GetInvalidTimerId() { return InvalidTimerId; }
	bool IsValidSelectedTimer() const { return SelectedTimerId != InvalidTimerId; }
	uint32 GetSelectedTimer() const { return SelectedTimerId; }
	void SetSelectedTimer(uint32 TimerId);
	void ToggleTimingViewMainGraphEventSeries(uint32 InTimerId);

	void OnThreadFilterChanged();

	void ResetCallersAndCallees();
	void UpdateCallersAndCallees();
	TSharedRef<FTimerButterflyAggregator> GetTimerButterflyAggregator() const { return TimerButterflyAggregator; }

	void UpdateAggregatedTimerStats();
	void UpdateAggregatedCounterStats();

	ETimingEventsColoringMode GetColoringMode() const { return ColoringMode; }
	void SetColoringMode(ETimingEventsColoringMode InColoringMode) { ColoringMode = InColoringMode; }

	uint32 GetEventDepthLimitMin() const { return EventDepthLimitMin; }
	void SetEventDepthLimitMin(uint32 InEventDepthLimit);

	uint32 GetEventDepthLimitMax() const { return EventDepthLimitMax; }
	uint32 GetEventDepthLimit() const { return EventDepthLimitMax; } // for backward compatibility
	void SetEventDepthLimitMax(uint32 InEventDepthLimit);

	/** Cached accessor for the annotations extender. Lookup happens once; pointer is stable
	 *  for process lifetime (same assumption as STimingView::CachedAnnotationExtender). */
	UE::Insights::TimingProfiler::FUserAnnotationsTimingViewExtender* GetUserAnnotationsExtender();

private:
	/** Binds our UI commands to delegates. */
	void BindCommands();

	/** Called to spawn the Timing Profiler major tab. */
	TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args);

	bool CanSpawnTab(const FSpawnTabArgs& Args) const;

	/** Callback called when the Timing Profiler major tab is closed. */
	void OnTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	/** Updates this manager, done through FCoreTicker. */
	bool Tick(float DeltaTime);

	void FinishTimerButterflyAggregation();

	void AssignProfilerWindow(const TSharedRef<STimingProfilerWindow>& InProfilerWindow)
	{
		ProfilerWindowWeakPtr = InProfilerWindow;
	}

	void RemoveProfilerWindow()
	{
		ProfilerWindowWeakPtr.Reset();
	}

private:
	bool bIsInitialized = false;
	bool bIsAvailable = false;

	/** If the tab is registered in the tab manager. */
	bool bIsTabRegistered = false;

	FAvailabilityCheck AvailabilityCheck;

	/** The delegate to be invoked when this manager ticks. */
	FTickerDelegate OnTick;

	/** Handle to the registered OnTick. */
	FTSTicker::FDelegateHandle OnTickHandle;

	/** List of UI commands for this manager. This will be filled by this and corresponding classes. */
	TSharedRef<FUICommandList> CommandList;

	/** An instance of the Timing Profiler action manager. */
	FTimingProfilerActionManager ActionManager;

	/** A weak pointer to the Timing Insights window. */
	TWeakPtr<STimingProfilerWindow> ProfilerWindowWeakPtr;

	/** A reference to the log filter state. */
	TSharedRef<FLogFilter> Filter;

	/** If the Frames Track is visible or hidden. */
	bool bIsFramesTrackVisible = false;

	/** If the Timing View is visible or hidden. */
	bool bIsTimingViewVisible = false;

	/** If the Timers View is visible or hidden. */
	bool bIsTimersViewVisible = false;

	/** If the Callers Tree View is visible or hidden. */
	bool bIsCallersTreeViewVisible = false;

	/** If the Callees Tree View is visible or hidden. */
	bool bIsCalleesTreeViewVisible = false;

	/** If the Stats Counters View is visible or hidden. */
	bool bIsStatsCountersViewVisible = false;

	/** If the Log View is visible or hidden. */
	bool bIsLogViewVisible = false;

	/** If the Modules View is visible or hidden. */
	bool bIsModulesViewVisible = false;

	/** If the User Annotations View is visible or hidden. */
	bool bIsUserAnnotationsViewVisible = false;

	//////////////////////////////////////////////////

	double SelectionStartTime = 0.0;
	double SelectionEndTime = 0.0;

	static constexpr uint32 InvalidTimerId = uint32(-1);
	uint32 SelectedTimerId = InvalidTimerId;

	TSharedRef<FTimerButterflyAggregator> TimerButterflyAggregator;

	/** The id of the Timing Insights major tab. */
	FName MajorTabId;

	/** The name of the Timing Profiler log listing. */
	FName LogListingName;

	ETimingEventsColoringMode ColoringMode = ETimingEventsColoringMode::ByTimerName;
	uint32 EventDepthLimitMin = 0;
	uint32 EventDepthLimitMax = UnlimitedEventDepth;

	/** Cached process-lifetime extender pointer. See GetUserAnnotationsExtender(). */
	UE::Insights::TimingProfiler::FUserAnnotationsTimingViewExtender* CachedUserAnnotationsExtender = nullptr;

	/** A shared pointer to the global instance of the Timing Profiler manager. */
	static TSharedPtr<FTimingProfilerManager> Instance;
};

} // namespace UE::Insights::TimingProfiler
