// Copyright Epic Games, Inc. All Rights Reserved.

#include "CpuStackSampleTimingTrack.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/StackSamples.h"

// TraceInsights
#include "Insights/InsightsManager.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::CpuStackSampleTimingTrack"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FCpuStackSampleTimingTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FCpuStackSampleTimingTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FCpuStackSampleTimingTrack::ReadTimeline(TFunctionRef<void(const TraceServices::ITimingProfilerProvider::Timeline&)> Callback) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	const TraceServices::IStackSamplesProvider* StackSamplesProvider = TraceServices::ReadStackSamplesProvider(*Session.Get());
	if (StackSamplesProvider)
	{
		TraceServices::FProviderReadScopeLock _(*StackSamplesProvider);
		const TraceServices::ITimingProfilerProvider::Timeline* Timeline = StackSamplesProvider->GetTimeline(GetTimelineIndex());
		if (Timeline)
		{
			Callback(*Timeline);
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
