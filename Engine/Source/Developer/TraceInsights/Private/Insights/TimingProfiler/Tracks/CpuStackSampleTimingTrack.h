// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/TimingProfiler/Tracks/ThreadTimingTrackPrivate.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FCpuStackSampleTimingTrack : public FThreadTimingTrackImpl
{
	INSIGHTS_DECLARE_RTTI(FCpuStackSampleTimingTrack, FThreadTimingTrackImpl)

public:
	explicit FCpuStackSampleTimingTrack(FThreadTimingSharedState& InSharedState, const FString& InName, uint32 InSystemThreadId)
		: FThreadTimingTrackImpl(InSharedState, InName, nullptr, InSystemThreadId, InSystemThreadId)
	{
	}

protected:
	virtual bool ReadTimeline(TFunctionRef<void(const TraceServices::ITimingProfilerProvider::Timeline&)> Callback) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
