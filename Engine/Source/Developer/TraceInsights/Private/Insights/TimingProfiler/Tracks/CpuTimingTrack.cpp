// Copyright Epic Games, Inc. All Rights Reserved.

#include "CpuTimingTrack.h"

// TraceServices
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/TasksProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/ThreadTimingSharedState.h"
#include "Insights/ViewModels/ThreadTrackEvent.h"
#include "Insights/ViewModels/TooltipDrawState.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::CpuTimingTrack"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FCpuTimingTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FCpuTimingTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuTimingTrack::AddTaskInfo(FTooltipDrawState& InOutTooltip, const TraceServices::FTaskInfo& Task) const
{
	InOutTooltip.AddTextLine(FString::Printf(TEXT("-------- Task %" UINT64_FMT "%s --------"), Task.Id, Task.bTracked ? TEXT("") : TEXT(" (not tracked)")), FLinearColor::Green);

	if (Task.DebugName != nullptr)
	{
		InOutTooltip.AddTextLine(FString::Printf(TEXT("%s"), Task.DebugName), FLinearColor::Green);
	}

	ENamedThreads::Type ThreadInfo = (ENamedThreads::Type)Task.ThreadToExecuteOn;
	ENamedThreads::Type ThreadIndex = ENamedThreads::GetThreadIndex(ThreadInfo);

	auto FormatTaskTimestamp = [](double Timestamp) -> FString
	{
		return (Timestamp != TraceServices::FTaskInfo::InvalidTimestamp) ? FString::SanitizeFloat(Timestamp) : TEXT("[not set]");
	};

	auto FormatTaskTime = [](double Time) -> FString
	{
		return FormatTimeAuto(Time, 2);
	};

	auto GetTrackName = [this](uint32 InThreadId) -> FString
	{
		TSharedPtr<FCpuTimingTrack> Track = GetSharedState().GetCpuTrack(InThreadId);
		return Track.IsValid() ? Track->GetName() : TEXT("Unknown");
	};

	const TCHAR* TaskPri = ENamedThreads::GetTaskPriority(ThreadInfo) == ENamedThreads::NormalTaskPriority ? TEXT("Normal") : TEXT("High");

	if (ThreadIndex == ENamedThreads::AnyThread)
	{
		int32 ThreadPriIndex = ENamedThreads::GetThreadPriorityIndex(ThreadInfo);
		const TCHAR* ThreadPriStrs[] = { TEXT("Normal"), TEXT("High"), TEXT("Low") };
		const TCHAR* ThreadPri = ensure(ThreadPriIndex >= 0 && ThreadPriIndex < 3) ? ThreadPriStrs[ThreadPriIndex] : TEXT("Unknown");

		InOutTooltip.AddTextLine(
			FString::Printf(TEXT("%s Pri task on %s Pri worker (%s)"), TaskPri, ThreadPri, *GetTrackName(Task.StartedThreadId)),
			FLinearColor::Green);
	}
	else
	{
		const TCHAR* QueueStr = ENamedThreads::GetQueueIndex(ThreadInfo) == ENamedThreads::MainQueue ? TEXT("Main") : TEXT("Local");
		InOutTooltip.AddTextLine(
			FString::Printf(TEXT("%s Pri task on %s (%s queue)"), TaskPri, *GetTrackName(Task.StartedThreadId), QueueStr),
			FLinearColor::Green);
	}

	InOutTooltip.AddNameValueTextLine(TEXTVIEW("Created:"), FString::Printf(TEXT("%s on %s"),
		*FormatTaskTimestamp(Task.CreatedTimestamp),
		*GetTrackName(Task.CreatedThreadId)));

	InOutTooltip.AddNameValueTextLine(TEXTVIEW("Launched:"), FString::Printf(TEXT("%s (+%s) on %s"),
		*FormatTaskTimestamp(Task.LaunchedTimestamp),
		*FormatTaskTime(Task.LaunchedTimestamp - Task.CreatedTimestamp),
		*GetTrackName(Task.LaunchedThreadId)));

	InOutTooltip.AddNameValueTextLine(TEXTVIEW("Scheduled:"), FString::Printf(TEXT("%s (+%s) on %s"),
		*FormatTaskTimestamp(Task.ScheduledTimestamp),
		*FormatTaskTime(Task.ScheduledTimestamp - Task.LaunchedTimestamp),
		*GetTrackName(Task.ScheduledThreadId)));

	InOutTooltip.AddNameValueTextLine(TEXTVIEW("Started:"), FString::Printf(TEXT("%s (+%s)"),
		*FormatTaskTimestamp(Task.StartedTimestamp),
		*FormatTaskTime(Task.StartedTimestamp - Task.ScheduledTimestamp)));

	if (Task.FinishedTimestamp != TraceServices::FTaskInfo::InvalidTimestamp)
	{
		InOutTooltip.AddNameValueTextLine(TEXTVIEW("Finished:"), FString::Printf(TEXT("%s (+%s)"),
			*FormatTaskTimestamp(Task.FinishedTimestamp),
			*FormatTaskTime(Task.FinishedTimestamp - Task.StartedTimestamp)));

		if (Task.CompletedTimestamp != TraceServices::FTaskInfo::InvalidTimestamp)
		{
			InOutTooltip.AddNameValueTextLine(TEXTVIEW("Completed:"), FString::Printf(TEXT("%s (+%s) on %s"),
				*FormatTaskTimestamp(Task.CompletedTimestamp),
				*FormatTaskTime(Task.CompletedTimestamp - Task.FinishedTimestamp),
				*GetTrackName(Task.CompletedThreadId)));

			if (Task.DestroyedTimestamp != TraceServices::FTaskInfo::InvalidTimestamp)
			{
				InOutTooltip.AddNameValueTextLine(TEXTVIEW("Destroyed:"), FString::Printf(TEXT("%s (+%s) on %s"),
					*FormatTaskTimestamp(Task.DestroyedTimestamp),
					*FormatTaskTime(Task.DestroyedTimestamp - Task.CompletedTimestamp),
					*GetTrackName(Task.DestroyedThreadId)));
			}
		}
	}

	InOutTooltip.AddNameValueTextLine(TEXTVIEW("Prerequisite tasks:"), FString::Printf(TEXT("%d"), Task.Prerequisites.Num()));
	InOutTooltip.AddNameValueTextLine(TEXTVIEW("Subsequent tasks:"), FString::Printf(TEXT("%d"), Task.Subsequents.Num()));
	InOutTooltip.AddNameValueTextLine(TEXTVIEW("Parent tasks:"), FString::Printf(TEXT("%d"), Task.ParentTasks.Num()));
	InOutTooltip.AddNameValueTextLine(TEXTVIEW("Nested tasks:"), FString::Printf(TEXT("%d"), Task.NestedTasks.Num()));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuTimingTrack::PostInitTooltip(FTooltipDrawState& InOutTooltip, const FThreadTrackEvent& TooltipEvent, const TraceServices::IAnalysisSession& Session, const TCHAR* TimerName) const
{
	// tasks
	const TraceServices::ITasksProvider* TasksProvider = TraceServices::ReadTasksProvider(Session);
	if (TasksProvider != nullptr)
	{
		// info about a task
		const TraceServices::FTaskInfo* Task = TasksProvider->TryGetTask(GetThreadId(), TooltipEvent.GetStartTime());
		if (Task != nullptr && Task->FinishedTimestamp >= TooltipEvent.GetEndTime())
		{
			AddTaskInfo(InOutTooltip, *Task);
		}

		// info about blocking
		const TraceServices::FWaitingForTasks* Waiting = TasksProvider->TryGetWaiting(TimerName, GetThreadId(), TooltipEvent.GetStartTime());
		if (Waiting != nullptr && Waiting->Tasks.Num() > 0)
		{
			InOutTooltip.AddTextLine(TEXT("-------- Waiting for tasks --------"), FLinearColor::Red);
			constexpr int32 NumIdsOnRow = 4;
			TStringBuilder<1024> StringBuilder;

			// Add the first line of Task Id values.
			for (int32 Index = 0; Index < Waiting->Tasks.Num() && Index < NumIdsOnRow; ++Index)
			{
				StringBuilder.Appendf(TEXT("%d, "), Waiting->Tasks[Index]);
			}

			// Remove separators from last entry.
			if (Waiting->Tasks.Num() <= NumIdsOnRow)
			{
				StringBuilder.RemoveSuffix(2);
			}
			InOutTooltip.AddNameValueTextLine(FString::Printf(TEXT("Tasks[%d]:"), Waiting->Tasks.Num()), StringBuilder.ToView());
			StringBuilder.Reset();

			// Add the rest of the lines with an empty name so they appear as a multi line value.
			for (int32 Index = NumIdsOnRow; Index < Waiting->Tasks.Num(); ++Index)
			{
				StringBuilder.Appendf(TEXT("%d, "), Waiting->Tasks[Index]);

				if ((Index + 1) % NumIdsOnRow == 0)
				{
					InOutTooltip.AddNameValueTextLine(TEXT(""), StringBuilder.ToView());
					StringBuilder.Reset();
				}
			}
			if (StringBuilder.Len() > 1)
			{
				StringBuilder.RemoveSuffix(2);
				InOutTooltip.AddNameValueTextLine(TEXT(""), StringBuilder.ToView());
			}

			InOutTooltip.AddNameValueTextLine(TEXT("Started waiting:"),
				FString::Printf(TEXT("%s"), *FString::SanitizeFloat(Waiting->StartedTimestamp)));

			if (Waiting->FinishedTimestamp != TraceServices::FTaskInfo::InvalidTimestamp)
			{
				InOutTooltip.AddNameValueTextLine(TEXT("Finished waiting:"),
					FString::Printf(TEXT("%s (+%s)"),
						*FString::SanitizeFloat(Waiting->FinishedTimestamp),
						*FormatTimeAuto(Waiting->FinishedTimestamp - Waiting->StartedTimestamp, 2)));
			}
			else
			{
				InOutTooltip.AddNameValueTextLine(TEXT("Finished waiting:"), TEXT("[not set]"));
			}

			const int32 MaxWaitedTasksToList = 5;
			int32 NumTasksToList = FMath::Min(Waiting->Tasks.Num(), MaxWaitedTasksToList);
			for (int32 TaskIndex = 0; TaskIndex != NumTasksToList; ++TaskIndex)
			{
				const TraceServices::FTaskInfo* WaitedTask = TasksProvider->TryGetTask(Waiting->Tasks[TaskIndex]);
				if (WaitedTask != nullptr)
				{
					AddTaskInfo(InOutTooltip , *WaitedTask);
				}
			}
			if (NumTasksToList < Waiting->Tasks.Num())
			{
				InOutTooltip.AddTextLine(TEXT("[...]"), FLinearColor::Green);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
