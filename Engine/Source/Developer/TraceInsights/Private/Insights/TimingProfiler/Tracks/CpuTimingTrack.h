// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/TimingProfiler/Tracks/ThreadTimingTrackPrivate.h"

namespace TraceServices
{
	struct FTaskInfo;
}

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FCpuTimingTrack : public FThreadTimingTrackImpl
{
	INSIGHTS_DECLARE_RTTI(FCpuTimingTrack, FThreadTimingTrackImpl)

public:
	explicit FCpuTimingTrack(FThreadTimingSharedState& InSharedState, const FString& InName, const TCHAR* InGroupName, uint32 InTimelineIndex, uint32 InThreadId)
		: FThreadTimingTrackImpl(InSharedState, InName, InGroupName, InTimelineIndex, InThreadId)
	{
	}

protected:
	virtual void PostInitTooltip(FTooltipDrawState& InOutTooltip, const FThreadTrackEvent& TooltipEvent, const TraceServices::IAnalysisSession& Session, const TCHAR* TimerName) const override;

private:
	void AddTaskInfo(FTooltipDrawState& InOutTooltip, const TraceServices::FTaskInfo& Task) const;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
