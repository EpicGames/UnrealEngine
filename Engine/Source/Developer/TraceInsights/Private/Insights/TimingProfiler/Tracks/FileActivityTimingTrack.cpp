// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileActivityTimingTrack.h"

#include "Algo/BinarySearch.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

// TraceServices
#include "TraceServices/Model/LoadTimeProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/FileActivitySharedState.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingViewDrawHelper.h"
#include "Insights/ViewModels/TooltipDrawState.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::FileActivity"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* GetFileActivityTypeName(TraceServices::EFileActivityType Type)
{
	static_assert(TraceServices::FileActivityType_Open == 0, "TraceServices::EFileActivityType enum has changed!?");
	static_assert(TraceServices::FileActivityType_ReOpen == 1, "TraceServices::EFileActivityType enum has changed!?");
	static_assert(TraceServices::FileActivityType_Close == 2, "TraceServices::EFileActivityType enum has changed!?");
	static_assert(TraceServices::FileActivityType_Read == 3, "TraceServices::EFileActivityType enum has changed!?");
	static_assert(TraceServices::FileActivityType_Write == 4, "TraceServices::EFileActivityType enum has changed!?");
	static_assert(TraceServices::FileActivityType_Count == 5, "TraceServices::EFileActivityType enum has changed!?");
	static const TCHAR* GFileActivityTypeNames[] =
	{
		TEXT("Open"),
		TEXT("ReOpen"),
		TEXT("Close"),
		TEXT("Read"),
		TEXT("Write"),
		TEXT("Idle"), // virtual events added for cases where Close event is more than 1s away from last Open/Read/Write event.
		TEXT("NotClosed") // virtual events added when an Open activity never closes
	};
	return GFileActivityTypeNames[Type];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 GetFileActivityTypeColor(TraceServices::EFileActivityType Type)
{
	static const uint32 GFileActivityTypeColors[] =
	{
		0xFFCCAA33, // open
		0xFFBB9922, // reopen
		0xFF33AACC, // close
		0xFF33AA33, // read
		0xFFDD33CC, // write
		0x55333333, // idle
		0x55553333, // close
	};
	return GFileActivityTypeColors[Type];
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FFileActivityTimingTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FFileActivityTimingTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileActivityTimingTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	InOutTooltip.ResetContent();

	if (InTooltipEvent.CheckTrack(this) && InTooltipEvent.Is<FTimingEvent>())
	{
		const FTimingEvent& TooltipEvent = InTooltipEvent.As<FTimingEvent>();

		auto MatchEvent = [this, &TooltipEvent](double InStartTime, double InEndTime, uint32 InDepth)
		{
			return InDepth == TooltipEvent.GetDepth()
				&& InStartTime == TooltipEvent.GetStartTime()
				&& InEndTime == TooltipEvent.GetEndTime();
		};

		FTimingEventSearchParameters SearchParameters(TooltipEvent.GetStartTime(), TooltipEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch, MatchEvent);
		FindIoTimingEvent(SearchParameters, [this, &InOutTooltip, &TooltipEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FIoTimingEvent& InEvent)
		{
			const TraceServices::EFileActivityType ActivityType = static_cast<TraceServices::EFileActivityType>(InEvent.Type & 0x0F);
			const bool bHasFailed = ((InEvent.Type & 0xF0) != 0);

			FString TypeStr;
			uint32 TypeColor;
			if (bHasFailed)
			{
				TypeStr = TEXT("Failed ");
				TypeStr += GetFileActivityTypeName(ActivityType);
				TypeColor = 0xFFFF3333;
			}
			else
			{
				TypeStr = GetFileActivityTypeName(ActivityType);
				TypeColor = GetFileActivityTypeColor(ActivityType);
			}
			if (InEvent.ActualSize != InEvent.Size)
			{
				TypeStr += TEXT(" [!]");
			}
			FLinearColor TypeLinearColor = FLinearColor(FColor(TypeColor));
			TypeLinearColor.R *= 2.0f;
			TypeLinearColor.G *= 2.0f;
			TypeLinearColor.B *= 2.0f;
			InOutTooltip.AddTitle(TypeStr, TypeLinearColor);

			if (ensure(InEvent.FileActivityIndex >= 0 && InEvent.FileActivityIndex < SharedState.FileActivities.Num()))
			{
				const TSharedPtr<FIoFileActivity>& ActivityPtr = SharedState.FileActivities[InEvent.FileActivityIndex];
				check(ActivityPtr.IsValid());
				FIoFileActivity& Activity = *ActivityPtr;

				InOutTooltip.AddTitle(Activity.Path);
			}

			if (InEvent.FileHandle != uint64(-1))
			{
				const FString Value = FString::Printf(TEXT("0x%llX"), InEvent.FileHandle);
				InOutTooltip.AddNameValueTextLine(TEXT("File Handle:"), Value);
			}

			if (InEvent.ReadWriteHandle != uint64(-1))
			{
				const FString Value = FString::Printf(TEXT("0x%llX"), InEvent.ReadWriteHandle);
				InOutTooltip.AddNameValueTextLine(TEXT("Read/Write Handle:"), Value);
			}

			const double Duration = InEvent.EndTime - InEvent.StartTime;
			InOutTooltip.AddNameValueTextLine(TEXT("Duration:"), FormatTimeAuto(Duration));

			if (ActivityType == TraceServices::FileActivityType_Read || ActivityType == TraceServices::FileActivityType_Write)
			{
				InOutTooltip.AddNameValueTextLine(TEXT("Offset:"), FText::AsNumber(InEvent.Offset).ToString() + TEXT(" bytes"));
				InOutTooltip.AddNameValueTextLine(TEXT("Size:"), FText::AsNumber(InEvent.Size).ToString() + TEXT(" bytes"));
				FString ActualSizeStr = FText::AsNumber(InEvent.ActualSize).ToString() + TEXT(" bytes");
				if (InEvent.ActualSize != InEvent.Size)
				{
					ActualSizeStr += TEXT(" [!]");
				}
				InOutTooltip.AddNameValueTextLine(TEXT("Actual Size:"), ActualSizeStr);
			}

			if (!bIgnoreEventDepth)
			{
				InOutTooltip.AddNameValueTextLine(TEXT("Depth:"), FString::Printf(TEXT("%d"), InEvent.Depth));
			}

			InOutTooltip.UpdateLayout();
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFileActivityTimingTrack::FindIoTimingEvent(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FIoTimingEvent&)> InFoundPredicate) const
{
	return TTimingEventSearch<FIoTimingEvent>::Search(

		InParameters,

		// Search Predicate
		[this]
		(TTimingEventSearch<FIoTimingEvent>::FContext& InContext)
		{
			const TArray<FIoTimingEvent>& Events = SharedState.GetAllEvents();

			if (bIgnoreDuration)
			{
				// Events are sorted by start time.
				// Find the first event with StartTime >= searched StartTime.
				int32 StartIndex = Algo::LowerBoundBy(Events, InContext.GetParameters().StartTime,
					[](const FIoTimingEvent& Event) { return Event.StartTime; });

				for (int32 Index = StartIndex; Index < Events.Num(); ++Index)
				{
					const FIoTimingEvent& Event = Events[Index];

					if (bShowOnlyErrors && ((Event.Type & 0xF0) == 0))
					{
						continue;
					}

					ensure(Event.StartTime >= InContext.GetParameters().StartTime);

					if (Event.StartTime > InContext.GetParameters().EndTime)
					{
						break;
					}

					InContext.Check(Event.StartTime, Event.StartTime, bIgnoreEventDepth ? 0 : Event.Depth, Event);

					if (!InContext.ShouldContinueSearching())
					{
						break;
					}
				}
			}
			else
			{
				// Events are sorted by start time.
				// Find the first event with StartTime >= searched EndTime.
				int32 StartIndex = Algo::LowerBoundBy(Events, InContext.GetParameters().EndTime,
					[](const FIoTimingEvent& Event) { return Event.StartTime; });

				// Start at the last event with StartTime < searched EndTime.
				for (int32 Index = StartIndex - 1; Index >= 0; --Index)
				{
					const FIoTimingEvent& Event = Events[Index];

					if (bShowOnlyErrors && ((Event.Type & 0xF0) == 0))
					{
						continue;
					}

					if (Event.EndTime <= InContext.GetParameters().StartTime ||
						Event.StartTime >= InContext.GetParameters().EndTime)
					{
						continue;
					}

					InContext.Check(Event.StartTime, Event.EndTime, bIgnoreEventDepth ? 0 : Event.Depth, Event);

					if (!InContext.ShouldContinueSearching())
					{
						break;
					}
				}
			}
		},

		// No Filter Predicate

		// Found Predicate
		InFoundPredicate

		// No Payload Matched Predicate

	); // Search
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FOverviewFileActivityTimingTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

void FOverviewFileActivityTimingTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	const FTimingTrackViewport& Viewport = Context.GetViewport();

	for (const FIoTimingEvent& Event : SharedState.AllIoEvents)
	{
		const TraceServices::EFileActivityType ActivityType = static_cast<TraceServices::EFileActivityType>(Event.Type & 0x0F);
		const uint64 EventType = static_cast<uint64>(ActivityType);

		if (ActivityType >= TraceServices::FileActivityType_Count)
		{
			// Ignore "Idle" and "NotClosed" events.
			continue;
		}

		//const double EventEndTime = Event.EndTime; // keep duration of events
		const double EventEndTime = Event.StartTime; // make all 0 duration events

		if (EventEndTime <= Viewport.GetStartTime())
		{
			continue;
		}
		if (Event.StartTime >= Viewport.GetEndTime())
		{
			break;
		}

		const bool bHasFailed = ((Event.Type & 0xF0) != 0);

		if (bShowOnlyErrors && !bHasFailed)
		{
			continue;
		}

		uint32 Color = bHasFailed ? 0xFFAA0000 : GetFileActivityTypeColor(ActivityType);
		if (Event.ActualSize != Event.Size)
		{
			Color = (Color & 0xFF000000) | ((Color & 0xFEFEFE) >> 1);
		}

		Builder.AddEvent(Event.StartTime, EventEndTime, 0, Color,
			[&Event](float Width)
			{
				FString EventName;

				const bool bHasFailed = ((Event.Type & 0xF0) != 0);
				if (bHasFailed)
				{
					EventName += TEXT("Failed ");
				}

				const TraceServices::EFileActivityType ActivityType = static_cast<TraceServices::EFileActivityType>(Event.Type & 0x0F);
				EventName += GetFileActivityTypeName(ActivityType);

				if (Event.ActualSize != Event.Size)
				{
					EventName += TEXT(" [!]");
				}

				const float MinWidth = static_cast<float>(EventName.Len()) * 4.0f + 32.0f;
				if (Width > MinWidth)
				{
					const double Duration = Event.EndTime - Event.StartTime; // actual event duration
					FTimingEventsTrackDrawStateBuilder::AppendDurationToEventName(EventName, Duration);
				}

				return EventName;
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FOverviewFileActivityTimingTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindIoTimingEvent(InSearchParameters, [this, &FoundEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FIoTimingEvent& InEvent)
	{
		FoundEvent = MakeShared<FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth);
	});

	return FoundEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FOverviewFileActivityTimingTrack::BuildContextMenu(FMenuBuilder& InOutMenuBuilder)
{
	InOutMenuBuilder.BeginSection(TEXT("Misc"));
	{
		InOutMenuBuilder.AddMenuEntry(
			LOCTEXT("OverviewTrack_ShowOnlyErrors", "Show Only Errors"),
			LOCTEXT("OverviewTrack_ShowOnlyErrors_Tooltip", "Show only the events with errors"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FOverviewFileActivityTimingTrack::ToggleOnlyErrors),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FOverviewFileActivityTimingTrack::IsOnlyErrorsToggleOn)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	InOutMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FDetailedFileActivityTimingTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

void FDetailedFileActivityTimingTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	const FTimingTrackViewport& Viewport = Context.GetViewport();

	// Add IO file activity background events.
	if (bShowBackgroundEvents)
	{
		for (const TSharedPtr<FIoFileActivity>& Activity : SharedState.FileActivities)
		{
			if (Activity->EndTime <= Viewport.GetStartTime())
			{
				continue;
			}
			if (Activity->StartTime >= Viewport.GetEndTime())
			{
				break;
			}

			ensure(Activity->StartingDepth < FFileActivitySharedState::MaxLanes);

			Builder.AddEvent(Activity->StartTime, Activity->EndTime, Activity->StartingDepth, 0x55333333,
				[&Activity](float Width)
				{
					FString EventName = Activity->Path;

					const float MinWidth = static_cast<float>(EventName.Len()) * 4.0f + 32.0f;
					if (Width > MinWidth)
					{
						const double Duration = Activity->EndTime - Activity->StartTime;
						FTimingEventsTrackDrawStateBuilder::AppendDurationToEventName(EventName, Duration);
					}

					return EventName;
				});
		}
	}

	// Add IO file activity foreground events.
	for (const FIoTimingEvent& Event : SharedState.AllIoEvents)
	{
		if (Event.EndTime <= Viewport.GetStartTime())
		{
			continue;
		}
		if (Event.StartTime >= Viewport.GetEndTime())
		{
			break;
		}

		ensure(Event.Depth < FFileActivitySharedState::MaxLanes);
		const TraceServices::EFileActivityType ActivityType = static_cast<TraceServices::EFileActivityType>(Event.Type & 0x0F);

		const bool bHasFailed = ((Event.Type & 0xF0) != 0);

		if (bShowOnlyErrors && !bHasFailed)
		{
			continue;
		}

		uint32 Color = bHasFailed ? 0xFFAA0000 : GetFileActivityTypeColor(ActivityType);
		if (Event.ActualSize != Event.Size)
		{
			Color = (Color & 0xFF000000) | ((Color & 0xFEFEFE) >> 1);
		}

		Builder.AddEvent(Event.StartTime, Event.EndTime, Event.Depth, Color,
			[&Event, FileActivity= SharedState.FileActivities[Event.FileActivityIndex]](float Width)
			{
				FString EventName;

				const bool bHasFailed = ((Event.Type & 0xF0) != 0);
				if (bHasFailed)
				{
					EventName += TEXT("Failed ");
				}

				const TraceServices::EFileActivityType ActivityType = static_cast<TraceServices::EFileActivityType>(Event.Type & 0x0F);
				EventName += GetFileActivityTypeName(ActivityType);

				if (Event.ActualSize != Event.Size)
				{
					EventName += TEXT(" [!]");
				}

				if (ActivityType >= TraceServices::FileActivityType_Count)
				{
					EventName += " [";
					EventName += FileActivity->Path;
					EventName += "]";
				}

				const float MinWidth = static_cast<float>(EventName.Len()) * 4.0f + 32.0f;
				if (Width > MinWidth)
				{
					const double Duration = Event.EndTime - Event.StartTime;
					FTimingEventsTrackDrawStateBuilder::AppendDurationToEventName(EventName, Duration);
				}

				return EventName;
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FDetailedFileActivityTimingTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindIoTimingEvent(InSearchParameters, [this, &FoundEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const FIoTimingEvent& InEvent)
	{
		FoundEvent = MakeShared<FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth);
	});

	return FoundEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDetailedFileActivityTimingTrack::BuildContextMenu(FMenuBuilder& InOutMenuBuilder)
{
	InOutMenuBuilder.BeginSection(TEXT("Misc"));
	{
		InOutMenuBuilder.AddMenuEntry(
			LOCTEXT("ActivityTrack_ShowOnlyErrors", "Show Only Errors"),
			LOCTEXT("ActivityTrack_ShowOnlyErrors_Tooltip", "Show only the events with errors"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FDetailedFileActivityTimingTrack::ToggleOnlyErrors),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FDetailedFileActivityTimingTrack::IsOnlyErrorsToggleOn)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		InOutMenuBuilder.AddMenuEntry(
			LOCTEXT("ActivityTrack_ShowBackgroundEvents", "Show Background Events - O"),
			LOCTEXT("ActivityTrack_ShowBackgroundEvents_Tooltip", "Show background events for file activities."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FDetailedFileActivityTimingTrack::ToggleBackgroundEvents),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FDetailedFileActivityTimingTrack::AreBackgroundEventsVisible)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	InOutMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
