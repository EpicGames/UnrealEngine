// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/ViewModels/TimingEventsTrack.h"

class FTimingEventSearchParameters;

namespace UE::Insights::TimingProfiler
{

class FFileActivitySharedState;
struct FIoTimingEvent;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFileActivityTimingTrack : public FTimingEventsTrack
{
	INSIGHTS_DECLARE_RTTI(FFileActivityTimingTrack, FTimingEventsTrack)

public:
	explicit FFileActivityTimingTrack(FFileActivitySharedState& InSharedState, const FString& InName)
		: FTimingEventsTrack(InName)
		, SharedState(InSharedState)
	{
	}

	virtual ~FFileActivityTimingTrack() override = default;

	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;

	bool IsOnlyErrorsToggleOn() const { return bShowOnlyErrors; }
	void ToggleOnlyErrors() { bShowOnlyErrors = !bShowOnlyErrors; SetDirtyFlag(); }

protected:
	bool FindIoTimingEvent(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const FIoTimingEvent&)> InFoundPredicate) const;

protected:
	FFileActivitySharedState& SharedState;

	bool bIgnoreEventDepth = false;
	bool bIgnoreDuration = false;
	bool bShowOnlyErrors = false; // shows only the events with errors (for the Overview track)
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FOverviewFileActivityTimingTrack : public FFileActivityTimingTrack
{
public:
	explicit FOverviewFileActivityTimingTrack(FFileActivitySharedState& InSharedState)
		: FFileActivityTimingTrack(InSharedState, TEXT("I/O Overview"))
	{
		bIgnoreEventDepth = true;
		bIgnoreDuration = true;
		//bShowOnlyErrors = true;
	}

	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FDetailedFileActivityTimingTrack : public FFileActivityTimingTrack
{
public:
	explicit FDetailedFileActivityTimingTrack(FFileActivitySharedState& InSharedState)
		: FFileActivityTimingTrack(InSharedState, TEXT("I/O Activity"))
	{
		//bShowOnlyErrors = true;
	}

	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;

	bool AreBackgroundEventsVisible() const { return bShowBackgroundEvents; }
	void ToggleBackgroundEvents() { bShowBackgroundEvents = !bShowBackgroundEvents; SetDirtyFlag(); }

private:
	bool bShowBackgroundEvents = false; // shows the file activity background events; from the Open event to the last Read/Write event, for each activity
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
