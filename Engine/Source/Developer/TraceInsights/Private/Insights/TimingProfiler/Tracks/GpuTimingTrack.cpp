// Copyright Epic Games, Inc. All Rights Reserved.

#include "GpuTimingTrack.h"

#include "Fonts/FontMeasure.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/TimingProfiler/ViewModels/ThreadTimingSharedState.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingViewDrawHelper.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::GpuTimingTrack"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FGpuTimingTrack, FGpuQueueTimingTrack, FGpuFencesTimingTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FGpuTimingTrack)
INSIGHTS_IMPLEMENT_RTTI(FGpuQueueTimingTrack)
INSIGHTS_IMPLEMENT_RTTI(FGpuFencesTimingTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////
// FGpuQueueWorkTimingTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FGpuQueueWorkTimingTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuQueueWorkTimingTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	DrawLineEvents(Context, 1.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuQueueWorkTimingTrack::DrawLineEvents(const ITimingTrackDrawContext& Context, const float OffsetY) const
{
	const FTimingViewDrawHelper& Helper = *static_cast<const FTimingViewDrawHelper*>(&Context.GetHelper());

	if ((Context.GetEventFilter().IsValid() && Context.GetEventFilter()->FilterTrack(*this)) || HasCustomFilter())
	{
		Helper.DrawFadedLineEvents(GetDrawState(), *this, OffsetY, 0.1f);

		if (UpdateFilteredDrawStateOpacity())
		{
			Helper.DrawLineEvents(GetFilteredDrawState(), *this, OffsetY);
		}
		else
		{
			Helper.DrawFadedLineEvents(GetFilteredDrawState(), *this, OffsetY, GetFilteredDrawStateOpacity());
		}
	}
	else
	{
		Helper.DrawLineEvents(GetDrawState(), *this, OffsetY);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuQueueWorkTimingTrack::PostDraw(const ITimingTrackDrawContext& Context) const
{
	const bool bAreOverlaysVisible = GetSharedState().AreOverlaysVisibleInGpuQueueTracks();
	const bool bAreExtendedLinesVisible = GetSharedState().AreExtendedLinesVisibleInGpuQueueTracks();

	if (bAreOverlaysVisible || bAreExtendedLinesVisible)
	{
		float LineY1 = 0.0f;
		float LineY2 = 0.0f;
		ETimingTrackLocation LocalLocation = ETimingTrackLocation::None;
		TSharedPtr<FBaseTimingTrack> LocalParentTrack = GetParentTrack().Pin();
		if (LocalParentTrack)
		{
			LineY1 = LocalParentTrack->GetPosY();
			LineY2 = LineY1 + LocalParentTrack->GetHeight();
			LocalLocation = LocalParentTrack->GetLocation();
		}
		else
		{
			LineY1 = GetPosY();
			LineY2 = LineY1 + GetHeight();
			LocalLocation = GetLocation();
		}

		const FTimingTrackViewport& Viewport = Context.GetViewport();
		switch (LocalLocation)
		{
			case ETimingTrackLocation::Scrollable:
			{
				const float TopY = Viewport.GetPosY() + Viewport.GetTopOffset();
				if (LineY1 < TopY)
				{
					LineY1 = TopY;
				}
				const float BottomY = Viewport.GetPosY() + Viewport.GetHeight() - Viewport.GetBottomOffset();
				if (LineY2 > BottomY)
				{
					LineY2 = BottomY;
				}
				break;
			}
			case ETimingTrackLocation::TopDocked:
			{
				const float TopY = Viewport.GetPosY();
				if (LineY1 < TopY)
				{
					LineY1 = TopY;
				}
				const float BottomY = Viewport.GetPosY() + Viewport.GetTopOffset();
				if (LineY2 > BottomY)
				{
					LineY2 = BottomY;
				}
				break;
			}
			case ETimingTrackLocation::BottomDocked:
			{
				const float TopY = Viewport.GetPosY() + Viewport.GetHeight() - Viewport.GetBottomOffset();
				if (LineY1 < TopY)
				{
					LineY1 = TopY;
				}
				const float BottomY = Viewport.GetPosY() + Viewport.GetHeight();
				if (LineY2 > BottomY)
				{
					LineY2 = BottomY;
				}
				break;
			}
		}

		const float LineH = LineY2 - LineY1;
		if (LineH > 0.0f)
		{
			const FTimingViewDrawHelper& Helper = *static_cast<const FTimingViewDrawHelper*>(&Context.GetHelper());
			Helper.DrawContextSwitchMarkers(GetDrawState(), LineY1, LineH, 0.25f, bAreOverlaysVisible, bAreExtendedLinesVisible);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FGpuFencesTrackBuilder
////////////////////////////////////////////////////////////////////////////////////////////////////

FGpuFencesTrackBuilder::FGpuFencesTrackBuilder(FGpuFencesTimingTrack& InTrack, const FTimingTrackViewport& InViewport, float InFontScale)
	: Track(InTrack)
	, Viewport(InViewport)
	, FontMeasureService(FSlateApplication::Get().GetRenderer()->GetFontMeasureService())
	, Font(FAppStyle::Get().GetFontStyle("SmallFont"))
	, FontScale(InFontScale)
{
	Track.ResetCache();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuFencesTrackBuilder::AddFence(double Timestamp, TraceServices::EGpuFenceType Type, const TCHAR* Text)
{
	float X = Viewport.TimeToSlateUnitsRounded(Timestamp);

	AddTimeMarker(X, Type, Text);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuFencesTrackBuilder::Flush()
{
	if (LastMessage.IsEmpty())
	{
		return;
	}

	Flush(Viewport.GetWidth() - LastX2);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuFencesTrackBuilder::Flush(float AvailableTextW)
{
	if (LastMessage.IsEmpty())
	{
		return;
	}

	auto ToLiniarColorNoAlpha = [](uint32 Value)
		{
			const float R = static_cast<float>((Value & 0xFF000000) >> 24) / 255.0f;
			const float G = static_cast<float>((Value & 0x00FF0000) >> 16) / 255.0f;
			const float B = static_cast<float>((Value & 0x0000FF00) >> 8) / 255.0f;
			return FLinearColor(R, G, B);
		};

	constexpr FLinearColor SignalFenceColor = ToLiniarColorNoAlpha(0x3A9C7bFF); // Light Blue
	constexpr FLinearColor WaitFenceColor = ToLiniarColorNoAlpha(0xFFDC1AFF); // Yellow

	const FLinearColor Color = LastType == TraceServices::EGpuFenceType::SignalFence ? SignalFenceColor : WaitFenceColor;

	bool bAddNewBox = true;
	if (Track.TimeMarkerBoxes.Num() > 0)
	{
		FGpuFenceTimeMarkerBoxInfo& PrevBox = Track.TimeMarkerBoxes.Last();
		if (PrevBox.X + PrevBox.W == LastX1 &&
			PrevBox.Color.R == Color.R &&
			PrevBox.Color.G == Color.G &&
			PrevBox.Color.B == Color.B)
		{
			// Extend previous box instead.
			PrevBox.W += LastX2 - LastX1;
			bAddNewBox = false;
		}
	}

	if (bAddNewBox)
	{
		// Add new Box info to cache.
		FGpuFenceTimeMarkerBoxInfo& Box = Track.TimeMarkerBoxes.AddDefaulted_GetRef();
		Box.X = LastX1;
		Box.W = LastX2 - LastX1;
		Box.Color = Color;
		Box.Color.A = 0.25f;
	}

	if (AvailableTextW > 6.0f)
	{
		const int32 HorizontalOffset = FMath::RoundToInt((AvailableTextW - 2.0f) * FontScale);
		const int32 LastWholeCharacterIndexCategory = FontMeasureService->FindLastWholeCharacterIndexBeforeOffset(LastMessage, Font, HorizontalOffset, FontScale);

		if (LastWholeCharacterIndexCategory >= 0)
		{
			// Add new Text info to cache.
			FGpuFenceTextInfo& TextInfo = Track.TimeMarkerTexts.AddDefaulted_GetRef();
			TextInfo.X = LastX2 + 2.0f;
			TextInfo.Color = Color;
			if (LastWholeCharacterIndexCategory >= 0)
			{
				TextInfo.Text.AppendChars(*LastMessage, LastWholeCharacterIndexCategory + 1);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuFencesTrackBuilder::AddTimeMarker(const float X, TraceServices::EGpuFenceType Type, const TCHAR* Message)
{
	const float W = X - LastX2;
	constexpr float BoxWidth = 1.0f;

	if (W > 0.0f) // There is at least 1px from previous box?
	{
		// Flush previous marker (if any).
		Flush(W);

		// Begin new marker info.
		LastX1 = X;
		LastX2 = X + BoxWidth;
	}
	else if (W == 0.0f) // Adjacent to previous box?
	{
		// Same color as previous marker?
		if (Type == LastType)
		{
			// Extend previous box.
			LastX2++;
		}
		else
		{
			// Flush previous marker (if any).
			Flush(0.0f);

			// Begin new box.
			LastX1 = X;
			LastX2 = X + BoxWidth;
		}
	}
	else // Overlaps previous box?
	{
		// Same color as previous marker?
		if (Type == LastType)
		{
			// Keep previous box.
		}
		else
		{
			// Shrink previous box.
			LastX2--;

			if (LastX2 > LastX1)
			{
				// Flush previous marker (if any).
				Flush(0.0f);
			}

			// Begin new box.
			LastX1 = X;
			LastX2 = X + BoxWidth;
		}
	}

	// Save marker.
	LastType = Type;
	LastMessage = Message;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FGpuFencesTimingTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

FGpuFencesTimingTrack::FGpuFencesTimingTrack(FThreadTimingSharedState& InSharedState, const FString& InName, uint32 InQueueId)
	: FThreadTimingTrackImpl(InSharedState, InName, nullptr, 0, InQueueId)
	, WhiteBrush(FAppStyle::Get().GetBrush("WhiteBrush"))
	, Font(FAppStyle::Get().GetFontStyle("SmallFont"))
{
}

void FGpuFencesTimingTrack::Reset()
{
	FThreadTimingTrackImpl::Reset();

	ResetCache();
}

void FGpuFencesTimingTrack::ResetCache()
{
	TimeMarkerBoxes.Empty();
	TimeMarkerTexts.Empty();
}

void FGpuFencesTimingTrack::PreUpdate(const ITimingTrackUpdateContext& Context)
{
	const FTimingTrackViewport& Viewport = Context.GetViewport();
	if (IsDirty() || Viewport.IsHorizontalViewportDirty())
	{
		ClearDirtyFlag();

		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
			if (TimingProfilerProvider)
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

				FGpuFencesTrackBuilder Builder(*this, Viewport, Context.GetGeometry().Scale);

				TimingProfilerProvider->EnumerateGpuFences(GetThreadId(), Viewport.GetStartTime(), Viewport.GetEndTime(),
					[&Builder](const TraceServices::FGpuFenceWrapper& Fence)
					{
						using namespace TraceServices;

						if (Fence.FenceType == EGpuFenceType::SignalFence)
						{
							const FGpuSignalFence* SignalFence = Fence.Fence.Get<const FGpuSignalFence*>();
							TCHAR Name[32];
							FCString::Sprintf(Name, TEXT("%ld"), SignalFence->Value);
							Builder.AddFence(SignalFence->Timestamp, EGpuFenceType::SignalFence, Name);
						}
						else if (Fence.FenceType == EGpuFenceType::WaitFence)
						{
							const FGpuWaitFence* WaitFence = Fence.Fence.Get<const FGpuWaitFence*>();
							TCHAR Name[32];
							FCString::Sprintf(Name, TEXT("%ld"), WaitFence->Value);
							Builder.AddFence(WaitFence->Timestamp, EGpuFenceType::WaitFence, Name);
						}

						return TraceServices::EEnumerateResult::Continue;
					});

				Builder.Flush();
			}
		}

		SetNumLanes(TimeMarkerBoxes.Num() ? 1 : 0);
	}

	UpdateTrackHeight(Context);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuFencesTimingTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	//////////////////////////////////////////////////
	// Draw texts (strings are already truncated).
	FDrawContext& DrawContext = Context.GetDrawContext();
	const FTimingTrackViewport& Viewport = Context.GetViewport();

	const FTimingViewDrawHelper& Helper = *static_cast<const FTimingViewDrawHelper*>(&Context.GetHelper());
	const FTimingViewLayout& Layout = Viewport.GetLayout();
	const int32 EventTextLayerId = Helper.GetTextLayerId();
	const float TextY = GetPosY() - (FTimingViewLayout::NormalLayoutEventH - Layout.EventH) / 2.0f + 3.0f;

	const int32 NumTexts = TimeMarkerTexts.Num();

	constexpr float TextMinEventH = 7.0f;
	if (Layout.EventH > TextMinEventH)
	{
		float TextOpacity = 1.0f;
		if (Layout.EventH < FTimingViewLayout::NormalLayoutEventH)
		{
			TextOpacity *= (Layout.EventH - TextMinEventH + 1.0f) / (FTimingViewLayout::NormalLayoutEventH - TextMinEventH + 1.0f);
		}
		for (int32 TextIndex = 0; TextIndex < NumTexts; TextIndex++)
		{
			const FGpuFenceTextInfo& TextInfo = TimeMarkerTexts[TextIndex];

			if (TextInfo.Text.Len() > 0)
			{
				DrawContext.DrawText(EventTextLayerId, TextInfo.X, TextY, TextInfo.Text, Font, TextInfo.Color.CopyWithNewOpacity(TextInfo.Color.A * TextOpacity));
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuFencesTimingTrack::PostDraw(const ITimingTrackDrawContext& Context) const
{
	FDrawContext& DrawContext = Context.GetDrawContext();
	const FTimingTrackViewport& Viewport = Context.GetViewport();

	//////////////////////////////////////////////////
	// Draw vertical lines.
	// Multiple adjacent vertical lines with same color are merged into a single box.

	DrawContext.LayerId++;

	constexpr double MaxViewportSizeForExtendedLines = 3.0;
	const bool bExtentFenceLines = GetSharedState().AreGpuFencesExtendedLinesVisible() && Viewport.GetDuration() < MaxViewportSizeForExtendedLines;
	const bool bFenceTracks = GetSharedState().AreGpuFencesTracksVisible();

	float BoxY1 = GetPosY();
	float BoxY2 = GetPosY() + GetHeight();
	ETimingTrackLocation LocalLocation = GetLocation();

	auto ParentTrackShared = GetParentTrack().Pin();
	if (bFenceTracks && bExtentFenceLines && ParentTrack.IsValid())
	{
		// Substract the size of the child tracks above this track.
		float TopY = GetPosY() - ParentTrackShared->GetPosY();
		BoxY2 = GetPosY() + ParentTrackShared->GetHeight() - TopY;
		LocalLocation = ParentTrackShared->GetLocation();
	}

	switch (LocalLocation)
	{
	case ETimingTrackLocation::Scrollable:
	{
		const float TopY = Viewport.GetPosY() + Viewport.GetTopOffset();
		if (BoxY1 < TopY)
		{
			BoxY1 = TopY;
		}
		const float BottomY = Viewport.GetPosY() + Viewport.GetHeight() - Viewport.GetBottomOffset();
		if (BoxY2 > BottomY)
		{
			BoxY2 = BottomY;
		}
		break;
	}
	case ETimingTrackLocation::TopDocked:
	{
		const float TopY = Viewport.GetPosY();
		if (BoxY1 < TopY)
		{
			BoxY1 = TopY;
		}
		const float BottomY = Viewport.GetPosY() + Viewport.GetTopOffset();
		if (BoxY2 > BottomY)
		{
			BoxY2 = BottomY;
		}
		break;
	}
	case ETimingTrackLocation::BottomDocked:
	{
		const float TopY = Viewport.GetPosY() + Viewport.GetHeight() - Viewport.GetBottomOffset();
		if (BoxY1 < TopY)
		{
			BoxY1 = TopY;
		}
		const float BottomY = Viewport.GetPosY() + Viewport.GetHeight();
		if (BoxY2 > BottomY)
		{
			BoxY2 = BottomY;
		}
		break;
	}
	}

	float BoxH = BoxY2 - BoxY1;
	const int32 NumBoxes = TimeMarkerBoxes.Num();
	for (int32 BoxIndex = 0; BoxIndex < NumBoxes; BoxIndex++)
	{
		const FGpuFenceTimeMarkerBoxInfo& Box = TimeMarkerBoxes[BoxIndex];
		DrawContext.DrawBox(Box.X, BoxY1, Box.W, BoxH, WhiteBrush, Box.Color);
	}
	DrawContext.LayerId++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
