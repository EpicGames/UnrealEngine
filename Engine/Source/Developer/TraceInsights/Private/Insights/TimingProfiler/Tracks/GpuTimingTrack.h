// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/TimingProfiler/Tracks/ThreadTimingTrackPrivate.h"

class FSlateFontMeasure;

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FGpuTimingTrack : public FThreadTimingTrackImpl
{
	INSIGHTS_DECLARE_RTTI(FGpuTimingTrack, FThreadTimingTrackImpl)

public:
	static constexpr uint32 Gpu1ThreadId = uint32('GPU1');
	static constexpr uint32 Gpu2ThreadId = uint32('GPU2');

public:
	explicit FGpuTimingTrack(FThreadTimingSharedState& InSharedState, const FString& InName, const TCHAR* InGroupName, uint32 InTimelineIndex, uint32 InThreadId)
		: FThreadTimingTrackImpl(InSharedState, InName, InGroupName, InTimelineIndex, InThreadId)
	{
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FGpuQueueTimingTrack : public FThreadTimingTrackImpl
{
	INSIGHTS_DECLARE_RTTI(FGpuQueueTimingTrack, FThreadTimingTrackImpl)

public:
	explicit FGpuQueueTimingTrack(FThreadTimingSharedState& InSharedState, const FString& InName, uint32 InTimelineIndex, uint32 InQueueId)
		: FThreadTimingTrackImpl(InSharedState, InName, nullptr, InTimelineIndex, InQueueId)
	{
	}

	uint32 GetQueueId() const { return GetThreadId(); }
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FGpuQueueWorkTimingTrack : public FThreadTimingTrackImpl
{
	INSIGHTS_DECLARE_RTTI(FGpuQueueWorkTimingTrack, FThreadTimingTrackImpl)

public:
	explicit FGpuQueueWorkTimingTrack(FThreadTimingSharedState& InSharedState, const FString& InName, uint32 InTimelineIndex, uint32 InQueueId)
		: FThreadTimingTrackImpl(InSharedState, InName, nullptr, InTimelineIndex, InQueueId)
	{
	}

	uint32 GetQueueId() const { return GetThreadId(); }

	virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	virtual void PostDraw(const ITimingTrackDrawContext& Context) const override;

private:
	void DrawLineEvents(const ITimingTrackDrawContext& Context, const float OffsetY) const;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FGpuFenceTimeMarkerBoxInfo
{
	float X;
	float W;
	FLinearColor Color;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FGpuFenceTextInfo
{
	float X;
	FLinearColor Color;
	FString Text;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FGpuFencesTimingTrack : public FThreadTimingTrackImpl
{
	friend class FGpuFencesTrackBuilder;
	INSIGHTS_DECLARE_RTTI(FGpuFencesTimingTrack, FThreadTimingTrackImpl)

public:
	explicit FGpuFencesTimingTrack(FThreadTimingSharedState& InSharedState, const FString& InName, uint32 InQueueId);

	uint32 GetQueueId() const { return GetThreadId(); }

	virtual void Reset() override;
	virtual void PreUpdate(const ITimingTrackUpdateContext& Context) override;

	virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	virtual void PostDraw(const ITimingTrackDrawContext& Context) const override;
	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override {};
	virtual void BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override {}
	virtual const TSharedPtr<const ITimingEvent> GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const override { return nullptr; };

private:
	void ResetCache();

private:
	TArray<FGpuFenceTimeMarkerBoxInfo> TimeMarkerBoxes;
	TArray<FGpuFenceTextInfo> TimeMarkerTexts;

	const FSlateBrush* WhiteBrush;
	const FSlateFontInfo Font;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FGpuFencesTrackBuilder
{
public:
	explicit FGpuFencesTrackBuilder(FGpuFencesTimingTrack& InTrack, const FTimingTrackViewport& InViewport, float InFontScale);

	/**
	 * Non-copyable
	 */
	FGpuFencesTrackBuilder(const FGpuFencesTrackBuilder&) = delete;
	FGpuFencesTrackBuilder& operator=(const FGpuFencesTrackBuilder&) = delete;

	const FTimingTrackViewport& GetViewport() { return Viewport; }

	void AddFence(double Timestamp, TraceServices::EGpuFenceType Type, const TCHAR* Text);
	void Flush();

private:
	void Flush(float AvailableTextW);
	void AddTimeMarker(const float X, TraceServices::EGpuFenceType Type, const TCHAR* Message);

private:
	FGpuFencesTimingTrack& Track;
	const FTimingTrackViewport& Viewport;

	const TSharedRef<FSlateFontMeasure> FontMeasureService;
	const FSlateFontInfo Font;
	float FontScale;

	float LastX1;
	float LastX2;
	TraceServices::EGpuFenceType LastType;
	FString LastMessage;
};

} // namespace UE::Insights::TimingProfiler
