// Copyright Epic Games, Inc. All Rights Reserved.

#include "MarkersTimingTrack.h"

#include "Containers/StringFwd.h"
#include "DesktopPlatformModule.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"

// TraceServices
#include "TraceServices/Model/Log.h"
#include "TraceServices/Model/Screenshot.h"
#include "TraceServices/Model/Bookmarks.h"
#include "TraceServices/Model/Callstack.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/ViewModels/LogFilter.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Insights/MemoryProfiler/ViewModels/CallstackFormatting.h"
#include "Insights/Widgets/STimingView.h"

#include <limits>
#include <cmath>

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::FMarkersTimingTrack"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FMarkersTimingTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

FMarkersTimingTrack::FMarkersTimingTrack()
	: FBaseTimingTrack(TEXT("Markers (Bookmarks / Logs)"))
	, Header(*this)
	, WhiteBrush(FAppStyle::Get().GetBrush("WhiteBrush"))
	, Font(FAppStyle::Get().GetFontStyle("SmallFont"))
	, LastScreenshotId(TraceServices::FScreenshot::InvalidScreenshotId)
{
	SetValidLocations(ETimingTrackLocation::TopDocked | ETimingTrackLocation::BottomDocked);
	SetOrder(FTimingTrackOrder::Markers);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FMarkersTimingTrack::~FMarkersTimingTrack()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMarkersTimingTrack::Reset()
{
	FBaseTimingTrack::Reset();

	TimeMarkerBoxes.Reset();
	TimeMarkerTexts.Reset();

	bUseOnlyBookmarks = true;
	BookmarkCategory = nullptr;
	ScreenshotCategory = nullptr;

	Header.Reset();
	Header.SetIsInBackground(true);
	Header.SetCanBeCollapsed(true);

	ChangeNumber = 0;

	NumLogMessages = 0;
	NumDrawBoxes = 0;
	NumDrawTexts = 0;

	UpdateTrackNameAndHeight();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMarkersTimingTrack::UpdateTrackNameAndHeight()
{
	if (bUseOnlyBookmarks)
	{
		SetName(TEXT("Bookmarks"));
		SetHeight(14.0f);
	}
	else
	{
		SetName(TEXT("Logs"));
		SetHeight(28.0f);
	}

	Header.UpdateSize();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMarkersTimingTrack::PreUpdate(const ITimingTrackUpdateContext& Context)
{
	if (!BookmarkCategory)
	{
		UpdateCategory(BookmarkCategory, TEXT("LogBookmark"));
	}
	if (!ScreenshotCategory)
	{
		UpdateCategory(ScreenshotCategory, TEXT("Screenshot"));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMarkersTimingTrack::Update(const ITimingTrackUpdateContext& Context)
{
	Header.SetFontScale(Context.GetGeometry().Scale);
	Header.Update(Context);

	const FTimingTrackViewport& Viewport = Context.GetViewport();
	
	if (FTimingProfilerManager::Get().IsValid())
	{
		uint64 NewChangeNumber = FTimingProfilerManager::Get()->GetLogFilter()->GetChangeNumber();
		if (IsDirty() || Viewport.IsHorizontalViewportDirty() || ChangeNumber != NewChangeNumber)
		{
			ClearDirtyFlag();

			ChangeNumber = NewChangeNumber;
			
			UpdateDrawState(Context);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMarkersTimingTrack::PostUpdate(const ITimingTrackUpdateContext& Context)
{
	const float MouseY = static_cast<float>(Context.GetMousePosition().Y);
	SetHoveredState(MouseY >= GetPosY() && MouseY < GetPosY() + GetHeight());

	Header.PostUpdate(Context);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMarkersTimingTrack::UpdateDrawState(const ITimingTrackUpdateContext& Context)
{
	FTimeMarkerTrackBuilder Builder(*this, Context.GetViewport(), Context.GetGeometry().Scale);

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());
		Builder.BeginLog(LogProvider);

		LogProvider.EnumerateMessages(
			Builder.GetViewport().GetStartTime(),
			Builder.GetViewport().GetEndTime(),
			[&Builder](const TraceServices::FLogMessageInfo& Message) { Builder.AddLogMessage(Message); });

		Builder.EndLog();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMarkersTimingTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	FDrawContext& DrawContext = Context.GetDrawContext();
	const FTimingTrackViewport& Viewport = Context.GetViewport();

	// Draw background.
	const FLinearColor BackgroundColor(0.04f, 0.04f, 0.04f, 1.0f);
	DrawContext.DrawBox(0.0f, GetPosY(), Viewport.GetWidth(), GetHeight(), WhiteBrush, BackgroundColor);
	DrawContext.LayerId++;

	Header.Draw(Context);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMarkersTimingTrack::PostDraw(const ITimingTrackDrawContext& Context) const
{
	FDrawContext& DrawContext = Context.GetDrawContext();
	const FTimingTrackViewport& Viewport = Context.GetViewport();

	//////////////////////////////////////////////////
	// Draw vertical lines.
	// Multiple adjacent vertical lines with same color are merged into a single box.

	float BoxY, BoxH;
	if (IsCollapsed())
	{
		BoxY = GetPosY();
		BoxH = GetHeight();
	}
	else
	{
		BoxY = Viewport.GetPosY();
		BoxH = Viewport.GetHeight();
	}

	const int32 NumBoxes = TimeMarkerBoxes.Num();
	for (int32 BoxIndex = 0; BoxIndex < NumBoxes; BoxIndex++)
	{
		const FTimeMarkerBoxInfo& Box = TimeMarkerBoxes[BoxIndex];
		DrawContext.DrawBox(Box.X, BoxY, Box.W, BoxH, WhiteBrush, Box.Color);
	}
	DrawContext.LayerId++;
	NumDrawBoxes = NumBoxes;

	//////////////////////////////////////////////////
	// Draw texts (strings are already truncated).

	const float CategoryY = GetPosY() + 2.0f;
	const float MessageY = GetPosY() + (IsBookmarksTrack() ? 1.0f : 14.0f);

	const int32 NumTexts = TimeMarkerTexts.Num();
	for (int32 TextIndex = 0; TextIndex < NumTexts; TextIndex++)
	{
		const FTimeMarkerTextInfo& TextInfo = TimeMarkerTexts[TextIndex];

		if (!IsBookmarksTrack() && TextInfo.Category.Len() > 0)
		{
			DrawContext.DrawText(TextInfo.X, CategoryY, TextInfo.Category, Font, TextInfo.Color);
			NumDrawTexts++;
		}

		if (TextInfo.Message.Len() > 0)
		{
			DrawContext.DrawText(TextInfo.X, MessageY, TextInfo.Message, Font, TextInfo.Color);
			NumDrawTexts++;
		}
	}
	DrawContext.LayerId++;

	//////////////////////////////////////////////////

	Header.PostDraw(Context);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply FMarkersTimingTrack::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (IsVisible() && IsHeaderHovered())
		{
			ToggleCollapsed();
			Reply = FReply::Handled();
		}
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply FMarkersTimingTrack::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return OnMouseButtonDown(MyGeometry, MouseEvent);
}


////////////////////////////////////////////////////////////////////////////////////////////////////

void FMarkersTimingTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection(TEXT("Content"), LOCTEXT("ContextMenu_Section_Content", "Content"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ContextMenu_Bookmarks", "Bookmarks"),
			LOCTEXT("ContextMenu_Bookmarks_Desc", "Changes this track to show only the bookmarks."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FMarkersTimingTrack::SetBookmarksTrack),
						FCanExecuteAction(),
						FIsActionChecked::CreateSP(this, &FMarkersTimingTrack::IsBookmarksTrack)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ContextMenu_Logs", "Logs"),
			LOCTEXT("ContextMenu_Logs_Desc", "Changes this track to show all logs."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FMarkersTimingTrack::SetLogsTrack),
					  FCanExecuteAction(),
					  FIsActionChecked::CreateSP(this, &FMarkersTimingTrack::IsLogsTrack)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("MarkerLines"), LOCTEXT("ContextMenu_Section_MarkerLines", "Marker Lines"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ContextMenu_ToggleCollapsed", "Collapsed"),
			LOCTEXT("ContextMenu_ToggleCollapsed_Desc", "Whether the vertical marker lines are collapsed or expanded."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FMarkersTimingTrack::ToggleCollapsed),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FMarkersTimingTrack::IsCollapsed)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("Screenshot"), LOCTEXT("ContextMenu_Section_Screenshot", "Screenshot"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ContextMenu_SaveScreenshot", "Save Screenshot..."),
			LOCTEXT("ContextMenu_SaveScreenshot_Desc", "Save the hovered screenshot to a file."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FMarkersTimingTrack::SaveScreenshot_Execute),
				FCanExecuteAction::CreateSP(this, &FMarkersTimingTrack::SaveScreenshot_CanExecute)),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		LastScreenshotId = TryGetHoveredEventScreenshotId();
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double FMarkersTimingTrack::Snap(double Time, const double SnapTolerance)
{
	if (bUseOnlyBookmarks && !BookmarkCategory)
	{
		return Time;
	}

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());

		double SnapTime = std::numeric_limits<double>::infinity();
		double SnapDistance = std::numeric_limits<double>::infinity();

		if (bUseOnlyBookmarks)
		{
			LogProvider.EnumerateMessages(
				Time - SnapTolerance,
				Time + SnapTolerance,
				[&SnapTime, &SnapDistance, Time, this](const TraceServices::FLogMessageInfo& Message)
				{
					if (Message.Category == BookmarkCategory)
					{
						double Distance = FMath::Abs(Message.Time - Time);
						if (Distance < SnapDistance)
						{
							SnapDistance = Distance;
							SnapTime = Message.Time;
						}
					}
				});
		}
		else
		{
			LogProvider.EnumerateMessages(
				Time - SnapTolerance,
				Time + SnapTolerance,
				[&SnapTime, &SnapDistance, Time, this](const TraceServices::FLogMessageInfo& Message)
				{
					double Distance = FMath::Abs(Message.Time - Time);
					if (Distance < SnapDistance)
					{
						SnapDistance = Distance;
						SnapTime = Message.Time;
					}
				});
		}

		if (SnapDistance < SnapTolerance)
		{
			Time = SnapTime;
		}
	}

	return Time;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMarkersTimingTrack::UpdateCategory(const TraceServices::FLogCategoryInfo*& InOutCategory, const TCHAR* CategoryName)
{
	InOutCategory = nullptr;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());

		LogProvider.EnumerateCategories([this, CategoryName, &InOutCategory](const TraceServices::FLogCategoryInfo& Category)
		{
			if (Category.Name && FCString::Strcmp(Category.Name, CategoryName) == 0)
			{
				InOutCategory = &Category;
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

class FLogMessageTimingEvent : public FTimingEvent
{
	INSIGHTS_DECLARE_RTTI(FLogMessageTimingEvent, FTimingEvent)

public:
	FLogMessageTimingEvent(const TSharedRef<const FMarkersTimingTrack> InTrack, double InTime, uint64 InLogIndex)
		: FTimingEvent(InTrack, InTime, InTime, 0, InLogIndex)
	{
	}

	uint64 GetLogIndex() const { return GetType(); }
};

INSIGHTS_IMPLEMENT_RTTI(FLogMessageTimingEvent)

////////////////////////////////////////////////////////////////////////////////////////////////////

class FScreenShotTimingEvent : public FLogMessageTimingEvent
{
	INSIGHTS_DECLARE_RTTI(FScreenShotTimingEvent, FLogMessageTimingEvent)

public:
	FScreenShotTimingEvent(const TSharedRef<const FMarkersTimingTrack> InTrack, double InTime, uint64 InLogIndex, uint32 InScreenshotId)
		: FLogMessageTimingEvent(InTrack, InTime, InLogIndex)
		, ScreenshotId(InScreenshotId)
	{
	}

	uint32 GetScreenshotId() const { return ScreenshotId; }

private:
	uint32 ScreenshotId;
};

INSIGHTS_IMPLEMENT_RTTI(FScreenShotTimingEvent)

////////////////////////////////////////////////////////////////////////////////////////////////////

class FBookmarkTimingEvent : public FLogMessageTimingEvent
{
	INSIGHTS_DECLARE_RTTI(FBookmarkTimingEvent, FLogMessageTimingEvent)

public:
	FBookmarkTimingEvent(const TSharedRef<const FMarkersTimingTrack> InTrack, double InTime, uint64 InLogIndex, uint32 InCallstackId)
		: FLogMessageTimingEvent(InTrack, InTime, InLogIndex)
		, CallstackId(InCallstackId)
	{}

	uint32 GetCallstackId() const { return CallstackId; }

private:
	uint32 CallstackId;
};

INSIGHTS_IMPLEMENT_RTTI(FBookmarkTimingEvent)

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FMarkersTimingTrack::GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const
{
	const FTimingViewLayout& Layout = Viewport.GetLayout();

	const float DY = InPosY - GetPosY();

	if (DY >= 0 && DY < GetHeight())
	{
		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (!Session.IsValid())
		{
			return nullptr;
		}

		const double PosTime = Viewport.SlateUnitsToTime(InPosX);

		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());

		uint64 LogIndex = 0;
		const TraceServices::FLogCategoryInfo* Category = nullptr;
		double Time = 0.0;
		uint32 ScreenshotId = TraceServices::FScreenshot::InvalidScreenshotId;

		if (IsBookmarksTrack())
		{
			//TODO: Reverse enumeration with filter; see FTimeMarkerTrackBuilder::AddLogMessage
			LogProvider.EnumerateMessages(Viewport.GetStartTime(), PosTime,
				[&LogIndex, &Category, &Time, &ScreenshotId, this](const TraceServices::FLogMessageInfo& Message)
				{
					if (Message.Category == this->BookmarkCategory)
					{
						LogIndex = Message.Index;
						Category = Message.Category;
						Time = Message.Time;
					}
					else if (Message.Category == this->ScreenshotCategory)
					{
						LogIndex = Message.Index;
						Category = Message.Category;
						Time = Message.Time;
						check(Message.Line >= 0);
						ScreenshotId = Message.Line;
					}
				});
		}
		else
		{
			ELogVerbosity::Type VerbosityThreshold = FTimingProfilerManager::Get()->GetLogFilter()->GetVerbosityThreshold();
			const TSet<FName>&  EnabledLogCategories = FTimingProfilerManager::Get()->GetLogFilter()->GetEnabledLogCategories();

			// Enumerate through from the current position backwards until there is an appropriately filtered element 
			LogProvider.ReverseEnumerateMessages(PosTime, 0.0f,
			[&LogIndex, &Time, &Category, &ScreenshotId, &VerbosityThreshold, &EnabledLogCategories, this](const TraceServices::FLogMessageInfo& Message)
			{
				FStringView CategoryStr(Message.Category->Name);
				if (CategoryStr.StartsWith(TEXT("Log")))
				{
					CategoryStr.RightChopInline(3);
				}
				FName CleanedCategoryStr(CategoryStr);
				
				if (Message.Verbosity <= VerbosityThreshold && EnabledLogCategories.Contains(CleanedCategoryStr))
				{
					LogIndex = Message.Index;
					Time = Message.Time;
					Category = Message.Category;
					if (Message.Category == this->ScreenshotCategory)
					{
						check(Message.Line >= 0);
						ScreenshotId = Message.Line;
					}
					return true;
				}
				return false;
			});
		}

		if (Category == nullptr)
		{
			return nullptr;
		}

		if (Category == ScreenshotCategory)
		{
			return MakeShared<FScreenShotTimingEvent>(SharedThis(this), Time, LogIndex, ScreenshotId);
		}

		if (Category == BookmarkCategory)
		{
			uint32 CallstackId = 0;
			const TraceServices::IBookmarkProvider& BookmarkProvider = TraceServices::ReadBookmarkProvider(*Session.Get());
			BookmarkProvider.EnumerateBookmarks(
				Time - UE_SMALL_NUMBER,
				Time + UE_SMALL_NUMBER,
				[&CallstackId](const TraceServices::FBookmark& Bookmark)
				{
					CallstackId = Bookmark.CallstackId;
				});
			return MakeShared<FBookmarkTimingEvent>(SharedThis(this), Time, LogIndex, CallstackId);
		}

		return MakeShared<FLogMessageTimingEvent>(SharedThis(this), Time, LogIndex);
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMarkersTimingTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	InOutTooltip.ResetContent();
	InOutTooltip.UpdateLayout();

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	if (!InTooltipEvent.CheckTrack(this))
	{
		return;
	}

	if (!InTooltipEvent.Is<FLogMessageTimingEvent>())
	{
		return;
	}

	const FLogMessageTimingEvent& LogMessageTimingEvent = InTooltipEvent.As<FLogMessageTimingEvent>();
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());
		LogProvider.ReadMessage(LogMessageTimingEvent.GetLogIndex(),
			[&InOutTooltip, this](const TraceServices::FLogMessageInfo& Message)
			{
				const FLinearColor Color = FTimeMarkerTrackBuilder::GetColorByCategory(Message.Category->Name);
				if (IsLogsTrack())
				{
					const TCHAR* CategoryName = Message.Category->Name != nullptr ? Message.Category->Name : TEXT("");
					FStringView CategoryStr(CategoryName);
					if (CategoryStr.StartsWith(TEXT("Log")))
					{
						// Strip the "Log" prefix.
						CategoryStr.RightChopInline(3);
					}
					InOutTooltip.AddTextLine(CategoryStr, Color);
				}
				InOutTooltip.AddTextLine(Message.Message, Color);
				InOutTooltip.UpdateLayout();
			});
	}

	if (InTooltipEvent.Is<FScreenShotTimingEvent>())
	{
		const FScreenShotTimingEvent& Event = InTooltipEvent.As<FScreenShotTimingEvent>();

		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::IScreenshotProvider& ScreenshotProvider = TraceServices::ReadScreenshotProvider(*Session.Get());
		TSharedPtr<const TraceServices::FScreenshot> Screenshot = ScreenshotProvider.GetScreenshot(Event.GetScreenshotId());

		if (!Screenshot.IsValid())
		{
			return;
		}

		IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
		FImage Image;
		if (ImageWrapperModule.DecompressImage(Screenshot->Data.GetData(), Screenshot->Size, Image))
		{
			constexpr int32 MAX_WIDTH = 640;
			constexpr int32 MAX_HEIGHT = 480;

			int32 ResizedX = Screenshot->Width;
			int32 ResizedY = Screenshot->Height;

			if (ResizedX > MAX_WIDTH)
			{
				ResizedY = (ResizedY * MAX_WIDTH) / ResizedX;
				ResizedX = MAX_WIDTH;
			}

			if (ResizedY > MAX_HEIGHT)
			{
				ResizedX = (ResizedX * MAX_HEIGHT) / ResizedY;
				ResizedY = MAX_HEIGHT;
			}

			TSharedPtr<FSlateBrush> ImageBrush;
			if (Screenshot->Width != ResizedX || Screenshot->Height != ResizedY)
			{
				FImage ResizedImage;
				Image.ResizeTo(ResizedImage, ResizedX, ResizedY, ERawImageFormat::BGRA8, EGammaSpace::sRGB);
				ImageBrush = FSlateDynamicImageBrush::CreateWithImageData(FName(Screenshot->Name), FVector2D(ResizedX, ResizedY), TArray<uint8>(ResizedImage.RawData));
			}
			else
			{
				ImageBrush = FSlateDynamicImageBrush::CreateWithImageData(FName(Screenshot->Name), FVector2D(Screenshot->Width, Screenshot->Height), TArray<uint8>(Image.RawData));
			}
			InOutTooltip.SetImage(ImageBrush);
			InOutTooltip.UpdateLayout();
		}
	}

	if (InTooltipEvent.Is<FBookmarkTimingEvent>())
	{
		const FBookmarkTimingEvent& Event = InTooltipEvent.As<FBookmarkTimingEvent>();

		if (Event.GetCallstackId() != 0)
		{
			if (const TraceServices::ICallstacksProvider* CallstackProvider = TraceServices::ReadCallstacksProvider(*Session.Get()))
			{
				if (const TraceServices::FCallstack* Callstack = CallstackProvider->GetCallstack(Event.GetCallstackId()))
				{
					uint32 CallstackSize = Callstack->Num();
					for (uint8 Idx = 0; Idx < CallstackSize; ++Idx)
					{
						TStringBuilder<512> CallstackText;
						UE::Insights::FormatStackFrame(*Callstack->Frame(Idx), CallstackText, UE::Insights::EStackFrameFormatFlags::ModuleSymbolFileAndLine);
						InOutTooltip.AddTextLine(CallstackText, FTooltipDrawState::DefaultValueColor);
					}
					InOutTooltip.UpdateLayout();
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMarkersTimingTrack::OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	if (!InSelectedEvent.CheckTrack(this))
	{
		return;
	}

	if (!InSelectedEvent.Is<FLogMessageTimingEvent>())
	{
		return;
	}

	TStringBuilder<2048> Result;

	const FLogMessageTimingEvent& LogMessageTimingEvent = InSelectedEvent.As<FLogMessageTimingEvent>();
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());
		LogProvider.ReadMessage(LogMessageTimingEvent.GetLogIndex(),
			[&Result, this](const TraceServices::FLogMessageInfo& Message)
			{
				Result.Append(Message.Message);
			});
	}

	if (InSelectedEvent.Is<FBookmarkTimingEvent>())
	{
		const FBookmarkTimingEvent& Event = InSelectedEvent.As<FBookmarkTimingEvent>();

		if (Event.GetCallstackId() != 0)
		{
			if (const TraceServices::ICallstacksProvider* CallstackProvider = TraceServices::ReadCallstacksProvider(*Session.Get()))
			{
				if (const TraceServices::FCallstack* Callstack = CallstackProvider->GetCallstack(Event.GetCallstackId()))
				{
					uint32 CallstackSize = Callstack->Num();
					for (uint8 Idx = 0; Idx < CallstackSize; ++Idx)
					{
						TStringBuilder<512> CallstackText;
						UE::Insights::FormatStackFrame(*Callstack->Frame(Idx), CallstackText, UE::Insights::EStackFrameFormatFlags::ModuleSymbolFileAndLine);
						Result.Appendf(TEXT("\n%s"), *CallstackText);
					}
				}
			}
		}
	}

	// Copy callstack to the clipboard.
	FPlatformApplicationMisc::ClipboardCopy(*Result);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMarkersTimingTrack::SaveScreenshot_CanExecute()
{
	return LastScreenshotId != TraceServices::FScreenshot::InvalidScreenshotId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMarkersTimingTrack::SaveScreenshot_Execute()
{
	if (LastScreenshotId == TraceServices::FScreenshot::InvalidScreenshotId)
	{
		return;
	}

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	const TraceServices::IScreenshotProvider& ScreenshotProvider = TraceServices::ReadScreenshotProvider(*Session.Get());
	TSharedPtr<const TraceServices::FScreenshot> Screenshot = ScreenshotProvider.GetScreenshot(LastScreenshotId);

	if (!Screenshot.IsValid())
	{
		return;
	}

	TArray<FString> SaveFilenames;
	bool bDialogResult = false;

	FString DefaultFile = Screenshot->Name;
	DefaultFile.Append(TEXT(".png"));

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		const FString DefaultPath = FPaths::ProjectSavedDir();
		bDialogResult = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("SaveScreenshotTitle", "Save Screenshot").ToString(),
			DefaultPath,
			DefaultFile,
			TEXT("Portable Network Graphics File (*.png)|*.png"),
			EFileDialogFlags::None,
			SaveFilenames
		);
	}

	if (!bDialogResult || SaveFilenames.Num() == 0)
	{
		return;
	}

	FString& Path = SaveFilenames[0];

	FFileHelper::SaveArrayToFile(Screenshot->Data, *Path);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FMarkersTimingTrack::TryGetHoveredEvent()
{
	TSharedPtr<STimingProfilerWindow> Window = FTimingProfilerManager::Get()->GetProfilerWindow();
	if (!Window.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<STimingView> TimingView = Window->GetTimingView();
	if (!TimingView.IsValid())
	{
		return nullptr;
	}

	const TSharedPtr<const ITimingEvent> HoveredTimingEvent = TimingView->GetHoveredEvent();
	if (!HoveredTimingEvent.IsValid())
	{
		return nullptr;
	}

	if (!HoveredTimingEvent->CheckTrack(this))
	{
		return nullptr;
	}

	return HoveredTimingEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FMarkersTimingTrack::TryGetHoveredEventScreenshotId()
{
	const TSharedPtr<const ITimingEvent> Event = TryGetHoveredEvent();

	if (Event.IsValid() && Event->Is<FScreenShotTimingEvent>())
	{
		return Event->As<FScreenShotTimingEvent>().GetScreenshotId();
	}
	else
	{
		return TraceServices::FScreenshot::InvalidScreenshotId;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimeMarkerTrackBuilder
////////////////////////////////////////////////////////////////////////////////////////////////////

FTimeMarkerTrackBuilder::FTimeMarkerTrackBuilder(FMarkersTimingTrack& InTrack, const FTimingTrackViewport& InViewport, float InFontScale)
	: Track(InTrack)
	, Viewport(InViewport)
	, FontMeasureService(FSlateApplication::Get().GetRenderer()->GetFontMeasureService())
	, Font(FAppStyle::Get().GetFontStyle("SmallFont"))
	, FontScale(InFontScale)
{
	Track.ResetCache();
	Track.NumLogMessages = 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeMarkerTrackBuilder::BeginLog(const TraceServices::ILogProvider& LogProvider)
{
	LogProviderPtr = &LogProvider;

	LastX1 = -1000.0f;
	LastX2 = -1000.0f;
	LastLogIndex = 0;
	LastVerbosity = ELogVerbosity::NoLogging;
	LastCategory = nullptr;
	LastMessage = nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeMarkerTrackBuilder::AddLogMessage(const TraceServices::FLogMessageInfo& Message)
{
	Track.NumLogMessages++;

	// Add also the log message immediately on the left of the screen (if any).
	if (Track.NumLogMessages == 1 && Message.Index > 0)
	{
		// Note: The log message at Index-1 may fail the filter (i.e. not a bookmark or screenshot), so it might not be displayed!
		//TODO: Search API like: LogProviderPtr->SearchMessage(StartIndex, ESearchDirection::Backward, LambdaPredicate, bResolveFormatString);
		LogProviderPtr->ReadMessage(
			Message.Index - 1,
			[this](const TraceServices::FLogMessageInfo& Message) { AddLogMessage(Message); });
	}

	check(Message.Category != nullptr);
	
	FStringView CategoryStr(Message.Category->Name);
	if (CategoryStr.StartsWith(TEXT("Log")))
	{
		CategoryStr.RightChopInline(3);
	}
	FName CleanedCategoryStr(CategoryStr);

	// Check whether we are using the bookmarks track and if the current message is allowed
	if (Track.IsBookmarksTrack())
	{
		if (!(Message.Category == Track.BookmarkCategory || Message.Category == Track.ScreenshotCategory))
		{
			return;
		}
	}
		
	// Check whether we can allow the message to be added to the logs track
	else if (Track.IsLogsTrack() && FTimingProfilerManager::Get().IsValid())
	{
		ELogVerbosity::Type VerbosityThreshold = FTimingProfilerManager::Get()->GetLogFilter()->GetVerbosityThreshold();
		const TSet<FName>&  EnabledLogCategories = FTimingProfilerManager::Get()->GetLogFilter()->GetEnabledLogCategories();
		if (!(Message.Verbosity <= VerbosityThreshold && EnabledLogCategories.Contains(CleanedCategoryStr)))
		{
			return;
		}
	}

	float X = Viewport.TimeToSlateUnitsRounded(Message.Time);
	if (X < 0.0f)
	{
		X = -1.0f;
	}
	
	const TCHAR* CategoryName = Message.Category->Name != nullptr ? Message.Category->Name : TEXT("");
	AddTimeMarker(X, Message.Index, Message.Verbosity, CategoryName, Message.Message);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FTimeMarkerTrackBuilder::GetColorByCategory(const TCHAR* const Category)
{
	// Strip the "Log" prefix.
	FStringView CategoryStr(Category);
	if (CategoryStr.StartsWith(TEXT("Log")))
	{
		CategoryStr.RightChopInline(3);
	}

	uint32 Hash = 0;
	for (const TCHAR* c = CategoryStr.GetData(); *c; ++c)
	{
		Hash = (Hash + *c) * 0x2c2c57ed;
	}

	// Divided by 128.0 in order to force bright colors.
	return FLinearColor(((Hash >> 16) & 0xFF) / 128.0f, ((Hash >> 8) & 0xFF) / 128.0f, (Hash & 0xFF) / 128.0f, 1.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FTimeMarkerTrackBuilder::GetColorByVerbosity(const ELogVerbosity::Type Verbosity)
{
	static FLinearColor Colors[] =
	{
		FLinearColor(0.0f, 0.0f, 0.0f, 1.0f), // NoLogging
		FLinearColor(1.0f, 0.0f, 0.0f, 1.0f), // Fatal
		FLinearColor(1.0f, 0.1f, 0.1f, 1.0f), // Error
		FLinearColor(0.7f, 0.5f, 0.0f, 1.0f), // Warning
		FLinearColor(0.0f, 0.7f, 0.0f, 1.0f), // Display
		FLinearColor(0.0f, 0.7f, 1.0f, 1.0f), // Log
		FLinearColor(0.7f, 0.7f, 0.7f, 1.0f), // Verbose
		FLinearColor(1.0f, 1.0f, 1.0f, 1.0f), // VeryVerbose
	};
	static_assert(sizeof(Colors) / sizeof(FLinearColor) == (int)ELogVerbosity::Type::All + 1, "ELogVerbosity::Type has changed!?");
	//return Colors[Verbosity & ELogVerbosity::VerbosityMask];
	return Colors[Verbosity & 7]; // using 7 instead of ELogVerbosity::VerbosityMask (15) to make static analyzer happy
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeMarkerTrackBuilder::Flush(float AvailableTextW)
{
	// Is last marker valid?
	if (LastCategory != nullptr)
	{
		const FLinearColor Color = GetColorByCategory(LastCategory);

		bool bAddNewBox = true;
		if (Track.TimeMarkerBoxes.Num() > 0)
		{
			FTimeMarkerBoxInfo& PrevBox = Track.TimeMarkerBoxes.Last();
			if (PrevBox.X + PrevBox.W == LastX1 &&
				PrevBox.Color.R == Color.R &&
				PrevBox.Color.G == Color.G &&
				PrevBox.Color.B == Color.B)
			{
				// Extend previous box instead.
				PrevBox.W += LastX2 - LastX1;
				bAddNewBox = false;
			}
		}

		if (bAddNewBox)
		{
			// Add new Box info to cache.
			FTimeMarkerBoxInfo& Box = Track.TimeMarkerBoxes.AddDefaulted_GetRef();
			Box.X = LastX1;
			Box.W = LastX2 - LastX1;
			Box.Color = Color;
			Box.Color.A = 0.25f;
		}

		if (AvailableTextW > 6.0f)
		{
			// Strip the "Log" prefix.
			FStringView CategoryStr(LastCategory);
			if (CategoryStr.StartsWith(TEXT("Log")))
			{
				CategoryStr.RightChopInline(3);
			}

			const int32 HorizontalOffset = FMath::RoundToInt((AvailableTextW - 2.0f) * FontScale);
			const int32 LastWholeCharacterIndexCategory = FontMeasureService->FindLastWholeCharacterIndexBeforeOffset(CategoryStr, Font, HorizontalOffset, FontScale);
			const int32 LastWholeCharacterIndexMessage = FontMeasureService->FindLastWholeCharacterIndexBeforeOffset(LastMessage, Font, HorizontalOffset, FontScale);

			if (LastWholeCharacterIndexCategory >= 0 ||
				LastWholeCharacterIndexMessage >= 0)
			{
				// Add new Text info to cache.
				FTimeMarkerTextInfo& TextInfo = Track.TimeMarkerTexts.AddDefaulted_GetRef();
				TextInfo.X = LastX2 + 2.0f;
				TextInfo.Color = Color;
				TextInfo.LogIndex = LastLogIndex;
				if (LastWholeCharacterIndexCategory >= 0)
				{
					TextInfo.Category.AppendChars(CategoryStr.GetData(), LastWholeCharacterIndexCategory + 1);
				}
				if (LastWholeCharacterIndexMessage >= 0)
				{
					TextInfo.Message.AppendChars(LastMessage, LastWholeCharacterIndexMessage + 1);
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeMarkerTrackBuilder::AddTimeMarker(const float X, const uint64 LogIndex, const ELogVerbosity::Type Verbosity, const TCHAR* const Category, const TCHAR* Message)
{
	const float W = X - LastX2;

	if (W > 0.0f) // There is at least 1px from previous box?
	{
		// Flush previous marker (if any).
		Flush(W);

		// Begin new marker info.
		LastX1 = X;
		LastX2 = X + 1.0f;
	}
	else if (W == 0.0f) // Adjacent to previous box?
	{
		// Same color as previous marker?
		if (Category == LastCategory)
		{
			// Extend previous box.
			LastX2++;
		}
		else
		{
			// Flush previous marker (if any).
			Flush(0.0f);

			// Begin new box.
			LastX1 = X;
			LastX2 = X + 1.0f;
		}
	}
	else // Overlaps previous box?
	{
		// Same color as previous marker?
		if (Category == LastCategory)
		{
			// Keep previous box.
		}
		else
		{
			// Shrink previous box.
			LastX2--;

			if (LastX2 > LastX1)
			{
				// Flush previous marker (if any).
				Flush(0.0f);
			}

			// Begin new box.
			LastX1 = X;
			LastX2 = X + 1.0f;
		}
	}

	// Save marker.
	LastCategory = Category;
	LastVerbosity = Verbosity;
	LastLogIndex = LogIndex;
	LastMessage = Message;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeMarkerTrackBuilder::EndLog()
{
	Flush(Viewport.GetWidth() - LastX2);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
