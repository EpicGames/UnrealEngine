// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Fonts/SlateFontInfo.h"

// TraceInsights
#include "Insights/ViewModels/BaseTimingTrack.h"
#include "Insights/ViewModels/TrackHeader.h"

class ITimingEvent;
class FSlateFontMeasure;
struct FSlateBrush;

namespace TraceServices
{
	struct FLogCategoryInfo;
	struct FLogMessageInfo;
	class ILogProvider;
}

namespace UE::Insights { class FDrawContext; }

class FTimingTrackViewport;

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FTimeMarkerBoxInfo
{
	float X;
	float W;
	FLinearColor Color;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FTimeMarkerTextInfo
{
	float X;
	FLinearColor Color;
	FString Category; // truncated Category string
	FString Message; // truncated Message string
	uint64 LogIndex;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMarkersTimingTrack : public FBaseTimingTrack
{
	friend class FTimeMarkerTrackBuilder;

	INSIGHTS_DECLARE_RTTI(FMarkersTimingTrack, FBaseTimingTrack)

public:
	FMarkersTimingTrack();
	virtual ~FMarkersTimingTrack();

	virtual void Reset() override;

	bool IsCollapsed() const { return Header.IsCollapsed(); }
	void Expand() { Header.SetIsCollapsed(false); }
	void Collapse() { Header.SetIsCollapsed(true); }
	void ToggleCollapsed() { Header.ToggleCollapsed(); }

	bool IsBookmarksTrack() const { return bUseOnlyBookmarks; }
	bool IsLogsTrack() const { return !bUseOnlyBookmarks; }
	void SetBookmarksTrackFlag(bool bInUseOnlyBookmarks)
	{
		bUseOnlyBookmarks = bInUseOnlyBookmarks;
		UpdateTrackNameAndHeight();
	}
	void SetBookmarksTrack() { SetBookmarksTrackFlag(true); SetDirtyFlag(); }
	void SetLogsTrack() { SetBookmarksTrackFlag(false); SetDirtyFlag(); }

	bool SaveScreenshot_CanExecute();
	void SaveScreenshot_Execute();

	// Stats
	int32 GetNumLogMessages() const { return NumLogMessages; }
	int32 GetNumBoxes() const { return TimeMarkerBoxes.Num(); }
	int32 GetNumTexts() const { return TimeMarkerTexts.Num(); }

	// FBaseTimingTrack
	virtual void PreUpdate(const ITimingTrackUpdateContext& Context) override;
	virtual void Update(const ITimingTrackUpdateContext& Context) override;
	virtual void PostUpdate(const ITimingTrackUpdateContext& Context) override;
	virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	virtual void PostDraw(const ITimingTrackDrawContext& Context) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	virtual const TSharedPtr<const ITimingEvent> GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const override;
	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;
	virtual void OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const override;

	double Snap(double Time, double SnapTolerance);

private:
	void ResetCache()
	{
		TimeMarkerBoxes.Reset();
		TimeMarkerTexts.Reset();
	}

	void UpdateTrackNameAndHeight();
	void UpdateDrawState(const ITimingTrackUpdateContext& Context);

	void UpdateCategory(const TraceServices::FLogCategoryInfo*& InOutCategory, const TCHAR* CategoryName);

	const TSharedPtr<const ITimingEvent> TryGetHoveredEvent();
	uint32 TryGetHoveredEventScreenshotId();

private:
	TArray<FTimeMarkerBoxInfo> TimeMarkerBoxes;
	TArray<FTimeMarkerTextInfo> TimeMarkerTexts;

	bool bUseOnlyBookmarks = true; // If true, uses only bookmarks; otherwise it uses all log messages.
	const TraceServices::FLogCategoryInfo* BookmarkCategory = nullptr;
	const TraceServices::FLogCategoryInfo* ScreenshotCategory = nullptr;

	FTrackHeader Header;

	uint64 ChangeNumber = 0;

	// Stats
	int32 NumLogMessages = 0;
	mutable int32 NumDrawBoxes = 0;
	mutable int32 NumDrawTexts = 0;

	// Slate resources
	const FSlateBrush* WhiteBrush = nullptr;
	const FSlateFontInfo Font;

	uint32 LastScreenshotId = 0;
	mutable FString HoveredCategory;
	mutable FString HoveredMessage;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimeMarkerTrackBuilder
{
public:
	explicit FTimeMarkerTrackBuilder(FMarkersTimingTrack& InTrack, const FTimingTrackViewport& InViewport, float InFontScale);

	/**
	 * Non-copyable
	 */
	FTimeMarkerTrackBuilder(const FTimeMarkerTrackBuilder&) = delete;
	FTimeMarkerTrackBuilder& operator=(const FTimeMarkerTrackBuilder&) = delete;

	const FTimingTrackViewport& GetViewport() { return Viewport; }

	void BeginLog(const TraceServices::ILogProvider& LogProvider);
	void AddLogMessage(const TraceServices::FLogMessageInfo& Message);
	void EndLog();

	static FLinearColor GetColorByCategory(const TCHAR* const Category);
	static FLinearColor GetColorByVerbosity(const ELogVerbosity::Type Verbosity);

private:
	void Flush(float AvailableTextW);
	void AddTimeMarker(const float X, const uint64 LogIndex, const ELogVerbosity::Type Verbosity, const TCHAR* const Category, const TCHAR* Message);

private:
	FMarkersTimingTrack& Track;
	const FTimingTrackViewport& Viewport;

	const TSharedRef<FSlateFontMeasure> FontMeasureService;
	const FSlateFontInfo Font;
	float FontScale;

	const TraceServices::ILogProvider* LogProviderPtr; // valid only between BeginLog() and EndLog()

	float LastX1;
	float LastX2;
	uint64 LastLogIndex;
	ELogVerbosity::Type LastVerbosity;
	const TCHAR* LastCategory;
	const TCHAR* LastMessage;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
