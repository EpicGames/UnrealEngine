// Copyright Epic Games, Inc. All Rights Reserved.

#include "RegionsTimingTrack.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/Regions.h"

// TraceInsightsCore
#include "InsightsCore/Common/Log.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Filter/ViewModels/FilterConfigurator.h"
#include "InsightsCore/Filter/ViewModels/Filters.h"

// TraceInsights
#include "Insights/Common/InsightsMenuBuilder.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/TimingProfilerCommon.h"
#include "Insights/TimingProfiler/ViewModels/TimingRegionsSharedState.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingViewDrawHelper.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::TimingRegions"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimingRegionsTrackFilterContext
////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingRegionsTrackFilterContext : public FBaseFilterContext
{
public:
	FTimingRegionsTrackFilterContext(const FTimingRegionsTrack& InTrack)
		: Track(InTrack)
	{
	}
	virtual ~FTimingRegionsTrackFilterContext() = default;

	virtual bool HasData(int32 Key) const override
	{
		switch (Key)
		{
		case static_cast<int32>(EFilterField::StartTime):
		case static_cast<int32>(EFilterField::EndTime):
		case static_cast<int32>(EFilterField::Duration):
		case static_cast<int32>(EFilterField::TrackName):
		case static_cast<int32>(EFilterField::RegionName):
		case static_cast<int32>(EFilterField::RegionCategory):
			return true;
		default:
			return false;
		}
	}

	virtual EFilterDataType GetDataType(int32 Key) const override
	{
		switch (Key)
		{
		case static_cast<int32>(EFilterField::StartTime):
		case static_cast<int32>(EFilterField::EndTime):
		case static_cast<int32>(EFilterField::Duration):
			return EFilterDataType::Double;
		case static_cast<int32>(EFilterField::TrackName):
		case static_cast<int32>(EFilterField::RegionName):
		case static_cast<int32>(EFilterField::RegionCategory):
			return EFilterDataType::String;
		default:
			return EFilterDataType::Void;
		}
	}

	virtual double GetDataAsDouble(int32 Key) const override
	{
		if (Region)
		{
			if (Key == static_cast<int32>(EFilterField::StartTime))
			{
				return Region->BeginTime;
			}
			if (Key == static_cast<int32>(EFilterField::EndTime))
			{
				return Region->EndTime;
			}
			if (Key == static_cast<int32>(EFilterField::Duration))
			{
				return Region->EndTime - Region->BeginTime;
			}
		}
		return 0.0;
	}

	virtual const FString& GetDataAsString(int32 Key) const override
	{
		if (Key == static_cast<int32>(EFilterField::TrackName))
		{
			return Track.GetName();
		}
		if (Region)
		{
			if (Key == static_cast<int32>(EFilterField::RegionName))
			{
				check(Region->Timer);
				RegionName = Region->Timer->Name;
				return RegionName;
			}
			if (Key == static_cast<int32>(EFilterField::RegionCategory))
			{
				check(Region->Timer);
				if (Region->Timer->Category)
				{
					RegionCategory = Region->Timer->Category->Name;
					return RegionCategory;
				}
			}
		}
		return EmptyString;
	}

	void SetRegion(const TraceServices::FTimeRegion& InRegion)
	{
		Region = &InRegion;
	}

private:
	const FTimingRegionsTrack& Track;
	const TraceServices::FTimeRegion* Region = nullptr;
	mutable FString RegionName;
	mutable FString RegionCategory;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimingRegionsTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FTimingRegionsTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingRegionsTrack::~FTimingRegionsTrack()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingRegionsTrack::SetRegionsCategory(const TCHAR* InRegionsCategory)
{
	RegionsCategory = InRegionsCategory;
	SetName(FString(TEXT("Timing Regions - ")) + InRegionsCategory);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingRegionsTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	FTimingEventsTrack::BuildContextMenu(MenuBuilder);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingRegionsTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	if (InTooltipEvent.CheckTrack(this) && InTooltipEvent.Is<FTimingEvent>())
	{
		const FTimingEvent& TooltipEvent = InTooltipEvent.As<FTimingEvent>();

		auto MatchEvent = [this, &TooltipEvent](double InStartTime, double InEndTime, uint32 InDepth)
		{
			return InDepth == TooltipEvent.GetDepth()
				&& InStartTime == TooltipEvent.GetStartTime()
				&& InEndTime == TooltipEvent.GetEndTime();
		};

		FTimingEventSearchParameters SearchParameters(TooltipEvent.GetStartTime(), TooltipEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch, MatchEvent);
		FindRegionEvent(SearchParameters,
			[this, &InOutTooltip, &TooltipEvent]
			(double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const TraceServices::FTimeRegion& InRegion)
			{
				check(InRegion.Timer);
				InOutTooltip.Reset();
				InOutTooltip.AddTitle(InRegion.Timer->Name, FLinearColor::White);
				InOutTooltip.AddNameValueTextLine(TEXT("Duration:"), FormatTimeAuto(InRegion.EndTime - InRegion.BeginTime));
				InOutTooltip.AddNameValueTextLine(TEXT("Depth:"), FString::FromInt(InRegion.Depth));
				if (InRegion.Timer->Category && InRegion.Timer->Category->Name)
				{
					InOutTooltip.AddNameValueTextLine(TEXT("Category:"), InRegion.Timer->Category->Name);
				}
				InOutTooltip.UpdateLayout();
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingRegionsTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	const FTimingTrackViewport& Viewport = Context.GetViewport();

	const TraceServices::IRegionProvider& RegionProvider = TraceServices::ReadRegionProvider(*Session);
	TraceServices::FProviderReadScopeLock RegionProviderScopedLock(RegionProvider);

	FStopwatch Stopwatch;
	Stopwatch.Start();

	// We are counting only non-empty lanes, so we can collapse empty ones in the visualization.
	int32 CurDepth = 0;
	const TraceServices::IRegionTimeline* Timeline = RegionProvider.GetTimelineForCategory(RegionsCategory);
	check(Timeline)

	Timeline->EnumerateLanes(
		[this, Viewport, &CurDepth, &Builder]
		(const TraceServices::FRegionLane& Lane, const int32 Depth)
		{
			bool RegionHadEvents = false;
			Lane.EnumerateRegions(Viewport.GetStartTime(), Viewport.GetEndTime(),
				[this, &Builder, &RegionHadEvents, &CurDepth]
				(const TraceServices::FTimeRegion& Region) -> bool
				{
					check(Region.Timer);
					RegionHadEvents = true;
					uint32 EventColor = SharedState.bColorRegionsByCategory ?
						FTimingEvent::ComputeEventColor(Region.Timer->Category ? Region.Timer->Category->Name : nullptr) :
						FTimingEvent::ComputeEventColor(Region.Timer->Name);
					Builder.AddEvent(Region.BeginTime, Region.EndTime,CurDepth, Region.Timer->Name, 0, EventColor);
					return true;
				});

			if (RegionHadEvents)
			{
				CurDepth++;
			}
		});

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	UE_CLOGF(TotalTime > 1.0, LogTimingProfiler, Verbose, "[Regions] Updated draw state in %ls.", *FormatTimeAuto(TotalTime));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingRegionsTrack::BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	const TSharedPtr<ITimingEventFilter> EventFilterPtr = Context.GetEventFilter();
	if (EventFilterPtr.IsValid() && EventFilterPtr->FilterTrack(*this))
	{
		bool bFilterOnlyByEventType = false; // this is the most often use case, so the below code tries to optimize it
		TCHAR* FilterEventType = 0;
		if (EventFilterPtr->Is<FTimingEventFilterByEventType>())
		{
			bFilterOnlyByEventType = true;
			const FTimingEventFilterByEventType& EventFilter = EventFilterPtr->As<FTimingEventFilterByEventType>();
			FilterEventType = reinterpret_cast<TCHAR*>(EventFilter.GetEventType());
		}

		if (bFilterOnlyByEventType)
		{
			const TraceServices::IRegionProvider& RegionProvider = TraceServices::ReadRegionProvider(*Session);
			TraceServices::FProviderReadScopeLock RegionProviderScopedLock(RegionProvider);

			const FTimingTrackViewport& Viewport = Context.GetViewport();

			int32 CurDepth = 0;
			const TraceServices::IRegionTimeline* Timeline = RegionProvider.GetTimelineForCategory(RegionsCategory);
			check(Timeline)
			Timeline->EnumerateLanes(
				[this, Viewport, &CurDepth, &Builder, FilterEventType]
				(const TraceServices::FRegionLane& Lane, const int32 Depth)
				{
					bool RegionHadEvents = false;
					Lane.EnumerateRegions(Viewport.GetStartTime(), Viewport.GetEndTime(),
						[&Builder, &RegionHadEvents, &CurDepth, FilterEventType]
						(const TraceServices::FTimeRegion& Region) -> bool
						{
							check(Region.Timer);
							if (Region.Timer->Name == FilterEventType)
							{
								Builder.AddEvent(Region.BeginTime, Region.EndTime, CurDepth, Region.Timer->Name);
							}
							RegionHadEvents = true;
							return true;
						});

					if (RegionHadEvents)
					{
						CurDepth++;
					}
				});
		}
		else // generic filter
		{
			//TODO: if (EventFilterPtr->FilterEvent(TimingEvent))
		}
	}

	if (HasCustomFilter())
	{
		FTimingRegionsTrackFilterContext FilterContext(*this);
		const FTimingTrackViewport& Viewport = Context.GetViewport();

		const TraceServices::IRegionProvider& RegionProvider = TraceServices::ReadRegionProvider(*Session);
		TraceServices::FProviderReadScopeLock RegionProviderScopedLock(RegionProvider);

		int32 CurDepth = 0;
		const TraceServices::IRegionTimeline* Timeline = RegionProvider.GetTimelineForCategory(RegionsCategory);
		check(Timeline)
		Timeline->EnumerateLanes(
			[this, &Viewport, &CurDepth, &Builder, &FilterContext]
			(const TraceServices::FRegionLane& Lane, const int32 Depth)
			{
				bool RegionHadEvents = false;
				Lane.EnumerateRegions(Viewport.GetStartTime(), Viewport.GetEndTime(),
					[&Builder, &RegionHadEvents, &CurDepth, &FilterContext, this]
					(const TraceServices::FTimeRegion& Region) -> bool
					{
						check(Region.Timer);
						FilterContext.SetRegion(Region);
						if (FilterConfigurator->ApplyFilters(FilterContext))
						{
							Builder.AddEvent(Region.BeginTime, Region.EndTime, CurDepth, Region.Timer->Name);
						}
						RegionHadEvents = true;
						return true;
					});

				if (RegionHadEvents)
				{
					CurDepth++;
				}
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FTimingRegionsTrack::SearchEvent(
	const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindRegionEvent(InSearchParameters,
		[this, &FoundEvent]
		(double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const TraceServices::FTimeRegion& InRegion)
		{
			check(InRegion.Timer);
			FoundEvent = MakeShared<FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth, reinterpret_cast<uint64>(InRegion.Timer->Name));
		});

	return FoundEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingRegionsTrack::FindRegionEvent(
	const FTimingEventSearchParameters& InParameters,
	TFunctionRef<void(double, double, uint32, const TraceServices::FTimeRegion&)> InFoundPredicate) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return false;
	}

	// If the query start time is larger than the end of the session return false.
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const double SessionDuration = Session->GetDurationSeconds();;
		if (InParameters.StartTime > SessionDuration)
		{
			return false;
		}
	}

	FTimingRegionsTrackFilterContext FilterContext(*this);

	return TTimingEventSearch<TraceServices::FTimeRegion>::Search(

		InParameters,

		// Search Predicate
		[this]
		(TTimingEventSearch<TraceServices::FTimeRegion>::FContext& InContext)
		{
			TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
			if (Session.IsValid())
			{
				const TraceServices::IRegionProvider& RegionProvider = TraceServices::ReadRegionProvider(*Session);
				TraceServices::FProviderReadScopeLock RegionProviderScopedLock(RegionProvider);

				const TraceServices::IRegionTimeline* Timeline = RegionProvider.GetTimelineForCategory(RegionsCategory);
				check(Timeline)

				auto Callback =
					[&InContext]
					(const TraceServices::FTimeRegion& Region) -> bool
					{
						InContext.Check(Region.BeginTime, Region.EndTime, Region.Depth, Region);
						return InContext.ShouldContinueSearching();
					};

				if (InContext.GetParameters().SearchDirection == FTimingEventSearchParameters::ESearchDirection::Forward)
				{
					Timeline->EnumerateRegions(InContext.GetParameters().StartTime, InContext.GetParameters().EndTime, Callback);
				}
				else
				{
					Timeline->EnumerateRegionsBackwards(InContext.GetParameters().EndTime, InContext.GetParameters().StartTime, Callback);
				}
			}
		},

		// Filter Predicate
		[&FilterContext, &InParameters]
		(double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimeRegion& Region)
		{
			if (!InParameters.FilterExecutor.IsValid())
			{
				return true;
			}

			check(Region.Timer);
			FilterContext.SetRegion(Region);
			return InParameters.FilterExecutor->ApplyFilters(FilterContext);
		},

		// Found Predicate
		InFoundPredicate,

		// Payload Matched Predicate
		TTimingEventSearch<TraceServices::FTimeRegion>::NoMatch

	); // Search
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingRegionsTrack::HasCustomFilter() const
{
	return FilterConfigurator.IsValid() && !FilterConfigurator->IsEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingRegionsTrack::SetFilterConfigurator(TSharedPtr<FFilterConfigurator> InFilterConfigurator)
{
	if (FilterConfigurator != InFilterConfigurator)
	{
		FilterConfigurator = InFilterConfigurator;
		SetDirtyFlag();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingRegionsTrack::OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const
{
	if (InSelectedEvent.CheckTrack(this) && InSelectedEvent.Is<FTimingEvent>())
	{
		const FTimingEvent& TrackEvent = InSelectedEvent.As<FTimingEvent>();

		// The pointer should be safe to access because it is stored in the Session string store.
		FString EventName(reinterpret_cast<const TCHAR*>(TrackEvent.GetType()));
		FTimingEventsTrackDrawStateBuilder::AppendDurationToEventName(EventName, TrackEvent.GetDuration());

		FPlatformApplicationMisc::ClipboardCopy(*EventName);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
