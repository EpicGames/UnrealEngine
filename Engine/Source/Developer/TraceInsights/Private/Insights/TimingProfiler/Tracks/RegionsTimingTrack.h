// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace TraceServices
{
	struct FTimeRegion;
}

// TraceInsights
#include "Insights/ViewModels/TimingEventsTrack.h"

namespace UE::Insights { class FFilterConfigurator; }

namespace UE::Insights::TimingProfiler
{

class FTimingRegionsSharedState;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingRegionsTrack : public FTimingEventsTrack
{
	INSIGHTS_DECLARE_RTTI(FTimingRegionsTrack, FTimingEventsTrack)

public:
	explicit FTimingRegionsTrack(FTimingRegionsSharedState& InSharedState)
		: FTimingEventsTrack(TEXT("Timing Regions"))
		, SharedState(InSharedState)
	{
	}

	virtual ~FTimingRegionsTrack() override;

	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;
	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;
	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void SetFilterConfigurator(TSharedPtr<FFilterConfigurator> InFilterConfigurator) override;
	virtual bool HasCustomFilter() const override;
	virtual void OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const override;

	void SetRegionsCategory(const TCHAR* InRegionsCategory);

private:
	bool FindRegionEvent(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const TraceServices::FTimeRegion&)> InFoundPredicate) const;

private:
	FTimingRegionsSharedState& SharedState;

	TSharedPtr<FFilterConfigurator> FilterConfigurator;
	const TCHAR* RegionsCategory = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
