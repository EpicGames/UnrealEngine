// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/TimingProfiler/Tracks/ThreadTimingTrack.h"
#include "ThreadTimingTrackPrivate.h"

#include "Async/TaskGraphInterfaces.h" // for ENamedThreads
#include "CborReader.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Serialization/MemoryReader.h"

// TraceServices
#include "TraceServices/Containers/Timelines.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Threads.h"
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Filter/ViewModels/FilterConfigurator.h"
#include "InsightsCore/Filter/ViewModels/Filters.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Tracks/CpuTimingTrack.h"
#include "Insights/TimingProfiler/UserAnnotationsTimingViewExtender.h"
#include "Insights/TimingProfiler/ViewModels/ThreadTimingSharedState.h"
#include "Insights/ViewModels/ThreadTrackEvent.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingViewDrawHelper.h"
#include "Insights/ViewModels/TooltipDrawState.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::ThreadTiming"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FThreadTimingTrackFilterContext
////////////////////////////////////////////////////////////////////////////////////////////////////

class FThreadTimingTrackFilterContext : public FBaseFilterContext
{
public:
	FThreadTimingTrackFilterContext(const FThreadTimingTrackImpl& InTrack)
		: Track(InTrack)
	{
	}
	virtual ~FThreadTimingTrackFilterContext() = default;

	virtual bool HasData(int32 Key) const override
	{
		switch (Key)
		{
		case static_cast<int32>(EFilterField::StartTime):
		case static_cast<int32>(EFilterField::EndTime):
		case static_cast<int32>(EFilterField::Duration):
		case static_cast<int32>(EFilterField::TrackName):
		case static_cast<int32>(EFilterField::TimerId):
		case static_cast<int32>(EFilterField::TimerName):
		case static_cast<int32>(EFilterField::Metadata):
			return true;
		default:
			return false;
		}
	}

	virtual EFilterDataType GetDataType(int32 Key) const override
	{
		switch (Key)
		{
		case static_cast<int32>(EFilterField::StartTime):
		case static_cast<int32>(EFilterField::EndTime):
		case static_cast<int32>(EFilterField::Duration):
			return EFilterDataType::Double;
		case static_cast<int32>(EFilterField::TrackName):
			return EFilterDataType::String;
		case static_cast<int32>(EFilterField::TimerId):
		case static_cast<int32>(EFilterField::TimerName): // exposed as TimerId
		case static_cast<int32>(EFilterField::Metadata):
			return EFilterDataType::Int64;
		default:
			return EFilterDataType::Void;
		}
	}

	virtual int64 GetDataAsInt64(int32 Key) const override
	{
		if (Key == static_cast<int32>(EFilterField::TimerId) ||
			Key == static_cast<int32>(EFilterField::TimerName)) // exposed as TimerId
		{
			check(TimerReader);
			const TraceServices::FTimingProfilerTimer* Timer = TimerReader->GetTimer(TimerIndex);
			return ensure(Timer != nullptr) ? int64(Timer->Id) : int64(~0u);
		}
		if (Key == static_cast<int32>(EFilterField::Metadata))
		{
			return int64(TimerIndex);
		}
		return 0;
	}

	virtual double GetDataAsDouble(int32 Key) const override
	{
		if (Key == static_cast<int32>(EFilterField::StartTime))
		{
			return EventStartTime;
		}
		if (Key == static_cast<int32>(EFilterField::EndTime))
		{
			return EventEndTime;
		}
		if (Key == static_cast<int32>(EFilterField::Duration))
		{
			return EventEndTime - EventStartTime;
		}
		return 0.0;
	}

	virtual const FString& GetDataAsString(int32 Key) const override
	{
		if (Key == static_cast<int32>(EFilterField::TrackName))
		{
			return Track.GetName();
		}
		return EmptyString;
	}

	void SetTimerReader(const TraceServices::ITimingProfilerTimerReader& InTimerReader)
	{
		TimerReader = &InTimerReader;
	}

	void SetEvent(double StartTime, double EndTime, uint32 Depth, const TraceServices::FTimingProfilerEvent& Event)
	{
		EventStartTime = StartTime;
		EventEndTime = EndTime;
		EventDepth = Depth;
		TimerIndex = Event.TimerIndex;
	}

private:
	const FThreadTimingTrackImpl& Track;
	const TraceServices::ITimingProfilerTimerReader* TimerReader;
	double EventStartTime = 0.0;
	double EventEndTime = 0.0;
	uint32 EventDepth = 0;
	uint32 TimerIndex = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FThreadTimingTrackImpl
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FThreadTimingTrack);
INSIGHTS_IMPLEMENT_RTTI(FThreadTimingTrackImpl)

////////////////////////////////////////////////////////////////////////////////////////////////////

static void AppendMetadataToTooltip(FTooltipDrawState& InOutTooltip, const TraceServices::FMetadataSpec* InMetadataSpec, TArrayView<const uint8>& InMetadata)
{
	FMemoryReaderView MemoryReader(InMetadata);
	FCborReader CborReader(&MemoryReader, ECborEndianness::StandardCompliant);
	FCborContext Context;

	// We have 3 cases:
	// 1. InMetadataSpec in not null and InMetadata is a list of values (dynamic metadata)
	// 2. InMetadataSpec is null and InMetadata is a list of values (dynamic metadata without a spec emitted or saved)
	// 3. InMetadataSpec is null and InMetadata is a map of key value pairs. (static metadata)
	bool bIsMap = false;
	bool bHasReadFirstValue = false;
	if (InMetadataSpec == nullptr)
	{
		if (!CborReader.ReadNext(Context))
		{
			InOutTooltip.AddTitle(TEXTVIEW("<empty metadata>"), FLinearColor(1.0f, 0.5f, 0.5f, 1.0f));
			return;
		}

		if (Context.MajorType() == ECborCode::Map)
		{
			bIsMap = true;
		}
		else
		{
			bHasReadFirstValue = true;
		}
	}

	InOutTooltip.AddTitle(TEXTVIEW("Metadata..."), FLinearColor(0.3f, 0.3f, 0.3f, 1.0f));

	for (uint32 Index = 0; true; ++Index)
	{
		FString Key;

		if (InMetadataSpec)
		{
			if (Index < static_cast<uint32>(InMetadataSpec->FieldNames.Num()))
			{
				Key = InMetadataSpec->FieldNames[Index];
				Key += TEXT(":");
			}
			else
			{
				Key = TEXT(""); // undefined field
			}
		}
		else if (bIsMap)
		{
			// Read key
			if (!CborReader.ReadNext(Context) || !Context.IsString())
			{
				break;
			}

			Key = FString::ConstructFromPtrSize(Context.AsCString(), static_cast<int32>(Context.AsLength()));
			Key += TEXT(":");
		}

		// Read value
		if (Index > 0 || !bHasReadFirstValue) // Skip reading if we already read the first value.
		{
			if (!CborReader.ReadNext(Context))
			{
				break;
			}
		}

		switch (Context.MajorType())
		{

		case ECborCode::Int:
		{
			int64 Value = Context.AsInt();
			FString ValueStr;
			if (Value > 999'999'999LL)
			{
				ValueStr += FString::Printf(TEXT("0x%llX"), Value);
			}
			else
			{
				ValueStr += FString::Printf(TEXT("%lld"), Value);
			}
			InOutTooltip.AddNameValueTextLine(Key, ValueStr);
			continue;
		}

		case ECborCode::Uint:
		{
			uint64 Value = Context.AsUInt();
			FString ValueStr;
			if (Value > 999'999'999ULL)
			{
				ValueStr = FString::Printf(TEXT("0x%llX"), Value);
			}
			else
			{
				ValueStr = FString::Printf(TEXT("%llu"), Value);
			}
			InOutTooltip.AddNameValueTextLine(Key, ValueStr);
			continue;
		}

		case ECborCode::TextString:
		{
			FString Value = Context.AsString();
			InOutTooltip.AddNameValueTextLine(Key, Value);
			continue;
		}

		case ECborCode::ByteString:
		{
			FString ValueStr = FString::ConstructFromPtrSize(Context.AsCString(), static_cast<int32>(Context.AsLength()));
			InOutTooltip.AddNameValueTextLine(Key, ValueStr);
			continue;
		}

		} // switch

		if (Context.RawCode() == (ECborCode::Prim|ECborCode::Value_4Bytes))
		{
			float Value = Context.AsFloat();
			FString ValueStr = FString::Printf(TEXT("%f"), Value);
			InOutTooltip.AddNameValueTextLine(Key, ValueStr);
			continue;
		}

		if (Context.RawCode() == (ECborCode::Prim|ECborCode::Value_8Bytes))
		{
			double Value = Context.AsDouble();
			FString ValueStr = FString::Printf(TEXT("%g"), Value);
			InOutTooltip.AddNameValueTextLine(Key, ValueStr);
			continue;
		}

		if (Context.RawCode() == (ECborCode::Prim | ECborCode::False))
		{
			InOutTooltip.AddNameValueTextLine(Key, TEXTVIEW("false"));
			continue;
		}

		if (Context.RawCode() == (ECborCode::Prim | ECborCode::True))
		{
			InOutTooltip.AddNameValueTextLine(Key, TEXTVIEW("true"));
			continue;
		}

		InOutTooltip.AddNameValueTextLine(Key, TEXTVIEW("???")); // unknown field type

		if (Context.MajorType() == ECborCode::Array)
		{
			CborReader.SkipContainer(ECborCode::Array);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static void AppendMetadataToString(FString& InOutStr, TArrayView<const uint8>& InMetadata)
{
	FMemoryReaderView MemoryReader(InMetadata);
	FCborReader CborReader(&MemoryReader, ECborEndianness::StandardCompliant);
	FCborContext Context;

	if (!CborReader.ReadNext(Context))
	{
		InOutStr += TEXTVIEW(" - <empty metadata>");
		return;
	}

	bool bIsMap = false;
	if (Context.MajorType() == ECborCode::Map)
	{
		bIsMap = true;
	}

	for (uint32 Index = 0; true; ++Index)
	{
		if (bIsMap)
		{
			// Read key
			if (!CborReader.ReadNext(Context) || !Context.IsString())
			{
				break;
			}
		}

		if (Index > 0 || bIsMap) // Skip reading if we already read the first value
		{
			// Read value
			if (!CborReader.ReadNext(Context))
			{
				break;
			}
		}

		if (Index == 0)
		{
			InOutStr += TEXTVIEW(" - ");
		}
		else
		{
			InOutStr += TEXTVIEW(", ");
		}

		switch (Context.MajorType())
		{

		case ECborCode::Int:
		{
			int64 Value = Context.AsInt();
			if (Value > 999'999'999LL)
			{
				InOutStr += FString::Printf(TEXT("0x%llX"), Value);
			}
			else
			{
				InOutStr += FString::Printf(TEXT("%lld"), Value);
			}
			continue;
		}

		case ECborCode::Uint:
		{
			uint64 Value = Context.AsUInt();
			if (Value > 999'999'999ULL)
			{
				InOutStr += FString::Printf(TEXT("0x%llX"), Value);
			}
			else
			{
				InOutStr += FString::Printf(TEXT("%llu"), Value);
			}
			continue;
		}

		case ECborCode::TextString:
		{
			InOutStr += Context.AsString();
			continue;
		}

		case ECborCode::ByteString:
		{
			InOutStr.AppendChars(Context.AsCString(), static_cast<int32>(Context.AsLength()));
			continue;
		}

		} // switch

		if (Context.RawCode() == (ECborCode::Prim | ECborCode::Value_4Bytes))
		{
			float Value = Context.AsFloat();
			InOutStr += FString::Printf(TEXT("%f"), Value);
			continue;
		}

		if (Context.RawCode() == (ECborCode::Prim | ECborCode::Value_8Bytes))
		{
			double Value = Context.AsDouble();
			InOutStr += FString::Printf(TEXT("%g"), Value);
			continue;
		}

		if (Context.RawCode() == (ECborCode::Prim | ECborCode::False))
		{
			InOutStr += TEXTVIEW("false");
			continue;
		}

		if (Context.RawCode() == (ECborCode::Prim | ECborCode::True))
		{
			InOutStr += TEXTVIEW("true");
			continue;
		}

		InOutStr += TEXTVIEW("???"); // unknown field type

		if (Context.MajorType() == ECborCode::Array)
		{
			CborReader.SkipContainer(ECborCode::Array);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static void AppendMetadataToString(FString& InOutStr, const FString& InFormat, TArrayView<const uint8>& InMetadata)
{
	if (InFormat.IsEmpty())
	{
		return;
	}

	FMemoryReaderView MemoryReader(InMetadata);
	FCborReader CborReader(&MemoryReader, ECborEndianness::StandardCompliant);
	FCborContext Context;

	FString Format = InFormat;

	Format.TrimStartInline();
	if (Format.StartsWith(TEXT("- ")))
	{
		Format.RightChopInline(2);
	}

	InOutStr += TEXTVIEW(" - ");

	const FString Specifiers = TEXT("diuoxXfFeEgGaAcspn");
	auto GetNextFormatSection = [&Format, &Specifiers]()
	{
		bool bIsInFormatSpecifier = false;
		for (int32 Index = 0; Index < Format.Len(); ++Index)
		{
			if (bIsInFormatSpecifier)
			{
				int32 SpecIndex = -1;
				if (Specifiers.FindChar(Format[Index], SpecIndex))
				{
					FString NextFormat = Format.Left(Index+1);
					Format.MidInline(Index+1);
					return NextFormat;
				}
			}

			if (Format[Index] == TEXT('%'))
			{
				bIsInFormatSpecifier = !bIsInFormatSpecifier;
			}
		}

		FString Copy = Format;
		Format.Empty();
		return Copy;
	};

	for (uint32 Index = 0; !Format.IsEmpty(); ++Index)
	{
		// Read value
		if (!CborReader.ReadNext(Context))
		{
			break;
		}

		constexpr int MaxLength = 1024;
		TCHAR Data[MaxLength];

		auto AddValueToName = [&InOutStr](TCHAR* Dest, int MaxLength, const FString& Format, auto Value)
		{
			int32 Result = FCString::Snprintf(Dest, MaxLength, reinterpret_cast<TCHAR const(&)[1]>(**Format), Value);
			if (Result > 0)
			{
				InOutStr.Append(Dest);
			}
		};

		switch (Context.MajorType())
		{

		case ECborCode::Int:
		{
			int64 Value = Context.AsInt();
			AddValueToName(Data, MaxLength, GetNextFormatSection(), Value);
			continue;
		}

		case ECborCode::Uint:
		{
			uint64 Value = Context.AsUInt();
			AddValueToName(Data, MaxLength, GetNextFormatSection(), Value);
			continue;
		}

		case ECborCode::TextString:
		{
			AddValueToName(Data, MaxLength, GetNextFormatSection(), *Context.AsString());
			continue;
		}

		case ECborCode::ByteString:
		{
			AddValueToName(Data, FMath::Min(MaxLength, (int)Context.AsLength()), GetNextFormatSection(), Context.AsCString());
			continue;
		}

		} // switch

		if (Context.RawCode() == (ECborCode::Prim | ECborCode::Value_4Bytes))
		{
			AddValueToName(Data, MaxLength, GetNextFormatSection(), Context.AsFloat());
			continue;
		}

		if (Context.RawCode() == (ECborCode::Prim | ECborCode::Value_8Bytes))
		{
			AddValueToName(Data, MaxLength, GetNextFormatSection(), Context.AsDouble());
			continue;
		}

		if (Context.RawCode() == (ECborCode::Prim | ECborCode::False))
		{
			AddValueToName(Data, MaxLength, GetNextFormatSection(), false);
			continue;
		}

		if (Context.RawCode() == (ECborCode::Prim | ECborCode::True))
		{
			AddValueToName(Data, MaxLength, GetNextFormatSection(), true);
			continue;
		}

		GetNextFormatSection();
		AddValueToName(Data, MaxLength, FString(TEXT("%s")), TEXT("???")); // unknown field type

		if (Context.MajorType() == ECborCode::Array)
		{
			CborReader.SkipContainer(ECborCode::Array);
		}
	}

	// Append what's left of the format string.
	InOutStr.Append(Format);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void AppendInfoToTimingEventDisplayName(
	FString& InOutDisplayName,
	double EventStartTime,
	double EventEndTime,
	uint32 TimerIndex,
	const TraceServices::FTimingProfilerTimer* Timer,
	const TraceServices::ITimingProfilerProvider& TimingProfilerProvider,
	const TraceServices::ITimingProfilerTimerReader& TimerReader)
{
	const double Duration = EventEndTime - EventStartTime;
	FTimingEventsTrackDrawStateBuilder::AppendDurationToEventName(InOutDisplayName, Duration);

	TArrayView<const uint8> Metadata;
	if (int32(TimerIndex) < 0) // has static metadata
	{
		Metadata = TimerReader.GetMetadata(TimerIndex);
	}
	const TraceServices::FMetadataSpec* MetadataSpec = nullptr;
	if (Timer->HasValidMetadataSpecId()) // dynamic metadata with spec
	{
		MetadataSpec = TimingProfilerProvider.GetMetadataSpec(Timer->MetadataSpecId);
		if (MetadataSpec)
		{
			AppendMetadataToString(InOutDisplayName, MetadataSpec->Format, Metadata);
			return;
		}
	}
	if (Metadata.Num() > 0)
	{
		AppendMetadataToString(InOutDisplayName, Metadata);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void AddTimingEventToBuilder(
	ITimingEventsTrackDrawStateBuilder& Builder,
	double EventStartTime,
	double EventEndTime,
	uint32 EventDepth,
	uint32 TimerIndex,
	const TraceServices::FTimingProfilerTimer* Timer,
	const TraceServices::ITimingProfilerProvider& TimingProfilerProvider,
	const TraceServices::ITimingProfilerTimerReader& TimerReader)
{
	if (EventDepth >= FTimingProfilerManager::Get()->GetEventDepthLimit())
	{
		return;
	}

	uint32 EventColor;
	switch (FTimingProfilerManager::Get()->GetColoringMode())
	{
		case ETimingEventsColoringMode::ByTimerName:
			EventColor = FTimingEvent::ComputeEventColor(Timer->Name);
			break;
		case ETimingEventsColoringMode::ByTimerNameAndMetadata:
			EventColor = int32(TimerIndex) < 0 // has metadata?
				? FTimingEvent::ComputeEventColor(Timer->Name, TimerReader.GetMetadata(TimerIndex))
				: FTimingEvent::ComputeEventColor(Timer->Name);
			break;
		case ETimingEventsColoringMode::ByTimerId:
			EventColor = FTimingEvent::ComputeEventColor(Timer->Id);
			break;
		case ETimingEventsColoringMode::BySourceFile:
			EventColor = FTimingEvent::ComputeEventColor(Timer->File);
			break;
		case ETimingEventsColoringMode::ByDuration:
		{
			const double EventDuration = EventEndTime - EventStartTime;
			EventColor = (EventDuration >= 0.01)     ? 0xFF883333 : // red:    >= 10ms
						 (EventDuration >= 0.001)    ? 0xFF998833 : // yellow: [1ms .. 10ms)
						 (EventDuration >= 0.0001)   ? 0xFF338833 : // green:  [100us .. 1ms)
						 (EventDuration >= 0.00001)  ? 0xFF338888 : // cyan:   [10us .. 100us)
						 (EventDuration >= 0.000001) ? 0xFF333388 : // blue:   [1us .. 10us)
						                               0xFF888888;  // gray:   < 1us
			break;
		}
		default:
			EventColor = 0xFF000000;
	}

	Builder.AddEvent(EventStartTime, EventEndTime, EventDepth, EventColor,
		[TimerIndex, Timer, EventStartTime, EventEndTime, &TimingProfilerProvider, &TimerReader]
		(float Width)
		{
			FString EventName = Timer->Name;

			const float MinWidth = static_cast<float>(EventName.Len()) * 4.0f + 32.0f;
			if (Width > MinWidth)
			{
				AppendInfoToTimingEventDisplayName(EventName, EventStartTime, EventEndTime, TimerIndex, Timer, TimingProfilerProvider, TimerReader);
			}

			return EventName;
		}
	); // Builder.AddEvent
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingTrackImpl::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	const FTimingTrackViewport& Viewport = Context.GetViewport();

	ReadTimers(
		[this, &Builder, &Viewport]
		(const TraceServices::ITimingProfilerProvider& TimingProfilerProvider, const TraceServices::ITimingProfilerTimerReader& TimerReader)
		{
			ReadTimeline(
				[this, &Builder, &Viewport, &TimingProfilerProvider, &TimerReader]
				(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
				{
					if (FTimingEventsTrack::bUseDownSampling)
					{
						const double SecondsPerPixel = 1.0 / Viewport.GetScaleX();
						Timeline.EnumerateEventsDownSampled(Viewport.GetStartTime(), Viewport.GetEndTime(), SecondsPerPixel,
							[this, &Builder, &TimingProfilerProvider, &TimerReader]
							(double StartTime, double EndTime, uint32 Depth, const TraceServices::FTimingProfilerEvent& Event)
							{
								const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(Event.TimerIndex);
								if (ensure(Timer != nullptr))
								{
									AddTimingEventToBuilder(Builder, StartTime, EndTime, Depth, Event.TimerIndex, Timer, TimingProfilerProvider, TimerReader);
								}
								else
								{
									Builder.AddEvent(StartTime, EndTime, Depth, 0xFF000000, [&Event](float) { return FString::Printf(TEXT("[%u]"), Event.TimerIndex); });
								}
								return TraceServices::EEventEnumerate::Continue;
							}
						);
					}
					else
					{
						Timeline.EnumerateEvents(Viewport.GetStartTime(), Viewport.GetEndTime(),
							[this, &Builder, &TimingProfilerProvider, &TimerReader]
							(double StartTime, double EndTime, uint32 Depth, const TraceServices::FTimingProfilerEvent& Event)
							{
								const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(Event.TimerIndex);
								if (ensure(Timer != nullptr))
								{
									AddTimingEventToBuilder(Builder, StartTime, EndTime, Depth, Event.TimerIndex, Timer, TimingProfilerProvider, TimerReader);
								}
								else
								{
									Builder.AddEvent(StartTime, EndTime, Depth, 0xFF000000, [&Event](float) { return FString::Printf(TEXT("[%u]"), Event.TimerIndex); });
								}
								return TraceServices::EEventEnumerate::Continue;
							}
						);
					}
				}
			); // ReadTimeline
		}
	); // ReadTimers
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingTrackImpl::BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	struct FPendingEventInfo
	{
		double StartTime;
		double EndTime;
		uint32 Depth;
		uint32 TimerIndex;
	};

	const TSharedPtr<ITimingEventFilter> EventFilterPtr = Context.GetEventFilter();
	if (EventFilterPtr.IsValid() && EventFilterPtr->FilterTrack(*this))
	{
		bool bFilterOnlyByEventType = false; // this is the most often use case, so the below code tries to optimize it
		uint64 FilterEventType = 0;
		if (EventFilterPtr->Is<FTimingEventFilterByEventType>())
		{
			bFilterOnlyByEventType = true;
			const FTimingEventFilterByEventType& EventFilter = EventFilterPtr->As<FTimingEventFilterByEventType>();
			FilterEventType = EventFilter.GetEventType();
		}

		const FTimingTrackViewport& Viewport = Context.GetViewport();

		ReadTimers(
			[this, &Builder, &EventFilterPtr, bFilterOnlyByEventType, FilterEventType, &Viewport]
			(const TraceServices::ITimingProfilerProvider& TimingProfilerProvider, const TraceServices::ITimingProfilerTimerReader& TimerReader)
			{
				if (bFilterOnlyByEventType)
				{
					//TODO: Add a setting to switch this on/off
					#if 1
					{
						ReadTimeline(
							[this, &Builder, FilterEventType, &Viewport, &TimingProfilerProvider, &TimerReader]
							(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
							{
								TArray<TArray<FPendingEventInfo>> FilteredEvents;

								TraceServices::ITimeline<TraceServices::FTimingProfilerEvent>::EnumerateAsyncParams Params;
								Params.IntervalStart = Viewport.GetStartTime();
								Params.IntervalEnd = Viewport.GetEndTime();
								Params.Resolution = 0.0;
								Params.SetupCallback =
									[&FilteredEvents](uint32 NumTasks)
									{
										FilteredEvents.AddDefaulted(NumTasks);
									};
								Params.EventRangeCallback =
									[this, &Builder, FilterEventType, &TimerReader, &FilteredEvents]
									(double StartTime, double EndTime, uint32 Depth, const TraceServices::FTimingProfilerEvent& Event, uint32 TaskIndex)
									{
										const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(Event.TimerIndex);
										if (ensure(Timer != nullptr))
										{
											if (Timer->Id == FilterEventType)
											{
												FPendingEventInfo TimelineEvent;
												TimelineEvent.StartTime = StartTime;
												TimelineEvent.EndTime = EndTime;
												TimelineEvent.Depth = Depth;
												TimelineEvent.TimerIndex = Event.TimerIndex;
												FilteredEvents[TaskIndex].Add(TimelineEvent);
											}
										}
										return TraceServices::EEventEnumerate::Continue;
									};

								// Note: Enumerating events for filtering should not use downsampling.
								Timeline.EnumerateEventsDownSampledAsync(Params);

								for (TArray<FPendingEventInfo>& Array : FilteredEvents)
								{
									for (FPendingEventInfo& TimelineEvent : Array)
									{
										const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(TimelineEvent.TimerIndex);
										AddTimingEventToBuilder(Builder, TimelineEvent.StartTime, TimelineEvent.EndTime, TimelineEvent.Depth, TimelineEvent.TimerIndex, Timer, TimingProfilerProvider, TimerReader);
									}
								}
							}); // ReadTimeline
					}
					#else
					{
						ReadTimeline(
							[this, &Builder, FilterEventType, &Viewport, &TimingProfilerProvider, &TimerReader]
							(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
							{
								// Note: Enumerating events for filtering should not use downsampling.
								Timeline.EnumerateEventsDownSampled(Viewport.GetStartTime(), Viewport.GetEndTime(), 0,
									[this, &Builder, FilterEventType, &TimingProfilerProvider, &TimerReader]
									(double StartTime, double EndTime, uint32 Depth, const TraceServices::FTimingProfilerEvent& Event)
									{
										const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(Event.TimerIndex);
										if (ensure(Timer != nullptr))
										{
											if (Timer->Id == FilterEventType)
											{
												AddTimingEventToBuilder(Builder, StartTime, EndTime, Depth, Event.TimerIndex, Timer, TimingProfilerProvider, TimerReader);
											}
										}
										return TraceServices::EEventEnumerate::Continue;
									}
								);
							}
						); // ReadTimeline
					}
					#endif
				}
				else // generic filter
				{
					ReadTimeline(
						[this, &Builder, &EventFilterPtr, &Viewport, &TimingProfilerProvider, &TimerReader]
						(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
						{
							// Note: Enumerating events for filtering should not use downsampling.
							//const double SecondsPerPixel = 1.0 / Viewport.GetScaleX();
							//Timeline.EnumerateEventsDownSampled(Viewport.GetStartTime(), Viewport.GetEndTime(), SecondsPerPixel,
							Timeline.EnumerateEvents(Viewport.GetStartTime(), Viewport.GetEndTime(),
								[this, &Builder, &EventFilterPtr, &TimingProfilerProvider, &TimerReader]
								(double StartTime, double EndTime, uint32 Depth, const TraceServices::FTimingProfilerEvent& Event)
								{
									const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(Event.TimerIndex);
									if (ensure(Timer != nullptr))
									{
										FThreadTrackEvent TimingEvent(SharedThis(this), StartTime, EndTime, Depth);
										TimingEvent.SetTimerId(Timer->Id);
										TimingEvent.SetTimerIndex(Event.TimerIndex);

										if (EventFilterPtr->FilterEvent(TimingEvent))
										{
											AddTimingEventToBuilder(Builder, StartTime, EndTime, Depth, Event.TimerIndex, Timer, TimingProfilerProvider, TimerReader);
										}
									}
									return TraceServices::EEventEnumerate::Continue;
								}
							);
						}
					); // ReadTimeline
				}
			}
		); // ReadTimers
	}

	if (HasCustomFilter()) // Custom filter (from the filtering widget)
	{
		const FTimingTrackViewport& Viewport = Context.GetViewport();

		ReadTimers(
			[this, &Builder, &Viewport]
			(const TraceServices::ITimingProfilerProvider& TimingProfilerProvider, const TraceServices::ITimingProfilerTimerReader& TimerReader)
			{
				ReadTimeline(
					[this, &Builder, &Viewport, &TimingProfilerProvider, &TimerReader]
					(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
					{
						TArray<TArray<FPendingEventInfo>> FilteredEvents;
						TArray<FThreadTimingTrackFilterContext> FilterContexts;

						TraceServices::ITimeline<TraceServices::FTimingProfilerEvent>::EnumerateAsyncParams Params;
						constexpr uint32 LargeTimelineThreshold = 50 * 1000 * 1000;
						if (Timeline.GetEventCount() > LargeTimelineThreshold)
						{
							if (FilterConfigurator->IsKeyUsed(static_cast<int32>(EFilterField::Metadata)))
							{
								Params.MaxOccupancy = 0.75f; // This filter can be slow so reduce occupancy to avoid starvation.
							}
						}

						Params.IntervalStart = Viewport.GetStartTime();
						Params.IntervalEnd = Viewport.GetEndTime();

						// Note: Enumerating events for filtering should not use downsampling.
						Params.Resolution = 0.0;
						Params.SetupCallback =
							[this, &FilteredEvents, &FilterContexts, &TimerReader]
							(uint32 NumTasks)
							{
								FilteredEvents.AddDefaulted(NumTasks);
								FilterContexts.Reserve(NumTasks);
								for (uint32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
								{
									FThreadTimingTrackFilterContext& FilterContext = FilterContexts.Add_GetRef(FThreadTimingTrackFilterContext(*this));
									FilterContext.SetTimerReader(TimerReader);
								}
							};
						Params.EventRangeCallback =
							[this, &Builder, &FilteredEvents, &FilterContexts]
							(double StartTime, double EndTime, uint32 Depth, const TraceServices::FTimingProfilerEvent& Event, uint32 TaskIndex)
							{
								FThreadTimingTrackFilterContext& FilterContext = FilterContexts[TaskIndex];
								FilterContext.SetEvent(StartTime, EndTime, Depth, Event);
								if (FilterConfigurator->ApplyFilters(FilterContext))
								{
									FPendingEventInfo TimelineEvent;
									TimelineEvent.StartTime = StartTime;
									TimelineEvent.EndTime = EndTime;
									TimelineEvent.Depth = Depth;
									TimelineEvent.TimerIndex = Event.TimerIndex;
									FilteredEvents[TaskIndex].Add(TimelineEvent);
								}
								return TraceServices::EEventEnumerate::Continue;
							};

						Timeline.EnumerateEventsDownSampledAsync(Params);

						for (TArray<FPendingEventInfo>& Array : FilteredEvents)
						{
							for (FPendingEventInfo& TimelineEvent : Array)
							{
								const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(TimelineEvent.TimerIndex);
								AddTimingEventToBuilder(Builder, TimelineEvent.StartTime, TimelineEvent.EndTime, TimelineEvent.Depth, TimelineEvent.TimerIndex, Timer, TimingProfilerProvider, TimerReader);
							}
						}
					}
				); // ReadTimeline
			}
		); // ReadTimers
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingTrackImpl::PostDraw(const ITimingTrackDrawContext& Context) const
{
	for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
	{
		Track->PostDraw(Context);
	}

	const TSharedPtr<const ITimingEvent> SelectedEventPtr = Context.GetSelectedEvent();
	if (SelectedEventPtr.IsValid() &&
		SelectedEventPtr->CheckTrack(this) &&
		SelectedEventPtr->Is<FThreadTrackEvent>())
	{
		const FThreadTrackEvent& SelectedEvent = SelectedEventPtr->As<FThreadTrackEvent>();
		const ITimingViewDrawHelper& Helper = Context.GetHelper();

		ReadTimers(
			[this, &Context, &SelectedEvent, &Helper]
			(const TraceServices::ITimingProfilerProvider& TimingProfilerProvider, const TraceServices::ITimingProfilerTimerReader& TimerReader)
			{
				const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(SelectedEvent.GetTimerIndex());
				if (Timer != nullptr)
				{
					FString TimerName(Timer->Name);

					const double SelectedEventDuration = SelectedEvent.GetDuration();
					TStringBuilder<1024> StringBuilder;
					StringBuilder.Appendf(TEXT("  Incl.: %s"), *FormatTimeAuto(SelectedEventDuration, 2));
					if (SelectedEventDuration != std::numeric_limits<double>::infinity())
					{
						StringBuilder.Appendf(TEXT("   Excl.: %s"), *FormatTimeAuto(SelectedEvent.GetExclusiveTime(), 2));
					}
					FString StatsText(StringBuilder.ToView());

					if (Timer->File)
					{
						FString SourceFile = FPaths::GetCleanFilename(FString(Timer->File));
						FString SourceFileAndLine = FString::Printf(TEXT("%s (%d)"), *SourceFile, Timer->Line);
						DrawSelectedEventInfoEx(StatsText, TimerName, SourceFileAndLine, Context.GetViewport(), Context.GetDrawContext(), Helper.GetWhiteBrush(), Helper.GetEventFont());
					}
					else
					{
						DrawSelectedEventInfoEx(StatsText, TimerName, FString(), Context.GetViewport(), Context.GetDrawContext(), Helper.GetWhiteBrush(), Helper.GetEventFont());
					}
				}
			}
		); // ReadTimers
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingTrackImpl::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	if (!IsChildTrack())
	{
		InOutTooltip.ResetContent();
	}

	if (InTooltipEvent.CheckTrack(this) && InTooltipEvent.Is<FThreadTrackEvent>())
	{
		const FThreadTrackEvent& TooltipEvent = InTooltipEvent.As<FThreadTrackEvent>();

		TSharedPtr<FThreadTrackEvent> ParentTimingEvent;
		TSharedPtr<FThreadTrackEvent> RootTimingEvent;
		GetParentAndRoot(TooltipEvent, ParentTimingEvent, RootTimingEvent);

		ReadTimers(
			[this, &InOutTooltip, &TooltipEvent, &ParentTimingEvent, &RootTimingEvent]
			(const TraceServices::ITimingProfilerProvider& TimingProfilerProvider, const TraceServices::ITimingProfilerTimerReader& TimerReader)
			{
				// Add the timer name.
				const uint32 TimerIndex = TooltipEvent.GetTimerIndex();
				const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(TimerIndex);
				const TCHAR* TimerName = (Timer != nullptr) ? Timer->Name : TEXT("N/A");
				InOutTooltip.AddTitle(TimerName);

				TArrayView<const uint8> Metadata;
				const TraceServices::FMetadataSpec* MetadataSpec = nullptr;
				if (int32(TimerIndex) < 0) // has metadata? //-V547
				{
					Metadata = TimerReader.GetMetadata(TimerIndex);
					if (Timer && Timer->HasValidMetadataSpecId())
					{
						MetadataSpec = TimingProfilerProvider.GetMetadataSpec(Timer->MetadataSpecId);
					}

#if 0 // for debugging
					// Add the full timer name (i.e. with formatting/metadata).
					FString TimerNameEx = TimerName;
					if (MetadataSpec)
					{
						AppendMetadataToString(TimerNameEx, MetadataSpec->Format, Metadata);
					}
					else
					{
						AppendMetadataToString(TimerNameEx, Metadata);
					}
					InOutTooltip.AddTitle(TimerNameEx, FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
#endif
				}

				double SessionDuration = std::numeric_limits<double>::lowest();
				TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
				if (Session.IsValid())
				{
					TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
					SessionDuration = Session->GetDurationSeconds();
				}

				double TooltipEventDuration = TooltipEvent.GetDuration();
				const bool bIsInfinite = (TooltipEventDuration == std::numeric_limits<double>::infinity());
				if (bIsInfinite)
				{
					TooltipEventDuration = SessionDuration - TooltipEvent.GetStartTime();
				}

				if (TooltipEvent.GetDepth() > 0 &&
					ParentTimingEvent.IsValid() &&
					ParentTimingEvent->GetDuration() > 0.0)
				{
					double ParentTimingEventDuration = ParentTimingEvent->GetDuration();
					if (ParentTimingEventDuration == std::numeric_limits<double>::infinity())
					{
						ParentTimingEventDuration = SessionDuration - ParentTimingEvent->GetStartTime();
					}
					if (ParentTimingEventDuration > 0.0)
					{
						const TraceServices::FTimingProfilerTimer* ParentTimer = TimerReader.GetTimer(ParentTimingEvent->GetTimerIndex());
						const TCHAR* ParentTimerName = (ParentTimer != nullptr) ? ParentTimer->Name : TEXT("N/A");
						FNumberFormattingOptions FormattingOptions;
						FormattingOptions.MaximumFractionalDigits = 2;
						const FString ValueStr = FString::Printf(TEXT("%s %s"), *FText::AsPercent(TooltipEventDuration / ParentTimingEventDuration, &FormattingOptions).ToString(), ParentTimerName);
						InOutTooltip.AddNameValueTextLine(TEXTVIEW("% of Parent:"), ValueStr);
					}
				}

				if (TooltipEvent.GetDepth() > 1 &&
					RootTimingEvent.IsValid() &&
					RootTimingEvent->GetDuration() > 0.0)
				{
					double RootTimingEventDuration = RootTimingEvent->GetDuration();
					if (RootTimingEventDuration == std::numeric_limits<double>::infinity())
					{
						RootTimingEventDuration = SessionDuration - RootTimingEvent->GetStartTime();
					}
					if (RootTimingEventDuration > 0.0)
					{
						const TraceServices::FTimingProfilerTimer* RootTimer = TimerReader.GetTimer(RootTimingEvent->GetTimerIndex());
						const TCHAR* RootTimerName = (RootTimer != nullptr) ? RootTimer->Name : TEXT("N/A");
						FNumberFormattingOptions FormattingOptions;
						FormattingOptions.MaximumFractionalDigits = 2;
						const FString ValueStr = FString::Printf(TEXT("%s %s"), *FText::AsPercent(TooltipEventDuration / RootTimingEventDuration, &FormattingOptions).ToString(), RootTimerName);
						InOutTooltip.AddNameValueTextLine(TEXTVIEW("% of Root:"), ValueStr);
					}
				}

				{
					FString InclusiveTimeValue = FormatTimeAuto(TooltipEventDuration, 2);
					if (bIsInfinite)
					{
						InclusiveTimeValue = FString::Printf(TEXT("\u221E (%s)"), *InclusiveTimeValue);
					}
					InOutTooltip.AddNameValueTextLine(TEXTVIEW("Inclusive Time:"), InclusiveTimeValue);
				}

				if (TooltipEventDuration > 0.0 && !bIsInfinite)
				{
					const double ExclusiveTimePercent = TooltipEvent.GetExclusiveTime() / TooltipEventDuration;
					FNumberFormattingOptions FormattingOptions;
					FormattingOptions.MaximumFractionalDigits = 2;
					const FString ExclStr = FString::Printf(TEXT("%s (%s)"), *FormatTimeAuto(TooltipEvent.GetExclusiveTime(), 2), *FText::AsPercent(ExclusiveTimePercent, &FormattingOptions).ToString());
					InOutTooltip.AddNameValueTextLine(TEXTVIEW("Exclusive Time:"), ExclStr);
				}

				InOutTooltip.AddNameValueTextLine(TEXTVIEW("Depth:"), FString::Printf(TEXT("%d"), TooltipEvent.GetDepth()));

				// Add detailed metadata.
				if (int32(TimerIndex) < 0) // has metadata? //-V547
				{
					AppendMetadataToTooltip(InOutTooltip, MetadataSpec, Metadata);
				}

				if (Session.IsValid())
				{
					PostInitTooltip(InOutTooltip, TooltipEvent, *Session.Get(), TimerName);
				}

				// Append user-annotation info if the hovered event has a matching event-anchored annotation.
				if (FUserAnnotationsTimingViewExtender* AnnotationExt =
						FTimingProfilerManager::Get()->GetUserAnnotationsExtender())
				{
					const FString AnnotationTimerName = (Timer && Timer->Name) ? FString(Timer->Name) : FString();
					AnnotationExt->AppendEventAnnotationsToTooltip(InOutTooltip, GetName(), AnnotationTimerName,
						TooltipEvent.GetStartTime(), TooltipEvent.GetEndTime());
				}
			}
		); // ReadTimers
	}
	else
	{
		for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
		{
			Track->InitTooltip(InOutTooltip, InTooltipEvent);
		}
	}

	InOutTooltip.UpdateLayout();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingTrackImpl::GetParentAndRoot(const FThreadTrackEvent& TimingEvent, TSharedPtr<FThreadTrackEvent>& OutParentTimingEvent, TSharedPtr<FThreadTrackEvent>& OutRootTimingEvent) const
{
	if (TimingEvent.GetDepth() > 0)
	{
		ReadTimeline(
			[this, &TimingEvent, &OutParentTimingEvent, &OutRootTimingEvent]
			(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
			{
				const double Time = (TimingEvent.GetStartTime() + TimingEvent.GetEndTime()) / 2;
				TimelineEventInfo EventInfo;
				if (Timeline.GetEventInfo(Time, 0, TimingEvent.GetDepth() - 1, EventInfo))
				{
					OutParentTimingEvent = CreateEvent(EventInfo, TimingEvent.GetTrack(), TimingEvent.GetDepth() - 1);
				}
				if (Timeline.GetEventInfo(Time, 0, 0, EventInfo))
				{
					OutRootTimingEvent = CreateEvent(EventInfo, TimingEvent.GetTrack(), 0);
				}
			}
		); // ReadTimeline
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FThreadTimingTrackImpl::GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const
{
	TSharedPtr<FThreadTrackEvent> TimingEvent;

	const FTimingViewLayout& Layout = Viewport.GetLayout();

	float TopLaneY = GetPosY() + Layout.TimelineDY;
	float TrackLanesHeight = GetHeight();

	for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
	{
		float HeaderDY = InPosY - Track->GetPosY();
		float TrackHeightWithPadding = Track->GetHeight() + Layout.ChildTimelineDY;
		if (HeaderDY >= 0.0f && HeaderDY < TrackHeightWithPadding)
		{
			return Track->GetEvent(InPosX, InPosY, Viewport);
		}

		TopLaneY += TrackHeightWithPadding;
		TrackLanesHeight -= TrackHeightWithPadding;
	}

	const float DY = InPosY - TopLaneY;

	// If mouse is not above first sub-track or below last sub-track...
	if (DY >= 0 && DY < TrackLanesHeight)
	{
		const int32 Depth = static_cast<int32>(DY / (Layout.EventH + Layout.EventDY));

		const double SecondsPerPixel = 1.0 / Viewport.GetScaleX();

		const double EventTime = Viewport.SlateUnitsToTime(InPosX);

		double SessionDuration = std::numeric_limits<double>::lowest();
		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			SessionDuration = Session->GetDurationSeconds();
		}

		if (EventTime <= SessionDuration)
		{
			ReadTimeline(
				[this, &EventTime, &Depth, &TimingEvent, &SecondsPerPixel]
				(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
				{
					TimelineEventInfo EventInfo;
					if (Timeline.GetEventInfo(EventTime, 2 * SecondsPerPixel, Depth, EventInfo))
					{
						TimingEvent = CreateEvent(EventInfo, SharedThis(this), Depth);
					}
				}
			); // ReadTimeline
		}
	}

	return TimingEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FThreadTimingTrackImpl::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<FThreadTrackEvent> FoundEvent;

	FindTimingProfilerEvent(InSearchParameters,
		[this, &FoundEvent]
		(double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const TraceServices::FTimingProfilerEvent& InFoundEvent)
		{
			FoundEvent = MakeShared<FThreadTrackEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth);
			FoundEvent->SetTimerIndex(InFoundEvent.TimerIndex);

			uint32 TimerId = 0;
			bool bIsValidTimerId = TimerIndexToTimerId(InFoundEvent.TimerIndex, TimerId);
			if (bIsValidTimerId)
			{
				FoundEvent->SetTimerId(TimerId);
			}
		});

	return FoundEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingTrackImpl::UpdateEventStats(ITimingEvent& InOutEvent) const
{
	if (InOutEvent.CheckTrack(this) && InOutEvent.Is<FThreadTrackEvent>())
	{
		FThreadTrackEvent& TrackEvent = InOutEvent.As<FThreadTrackEvent>();
		if (!TrackEvent.IsExclusiveTimeComputed())
		{
			ReadTimeline(
				[&TrackEvent]
				(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
				{
					TimelineEventInfo EventInfo;
					bool bIsFound = Timeline.GetEventInfo(TrackEvent.GetStartTime(), 0.0, TrackEvent.GetDepth(), EventInfo);
					if (bIsFound)
					{
						TrackEvent.SetExclusiveTime(EventInfo.ExclTime);
						TrackEvent.SetIsExclusiveTimeComputed(true);
					}
				}
			); // ReadTimeline
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingTrackImpl::OnEventSelected(const ITimingEvent& InSelectedEvent) const
{
	if (InSelectedEvent.CheckTrack(this) && InSelectedEvent.Is<FThreadTrackEvent>())
	{
		const FThreadTrackEvent& TrackEvent = InSelectedEvent.As<FThreadTrackEvent>();

		// Select the timer node corresponding to timing event type of selected timing event.
		FTimingProfilerManager::Get()->SetSelectedTimer(TrackEvent.GetTimerId());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingTrackImpl::OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const
{
	if (InSelectedEvent.CheckTrack(this) && InSelectedEvent.Is<FThreadTrackEvent>())
	{
		const FThreadTrackEvent& TrackEvent = InSelectedEvent.As<FThreadTrackEvent>();

		ReadTimers(
			[this, &TrackEvent]
			(const TraceServices::ITimingProfilerProvider& TimingProfilerProvider, const TraceServices::ITimingProfilerTimerReader& TimerReader)
			{
				const TraceServices::FTimingProfilerTimer* TimerPtr = TimerReader.GetTimer(TrackEvent.GetTimerIndex());

				if (TimerPtr)
				{
					FString EventName(TimerPtr->Name);

					AppendInfoToTimingEventDisplayName(EventName, TrackEvent.GetStartTime(), TrackEvent.GetEndTime(), TrackEvent.GetTimerIndex(), TimerPtr, TimingProfilerProvider, TimerReader);

					// Copy name of selected timing event to clipboard.
					FPlatformApplicationMisc::ClipboardCopy(*EventName);
				}
			}
		); // ReadTimers
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingTrackImpl::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	if (GetGroupName() != nullptr)
	{
		MenuBuilder.BeginSection("CpuThread", LOCTEXT("ContextMenu_Section_CpuThread", "CPU Thread"));
		{
			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("CpuThreadGroupFmt", "Group: {0}"), FText::FromString(GetGroupName())),
				FText(),
				FSlateIcon(),
				FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([]() { return false; })),
				NAME_None,
				EUserInterfaceActionType::Button
			);

			const FString ThreadIdStr = FString::Printf(TEXT("%s%u (0x%X)"), ThreadId & 0x70000000 ? TEXT("*") : TEXT(""), ThreadId & ~0x70000000, ThreadId);
			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("CpuThreadIdFmt", "Thread Id: {0}"), FText::FromString(ThreadIdStr)),
				FText(),
				FSlateIcon(),
				FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([]() { return false; })),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		MenuBuilder.EndSection();
	}

	for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
	{
		Track->BuildContextMenu(MenuBuilder);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingTrackImpl::FindTimingProfilerEvent(
	const FThreadTrackEvent& InTimingEvent,
	TFunctionRef<void(double, double, uint32, const TraceServices::FTimingProfilerEvent&)> InFoundPredicate) const
{
	auto MatchEvent =
		[&InTimingEvent]
		(double InStartTime, double InEndTime, uint32 InDepth)
		{
			return InDepth == InTimingEvent.GetDepth()
				&& InStartTime == InTimingEvent.GetStartTime()
				&& InEndTime == InTimingEvent.GetEndTime();
		};

	const double Time = (InTimingEvent.GetStartTime() + InTimingEvent.GetEndTime()) / 2;
	FTimingEventSearchParameters SearchParameters(Time, Time, ETimingEventSearchFlags::StopAtFirstMatch, MatchEvent);
	SearchParameters.SearchHandle = &InTimingEvent.GetSearchHandle();
	return FindTimingProfilerEvent(SearchParameters, InFoundPredicate);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingTrackImpl::FindTimingProfilerEvent(
	const FTimingEventSearchParameters& InParameters,
	TFunctionRef<void(double, double, uint32, const TraceServices::FTimingProfilerEvent&)> InFoundPredicate) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return false;
	}

	FThreadTimingTrackFilterContext FilterContext(*this);

	return TTimingEventSearch<TraceServices::FTimingProfilerEvent>::Search(

		InParameters,

		// Search Predicate
		[this, &FilterContext]
		(TTimingEventSearch<TraceServices::FTimingProfilerEvent>::FContext& InContext)
		{
			ReadTimers(
				[this, &InContext, &FilterContext]
				(const TraceServices::ITimingProfilerProvider& TimingProfilerProvider, const TraceServices::ITimingProfilerTimerReader& TimerReader)
				{
					FilterContext.SetTimerReader(TimerReader);

					ReadTimeline(
						[&InContext]
						(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
						{
							auto Callback =
								[&InContext]
								(double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event)
								{
									InContext.Check(EventStartTime, EventEndTime, EventDepth, Event);
									return InContext.ShouldContinueSearching() ?
										TraceServices::EEventEnumerate::Continue :
										TraceServices::EEventEnumerate::Stop;
								};

							if (InContext.GetParameters().SearchDirection == FTimingEventSearchParameters::ESearchDirection::Forward)
							{
								Timeline.EnumerateEvents(InContext.GetParameters().StartTime, InContext.GetParameters().EndTime, Callback);
							}
							else
							{
								Timeline.EnumerateEventsBackwards(InContext.GetParameters().EndTime, InContext.GetParameters().StartTime, Callback);
							}
						}
					); // ReadTimeline
				}
			); // ReadTimers
		},

		// Filter Predicate
		[&FilterContext, &InParameters]
		(double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event)
		{
			if (!InParameters.FilterExecutor.IsValid())
			{
				return true;
			}

			FilterContext.SetEvent(EventStartTime, EventEndTime, EventDepth, Event);
			return InParameters.FilterExecutor->ApplyFilters(FilterContext);
		},

		// Found Predicate
		InFoundPredicate,

		// Payload Matched Predicate
		TTimingEventSearch<TraceServices::FTimingProfilerEvent>::NoMatch,

		// Cache
		&SearchCache

	); // Search
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FThreadTrackEvent> FThreadTimingTrackImpl::CreateEvent(const TimelineEventInfo& InEventInfo, const TSharedRef<const FBaseTimingTrack> InTrack, int32 InDepth) const
{
	TSharedRef<FThreadTrackEvent> ThreadTrackEvent = MakeShared<FThreadTrackEvent>(InTrack, InEventInfo.StartTime, InEventInfo.EndTime, InDepth);
	FThreadTrackEvent& Event = *ThreadTrackEvent;

	Event.SetExclusiveTime(InEventInfo.ExclTime);
	Event.SetIsExclusiveTimeComputed(true);
	Event.SetTimerIndex(InEventInfo.Event.TimerIndex);

	uint32 TimerId = 0;
	bool bIsValidTimerId = TimerIndexToTimerId(InEventInfo.Event.TimerIndex, TimerId);
	if (bIsValidTimerId)
	{
		Event.SetTimerId(TimerId);
	}

	return ThreadTrackEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingTrackImpl::TimerIndexToTimerId(uint32 InTimerIndex, uint32& OutTimerId) const
{
	bool bIsValidTimerId = false;

	ReadTimers(
		[&bIsValidTimerId, InTimerIndex, &OutTimerId]
		(const TraceServices::ITimingProfilerProvider& TimingProfilerProvider, const TraceServices::ITimingProfilerTimerReader& TimerReader)
		{
			const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(InTimerIndex);
			if (Timer)
			{
				OutTimerId = Timer->Id;
				bIsValidTimerId = true;
			}
		}
	); // ReadTimers

	return bIsValidTimerId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FThreadTimingTrackImpl::GetDepthAt(double Time) const
{
	int32 Depth = 0;

	ReadTimeline(
		[Time, &Depth]
		(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
		{
			Depth = Timeline.GetDepthAt(Time);
		}
	); // ReadTimeline

	return Depth;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingTrackImpl::HasCustomFilter() const
{
	return FilterConfigurator.IsValid() && !FilterConfigurator->IsEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingTrackImpl::SetFilterConfigurator(TSharedPtr<FFilterConfigurator> InFilterConfigurator)
{
	if (FilterConfigurator != InFilterConfigurator)
	{
		FilterConfigurator = InFilterConfigurator;
		SetDirtyFlag();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<const ITimingEvent> FThreadTimingTrackImpl::FindMaxEventInstance(uint32 InTimerId, double InStartTime, double InEndTime, double InMaxValidTime) const
{
	TSharedPtr<FThreadTrackEvent> TimingEvent;

	ReadTimers(
		[this, InTimerId, InStartTime, InEndTime, InMaxValidTime, &TimingEvent]
		(const TraceServices::ITimingProfilerProvider& TimingProfilerProvider, const TraceServices::ITimingProfilerTimerReader& TimerReader)
		{
			ReadTimeline(
				[this, InTimerId, InStartTime, InEndTime, InMaxValidTime, &TimingEvent, &TimerReader]
				(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
				{
					struct FCandidateEvent
					{
						double StartTime = 0.0f;
						double EndTime = -std::numeric_limits<double>::infinity();
						uint32 Depth = ~0u;
						uint32 TimerIndex = 0;
						double GetDuration() const { return EndTime - StartTime; }
						bool IsValid() const { return Depth != ~0u; }
					};
					TArray<FCandidateEvent> CandidateEvents;

					TraceServices::ITimeline<TraceServices::FTimingProfilerEvent>::EnumerateAsyncParams Params;
					Params.IntervalStart = InStartTime;
					Params.IntervalEnd = InEndTime;
					Params.Resolution = 0.0;
					Params.SetupCallback =
						[&CandidateEvents](uint32 NumTasks)
						{
							CandidateEvents.AddDefaulted(NumTasks);
						};
					Params.EventRangeCallback =
						[InTimerId, &TimerReader, &CandidateEvents, InMaxValidTime]
						(double StartTime, double EndTime, uint32 Depth, const TraceServices::FTimingProfilerEvent& Event, uint32 TaskIndex)
						{
							const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(Event.TimerIndex);
							if (ensure(Timer != nullptr) && Timer->Id == InTimerId)
							{
								if (EndTime == std::numeric_limits<double>::infinity())
								{
									EndTime = InMaxValidTime;
								}
								const double EventDuration = EndTime - StartTime;
								if (EventDuration > CandidateEvents[TaskIndex].GetDuration())
								{
									CandidateEvents[TaskIndex].StartTime = StartTime;
									CandidateEvents[TaskIndex].EndTime = EndTime;
									CandidateEvents[TaskIndex].Depth = Depth;
									CandidateEvents[TaskIndex].TimerIndex = Event.TimerIndex;
								}
							}
							return TraceServices::EEventEnumerate::Continue;
						};

					// Note: Enumerating events for filtering should not use downsampling.
					Timeline.EnumerateEventsDownSampledAsync(Params);

					FCandidateEvent BestMatch;
					for (const FCandidateEvent& Event : CandidateEvents)
					{
						if (Event.GetDuration() > BestMatch.GetDuration())
						{
							BestMatch = Event;
						}
					}
					if (BestMatch.IsValid())
					{
						TimingEvent = MakeShared<FThreadTrackEvent>(SharedThis(this), BestMatch.StartTime, BestMatch.EndTime, BestMatch.Depth);
						TimingEvent->SetTimerId(InTimerId);
						TimingEvent->SetTimerIndex(BestMatch.TimerIndex);
					}
				}
			); // ReadTimeline
		}
	); // ReadTimers

	return TimingEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<const ITimingEvent> FThreadTimingTrackImpl::FindMinEventInstance(uint32 InTimerId, double InStartTime, double InEndTime, double InMaxValidTime) const
{
	TSharedPtr<FThreadTrackEvent> TimingEvent;

	ReadTimers(
		[this, InTimerId, InStartTime, InEndTime, InMaxValidTime, &TimingEvent]
		(const TraceServices::ITimingProfilerProvider& TimingProfilerProvider, const TraceServices::ITimingProfilerTimerReader& TimerReader)
		{
			ReadTimeline(
				[this, InTimerId, InStartTime, InEndTime, InMaxValidTime, &TimingEvent, &TimerReader]
				(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
				{
					struct FCandidateEvent
					{
						double StartTime = 0.0;
						double EndTime = std::numeric_limits<double>::infinity();
						uint32 Depth = ~0u;
						uint32 TimerIndex = 0;
						double GetDuration() const { return EndTime - StartTime; }
						bool IsValid() const { return Depth != ~0u; }
					};
					TArray<FCandidateEvent> CandidateEvents;

					TraceServices::ITimeline<TraceServices::FTimingProfilerEvent>::EnumerateAsyncParams Params;
					Params.IntervalStart = InStartTime;
					Params.IntervalEnd = InEndTime;
					Params.Resolution = 0.0;
					Params.SetupCallback =
						[&CandidateEvents](uint32 NumTasks)
						{
							CandidateEvents.AddDefaulted(NumTasks);
						};
					Params.EventRangeCallback =
						[InTimerId, &TimerReader, &CandidateEvents, InMaxValidTime]
						(double StartTime, double EndTime, uint32 Depth, const TraceServices::FTimingProfilerEvent& Event, uint32 TaskIndex)
						{
							const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(Event.TimerIndex);
							if (ensure(Timer != nullptr) && Timer->Id == InTimerId)
							{
								if (EndTime == std::numeric_limits<double>::infinity())
								{
									EndTime = InMaxValidTime;
								}
								const double EventDuration = EndTime - StartTime;
								if (EventDuration < CandidateEvents[TaskIndex].GetDuration())
								{
									CandidateEvents[TaskIndex].StartTime = StartTime;
									CandidateEvents[TaskIndex].EndTime = EndTime;
									CandidateEvents[TaskIndex].Depth = Depth;
									CandidateEvents[TaskIndex].TimerIndex = Event.TimerIndex;
								}
							}
							return TraceServices::EEventEnumerate::Continue;
						};

					// Note: Enumerating events for filtering should not use downsampling.
					Timeline.EnumerateEventsDownSampledAsync(Params);

					FCandidateEvent BestMatch;
					for (const FCandidateEvent& Event : CandidateEvents)
					{
						if (Event.GetDuration() < BestMatch.GetDuration())
						{
							BestMatch = Event;
						}
					}
					if (BestMatch.IsValid())
					{
						TimingEvent = MakeShared<FThreadTrackEvent>(SharedThis(this), BestMatch.StartTime, BestMatch.EndTime, BestMatch.Depth);
						TimingEvent->SetTimerId(InTimerId);
						TimingEvent->SetTimerIndex(BestMatch.TimerIndex);
					}
				}
			); // ReadTimeline
		}
	); // ReadTimers

	return TimingEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingTrackImpl::ReadTimers(TFunctionRef<void(const TraceServices::ITimingProfilerProvider&, const TraceServices::ITimingProfilerTimerReader&)> Callback) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();
			Callback(*TimingProfilerProvider, TimerReader);
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingTrackImpl::ReadTimeline(TFunctionRef<void(const TraceServices::ITimingProfilerProvider::Timeline&)> Callback) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			TimingProfilerProvider->ReadTimeline(GetTimelineIndex(),
				[&Callback]
				(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
				{
					Callback(Timeline);
				});
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
