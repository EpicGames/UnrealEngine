// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceServices
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsights
#include "Insights/TimingProfiler/Tracks/ThreadTimingTrack.h"
#include "Insights/ViewModels/TimingEventSearch.h" // for TTimingEventSearchCache
#include "Insights/ViewModels/TimingEventsTrack.h"

class FThreadTrackEvent;
class FTimingEventSearchParameters;

namespace UE::Insights { class FFilterConfigurator; }

namespace UE::Insights::TimingProfiler
{

class FThreadTimingSharedState;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FThreadTimingTrackImpl : public FThreadTimingTrack
{
	INSIGHTS_DECLARE_RTTI(FThreadTimingTrackImpl, FThreadTimingTrack)

public:
	typedef typename TraceServices::ITimeline<TraceServices::FTimingProfilerEvent>::FTimelineEventInfo TimelineEventInfo;

	explicit FThreadTimingTrackImpl(FThreadTimingSharedState& InSharedState, const FString& InName, const TCHAR* InGroupName, uint32 InTimelineIndex, uint32 InThreadId)
		: FThreadTimingTrack(InName)
		, SharedState(InSharedState)
		, GroupName(InGroupName)
		, TimelineIndex(InTimelineIndex)
		, ThreadId(InThreadId)
	{
	}

	virtual ~FThreadTimingTrackImpl() override = default;

	const TCHAR* GetGroupName() const { return GroupName; };

	uint32 GetTimelineIndex() const { return TimelineIndex; }

	//////////////////////////////////////////////////
	// FThreadTimingTrack

	virtual uint32 GetThreadId() const override { return ThreadId; }

	virtual int32 GetDepthAt(double Time) const override;

	//////////////////////////////////////////////////
	// FBaseTimingTrack

	virtual void PostDraw(const ITimingTrackDrawContext& Context) const override;

	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;

	virtual const TSharedPtr<const ITimingEvent> GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const override;
	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;

	virtual void UpdateEventStats(ITimingEvent& InOutEvent) const override;
	virtual void OnEventSelected(const ITimingEvent& InSelectedEvent) const override;
	virtual void OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;

	virtual void SetFilterConfigurator(TSharedPtr<FFilterConfigurator> InFilterConfigurator) override;

	//////////////////////////////////////////////////
	// FTimingEventsTrack

	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;

	//////////////////////////////////////////////////

	TSharedPtr<const ITimingEvent> FindMaxEventInstance(uint32 TimerId, double StartTime, double EndTime, double MaxValidTime) const;
	TSharedPtr<const ITimingEvent> FindMinEventInstance(uint32 TimerId, double StartTime, double EndTime, double MaxValidTime) const;

protected:
	FThreadTimingSharedState& GetSharedState() const { return SharedState; }

	//////////////////////////////////////////////////
	// FTimingEventsTrack

	virtual bool HasCustomFilter() const override;

	//////////////////////////////////////////////////

	virtual void PostInitTooltip(FTooltipDrawState& InOutTooltip, const FThreadTrackEvent& TooltipEvent, const TraceServices::IAnalysisSession& Session, const TCHAR* TimerName) const {}

	virtual bool ReadTimers(TFunctionRef<void(const TraceServices::ITimingProfilerProvider&, const TraceServices::ITimingProfilerTimerReader&)> Callback) const;
	virtual bool ReadTimeline(TFunctionRef<void(const TraceServices::ITimingProfilerProvider::Timeline&)> Callback) const;

private:
	bool FindTimingProfilerEvent(const FThreadTrackEvent& InTimingEvent, TFunctionRef<void(double, double, uint32, const TraceServices::FTimingProfilerEvent&)> InFoundPredicate) const;
	bool FindTimingProfilerEvent(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const TraceServices::FTimingProfilerEvent&)> InFoundPredicate) const;

	void GetParentAndRoot(const FThreadTrackEvent& TimingEvent,
						  TSharedPtr<FThreadTrackEvent>& OutParentTimingEvent,
						  TSharedPtr<FThreadTrackEvent>& OutRootTimingEvent) const;

	TSharedRef<FThreadTrackEvent> CreateEvent(const TimelineEventInfo& InEventInfo, const TSharedRef<const FBaseTimingTrack> InTrack, int32 InDepth) const;
	bool TimerIndexToTimerId(uint32 InTimerIndex, uint32& OutTimerId) const;

private:
	FThreadTimingSharedState& SharedState;

	TSharedPtr<FFilterConfigurator> FilterConfigurator;

	const TCHAR* GroupName = nullptr;
	uint32 TimelineIndex = 0;
	uint32 ThreadId = 0;

	// Search cache
	mutable TTimingEventSearchCache<TraceServices::FTimingProfilerEvent> SearchCache;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
