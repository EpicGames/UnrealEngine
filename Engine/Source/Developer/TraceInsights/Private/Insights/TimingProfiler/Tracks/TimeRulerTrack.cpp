// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimeRulerTrack.h"

#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Rendering/DrawElements.h"
#include "Styling/AppStyle.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/ViewModels/TimeMarker.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/ViewModels/DrawHelpers.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/Widgets/SLogView.h"

#include <limits>

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::FTimeRulerTrack"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FTimeRulerTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimeRulerTrack::FTimeRulerTrack()
	: FBaseTimingTrack(TEXT("Time Ruler"))
	, WhiteBrush(FAppStyle::Get().GetBrush("WhiteBrush"))
	, Font(FAppStyle::Get().GetFontStyle("SmallFont"))
	, CrtMousePosTime(0.0)
	, CrtMousePosTextWidth(0.0f)
{
	SetValidLocations(ETimingTrackLocation::TopDocked);
	SetOrder(FTimingTrackOrder::TimeRuler);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimeRulerTrack::~FTimeRulerTrack()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeRulerTrack::Reset()
{
	FBaseTimingTrack::Reset();

	bIsSelecting = false;
	SelectionStartTime = 0.0;
	SelectionEndTime = 0.0;

	TimeMarkers.Reset();
	bIsScrubbing = false;

	constexpr float TimeRulerHeight = 24.0f;
	SetHeight(TimeRulerHeight);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeRulerTrack::SetSelection(const bool bInIsSelecting, const double InSelectionStartTime, const double InSelectionEndTime)
{
	bIsSelecting = bInIsSelecting;
	SelectionStartTime = InSelectionStartTime;
	SelectionEndTime = InSelectionEndTime;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeRulerTrack::AddTimeMarker(TSharedRef<FTimeMarker> InTimeMarker)
{
	TimeMarkers.Add(InTimeMarker);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeRulerTrack::RemoveTimeMarker(TSharedRef<FTimeMarker> InTimeMarker)
{
	TimeMarkers.Remove(InTimeMarker);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeRulerTrack::RemoveAllTimeMarkers()
{
	TimeMarkers.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTimeMarker> FTimeRulerTrack::GetTimeMarkerByName(const FString& InTimeMarkerName)
{
	for (TSharedRef<FTimeMarker>& TimeMarker : TimeMarkers)
	{
		if (TimeMarker->GetName().Equals(InTimeMarkerName))
		{
			return TimeMarker;
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTimeMarker> FTimeRulerTrack::GetTimeMarkerAtPos(const FVector2D& InPosition, const FTimingTrackViewport& InViewport)
{
	TSharedPtr<FTimeMarker> ClosestTimeMarker = nullptr;

	constexpr float TimeMarkerBoxHeight = 12.0f;
	const float InPositionY = static_cast<float>(InPosition.Y);
	if (InPositionY >= GetPosY() && InPositionY < GetPosY() + TimeMarkerBoxHeight)
	{
		const float InPositionX = static_cast<float>(InPosition.X);
		float MinDX = 42.0f;
		for (TSharedRef<FTimeMarker>& TimeMarker : TimeMarkers)
		{
			if (TimeMarker->IsVisible())
			{
				const float MarkerX = InViewport.TimeToSlateUnitsRounded(TimeMarker->GetTime());
				const float DX = FMath::Abs(InPositionX - MarkerX);
				if (DX <= MinDX)
				{
					MinDX = DX;
					ClosestTimeMarker = TimeMarker;
				}
			}
		}
	}

	return ClosestTimeMarker;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeRulerTrack::StartScrubbing(TSharedRef<FTimeMarker> InTimeMarker)
{
	// Move the scrubbing time marker at the end of sorting list to be draw on top of other markers.
	TimeMarkers.Remove(InTimeMarker);
	TimeMarkers.Add(InTimeMarker);

	ScrubbingTimeMarker = InTimeMarker;
	ScrubbingTimeMarker->StartDragging();

	bIsScrubbing = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeRulerTrack::StopScrubbing()
{
	if (bIsScrubbing)
	{
		ScrubbingTimeMarker->StopDragging();
		bIsScrubbing = false;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeRulerTrack::PostUpdate(const ITimingTrackUpdateContext& Context)
{
	//constexpr float HeaderWidth = 100.0f;
	//constexpr float HeaderHeight = 14.0f;

	const float MouseY = static_cast<float>(Context.GetMousePosition().Y);
	if (MouseY >= GetPosY() && MouseY < GetPosY() + GetHeight())
	{
		SetHoveredState(true);
		//const float MouseX = static_cast<float>(Context.GetMousePosition().X);
		//SetHeaderHoveredState(MouseX < HeaderWidth && MouseY < GetPosY() + HeaderHeight);
	}
	else
	{
		SetHoveredState(false);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeRulerTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	FDrawContext& DrawContext = Context.GetDrawContext();
	const FTimingTrackViewport& Viewport = Context.GetViewport();

	const float MinorTickMark = 5.0f;
	const float MajorTickMark = 20 * MinorTickMark;

	const float MinorTickMarkHeight = 5.0f;
	const float MajorTickMarkHeight = 11.0f;

	const float TextY = GetPosY() + MajorTickMarkHeight;

	double MinorTickMarkTime = Viewport.GetDurationForViewportDX(MinorTickMark);
	double MajorTickMarkTime = Viewport.GetDurationForViewportDX(MajorTickMark);

	double VX = Viewport.GetStartTime() * Viewport.GetScaleX();
	double MinorN = FMath::FloorToDouble(VX / static_cast<double>(MinorTickMark));
	double MajorN = FMath::FloorToDouble(VX / static_cast<double>(MajorTickMark));
	float MinorOX = static_cast<float>(FMath::RoundToDouble(MinorN * static_cast<double>(MinorTickMark) - VX));
	float MajorOX = static_cast<float>(FMath::RoundToDouble(MajorN * static_cast<double>(MajorTickMark) - VX));

	// Draw the time ruler's background.
	FDrawHelpers::DrawBackground(DrawContext, WhiteBrush, Viewport, GetPosY(), GetHeight());

	// Draw the minor tick marks.
	for (float X = MinorOX; X < Viewport.GetWidth(); X += MinorTickMark)
	{
		const bool bIsTenth = ((int32)(((X - MajorOX) / MinorTickMark) + 0.4f) % 2 == 0);
		const float MinorTickH = bIsTenth ? MinorTickMarkHeight : MinorTickMarkHeight - 1.0f;
		DrawContext.DrawBox(X, GetPosY(), 1.0f, MinorTickH, WhiteBrush,
			bIsTenth ? FLinearColor(0.3f, 0.3f, 0.3f, 1.0f) : FLinearColor(0.25f, 0.25f, 0.25f, 1.0f));
	}
	// Draw the major tick marks.
	for (float X = MajorOX; X < Viewport.GetWidth(); X += MajorTickMark)
	{
		DrawContext.DrawBox(X, GetPosY(), 1.0f, MajorTickMarkHeight, WhiteBrush, FLinearColor(0.4f, 0.4f, 0.4f, 1.0f));
	}
	DrawContext.LayerId++;

	const double DT = static_cast<double>(MajorTickMark) / Viewport.GetScaleX();
	const double Precision = FMath::Max(DT / 10.0, FTimeValue::Nanosecond);

	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const float FontScale = DrawContext.Geometry.Scale;

	// Draw the time at major tick marks.
	for (float X = MajorOX; X < Viewport.GetWidth() + MajorTickMark; X += MajorTickMark)
	{
		const double T = Viewport.SlateUnitsToTime(X);
		FString Text = FormatTime(T, Precision);
		const float TextWidth = static_cast<float>(FontMeasureService->Measure(Text, Font, FontScale).X / FontScale);
		DrawContext.DrawText(X - TextWidth / 2, TextY, Text, Font,
			(T < Viewport.GetMinValidTime() || T >= Viewport.GetMaxValidTime()) ? FLinearColor(0.7f, 0.5f, 0.5f, 1.0f) : FLinearColor(0.8f, 0.8f, 0.8f, 1.0f));
	}
	DrawContext.LayerId++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeRulerTrack::PostDraw(const ITimingTrackDrawContext& Context) const
{
	FDrawContext& DrawContext = Context.GetDrawContext();
	const FTimingTrackViewport& Viewport = Context.GetViewport();
	const FVector2D& MousePosition = Context.GetMousePosition();

	const bool bShowMousePos = !MousePosition.IsZero() && !bIsScrubbing;
	const bool bIsMouseOver = bShowMousePos && MousePosition.Y >= GetPosY() && MousePosition.Y < GetPosY() + GetHeight();

	if (bShowMousePos)
	{
		const FLinearColor MousePosLineColor(0.9f, 0.9f, 0.9f, 0.1f);
		const FLinearColor MousePosTextBackgroundColor(0.9f, 0.9f, 0.9f, 1.0f);
		FLinearColor MousePosTextForegroundColor(0.1f, 0.1f, 0.1f, 1.0f);

		// Time at current mouse position.
		FString MousePosText;
		const double MousePosTime = Viewport.SlateUnitsToTime(static_cast<float>(MousePosition.X));
		CrtMousePosTime = MousePosTime;

		const double DT = 100.0 / Viewport.GetScaleX();
		const double MousePosPrecision = FMath::Max(DT / 100.0, FTimeValue::Nanosecond);
		if (bIsMouseOver)
		{
			// If mouse is hovering the time ruler, format time with a better precision (split seconds in ms, us, ns and ps).
			MousePosText = FormatTimeSplit(MousePosTime, MousePosPrecision);
		}
		else
		{
			// Format current time with one more digit than the time at major tick marks.
			MousePosText = FormatTime(MousePosTime, MousePosPrecision);
		}

		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const float FontScale = DrawContext.Geometry.Scale;

		const float MousePosTextWidth = FMath::RoundToFloat(static_cast<float>(FontMeasureService->Measure(MousePosText, Font, FontScale).X / FontScale));

		if (!FMath::IsNearlyEqual(CrtMousePosTextWidth, MousePosTextWidth))
		{
			// Animate the box's width (to avoid flickering).
			CrtMousePosTextWidth = CrtMousePosTextWidth * 0.6f + MousePosTextWidth * 0.4f;
		}

		const float TextY = GetPosY() + 11.0f;

		float X = static_cast<float>(MousePosition.X);
		float W = CrtMousePosTextWidth + 4.0f;
		if (bIsSelecting && SelectionStartTime < SelectionEndTime)
		{
			// While selecting, display the current time on either left or right side of the selected time range (i.e. to not overlap the selection arrows).
			float SelectionX1 = Viewport.TimeToSlateUnitsRounded(SelectionStartTime);
			float SelectionX2 = Viewport.TimeToSlateUnitsRounded(SelectionEndTime);
			if (FMath::Abs(X - SelectionX1) > FMath::Abs(SelectionX2 - X))
			{
				X = SelectionX2 + W / 2.0f;
			}
			else
			{
				X = SelectionX1 - W / 2.0f;
			}
			MousePosTextForegroundColor = FLinearColor(0.01f, 0.05f, 0.2f, 1.0f);
		}
		else
		{
			// Draw horizontal line at mouse position.
			//DrawContext.DrawBox(0.0f, static_cast<float>(MousePosition.Y), Viewport.Width, 1.0f, WhiteBrush, MousePosLineColor);

			// Draw vertical line at mouse position.
			DrawContext.DrawBox(static_cast<float>(MousePosition.X), Viewport.GetPosY(), 1.0f, Viewport.GetHeight(), WhiteBrush, MousePosLineColor);

			// Stroke the vertical line above current time box.
			DrawContext.DrawBox(static_cast<float>(MousePosition.X), GetPosY(), 1.0f, TextY - GetPosY(), WhiteBrush, MousePosTextBackgroundColor);
		}

		// Fill the current time box.
		DrawContext.DrawBox(X - W / 2.0f, TextY, W, 12.0f, WhiteBrush, MousePosTextBackgroundColor);
		DrawContext.LayerId++;

		// Draw current time text.
		DrawContext.DrawText(X - MousePosTextWidth / 2.0f, TextY, MousePosText, Font, MousePosTextForegroundColor);
		DrawContext.LayerId++;
	}

	// Draw the time markers.
	for (const TSharedRef<FTimeMarker>& TimeMarker : TimeMarkers)
	{
		DrawTimeMarker(Context, *TimeMarker);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeRulerTrack::DrawTimeMarker(const ITimingTrackDrawContext& Context, const FTimeMarker& TimeMarker) const
{
	if (!TimeMarker.IsVisible())
	{
		return;
	}

	const FTimingTrackViewport& Viewport = Context.GetViewport();

	const float TimeMarkerX = Viewport.TimeToSlateUnitsRounded(TimeMarker.GetTime());
	const float HalfWidth = TimeMarker.GetCrtTextWidth() / 2;

	if (TimeMarkerX < -HalfWidth || TimeMarkerX > Viewport.GetWidth() + HalfWidth)
	{
		return;
	}

	const float TimeMarkerY = GetPosY();
	constexpr float BoxHeight = 12.0f;

	FDrawContext& DrawContext = Context.GetDrawContext();

	// Draw the vertical line.
	DrawContext.DrawBox(TimeMarkerX, TimeMarkerY, 1.0f, Viewport.GetPosY() + Viewport.GetHeight() - TimeMarkerY, WhiteBrush, TimeMarker.GetColor());
	DrawContext.LayerId++;

	const FVector2D& MousePosition = Context.GetMousePosition();

	const bool bIsMouseOverTrack =  !MousePosition.IsZero() &&
									MousePosition.Y >= GetPosY() &&
									MousePosition.Y < GetPosY() + GetHeight();

	constexpr float FixedHalfWidth = 42.0f;
	const bool bIsMouseOverMarker = bIsMouseOverTrack &&
									MousePosition.Y >= GetPosY() + TimeMarkerY &&
									MousePosition.Y < GetPosY() + TimeMarkerY + BoxHeight &&
									MousePosition.X >= TimeMarkerX - FixedHalfWidth &&
									MousePosition.X < TimeMarkerX + FixedHalfWidth;

	// Decide what text to display.
	FString TimeMarkerText;
	if (bIsMouseOverTrack)
	{
		if (TimeMarker.GetName().Len() > 0)
		{
			TimeMarkerText = TimeMarker.GetName() + TEXT(": ");
		}

		// Format time value with one more digit than the time at major tick marks.
		const double DT = 100.0 / Viewport.GetScaleX();
		const double Precision = FMath::Max(DT / 100.0, FTimeValue::Nanosecond);

		if (bIsMouseOverMarker)
		{
			// If mouse is hovering the time marker, format time with a better precision (split seconds in ms, us, ns and ps).
			TimeMarkerText += FormatTimeSplit(TimeMarker.GetTime(), Precision);
		}
		else
		{
			TimeMarkerText += FormatTime(TimeMarker.GetTime(), Precision);
		}
	}
	else
	{
		if (TimeMarker.GetName().Len() > 0)
		{
			TimeMarkerText = TimeMarker.GetName();
		}
		else
		{
			TimeMarkerText = TEXT("T");
		}
	}

	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const float FontScale = DrawContext.Geometry.Scale;

	const float TextWidth = FMath::RoundToFloat(static_cast<float>(FontMeasureService->Measure(TimeMarkerText, Font, FontScale).X / FontScale));

	if (!FMath::IsNearlyEqual(TimeMarker.GetCrtTextWidth(), TextWidth))
	{
		// Animate the box's width (to avoid flickering).
		TimeMarker.SetCrtTextWidthAnimated(TextWidth);
	}

	const FLinearColor TextBackgroundColor(TimeMarker.GetColor().CopyWithNewOpacity(1.0f));
	const FLinearColor TextForegroundColor(0.07f, 0.07f, 0.07f, 1.0f);

	// Fill the time marker box.
	const float BoxWidth = TimeMarker.GetCrtTextWidth() + 4.0f;
	DrawContext.DrawBox(TimeMarkerX - BoxWidth / 2, TimeMarkerY, BoxWidth, BoxHeight, WhiteBrush, TextBackgroundColor);
	DrawContext.LayerId++;

	// Draw time marker text.
	DrawContext.DrawText(TimeMarkerX - TextWidth / 2, TimeMarkerY, TimeMarkerText, Font, TextForegroundColor);
	DrawContext.LayerId++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeRulerTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	TArray<TSharedRef<FTimeMarker>> VisibleTimeMarkers;
	for (TSharedRef<FTimeMarker>& TimeMarker : TimeMarkers)
	{
		if (TimeMarker->IsVisible())
		{
			VisibleTimeMarkers.Add(TimeMarker);
		}
	}

	FText CrtMousePosTimeText = FText::FromString(FormatTimeAuto(CrtMousePosTime, 2));

	if (VisibleTimeMarkers.Num() > 0)
	{
		// Sort TimeMarkers by name.
		VisibleTimeMarkers.Sort([](const TSharedRef<FTimeMarker>& A, const TSharedRef<FTimeMarker>& B) -> bool { return A->GetName().Compare(B->GetName()) <= 0; });

		MenuBuilder.BeginSection("TimeMarkers", LOCTEXT("ContextMenu_Section_TimeMarkers", "Time Markers"));

		for (TSharedRef<FTimeMarker>& TimeMarker : VisibleTimeMarkers)
		{
			FUIAction Action_MoveTimeMarker
			(
				FExecuteAction::CreateSP(this, &FTimeRulerTrack::ContextMenu_MoveTimeMarker_Execute, TimeMarker),
				FCanExecuteAction()
			);
			const FString& MarkerNameString = TimeMarker->GetName();
			const FText MarkerNameText = FText::FromString((MarkerNameString.Len() > 0) ? MarkerNameString : TEXT("T"));
			MenuBuilder.AddMenuEntry
			(
				FText::Format(LOCTEXT("ContextMenu_MoveTimeMarker_Fmt", "Move Time Marker '{0}' (\u2192 {1})"), MarkerNameText, CrtMousePosTimeText),
				FText::Format(LOCTEXT("ContextMenu_MoveTimeMarker_Desc_Fmt", "Move the time marker '{0}' at the current mouse position ({1})."), MarkerNameText, CrtMousePosTimeText),
				FSlateIcon(),
				Action_MoveTimeMarker,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}

		MenuBuilder.EndSection();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimeRulerTrack::ContextMenu_MoveTimeMarker_Execute(TSharedRef<FTimeMarker> InTimeMarker)
{
	InTimeMarker->SetTime(CrtMousePosTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
