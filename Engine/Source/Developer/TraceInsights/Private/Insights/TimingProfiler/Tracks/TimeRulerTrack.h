// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Fonts/SlateFontInfo.h"
#include "Templates/SharedPointer.h"

// TraceInsights
#include "Insights/ViewModels/BaseTimingTrack.h"
#include "Insights/ITimingViewSession.h"

class FMenuBuilder;
struct FSlateBrush;

class FTimingTrackViewport;

namespace UE::Insights::TimingProfiler
{

class FTimeMarker;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimeRulerTrack : public FBaseTimingTrack
{
	INSIGHTS_DECLARE_RTTI(FTimeRulerTrack, FBaseTimingTrack)

public:
	FTimeRulerTrack();
	virtual ~FTimeRulerTrack();

	virtual void Reset() override;

	void SetSelection(const bool bInIsSelecting, const double InSelectionStartTime, const double InSelectionEndTime);

	TArray<TSharedRef<FTimeMarker>>& GetTimeMarkers() { return TimeMarkers; }
	const TArray<TSharedRef<FTimeMarker>>& GetTimeMarkers() const { return TimeMarkers; }
	void AddTimeMarker(TSharedRef<FTimeMarker> InTimeMarker);
	void RemoveTimeMarker(TSharedRef<FTimeMarker> InTimeMarker);
	void RemoveAllTimeMarkers();

	TSharedPtr<FTimeMarker> GetTimeMarkerByName(const FString& InTimeMarkerName);
	TSharedPtr<FTimeMarker> GetTimeMarkerAtPos(const FVector2D& InPosition, const FTimingTrackViewport& InViewport);

	bool IsScrubbing() const { return bIsScrubbing; }
	TSharedRef<FTimeMarker> GetScrubbingTimeMarker() { return TimeMarkers.Last(); }
	void StartScrubbing(TSharedRef<FTimeMarker> InTimeMarker);
	void StopScrubbing();

	virtual void PostUpdate(const ITimingTrackUpdateContext& Context) override;
	void Draw(const ITimingTrackDrawContext& Context) const override;
	void PostDraw(const ITimingTrackDrawContext& Context) const override;

	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;

private:
	void DrawTimeMarker(const ITimingTrackDrawContext& Context, const FTimeMarker& TimeMarker) const;
	void ContextMenu_MoveTimeMarker_Execute(TSharedRef<FTimeMarker> InTimeMarker);

private:
	// Slate resources
	const FSlateBrush* WhiteBrush;
	const FSlateFontInfo Font;

	bool bIsSelecting;
	double SelectionStartTime;
	double SelectionEndTime;

	// The last time value at mouse position. Updated in PostDraw.
	mutable double CrtMousePosTime;

	// The smoothed width of "the text at mouse position" to avoid flickering. Updated in PostDraw.
	mutable float CrtMousePosTextWidth;

	/**
	 * The sorted list of all registered time markers. It defines the draw order of time markers.
	 * The time marker currently scrubbing will be moved at the end of the list in order to be displayed on top of other markers.
	 */
	TArray<TSharedRef<FTimeMarker>> TimeMarkers;

	TSharedPtr<FTimeMarker> ScrubbingTimeMarker;

	/** True if the user is currently dragging a time marker. */
	bool bIsScrubbing;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
