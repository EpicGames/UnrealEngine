// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserAnnotationsTimingTrack.h"

#include "Algo/Sort.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Styling/AppStyle.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "Insights/InsightsStyle.h"

// TraceInsights
#include "Insights/ITimingViewSession.h"
#include "Insights/TimingProfiler/Models/UserAnnotationStore.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingViewLayout.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Insights/Widgets/STimingView.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FUserAnnotationsTimingTrack)
INSIGHTS_IMPLEMENT_RTTI(FUserAnnotationTimingEvent)

namespace
{
	// Label geometry constants — shared by UpdateDrawState (for slot-width math) and Draw
	// (for rendering). Kept in one place so the two sides can't silently drift apart.
	constexpr float LabelIconSize = 10.0f;
	constexpr float LabelIconGap = 3.0f;
	constexpr float LabelPadH = 4.0f;
	constexpr float LabelPadV = 2.0f;
	constexpr float LabelFullBoxExtra = LabelIconSize + LabelIconGap + LabelPadH * 2.0f; // icon + gap + 2*pad
	constexpr float LabelIconOnlyBoxW = LabelIconSize + LabelPadH * 2.0f;                // icon + 2*pad

	/** Returns white on dark backgrounds, black on light. Keeps annotation text legible regardless of user-picked color. */
	FLinearColor GetContrastTextColor(const FLinearColor& BackgroundColor)
	{
		const bool bIsDarkColor = BackgroundColor.GetLuminance() < 0.4f;
		return bIsDarkColor ? FLinearColor::White : FLinearColor::Black;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FUserAnnotationsTimingTrack::FUserAnnotationsTimingTrack(TSharedPtr<FUserAnnotationStore> InAnnotationStore)
	: FBaseTimingTrack(TEXT("UserAnnotationsTrack"))
	, AnnotationStore(InAnnotationStore)
	, Header(*this)
	, WhiteBrush(FAppStyle::Get().GetBrush("WhiteBrush"))
	, Font(FAppStyle::Get().GetFontStyle("SmallFont"))
{
	SetName(TEXT("Annotations"));
	SetValidLocations(ETimingTrackLocation::TopDocked | ETimingTrackLocation::BottomDocked);
	SetOrder(FTimingTrackOrder::Markers + 1); // Just below system bookmarks

	// Track height is driven by content (auto-hidden when empty).
	Header.SetCanBeCollapsed(false);
	Header.SetIsCollapsed(false);

	SetHeight(20.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double FUserAnnotationsTimingTrack::Snap(double Time, double SnapTolerance) const
{
	if (!AnnotationStore.IsValid())
	{
		return Time;
	}

	double SnapTime = Time;
	double SnapDistance = SnapTolerance; // Only snap if closer than tolerance

	for (const FUserAnnotation& Annotation : AnnotationStore->GetAllAnnotations())
	{
		if (!Annotation.bVisible)
		{
			continue;
		}

		// Snap to annotation start time
		const double Distance = FMath::Abs(Annotation.Time - Time);
		if (Distance < SnapDistance)
		{
			SnapDistance = Distance;
			SnapTime = Annotation.Time;
		}

		// For range annotations, also snap to the end time
		if (Annotation.IsRange())
		{
			const double EndDistance = FMath::Abs(Annotation.EndTime - Time);
			if (EndDistance < SnapDistance)
			{
				SnapDistance = EndDistance;
				SnapTime = Annotation.EndTime;
			}
		}
	}

	return SnapTime;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingTrack::Reset()
{
	FBaseTimingTrack::Reset();
	Header.Reset();
	ResetCache();
	ChangeNumber = 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingTrack::PreUpdate(const ITimingTrackUpdateContext& Context)
{
	Header.UpdateSize();

	// Height collapses to 0 (hidden via Auto Hide Empty Tracks) when no annotation is in the visible view.
	const int32 TotalCount = AnnotationStore.IsValid() ? AnnotationStore->GetAllAnnotations().Num() : 0;
	const bool bHasAnnotations = (TotalCount > 0);

	int32 VisibleInView = 0;
	if (AnnotationStore.IsValid() && bHasAnnotations)
	{
		const FTimingTrackViewport& Viewport = Context.GetViewport();
		const double NewViewStart = Viewport.GetStartTime();
		const double NewViewEnd = Viewport.GetEndTime();
		const uint64 NewChangeNum = AnnotationStore->GetChangeNumber();
		// Reuse cached count when viewport bounds and store contents are unchanged.
		if (CachedVisibleInView >= 0
			&& CachedViewStartTime == NewViewStart
			&& CachedViewEndTime == NewViewEnd
			&& CachedVisibleChangeNumber == NewChangeNum)
		{
			VisibleInView = CachedVisibleInView;
		}
		else
		{
			AnnotationStore->EnumerateAnnotations(NewViewStart, NewViewEnd,
				[&VisibleInView](const FUserAnnotation& Annotation)
				{
					if (Annotation.bVisible)
					{
						++VisibleInView;
					}
				});
			CachedVisibleInView = VisibleInView;
			CachedViewStartTime = NewViewStart;
			CachedViewEndTime = NewViewEnd;
			CachedVisibleChangeNumber = NewChangeNum;
		}
	}

	float DesiredHeight = 0.0f;
	if (!bCanPersist)
	{
		DesiredHeight = 22.0f;
	}
	else if (VisibleInView > 0)
	{
		DesiredHeight = 22.0f;
	}
	const bool bExpanded = (DesiredHeight > 0.0f);

	// Snap to target once delta is sub-pixel so the lerp doesn't oscillate forever.
	const float CurrentHeight = GetHeight();
	if (FMath::Abs(CurrentHeight - DesiredHeight) < 0.5f)
	{
		if (CurrentHeight != DesiredHeight)
		{
			SetHeight(DesiredHeight);
		}
	}
	else if (CurrentHeight < DesiredHeight)
	{
		const float NewHeight = FMath::CeilToFloat(CurrentHeight * 0.9f + DesiredHeight * 0.1f);
		SetHeight(NewHeight);
	}
	else if (CurrentHeight > DesiredHeight)
	{
		const float NewHeight = FMath::FloorToFloat(CurrentHeight * 0.9f + DesiredHeight * 0.1f);
		SetHeight(NewHeight);
	}

	if (Header.IsCollapsed() == bExpanded)
	{
		Header.SetIsCollapsed(!bExpanded);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingTrack::Update(const ITimingTrackUpdateContext& Context)
{
	Header.Update(Context);

	const FTimingTrackViewport& Viewport = Context.GetViewport();

	const uint64 NewChangeNumber = AnnotationStore.IsValid() ? AnnotationStore->GetChangeNumber() : 0;

	if (IsDirty() || Viewport.IsHorizontalViewportDirty() || ChangeNumber != NewChangeNumber)
	{
		ClearDirtyFlag();
		ChangeNumber = NewChangeNumber;
		ResetCache();
		UpdateDrawState(Context);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingTrack::PostUpdate(const ITimingTrackUpdateContext& Context)
{
	Header.PostUpdate(Context);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingTrack::UpdateDrawState(const ITimingTrackUpdateContext& Context)
{
	if (!AnnotationStore.IsValid())
	{
		return;
	}

	const FTimingTrackViewport& Viewport = Context.GetViewport();
	const double ViewStartTime = Viewport.GetStartTime();
	const double ViewEndTime = Viewport.GetEndTime();
	const float ViewWidth = Viewport.GetWidth();

	const TSharedRef<FSlateFontMeasure> FontMeasureService =
		FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	// Sort annotations by time for consistent rendering
	TArray<const FUserAnnotation*> SortedAnnotations;
	AnnotationStore->EnumerateAnnotations(ViewStartTime, ViewEndTime,
		[&SortedAnnotations](const FUserAnnotation& Annotation)
		{
			SortedAnnotations.Add(&Annotation);
		});
	Algo::SortBy(SortedAnnotations, [](const FUserAnnotation* Annotation) { return Annotation->Time; });

	// Filter to visible only so the "next annotation X" lookahead below doesn't stop on
	// an invisible neighbor and underestimate the available label width.
	TArray<const FUserAnnotation*> VisibleAnnotations;
	VisibleAnnotations.Reserve(SortedAnnotations.Num());
	for (const FUserAnnotation* Annotation : SortedAnnotations)
	{
		if (Annotation->bVisible)
		{
			VisibleAnnotations.Add(Annotation);
		}
	}

	// Unified layout: every annotation gets a marker at its start X, and a label that
	// truncates (not culls) to the gap before the next annotation. Inspired by the
	// bookmarks/logs tracks; differs in that we fall back to a compact icon-only badge
	// below the text-fits threshold so the annotation is still locatable. Range
	// annotations additionally get a filled-rect or bracket overlay so narrow ranges
	// remain visible even when the label is too tight to show text.
	for (int32 i = 0; i < VisibleAnnotations.Num(); ++i)
	{
		const FUserAnnotation* Annotation = VisibleAnnotations[i];

		const float X1 = Viewport.TimeToSlateUnitsRounded(Annotation->Time);
		const float X2 = Annotation->IsRange() ? Viewport.TimeToSlateUnitsRounded(Annotation->EndTime) : X1;

		if (X2 < -100.0f || X1 > ViewWidth + 100.0f)
		{
			continue;
		}

		// Range overlay (filled rect / bracket) always emitted; at narrow widths the rect
		// is still drawn, and at sub-pixel widths the vertical marker below carries visibility.
		if (Annotation->IsRange())
		{
			FUserAnnotationRangeInfo RangeInfo;
			RangeInfo.X1 = FMath::Max(X1, -100.0f);
			RangeInfo.X2 = FMath::Min(X2, ViewWidth + 100.0f);
			RangeInfo.Color = Annotation->Color;
			RangeInfo.Id = Annotation->Id;
			RangeInfo.ThreadName = Annotation->ThreadName;
			RangeInfo.bHasEventAnchor = Annotation->HasEventAnchor();
			AnnotationRanges.Add(MoveTemp(RangeInfo));
		}

		// Marker (vertical line) always emitted.
		{
			FUserAnnotationBoxInfo BoxInfo;
			BoxInfo.X = X1 - 0.5f;
			BoxInfo.W = 1.0f;
			BoxInfo.Color = Annotation->Color;
			BoxInfo.ThreadName = Annotation->ThreadName;
			AnnotationBoxes.Add(BoxInfo);
		}

		// Label: truncate to the horizontal gap to the next visible annotation (or the viewport
		// edge for the last one). Truncation itself uses FindLastWholeCharacterIndexBeforeOffset
		// like the bookmarks/logs tracks do.
		{
			// Gate the slot-width test on unrounded X to keep the icon-only/no-label threshold
			// stable across zoom steps. (Rounded X can jitter ±1 px near the boundary, which
			// would cause a small badge to blink in/out during repeated zoom in/out.) Placement
			// still uses rounded X for pixel alignment.
			const float X1Precise = Viewport.TimeToSlateUnits(Annotation->Time);
			const float NextAnnotationXPrecise = (i + 1 < VisibleAnnotations.Num())
				? Viewport.TimeToSlateUnits(VisibleAnnotations[i + 1]->Time)
				: static_cast<float>(ViewWidth);
			const float LabelGap = 4.0f;
			const float WidthUntilNext = FMath::Max(0.0f, NextAnnotationXPrecise - (X1Precise + LabelPadH) - LabelGap);

			// Range (non-event) annotations center the label inside the range. Constrain by
			// both the range interior and the gap to the next annotation.
			const bool bCenterInRange = Annotation->IsRange() && !Annotation->HasEventAnchor();
			const float X2Precise = Annotation->IsRange()
				? Viewport.TimeToSlateUnits(Annotation->EndTime)
				: X1Precise;
			const float AvailableWidth = bCenterInRange
				? FMath::Min(WidthUntilNext, FMath::Max(0.0f, X2Precise - X1Precise))
				: WidthUntilNext;

			// Below icon-only width, the tick mark alone carries the annotation — no label box.
			if (AvailableWidth < LabelIconOnlyBoxW)
			{
				// Skip the label; marker + any range overlay already emitted above.
			}
			else
			{
				FUserAnnotationTextInfo TextInfo;
				TextInfo.Color = Annotation->Color;
				TextInfo.Id = Annotation->Id;
				TextInfo.Time = Annotation->Time;

				const float AvailableText = FMath::Max(0.0f, AvailableWidth - LabelFullBoxExtra);
				FString DisplayText = Annotation->Text;
				float MeasuredText = 0.0f;

				if (AvailableText > 0.0f && !DisplayText.IsEmpty())
				{
					// Find the last whole character that fits. Hard cut, no ellipsis — matches
					// MarkersTimingTrack::Flush.
					const int32 LastWholeIdx = FontMeasureService->FindLastWholeCharacterIndexBeforeOffset(
						FStringView(DisplayText), Font, FMath::RoundToInt(AvailableText));
					if (LastWholeIdx < 0)
					{
						// Not even one character fits — render icon-only.
						DisplayText.Empty();
					}
					else if (LastWholeIdx + 1 < DisplayText.Len())
					{
						DisplayText.LeftInline(LastWholeIdx + 1);
					}
					MeasuredText = FMath::Min(
						static_cast<float>(FontMeasureService->Measure(DisplayText, Font).X),
						AvailableText);
				}
				else
				{
					// No room for text at all — icon-only label.
					DisplayText.Empty();
				}

				const float BoxW = DisplayText.IsEmpty()
					? LabelIconOnlyBoxW
					: (MeasuredText + LabelFullBoxExtra);

				if (bCenterInRange && BoxW <= (X2 - X1))
				{
					const float RangeCenter = (X1 + X2) * 0.5f;
					// Cap right edge by the next annotation's X so a centered label doesn't paint over its marker.
					const float MaxRight = FMath::Min(X2, NextAnnotationXPrecise - LabelGap);
					TextInfo.X = FMath::Clamp(RangeCenter - BoxW * 0.5f, X1, MaxRight - BoxW);
				}
				else
				{
					TextInfo.X = X1 + LabelPadH;
				}

				TextInfo.Width = BoxW;
				TextInfo.Text = MoveTemp(DisplayText);
				AnnotationTexts.Add(MoveTemp(TextInfo));
			}
		}

		// Build event anchor overlay if this annotation is anchored to a specific timing event.
		// Use precise (unrounded) bounds for the view-overlap test so rounding jitter doesn't
		// flip anchor presence on/off across adjacent zoom levels. Generous margin for the same
		// reason — the callout stays stable near the viewport edges.
		if (Annotation->HasEventAnchor())
		{
			const float EvX1Precise = Viewport.TimeToSlateUnits(Annotation->EventStartTime);
			const float EvX2Precise = Viewport.TimeToSlateUnits(Annotation->EventEndTime);
			const float EvX1 = Viewport.TimeToSlateUnitsRounded(Annotation->EventStartTime);
			const float EvX2 = Viewport.TimeToSlateUnitsRounded(Annotation->EventEndTime);

			if (EvX2Precise >= -1000.0f && EvX1Precise <= ViewWidth + 1000.0f)
			{
				FUserAnnotationEventAnchorInfo AnchorInfo;
				AnchorInfo.X1 = FMath::Max(EvX1, -1000.0f);
				AnchorInfo.X2 = FMath::Min(EvX2, ViewWidth + 1000.0f);
				// Clamp to >= 0 so a malformed annotation with EndTime <= StartTime doesn't
				// propagate a negative width into downstream draw math.
				AnchorInfo.PreciseWidth = FMath::Max(0.0f, EvX2Precise - EvX1Precise);
				AnchorInfo.Depth = Annotation->EventDepth;
				AnchorInfo.Color = Annotation->Color;
				AnchorInfo.Text = Annotation->Text;
				AnchorInfo.ThreadName = Annotation->ThreadName;
				AnchorInfo.Id = Annotation->Id;
				EventAnchors.Add(MoveTemp(AnchorInfo));
			}
		}
	}

	// Drop labels that would overlap a previously-kept one; range labels are X-centered so re-sort by X before the sweep.
	if (AnnotationTexts.Num() > 1)
	{
		Algo::SortBy(AnnotationTexts, [](const FUserAnnotationTextInfo& T) { return T.X; });
		constexpr float CollisionGap = 2.0f;
		float KeptRightEdge = -FLT_MAX;
		AnnotationTexts.RemoveAll([&KeptRightEdge](const FUserAnnotationTextInfo& T)
		{
			if (T.X < KeptRightEdge + CollisionGap)
			{
				return true;
			}
			KeptRightEdge = T.X + T.Width;
			return false;
		});
	}

	// Drop labels that would render under the fixed "Annotations" header strip on the left edge.
	const float HeaderEffectiveW = static_cast<float>(FontMeasureService->Measure(GetName(), Font).X) + 6.0f;
	AnnotationTexts.RemoveAll([HeaderEffectiveW](const FUserAnnotationTextInfo& T) { return T.X < HeaderEffectiveW; });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	// Empty + writable → skip draw (header would bleed into the 4px strip). !bCanPersist
	// falls through so the read-only banner renders even with no annotations.
	const bool bEmpty = !AnnotationStore.IsValid() || AnnotationStore->GetAllAnnotations().Num() == 0;
	if (bEmpty && bCanPersist)
	{
		return;
	}

	// Mirror FTimingViewDrawHelper::DrawTrackHeader: hide header (fixed 14px) when track height drops below 4px.
	if (bCanPersist && GetHeight() < 4.0f)
	{
		return;
	}

	Header.Draw(Context);

	// Read-only banner: overlays the track with an explanation so the user understands why
	// annotations can't be created. Shown whenever the sidecar can't be written.
	if (!bCanPersist)
	{
		FDrawContext& BannerCtx = Context.GetDrawContext();
		const TSharedRef<FSlateFontMeasure> BannerMeasure =
			FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const FString BannerText = TEXT("Annotations disabled — trace folder or sidecar .ini is read-only.");
		const FVector2D BannerSize = BannerMeasure->Measure(BannerText, Font);
		const float BannerH = static_cast<float>(BannerSize.Y) + 4.0f;
		const float BannerY = GetPosY() + 1.0f;
		const float BannerX = 8.0f;
		BannerCtx.DrawBox(0.0f, BannerY, Context.GetViewport().GetWidth(), BannerH,
			WhiteBrush, FLinearColor(0.35f, 0.15f, 0.15f, 0.75f));
		BannerCtx.DrawText(BannerX, BannerY + 2.0f, BannerText, Font,
			FLinearColor(1.0f, 0.9f, 0.9f, 1.0f));

		// Nothing else to draw if the store is empty.
		if (bEmpty)
		{
			return;
		}
	}

	if (Header.IsCollapsed())
	{
		return;
	}

	FDrawContext& DrawContext = Context.GetDrawContext();
	const float TrackY = GetPosY();
	const float TrackH = GetHeight();

	const TSharedRef<FSlateFontMeasure> FontMeasureService =
		FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const FLinearColor TextBoxBorder(0.5f, 0.5f, 0.5f, 0.6f);
	const FSlateBrush* AnnotationBrush = FInsightsStyle::GetBrush("Icons.Annotation");

	// Range overlays (filled rect or bracket). Labels come from AnnotationTexts below.
	// Event-anchored highlights render on the thread track from PostDraw, not here.
	for (const FUserAnnotationRangeInfo& Range : AnnotationRanges)
	{
		const float RangeW = Range.X2 - Range.X1;
		if (Range.bHasEventAnchor)
		{
			// Bracket [====] — end caps + dashed line across the full span.
			const FLinearColor BracketColor(Range.Color.R, Range.Color.G, Range.Color.B, 0.9f);
			const float BracketCapH = FMath::Min(TrackH * 0.6f, 10.0f);
			const float BracketCapY = TrackY + (TrackH - BracketCapH) * 0.5f;
			const float LineY = TrackY + (TrackH - 1.0f) * 0.5f;

			DrawContext.DrawBox(Range.X1, BracketCapY, 1.0f, BracketCapH, WhiteBrush, BracketColor);
			DrawContext.DrawBox(Range.X2 - 1.0f, BracketCapY, 1.0f, BracketCapH, WhiteBrush, BracketColor);

			const float DashStartX = Range.X1 + 3.0f;
			const float DashEndX = Range.X2 - 2.0f;
			const float DashLen = 4.0f;
			const float DashGap = 3.0f;
			for (float X = DashStartX; X + 1.0f < DashEndX; X += DashLen + DashGap)
			{
				const float SegLen = FMath::Min(DashLen, DashEndX - X);
				if (SegLen >= 1.0f)
				{
					DrawContext.DrawBox(X, LineY, SegLen, 1.0f, WhiteBrush, BracketColor);
				}
			}
		}
		else
		{
			DrawContext.DrawBox(Range.X1, TrackY, RangeW, TrackH, WhiteBrush,
				FLinearColor(Range.Color.R, Range.Color.G, Range.Color.B, 0.3f));
		}
	}

	// Draw boxes (marker backgrounds) in the track row
	for (const FUserAnnotationBoxInfo& Box : AnnotationBoxes)
	{
		DrawContext.DrawBox(Box.X, TrackY, Box.W, TrackH, WhiteBrush,
			FLinearColor(Box.Color.R, Box.Color.G, Box.Color.B, 0.25f));
	}

	// Draw labels. Icon-only (empty Text) is emitted at narrow zoom as a minimal marker badge.
	// "Mg" probes the font's ascender + descender so BoxH stays constant regardless of which
	// text the label carries.
	const float LabelTextH = static_cast<float>(FontMeasureService->Measure(TEXT("Mg"), Font).Y);
	const float LabelBoxH = LabelTextH + LabelPadV * 2.0f;
	for (const FUserAnnotationTextInfo& TextInfo : AnnotationTexts)
	{
		const bool bIconOnly = TextInfo.Text.IsEmpty();
		const float BoxW = TextInfo.Width; // computed by UpdateDrawState; authoritative for hit-test too
		const float BoxX = TextInfo.X;
		const float BoxY = TrackY + (TrackH - LabelBoxH) * 0.5f;

		const FLinearColor LabelTintBg(TextInfo.Color.R, TextInfo.Color.G, TextInfo.Color.B, 0.85f);
		DrawContext.DrawBox(BoxX, BoxY, BoxW, LabelBoxH, WhiteBrush, LabelTintBg);
		DrawContext.DrawBox(BoxX, BoxY, BoxW, 1.0f, WhiteBrush, TextBoxBorder);
		DrawContext.DrawBox(BoxX, BoxY + LabelBoxH - 1.0f, BoxW, 1.0f, WhiteBrush, TextBoxBorder);
		DrawContext.DrawBox(BoxX, BoxY, 1.0f, LabelBoxH, WhiteBrush, TextBoxBorder);
		DrawContext.DrawBox(BoxX + BoxW - 1.0f, BoxY, 1.0f, LabelBoxH, WhiteBrush, TextBoxBorder);
		DrawContext.DrawBox(BoxX + LabelPadH, BoxY + (LabelBoxH - LabelIconSize) * 0.5f,
			LabelIconSize, LabelIconSize, AnnotationBrush, GetContrastTextColor(LabelTintBg));
		if (!bIconOnly)
		{
			DrawContext.DrawText(BoxX + LabelPadH + LabelIconSize + LabelIconGap, BoxY + LabelPadV,
				TextInfo.Text, Font, GetContrastTextColor(LabelTintBg));
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingTrack::PostDraw(const ITimingTrackDrawContext& Context) const
{
	// When the Annotations track is hidden via the Other menu, suppress every
	// annotation-related visual: event highlights, left-edge indicators, connectors,
	// and callouts. Floating Callouts toggle still only affects the callout box.
	if (!IsVisible())
	{
		return;
	}

	FDrawContext& DrawContext = Context.GetDrawContext();

	const TSharedRef<FSlateFontMeasure> FontMeasureService =
		FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const FLinearColor CalloutBorder(0.15f, 0.15f, 0.15f, 0.85f);

	// REVERTIBLE-PERF: cache keyed by FBaseTimingTrack* so hot draw loops skip per-entry string hashing.
	// Per-frame string lookups are now confined to the single resolve pass below; the inner loops only
	// follow the cached TWeakPtr in each info struct.
	TMap<FBaseTimingTrack*, TPair<float, float>> TrackBoundsCache;
	if (Session)
	{
		TSet<FString> NeededTrackNames;
		for (const FUserAnnotationRangeInfo& Range : AnnotationRanges)
		{
			if (!Range.bHasEventAnchor && !Range.ThreadName.IsEmpty()) { NeededTrackNames.Add(Range.ThreadName); }
		}
		for (const FUserAnnotationBoxInfo& Box : AnnotationBoxes)
		{
			if (!Box.ThreadName.IsEmpty()) { NeededTrackNames.Add(Box.ThreadName); }
		}
		for (const FUserAnnotationEventAnchorInfo& Anchor : EventAnchors)
		{
			if (!Anchor.ThreadName.IsEmpty()) { NeededTrackNames.Add(Anchor.ThreadName); }
		}
		if (NeededTrackNames.Num() > 0)
		{
			TrackBoundsCache.Reserve(NeededTrackNames.Num());
			// Temporary name -> TSharedPtr resolver for this frame (used to populate each info struct's ResolvedTrack).
			TMap<FString, TSharedPtr<FBaseTimingTrack>> NameToTrack;
			NameToTrack.Reserve(NeededTrackNames.Num());
			Session->EnumerateTracks([&TrackBoundsCache, &NeededTrackNames, &NameToTrack](TSharedPtr<FBaseTimingTrack> Track)
			{
				if (Track.IsValid() && Track->IsVisible() && NeededTrackNames.Contains(Track->GetName()))
				{
					TrackBoundsCache.Add(Track.Get(), TPair<float, float>(Track->GetPosY(), Track->GetHeight()));
					NameToTrack.Add(Track->GetName(), Track);
				}
			});

			// Single name->ptr resolve pass per Draw. After this, hot loops never touch ThreadName.
			// ResolvedTrack is mutable on the info structs so we can refresh it through const iteration.
			auto ResolveFromName = [&NameToTrack](const FString& InTrackName) -> TWeakPtr<FBaseTimingTrack>
			{
				if (InTrackName.IsEmpty()) { return TWeakPtr<FBaseTimingTrack>(); }
				if (const TSharedPtr<FBaseTimingTrack>* Found = NameToTrack.Find(InTrackName)) { return *Found; }
				return TWeakPtr<FBaseTimingTrack>();
			};
			for (const FUserAnnotationRangeInfo& Range : AnnotationRanges)
			{
				Range.ResolvedTrack = ResolveFromName(Range.ThreadName);
			}
			for (const FUserAnnotationBoxInfo& Box : AnnotationBoxes)
			{
				Box.ResolvedTrack = ResolveFromName(Box.ThreadName);
			}
			for (const FUserAnnotationEventAnchorInfo& Anchor : EventAnchors)
			{
				Anchor.ResolvedTrack = ResolveFromName(Anchor.ThreadName);
			}
		}
	}

	// Range bands and per-track point lines are markers gated by bShowAnnotationMarkers.
	if (bShowAnnotationMarkers)
	{
		for (const FUserAnnotationRangeInfo& Range : AnnotationRanges)
		{
			if (Range.bHasEventAnchor)
			{
				continue;
			}
			const float RangeW = Range.X2 - Range.X1;
			// REVERTIBLE-PERF: pointer-keyed lookup; ResolvedTrack populated above.
			if (TSharedPtr<FBaseTimingTrack> Track = Range.ResolvedTrack.Pin())
			{
				if (const TPair<float, float>* Bounds = TrackBoundsCache.Find(Track.Get()))
				{
					DrawContext.DrawBox(Range.X1, Bounds->Key, RangeW, Bounds->Value, WhiteBrush,
						FLinearColor(Range.Color.R, Range.Color.G, Range.Color.B, 0.15f));
				}
			}
		}

		for (const FUserAnnotationBoxInfo& Box : AnnotationBoxes)
		{
			// REVERTIBLE-PERF: pointer-keyed lookup; ResolvedTrack populated above.
			if (TSharedPtr<FBaseTimingTrack> Track = Box.ResolvedTrack.Pin())
			{
				if (const TPair<float, float>* Bounds = TrackBoundsCache.Find(Track.Get()))
				{
					DrawContext.DrawBox(Box.X, Bounds->Key, Box.W, Bounds->Value, WhiteBrush,
						FLinearColor(Box.Color.R, Box.Color.G, Box.Color.B, 0.25f));
				}
			}
		}
	}

	// Full-viewport range bands + point indicator lines (parity between range and point markers).
	if (bShowAnnotationMarkers)
	{
		const float ViewY = Context.GetViewport().GetPosY();
		const float ViewH = Context.GetViewport().GetHeight();
		// REVERTIBLE: viewport-wide range band — mirrors point lines for ranges without a target track.
		for (const FUserAnnotationRangeInfo& Range : AnnotationRanges)
		{
			if (Range.bHasEventAnchor)
			{
				continue;
			}
			const float RangeW = Range.X2 - Range.X1;
			const FLinearColor ExtColor(Range.Color.R, Range.Color.G, Range.Color.B, 0.08f);
			DrawContext.DrawBox(Range.X1, ViewY, RangeW, ViewH, WhiteBrush, ExtColor);
		}
		for (const FUserAnnotationBoxInfo& Box : AnnotationBoxes)
		{
			const FLinearColor ExtColor(Box.Color.R, Box.Color.G, Box.Color.B, 0.15f);
			DrawContext.DrawBox(Box.X, ViewY, Box.W, ViewH, WhiteBrush, ExtColor);
		}
		DrawContext.LayerId++;
	}

	// Event anchor highlights: tint the annotated event and the call stack beneath it.
	if (EventAnchors.Num() > 0)
	{
		const FTimingTrackViewport& Viewport = Context.GetViewport();
		const FTimingViewLayout& Layout = Viewport.GetLayout();

		// Scrollable region bounds; anything outside belongs to top/bottom-docked tracks.
		const float ScrollableTopY = Viewport.GetPosY() + Viewport.GetTopOffset();
		const float ScrollableBottomY = ScrollableTopY + Viewport.GetScrollableAreaHeight();

		// Callout placement records: collected in pass 1, coalesced in pass 2, drawn in pass 3.
		struct FCalloutPlacement
		{
			int32 AnchorIdx = INDEX_NONE;
			float BoxX = 0.0f;
			float BoxY = 0.0f;
			float BoxW = 0.0f;
			float BoxH = 0.0f;
			float EventY = 0.0f;
			float EventH = 0.0f;
			bool bCalloutAbove = true;
			int32 MergedCount = 0;
			FString DisplayText;
		};
		TArray<FCalloutPlacement> Placements;
		Placements.Reserve(EventAnchors.Num());

		const FSlateFontInfo& CalloutFont = Font;

		const float EventCalloutPadH = 6.0f;
		const float EventCalloutPadV = 4.0f;
		const float EventIconSize = 12.0f;
		const float EventIconGap = 4.0f;
		const float CalloutGap = 14.0f;
		const float ViewWidth = Viewport.GetWidth();
		// Cap callout text so long annotation messages don't overflow the box.
		const float MaxCalloutTextW = 240.0f;

		auto FitTextToWidth = [&](const FString& InText, float MaxWidth) -> FString
		{
			const FVector2D Size = FontMeasureService->Measure(InText, CalloutFont);
			if (static_cast<float>(Size.X) <= MaxWidth)
			{
				return InText;
			}
			const FVector2D EllipsisSize = FontMeasureService->Measure(TEXT("..."), CalloutFont);
			const float AvailableW = FMath::Max(MaxWidth - static_cast<float>(EllipsisSize.X), 0.0f);
			const int32 LastWholeIdx = FontMeasureService->FindLastWholeCharacterIndexBeforeOffset(
				FStringView(InText), CalloutFont, FMath::RoundToInt(AvailableW));
			if (LastWholeIdx < 0)
			{
				return FString(TEXT("..."));
			}
			return InText.Left(LastWholeIdx + 1) + TEXT("...");
		};

		// Reserve width for the worst-case "(+N)" merge suffix when checking overlap in pass 2.
		const FVector2D SuffixSize = FontMeasureService->Measure(TEXT(" (+99)"), CalloutFont);
		const float CoalesceSuffixSlack = FMath::CeilToFloat(static_cast<float>(SuffixSize.X));

		for (int32 AnchorIdx = 0; AnchorIdx < EventAnchors.Num(); ++AnchorIdx)
		{
			const FUserAnnotationEventAnchorInfo& Anchor = EventAnchors[AnchorIdx];
			// REVERTIBLE-PERF: pointer-keyed lookup; ResolvedTrack populated above.
			TSharedPtr<FBaseTimingTrack> AnchorTrack = Anchor.ResolvedTrack.Pin();
			if (!AnchorTrack.IsValid())
			{
				continue;
			}

			const TPair<float, float>* Bounds = TrackBoundsCache.Find(AnchorTrack.Get());
			if (!Bounds)
			{
				continue;
			}

			const float TrackY = Bounds->Key;
			const float TrackH = Bounds->Value;
			// Snap to pixel grid — otherwise sub-pixel Y drift between adjacent zoom levels
			// causes the event-highlight fill to flicker during repeated zoom in/out.
			const float EventY = FMath::RoundToFloat(TrackY + Layout.GetLaneY(static_cast<int32>(Anchor.Depth)));
			const float EventH = FMath::RoundToFloat(Layout.EventH);
			const float EventW = Anchor.X2 - Anchor.X1;

			// Depth limit: events at or beyond the configured depth are filtered out of
			// the thread track's draw state (see FTimingProfilerManager::GetEventDepthLimit),
			// so any callout or highlight anchored on them must also hide. Per Catalin's
			// review feedback.
			if (Anchor.Depth >= FTimingProfilerManager::Get()->GetEventDepthLimit())
			{
				continue;
			}

			// Clip to scrollable area: skip events that fall inside top/bottom-docked regions
			// so the highlight doesn't bleed into adjacent docked tracks.
			if (EventY + EventH <= ScrollableTopY || EventY >= ScrollableBottomY)
			{
				continue;
			}

			const float HighlightTopY = FMath::Max(EventY, ScrollableTopY);
			const float HighlightBottomY = FMath::Min(TrackY + TrackH, ScrollableBottomY);
			const float StackH = HighlightBottomY - HighlightTopY;
			const FLinearColor FillColor(Anchor.Color.R, Anchor.Color.G, Anchor.Color.B, 0.15f);
			if (bShowAnnotationMarkers && StackH > 0.0f)
			{
				DrawContext.DrawBox(Anchor.X1, HighlightTopY, EventW, StackH, WhiteBrush, FillColor);
			}

			// Compact mode events are ~4 px tall and a callout box would overlap adjacent tracks.
			if (!bShowFloatingAnnotations || Anchor.Text.IsEmpty() || Layout.bIsCompactMode)
			{
				continue;
			}

			// Use the visible portion of the event so connectors don't point outside the scrollable region.
			const float ClampedEventY = HighlightTopY;
			const float ClampedEventBottomY = HighlightBottomY;
			const float ClampedEventH = FMath::Max(ClampedEventBottomY - ClampedEventY, 0.0f);

			// Pass 1: compute placement; defer draw until after coalescing.
			const FString FittedText = FitTextToWidth(Anchor.Text, MaxCalloutTextW);
			const FVector2D TextSize = FontMeasureService->Measure(FittedText, CalloutFont);
			const float TextW = FMath::CeilToFloat(static_cast<float>(TextSize.X));
			const float TextH = FMath::CeilToFloat(static_cast<float>(TextSize.Y));
			const float BoxW = TextW + EventIconSize + EventIconGap + EventCalloutPadH * 2.0f;
			const float BoxH = TextH + EventCalloutPadV * 2.0f;

			float BoxX = FMath::Clamp(Anchor.X1, 0.0f, FMath::Max(ViewWidth - BoxW, 0.0f));
			BoxX = FMath::RoundToFloat(BoxX);

			const float AboveBoxY = ClampedEventY - BoxH - CalloutGap;
			const float BelowBoxY = ClampedEventBottomY + CalloutGap;
			float BoxY = (AboveBoxY >= ScrollableTopY) ? AboveBoxY : BelowBoxY;
			BoxY = FMath::Min(BoxY, ScrollableBottomY - BoxH);
			BoxY = FMath::Max(BoxY, ScrollableTopY);
			BoxY = FMath::RoundToFloat(BoxY);

			FCalloutPlacement P;
			P.AnchorIdx = AnchorIdx;
			P.BoxX = BoxX;
			P.BoxY = BoxY;
			P.BoxW = BoxW;
			P.BoxH = BoxH;
			P.EventY = ClampedEventY;
			P.EventH = ClampedEventH;
			P.bCalloutAbove = (BoxY + BoxH <= ClampedEventY);
			P.DisplayText = FittedText;
			Placements.Add(P);
		}

		// Pass 2: coalesce overlapping callouts on the same track into a "(+N)" suffix on the kept one.
		TArray<int32> KeptIdx;
		KeptIdx.Reserve(Placements.Num());
		for (int32 i = 0; i < Placements.Num(); ++i)
		{
			const FCalloutPlacement& Cur = Placements[i];
			bool bMerged = false;
			for (int32 k : KeptIdx)
			{
				const FCalloutPlacement& Kept = Placements[k];
				if (EventAnchors[Kept.AnchorIdx].ThreadName != EventAnchors[Cur.AnchorIdx].ThreadName)
				{
					continue;
				}
				// Inflate the kept box symmetrically by the worst-case "(+N)" suffix width.
				const float KeptLeft = Kept.BoxX - CoalesceSuffixSlack;
				const float KeptRight = Kept.BoxX + Kept.BoxW + CoalesceSuffixSlack;
				const bool bOverlapsX = (Cur.BoxX < KeptRight) && (Cur.BoxX + Cur.BoxW > KeptLeft);
				const bool bOverlapsY = (Cur.BoxY < Kept.BoxY + Kept.BoxH) && (Cur.BoxY + Cur.BoxH > Kept.BoxY);
				if (bOverlapsX && bOverlapsY)
				{
					Placements[k].MergedCount++;
					bMerged = true;
					break;
				}
			}
			if (!bMerged)
			{
				KeptIdx.Add(i);
			}
		}

		// Pass 3: draw kept callouts + connectors.
		const FSlateBrush* PostDrawAnnotationBrush = FInsightsStyle::GetBrush("Icons.Annotation");
		for (int32 k : KeptIdx)
		{
			FCalloutPlacement& P = Placements[k];
			const FUserAnnotationEventAnchorInfo& Anchor = EventAnchors[P.AnchorIdx];

			FString DisplayText = P.DisplayText;
			if (P.MergedCount > 0)
			{
				// Cap at 99 so the suffix never exceeds CoalesceSuffixSlack.
				const int32 DisplayCount = FMath::Min(P.MergedCount, 99);
				DisplayText = FString::Printf(TEXT("%s (+%d)"), *P.DisplayText, DisplayCount);
				const FVector2D TextSize2 = FontMeasureService->Measure(DisplayText, CalloutFont);
				const float TextW2 = FMath::CeilToFloat(static_cast<float>(TextSize2.X));
				const float TextH2 = FMath::CeilToFloat(static_cast<float>(TextSize2.Y));
				P.BoxW = TextW2 + EventIconSize + EventIconGap + EventCalloutPadH * 2.0f;
				P.BoxH = TextH2 + EventCalloutPadV * 2.0f;
				P.BoxX = FMath::RoundToFloat(FMath::Clamp(P.BoxX, 0.0f, FMath::Max(ViewWidth - P.BoxW, 0.0f)));
			}

			const FLinearColor TintedCalloutBg(Anchor.Color.R, Anchor.Color.G, Anchor.Color.B, 0.85f);
			DrawContext.DrawBox(P.BoxX, P.BoxY, P.BoxW, P.BoxH, WhiteBrush, TintedCalloutBg);
			DrawContext.DrawBox(P.BoxX, P.BoxY, P.BoxW, 1.0f, WhiteBrush, CalloutBorder);
			DrawContext.DrawBox(P.BoxX, P.BoxY + P.BoxH - 1.0f, P.BoxW, 1.0f, WhiteBrush, CalloutBorder);
			DrawContext.DrawBox(P.BoxX, P.BoxY, 1.0f, P.BoxH, WhiteBrush, CalloutBorder);
			DrawContext.DrawBox(P.BoxX + P.BoxW - 1.0f, P.BoxY, 1.0f, P.BoxH, WhiteBrush, CalloutBorder);
			const float IconX = FMath::RoundToFloat(P.BoxX + EventCalloutPadH);
			const float IconY = FMath::RoundToFloat(P.BoxY + (P.BoxH - EventIconSize) * 0.5f);
			DrawContext.DrawBox(IconX, IconY, EventIconSize, EventIconSize,
				PostDrawAnnotationBrush, GetContrastTextColor(TintedCalloutBg));
			const float TextX = FMath::RoundToFloat(IconX + EventIconSize + EventIconGap);
			DrawContext.DrawText(TextX, FMath::RoundToFloat(P.BoxY + EventCalloutPadV),
				DisplayText, CalloutFont, GetContrastTextColor(TintedCalloutBg));

			// Connector stem from the callout edge back to the event bar.
			const float VisibleEventX1 = FMath::Max(Anchor.X1, 0.0f);
			const float VisibleEventX2 = FMath::Min(Anchor.X2, ViewWidth);
			const float ConnX = FMath::RoundToFloat(FMath::Clamp(P.BoxX + 8.0f, VisibleEventX1, VisibleEventX2));
			const float ConnW = 3.0f;
			const FLinearColor ConnColor(Anchor.Color.R, Anchor.Color.G, Anchor.Color.B, 0.9f);

			const float ArrowHeadH = 5.0f;
			if (P.bCalloutAbove)
			{
				const float ConnTopY = FMath::RoundToFloat(P.BoxY + P.BoxH);
				const float ConnBotY = FMath::RoundToFloat(P.EventY);
				if (ConnBotY > ConnTopY)
				{
					DrawContext.DrawBox(ConnX - ConnW * 0.5f, ConnTopY,
						ConnW, ConnBotY - ConnTopY - ArrowHeadH, WhiteBrush, ConnColor);
					const float ArrowTopY = ConnBotY - ArrowHeadH;
					for (int32 Row = 0; Row < 5; ++Row)
					{
						const float RowF = static_cast<float>(Row);
						const float RowW = 5.0f - RowF;
						DrawContext.DrawBox(ConnX - RowW * 0.5f, ArrowTopY + RowF,
							RowW, 1.0f, WhiteBrush, ConnColor);
					}
				}
			}
			else
			{
				const float ConnTopY = FMath::RoundToFloat(P.EventY + P.EventH);
				const float ConnBotY = FMath::RoundToFloat(P.BoxY);
				if (ConnBotY > ConnTopY)
				{
					DrawContext.DrawBox(ConnX - ConnW * 0.5f, ConnTopY + ArrowHeadH,
						ConnW, ConnBotY - ConnTopY - ArrowHeadH, WhiteBrush, ConnColor);
					const float ArrowBotY = ConnTopY;
					for (int32 Row = 0; Row < 5; ++Row)
					{
						const float RowF = static_cast<float>(Row);
						const float RowW = 1.0f + RowF;
						DrawContext.DrawBox(ConnX - RowW * 0.5f, ArrowBotY + RowF,
							RowW, 1.0f, WhiteBrush, ConnColor);
					}
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply FUserAnnotationsTimingTrack::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Double-click on an annotation -> select its time range. Matches the convention used
	// on other timing-event tracks (per reviewer feedback).
	if (!Session || !AnnotationStore.IsValid())
	{
		return FReply::Unhandled();
	}

	// STimingView is the only implementer of ITimingViewSession; no RTTI available here.
	STimingView* TimingView = static_cast<STimingView*>(Session);
	const FTimingTrackViewport& Viewport = TimingView->GetViewport();

	const FVector2D LocalPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	const double TimeAtCursor = Viewport.SlateUnitsToTime(static_cast<float>(LocalPos.X));
	const double TimeTolerance = 4.0 / FMath::Max(Viewport.GetScaleX(), KINDA_SMALL_NUMBER);

	// Find the annotation under the cursor: range first (covers area), then nearest point.
	const FUserAnnotation* Selected = nullptr;
	for (const FUserAnnotation& Annotation : AnnotationStore->GetAllAnnotations())
	{
		if (!Annotation.bVisible)
		{
			continue;
		}
		if (Annotation.IsRange() && TimeAtCursor >= Annotation.Time && TimeAtCursor <= Annotation.EndTime)
		{
			Selected = &Annotation;
			break;
		}
	}
	if (!Selected)
	{
		double NearestDistance = TimeTolerance;
		for (const FUserAnnotation& Annotation : AnnotationStore->GetAllAnnotations())
		{
			if (!Annotation.bVisible || Annotation.IsRange())
			{
				continue;
			}
			const double Distance = FMath::Abs(Annotation.Time - TimeAtCursor);
			if (Distance < NearestDistance)
			{
				NearestDistance = Distance;
				Selected = &Annotation;
			}
		}
	}
	if (!Selected)
	{
		return FReply::Unhandled();
	}

	// Event-anchored: select the anchored event's [EventStartTime, EventEndTime].
	// Range: select [Time, EndTime]. Point: no duration to select, leave selection alone.
	if (Selected->HasEventAnchor())
	{
		const double Duration = Selected->EventEndTime - Selected->EventStartTime;
		if (Duration > 0.0)
		{
			TimingView->SelectTimeInterval(Selected->EventStartTime, Duration);
			return FReply::Handled();
		}
	}
	else if (Selected->IsRange())
	{
		const double Duration = Selected->EndTime - Selected->Time;
		if (Duration > 0.0)
		{
			TimingView->SelectTimeInterval(Selected->Time, Duration);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FUserAnnotationsTimingTrack::GetEvent(
	float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const
{
	if (!AnnotationStore.IsValid())
	{
		return nullptr;
	}

	const float Tolerance = 4.0f; // pixels
	const double ScaleX = Viewport.GetScaleX();
	if (ScaleX <= 0.0)
	{
		return nullptr;
	}
	const double Time = Viewport.SlateUnitsToTime(InPosX);
	const double TimeTolerance = Tolerance / ScaleX;

	// Check range annotations first (they cover an area)
	for (const FUserAnnotation& Annotation : AnnotationStore->GetAllAnnotations())
	{
		if (Annotation.IsRange() && Time >= Annotation.Time && Time <= Annotation.EndTime)
		{
			return MakeShared<FUserAnnotationTimingEvent>(
				SharedThis(this),
				Annotation.Time,
				Annotation.EndTime,
				Annotation.Id);
		}
	}

	// Fall back to point annotation nearest-distance check
	const FUserAnnotation* NearestAnnotation = nullptr;
	double NearestDistance = TimeTolerance;

	for (const FUserAnnotation& Annotation : AnnotationStore->GetAllAnnotations())
	{
		if (!Annotation.IsRange())
		{
			const double Distance = FMath::Abs(Annotation.Time - Time);
			if (Distance < NearestDistance)
			{
				NearestDistance = Distance;
				NearestAnnotation = &Annotation;
			}
		}
	}

	if (NearestAnnotation)
	{
		return MakeShared<FUserAnnotationTimingEvent>(
			SharedThis(this),
			NearestAnnotation->Time,
			NearestAnnotation->Id);
	}

	// Check if hovering over a text label (the label extends rightward from the marker)
	for (const FUserAnnotationTextInfo& TextInfo : AnnotationTexts)
	{
		if (TextInfo.Width > 0.0f && InPosX >= TextInfo.X && InPosX <= TextInfo.X + TextInfo.Width)
		{
			return MakeShared<FUserAnnotationTimingEvent>(
				SharedThis(this),
				TextInfo.Time,
				TextInfo.Id);
		}
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	if (!InTooltipEvent.CheckTrack(this) || !InTooltipEvent.Is<FUserAnnotationTimingEvent>())
	{
		return;
	}

	const FUserAnnotationTimingEvent& AnnotationEvent =
		InTooltipEvent.As<FUserAnnotationTimingEvent>();

	InOutTooltip.ResetContent();

	const FUserAnnotation* Annotation = AnnotationStore.IsValid()
		? AnnotationStore->FindAnnotation(AnnotationEvent.GetAnnotationId())
		: nullptr;

	if (Annotation)
	{
		if (Annotation->HasEventAnchor())
		{
			InOutTooltip.AddTitle(TEXT("Event Annotation"));
			InOutTooltip.AddNameValueTextLine(TEXT("Event:"), *Annotation->TimerName);
			InOutTooltip.AddNameValueTextLine(TEXT("Text:"), *Annotation->Text);
			InOutTooltip.AddNameValueTextLine(TEXT("Event Time:"),
				*FString::Printf(TEXT("%.6f s \u2014 %.6f s (%.6f s)"),
					Annotation->EventStartTime, Annotation->EventEndTime,
					Annotation->EventEndTime - Annotation->EventStartTime));
			InOutTooltip.AddNameValueTextLine(TEXT("Depth:"),
				*FString::Printf(TEXT("%u"), Annotation->EventDepth));
		}
		else if (Annotation->IsRange())
		{
			InOutTooltip.AddTitle(TEXT("Range Annotation"));
			InOutTooltip.AddNameValueTextLine(TEXT("Text:"), *Annotation->Text);
			InOutTooltip.AddNameValueTextLine(TEXT("Start:"),
				*FString::Printf(TEXT("%.6f s"), Annotation->Time));
			InOutTooltip.AddNameValueTextLine(TEXT("End:"),
				*FString::Printf(TEXT("%.6f s"), Annotation->EndTime));
			InOutTooltip.AddNameValueTextLine(TEXT("Duration:"),
				*FString::Printf(TEXT("%.6f s"), Annotation->EndTime - Annotation->Time));
		}
		else
		{
			InOutTooltip.AddTitle(TEXT("Time Annotation"));
			InOutTooltip.AddNameValueTextLine(TEXT("Text:"), *Annotation->Text);
			InOutTooltip.AddNameValueTextLine(TEXT("Time:"),
				*FString::Printf(TEXT("%.6f s"), Annotation->Time));
		}
		if (!Annotation->Description.IsEmpty())
		{
			// Flatten newlines for single-line tooltip display.
			FString FlatDesc = Annotation->Description;
			FlatDesc.ReplaceInline(TEXT("\r\n"), TEXT(" "));
			FlatDesc.ReplaceInline(TEXT("\n"), TEXT(" "));
			FlatDesc.ReplaceInline(TEXT("\r"), TEXT(" "));
			InOutTooltip.AddNameValueTextLine(TEXT("Description:"), *FlatDesc);
		}
		InOutTooltip.AddNameValueTextLine(TEXT("Game Frame:"),
			*FString::Printf(TEXT("%u"), Annotation->GameFrameNumber));
		InOutTooltip.AddNameValueTextLine(TEXT("Render Frame:"),
			*FString::Printf(TEXT("%u"), Annotation->RenderFrameNumber));
		if (!Annotation->ThreadName.IsEmpty())
		{
			InOutTooltip.AddNameValueTextLine(TEXT("Track:"), *Annotation->ThreadName);
		}
		if (!Annotation->Author.IsEmpty())
		{
			InOutTooltip.AddNameValueTextLine(TEXT("Author:"), *Annotation->Author);
		}
		InOutTooltip.AddNameValueTextLine(TEXT("Created:"),
			*Annotation->CreatedAt.ToString(TEXT("%Y-%m-%d %H:%M:%S")));
	}

	InOutTooltip.UpdateLayout();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingTrack::OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const
{
	if (!InSelectedEvent.CheckTrack(this) || !InSelectedEvent.Is<FUserAnnotationTimingEvent>())
	{
		return;
	}

	const FUserAnnotationTimingEvent& AnnotationEvent =
		InSelectedEvent.As<FUserAnnotationTimingEvent>();

	const FUserAnnotation* Annotation = AnnotationStore.IsValid()
		? AnnotationStore->FindAnnotation(AnnotationEvent.GetAnnotationId())
		: nullptr;

	if (Annotation)
	{
		FPlatformApplicationMisc::ClipboardCopy(*Annotation->Text);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
