// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Fonts/SlateFontInfo.h"
#include "Misc/Guid.h"

// TraceInsights
#include "Insights/ViewModels/BaseTimingTrack.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TrackHeader.h"

struct FSlateBrush;
class FTimingTrackViewport;

namespace UE::Insights::Timing { class ITimingViewSession; }

namespace UE::Insights::TimingProfiler
{

class FUserAnnotationStore;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Custom timing event for user annotations, used for hit testing and tooltips. */
class FUserAnnotationTimingEvent : public FTimingEvent
{
	INSIGHTS_DECLARE_RTTI(FUserAnnotationTimingEvent, FTimingEvent)

public:
	FUserAnnotationTimingEvent(const TSharedRef<const FBaseTimingTrack> InTrack,
		double InTime, const FGuid& InAnnotationId)
		: FTimingEvent(InTrack, InTime, InTime, 0 /*Depth*/, uint32(-1) /*Type*/)
		, AnnotationId(InAnnotationId)
	{
	}

	FUserAnnotationTimingEvent(const TSharedRef<const FBaseTimingTrack> InTrack,
		double InTime, double InEndTime, const FGuid& InAnnotationId)
		: FTimingEvent(InTrack, InTime, InEndTime, 0 /*Depth*/, uint32(-1) /*Type*/)
		, AnnotationId(InAnnotationId)
	{
	}

	const FGuid& GetAnnotationId() const { return AnnotationId; }

private:
	FGuid AnnotationId;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FUserAnnotationBoxInfo
{
	float X = 0.0f;
	float W = 0.0f;
	FLinearColor Color = FLinearColor::White;
	FString ThreadName; // Target track name for scoped rendering
	// REVERTIBLE-PERF: resolved track pointer (populated per Draw); lets hot loops skip string lookups.
	// mutable so const-accessed info structs in the const Draw() method can still refresh the cache.
	mutable TWeakPtr<FBaseTimingTrack> ResolvedTrack;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FUserAnnotationTextInfo
{
	float X = 0.0f;
	float Width = 0.0f;  // Measured text width for hit testing (0 when text doesn't render)
	FLinearColor Color = FLinearColor::White;
	FString Text;
	FGuid Id;     // Annotation ID for hit testing
	double Time = 0.0;  // Annotation time for creating timing events
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FUserAnnotationRangeInfo
{
	float X1 = 0.0f;      // Start pixel
	float X2 = 0.0f;      // End pixel
	FLinearColor Color = FLinearColor::White;
	FGuid Id;      // For hit testing
	FString ThreadName; // Target track name for scoped rendering
	bool bHasEventAnchor = false; // True if this range is anchored to a specific event (skip full-track overlay)
	// REVERTIBLE-PERF: resolved track pointer (populated per Draw); lets hot loops skip string lookups.
	// mutable so const-accessed info structs in the const Draw() method can still refresh the cache.
	mutable TWeakPtr<FBaseTimingTrack> ResolvedTrack;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Cached info for rendering an event-anchored annotation highlight on a function call bar. */
struct FUserAnnotationEventAnchorInfo
{
	float X1 = 0.0f;          // Start pixel of the event (pixel-rounded, for drawing)
	float X2 = 0.0f;          // End pixel of the event (pixel-rounded, for drawing)
	float PreciseWidth = 0.0f;// Unrounded pixel width; stable across zoom levels (no rounding jitter)
	uint32 Depth = 0;         // Depth in the track
	FLinearColor Color = FLinearColor::White;
	FString Text;
	FString ThreadName; // Track to render on
	FGuid Id;          // For hit testing
	// REVERTIBLE-PERF: resolved track pointer (populated per Draw); lets hot loops skip string lookups.
	// mutable so const-accessed info structs in the const Draw() method can still refresh the cache.
	mutable TWeakPtr<FBaseTimingTrack> ResolvedTrack;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Renders user annotations as colored vertical markers on the Insights timeline.
 * Follows the FMarkersTimingTrack pattern: cached draw state and change-number dirty checking.
 * Placement: TopDocked or BottomDocked, order = FTimingTrackOrder::Markers + 1 (below system Bookmarks).
 */
class FUserAnnotationsTimingTrack : public FBaseTimingTrack
{
	INSIGHTS_DECLARE_RTTI(FUserAnnotationsTimingTrack, FBaseTimingTrack)

public:
	explicit FUserAnnotationsTimingTrack(TSharedPtr<FUserAnnotationStore> InAnnotationStore);
	virtual ~FUserAnnotationsTimingTrack() = default;

	/** Sets the timing view session, used to resolve target tracks for scoped rendering. */
	void SetSession(UE::Insights::Timing::ITimingViewSession* InSession) { Session = InSession; }

	/**
	 * Swap in a shared annotation store. Used when another Insights window already loaded
	 * the same trace's sidecar — all windows viewing one trace share a single store so
	 * edits in any window are visible in all without reload.
	 */
	void SetStore(TSharedPtr<FUserAnnotationStore> InStore) { AnnotationStore = InStore; ChangeNumber = 0; ResetCache(); SetDirtyFlag(); }

	/** Controls whether floating event callout boxes are shown in PostDraw. Event highlights remain regardless. */
	bool GetShowFloatingAnnotations() const { return bShowFloatingAnnotations; }
	void SetShowFloatingAnnotations(bool bShow) { bShowFloatingAnnotations = bShow; }
	void ToggleShowFloatingAnnotations() { bShowFloatingAnnotations = !bShowFloatingAnnotations; }

	/** Controls visibility of annotation markers (vertical indicator lines and event highlights). */
	bool GetShowAnnotationMarkers() const { return bShowAnnotationMarkers; }
	void SetShowAnnotationMarkers(bool bShow) { bShowAnnotationMarkers = bShow; }

	/** When false, the track renders a "disabled — read-only sidecar" banner instead of normal content. */
	void SetCanPersist(bool bInCanPersist) { bCanPersist = bInCanPersist; }

	/** Snap a time value to the nearest visible user annotation within tolerance. */
	double Snap(double Time, double SnapTolerance) const;

	virtual void Reset() override;

	// FBaseTimingTrack
	virtual void PreUpdate(const ITimingTrackUpdateContext& Context) override;
	virtual void Update(const ITimingTrackUpdateContext& Context) override;
	virtual void PostUpdate(const ITimingTrackUpdateContext& Context) override;
	virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	virtual void PostDraw(const ITimingTrackDrawContext& Context) const override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual const TSharedPtr<const ITimingEvent> GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const override;
	virtual void InitTooltip(::FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;
	virtual void OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const override;

private:
	void ResetCache()
	{
		AnnotationBoxes.Reset();
		AnnotationTexts.Reset();
		AnnotationRanges.Reset();
		EventAnchors.Reset();
		CachedVisibleInView = -1;
	}

	void UpdateDrawState(const ITimingTrackUpdateContext& Context);

	TSharedPtr<FUserAnnotationStore> AnnotationStore;

	TArray<FUserAnnotationBoxInfo> AnnotationBoxes;
	TArray<FUserAnnotationTextInfo> AnnotationTexts;
	TArray<FUserAnnotationRangeInfo> AnnotationRanges;
	TArray<FUserAnnotationEventAnchorInfo> EventAnchors;

	FTrackHeader Header;

	uint64 ChangeNumber = 0;

	/** Cached visible-in-viewport count for PreUpdate height animation; -1 = invalid. */
	int32 CachedVisibleInView = -1;
	double CachedViewStartTime = 0.0;
	double CachedViewEndTime = 0.0;
	uint64 CachedVisibleChangeNumber = 0;

	/** Session pointer for resolving target tracks during PostDraw. Managed by extender. */
	UE::Insights::Timing::ITimingViewSession* Session = nullptr;

	/** When false, floating event callout boxes in PostDraw are hidden, but event highlights remain. */
	bool bShowFloatingAnnotations = true;
	bool bShowAnnotationMarkers = true;

	/** Mirrors FPerSessionData::bCanPersist — false when the sidecar folder or file is read-only. */
	bool bCanPersist = true;

	// Slate resources
	const FSlateBrush* WhiteBrush = nullptr;
	const FSlateFontInfo Font;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
