// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserAnnotationsTimingViewExtender.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformProcess.h"
#include "Internationalization/Text.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Misc/StringBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Styling/AppStyle.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceAnalysis
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Frames.h"
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsights
#include "Insights/Common/TraceMetadataFile.h"
#include "Insights/InsightsManager.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/TimingProfiler/Models/UserAnnotation.h"
#include "Insights/TimingProfiler/Models/UserAnnotationStore.h"
#include "Insights/TimingProfiler/Tracks/ThreadTimingTrack.h"
#include "Insights/TimingProfiler/Tracks/UserAnnotationsTimingTrack.h"
#include "Insights/ViewModels/ThreadTrackEvent.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/BaseTimingTrack.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingViewLayout.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::FUserAnnotationsTimingViewExtender"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace
{
	static const TCHAR* AnnotationsSettingsSection = TEXT("Insights.TimingProfiler.Annotations");
	static const TCHAR* AnnotationsSettingsBaseName = TEXT("UnrealInsightsSettings");

	/** Map STimingView's view name → panel "Source" column label. */
	FString GetSourceLabelFromViewName(FName ViewName)
	{
		if (ViewName == FInsightsManagerTabs::TimingProfilerTabId)
		{
			return TEXT("Timing");
		}
		if (ViewName == FInsightsManagerTabs::MemoryProfilerTabId)
		{
			return TEXT("Memory");
		}
		if (ViewName == FInsightsManagerTabs::LoadingProfilerTabId)
		{
			return TEXT("Asset Loading");
		}
		return FString();
	}

	/** HMS format at microsecond precision — matches other Insights panels. */
	FString FormatAnnotationTime(double Seconds)
	{
		return FormatTime(Seconds, FTimeValue::Microsecond);
	}

	/**
	 * Parse a user-entered time string. Accepts:
	 *   - plain seconds:        "42", "42.5", "15981.400659"
	 *   - single unit:          "42s", "500ms", "30m", "2h"
	 *   - multiple units:       "1h 5m 30.5s", "5m30s", "4h26m21.4s"
	 * Units: h (hours), m (minutes), s (seconds), ms (milliseconds), us (microseconds),
	 *        ns (nanoseconds). A trailing bare number is treated as seconds.
	 * Returns false if nothing usable was parsed.
	 */
	bool TryParseAnnotationTime(const FString& Input, double& OutSeconds)
	{
		FString Trimmed = Input;
		Trimmed.TrimStartAndEndInline();
		if (Trimmed.IsEmpty())
		{
			return false;
		}

		const TCHAR* Cur = *Trimmed;
		double Total = 0.0;
		bool bAnyTokenParsed = false;

		while (*Cur)
		{
			while (*Cur == TEXT(' ')) { ++Cur; }
			if (!*Cur) { break; }

			// Parse a number (optional sign, digits, optional dot, digits).
			const TCHAR* NumStart = Cur;
			if (*Cur == TEXT('-') || *Cur == TEXT('+')) { ++Cur; }
			while (FChar::IsDigit(*Cur)) { ++Cur; }
			if (*Cur == TEXT('.'))
			{
				++Cur;
				while (FChar::IsDigit(*Cur)) { ++Cur; }
			}
			if (Cur == NumStart || (Cur - NumStart == 1 && (*NumStart == TEXT('-') || *NumStart == TEXT('+'))))
			{
				return false;
			}
			const FString NumberStr(static_cast<int32>(Cur - NumStart), NumStart);
			const double Number = FCString::Atod(*NumberStr);
			while (*Cur == TEXT(' ')) { ++Cur; }

			// Identify the unit suffix (case-insensitive). "ms" / "us" / "ns" must be checked
			// before "m" / "u" / "n" — and before bare "s" for subseconds.
			double Multiplier = FTimeValue::Second;
			if ((Cur[0] == TEXT('m') || Cur[0] == TEXT('M')) && (Cur[1] == TEXT('s') || Cur[1] == TEXT('S')))
			{
				Multiplier = FTimeValue::Millisecond;
				Cur += 2;
			}
			else if ((Cur[0] == TEXT('u') || Cur[0] == TEXT('U')) && (Cur[1] == TEXT('s') || Cur[1] == TEXT('S')))
			{
				Multiplier = FTimeValue::Microsecond;
				Cur += 2;
			}
			else if ((Cur[0] == TEXT('n') || Cur[0] == TEXT('N')) && (Cur[1] == TEXT('s') || Cur[1] == TEXT('S')))
			{
				Multiplier = FTimeValue::Nanosecond;
				Cur += 2;
			}
			else if (Cur[0] == TEXT('h') || Cur[0] == TEXT('H'))
			{
				Multiplier = FTimeValue::Hour;
				Cur += 1;
			}
			else if (Cur[0] == TEXT('m') || Cur[0] == TEXT('M'))
			{
				Multiplier = FTimeValue::Minute;
				Cur += 1;
			}
			else if (Cur[0] == TEXT('s') || Cur[0] == TEXT('S'))
			{
				Multiplier = FTimeValue::Second;
				Cur += 1;
			}
			// else: no unit suffix — treat as seconds.

			Total += Number * Multiplier;
			bAnyTokenParsed = true;
		}

		if (!bAnyTokenParsed)
		{
			return false;
		}
		if (Total < 0.0)
		{
			return false;
		}
		OutSeconds = Total;
		return true;
	}

	/** Returns UnrealInsightsSettings.ini path, loading once into GConfig. Empty on failure. */
	const FString& GetAnnotationsSettingsIniPath()
	{
		static FString CachedPath;
		static bool bLoadAttempted = false;
		if (!bLoadAttempted)
		{
			bLoadAttempted = true;
			FConfigContext::ReadIntoGConfig().Load(AnnotationsSettingsBaseName, CachedPath);
		}
		return CachedPath;
	}

	bool LoadAnnotationsBool(const TCHAR* Key, bool bDefault)
	{
		bool bValue = bDefault;
		const FString& IniPath = GetAnnotationsSettingsIniPath();
		if (!IniPath.IsEmpty())
		{
			GConfig->GetBool(AnnotationsSettingsSection, Key, bValue, IniPath);
		}
		return bValue;
	}

	void SaveAnnotationsBool(const TCHAR* Key, bool bValue)
	{
		const FString& IniPath = GetAnnotationsSettingsIniPath();
		if (!IniPath.IsEmpty())
		{
			GConfig->SetBool(AnnotationsSettingsSection, Key, bValue, IniPath);
			GConfig->Flush(false, IniPath);
		}
	}
}

void FUserAnnotationsTimingViewExtender::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	FPerSessionData& Data = PerSessionDataMap.Add(&InSession);

	Data.MetadataFile = MakeShared<FTraceMetadataFile>();
	Data.Store = MakeShared<FUserAnnotationStore>(Data.MetadataFile);
	Data.Track = MakeShared<FUserAnnotationsTimingTrack>(Data.Store);
	Data.Track->SetSession(&InSession);

	const bool bTrackVisible = LoadAnnotationsBool(TEXT("bTrackVisible"), true);
	const bool bShowCallouts = LoadAnnotationsBool(TEXT("bShowFloatingCallouts"), true);
	const bool bShowMarkers = LoadAnnotationsBool(TEXT("bShowAnnotationMarkers"), true);
	Data.Track->SetVisibilityFlag(bTrackVisible);
	Data.Track->SetShowFloatingAnnotations(bShowCallouts);
	Data.Track->SetShowAnnotationMarkers(bShowMarkers);
	// Mirror onto the store so observers can check it via the store's weak ptr.
	Data.Store->SetAnnotationsVisible(bTrackVisible);

	InSession.AddTopDockedTrack(Data.Track);

	// Store is created empty here; populated later in EnsureSessionLoaded.
	OnAnnotationStoreChanged.Broadcast();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingViewExtender::OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	FPerSessionData* Data = PerSessionDataMap.Find(&InSession);
	if (Data)
	{
		if (Data->Track.IsValid())
		{
			Data->Track->SetSession(nullptr);
			InSession.RemoveTopDockedTrack(Data->Track);
		}
		PerSessionDataMap.Remove(&InSession);
	}

	// Reset N/P navigation anchor when a session ends so the next trace
	// starts navigation from its viewport center rather than the prior session.
	LastNavigatedTime = -1.0;

	// Notify subscribers to re-query; prevents stale pins on trace-switch in multi-window.
	OnAnnotationStoreChanged.Broadcast();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingViewExtender::Tick(const UE::Insights::Timing::FTimingViewExtenderTickParams& Params)
{
	FPerSessionData* Data = PerSessionDataMap.Find(&Params.Session);
	if (!Data || Data->bLoaded)
	{
		return;
	}
	EnsureSessionLoaded(*Data);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingViewExtender::EnsureAllSessionsLoaded()
{
	// Works around STimingView's extender-Tick gate (`if (Session)`) which skips early
	// frames on Memory/Asset Loading windows. Panel drives the load so its list populates
	// without needing Timing Insights clicked first.
	for (auto& Pair : PerSessionDataMap)
	{
		if (!Pair.Value.bLoaded)
		{
			EnsureSessionLoaded(Pair.Value);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingViewExtender::EnsureSessionLoaded(FPerSessionData& InData)
{
	if (InData.bLoaded)
	{
		return;
	}

	// Wait until we have a valid analysis session and trace name
	TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();
	if (!InsightsManager.IsValid())
	{
		return;
	}
	FPerSessionData* Data = &InData;

	const FString& TraceName = InsightsManager->GetTraceName();
	if (TraceName.IsEmpty())
	{
		return;
	}

	// Derive sidecar path from the trace name.
	// For TraceFile and TraceStore, CurrentTraceName is the full .utrace file path.
	// For live/direct traces, it's a descriptive string (not a file path).
	FString SidecarPath;

	const ETraceStreamType StreamType = InsightsManager->GetTraceStreamType();
	if (StreamType == ETraceStreamType::TraceFile || StreamType == ETraceStreamType::TraceStore)
	{
		SidecarPath = FTraceMetadataFile::GetSidecarPath(TraceName);
	}

	if (SidecarPath.IsEmpty())
	{
		// No sidecar path derivable — the trace name isn't a file path (e.g. live/direct
		// stream without a local file, remote UTS descriptor). Mark loaded so we don't
		// keep retrying; annotation UI is already gated out elsewhere via bCanPersist.
		Data->bLoaded = true;
		return;
	}

	// If another active session already loaded the same sidecar (user has the same trace
	// open in Timing + Memory + Asset Loading Insights), reuse its store so all three
	// windows see the same annotations in real time — no reload round-trip.
	for (auto& Pair : PerSessionDataMap)
	{
		FPerSessionData& Other = Pair.Value;
		if (&Other == Data)
		{
			continue;
		}
		if (Other.bLoaded && Other.SidecarFilePath == SidecarPath && Other.Store.IsValid())
		{
			Data->MetadataFile = Other.MetadataFile;
			Data->Store = Other.Store;
			Data->SidecarFilePath = SidecarPath;
			Data->bCanPersist = Other.bCanPersist;
			Data->bLoaded = true;
			if (Data->Track.IsValid())
			{
				Data->Track->SetStore(Data->Store);
				Data->Track->SetCanPersist(Data->bCanPersist);
			}
			// Store pointer was swapped to a sibling's — re-broadcast so subscribers rebind.
			OnAnnotationStoreChanged.Broadcast();
			return;
		}
	}

	Data->SidecarFilePath = SidecarPath;
	Data->MetadataFile->SetFilePath(SidecarPath);
	if (Data->MetadataFile->Load())
	{
		Data->Store->LoadFromMetadata();
	}
	// Mark loaded regardless of Load() result — if the sidecar is missing/corrupt the
	// store stays empty but we must not re-probe the folder every frame.
	Data->bLoaded = true;

	// bCanPersist = (file-RO-flag clear) AND (folder accepts new files via probe).
	// Probe catches ACL-denied folders / RO mounts that the file flag alone misses.
	const bool bFileExists = IFileManager::Get().FileExists(*SidecarPath);
	bool bCanWrite = !(bFileExists && IFileManager::Get().IsReadOnly(*SidecarPath));
	if (bCanWrite)
	{
		const FString ProbePath = FPaths::GetPath(SidecarPath) / FGuid::NewGuid().ToString() + TEXT(".probe");
		TUniquePtr<FArchive> ProbeWriter(IFileManager::Get().CreateFileWriter(*ProbePath));
		if (ProbeWriter.IsValid())
		{
			ProbeWriter.Reset();
			IFileManager::Get().Delete(*ProbePath, /*RequireExists*/ false, /*EvenReadOnly*/ false, /*Quiet*/ true);
		}
		else
		{
			bCanWrite = false;
		}
	}
	Data->bCanPersist = bCanWrite;

	if (Data->Track.IsValid())
	{
		Data->Track->SetCanPersist(bCanWrite);

		// One-time auto-hide on read-only sidecars with nothing to show. If the user re-enables
		// the track via the Other menu, the track itself renders a "disabled" banner so the
		// read-only state is obvious instead of silently suppressed.
		const bool bStoreEmpty = Data->Store.IsValid() && Data->Store->GetAllAnnotations().Num() == 0;
		if (!bCanWrite && bStoreEmpty && !Data->bAutoHiddenForReadOnly)
		{
			Data->Track->SetVisibilityFlag(false);
			Data->bAutoHiddenForReadOnly = true;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingViewExtender::AddAnnotationVisibilityToggles(
	FMenuBuilder& InMenuBuilder,
	UE::Insights::Timing::ITimingViewSession& InSession,
	FPerSessionData& InData)
{
	TWeakPtr<FUserAnnotationsTimingTrack> WeakTrack = InData.Track;
	TSharedPtr<FUserAnnotationStore> SharedStore = InData.Store;
	FUserAnnotationsTimingViewExtender* Self = this;
	UE::Insights::Timing::ITimingViewSession* SessionPtr = &InSession;

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("ShowAnnotationsTrack", "Show Annotations Track"),
		LOCTEXT("ShowAnnotationsTrack_Desc", "Show or hide the annotations track (persistent)"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([Self, SessionPtr]()
			{
				Self->ToggleAnnotationsTrackVisible(*SessionPtr);
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([WeakTrack]()
			{
				TSharedPtr<FUserAnnotationsTimingTrack> Track = WeakTrack.Pin();
				return Track.IsValid() && Track->IsVisible();
			})
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("ShowFloatingCallouts", "Show Floating Callouts"),
		LOCTEXT("ShowFloatingCallouts_Desc",
			"Show or hide floating annotation callouts on events (persistent). "
			"Disabled when the annotations track is hidden or there are no annotations."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([Self]()
			{
				Self->ToggleFloatingAnnotations();
			}),
			FCanExecuteAction::CreateLambda([WeakTrack, SharedStore]()
			{
				TSharedPtr<FUserAnnotationsTimingTrack> Track = WeakTrack.Pin();
				if (!Track.IsValid() || !Track->IsVisible())
				{
					return false;
				}
				return SharedStore.IsValid() && SharedStore->GetAllAnnotations().Num() > 0;
			}),
			FIsActionChecked::CreateLambda([Self]()
			{
				return Self->GetShowFloatingAnnotations();
			})
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("ShowAnnotationMarkers", "Show Annotation Markers"),
		LOCTEXT("ShowAnnotationMarkers_Desc",
			"Show or hide annotation markers on the timing tracks: range bands, point "
			"indicator lines, and event highlights (persistent). "
			"Disabled when the annotations track is hidden or there are no annotations."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([Self]()
			{
				Self->ToggleAnnotationMarkers();
			}),
			FCanExecuteAction::CreateLambda([WeakTrack, SharedStore]()
			{
				TSharedPtr<FUserAnnotationsTimingTrack> Track = WeakTrack.Pin();
				if (!Track.IsValid() || !Track->IsVisible())
				{
					return false;
				}
				return SharedStore.IsValid() && SharedStore->GetAllAnnotations().Num() > 0;
			}),
			FIsActionChecked::CreateLambda([Self]()
			{
				return Self->GetShowAnnotationMarkers();
			})
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FUserAnnotationsTimingViewExtender::ExtendGlobalContextMenu(
	UE::Insights::Timing::ITimingViewSession& InSession,
	FMenuBuilder& InMenuBuilder)
{
	FPerSessionData* Data = PerSessionDataMap.Find(&InSession);
	if (!Data || !Data->bLoaded || Data->SidecarFilePath.IsEmpty() || !Data->bCanPersist)
	{
		// No context menu entries when the trace has no writable sidecar (live/direct trace,
		// remote UTS without a local file, or a read-only .ini). Features are disabled entirely.
		return false;
	}

	// ITimingViewSession is implemented by STimingView.
	STimingView& TimingView = static_cast<STimingView&>(InSession);
	const double MousePosTime = TimingView.GetViewport().SlateUnitsToTime(
		static_cast<float>(TimingView.GetMousePosition().X));

	bool bAddedEntries = false;
	TSharedPtr<FUserAnnotationStore> SharedStore = Data->Store;

	const double SelStart = TimingView.GetSelectionStartTime();
	const double SelEnd = TimingView.GetSelectionEndTime();
	const bool bHasSelection = (SelEnd > SelStart);

	const TSharedPtr<const ITimingEvent> HoveredEvent = TimingView.GetHoveredEvent();
	// Accept any hovered timing event — CPU thread events, load-time events (Asset Loading),
	// allocation events (Memory Insights). Event anchoring works on any ITimingEvent.
	const bool bHoveredTimingEvent = HoveredEvent.IsValid();

	// Track visibility gates all add-annotation entries: when the track is hidden
	// via the Other menu, creating annotations would be invisible busywork.
	const bool bTrackVisible = Data->Track.IsValid() && Data->Track->IsVisible();
	const FText TrackHiddenReason = LOCTEXT("AddAnnotation_TrackHidden",
		"Enable the Annotations track via the Other menu to add annotations.");

	InMenuBuilder.BeginSection(TEXT("Annotations"), LOCTEXT("Annotations", "Annotations"));
	{
		// "Add Time Annotation Here..." (only when there is no selection)
		if (!bHasSelection)
		{
			FAnnotationContext PointCtx = BuildPointContext(TimingView, MousePosTime);
			InMenuBuilder.AddMenuEntry(
				LOCTEXT("AddTimeAnnotation", "Add Time Annotation Here..."),
				bTrackVisible
					? LOCTEXT("AddTimeAnnotation_Desc", "Add a text annotation at this time position")
					: TrackHiddenReason,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, SharedStore, PointCtx]()
					{
						ShowAnnotationDialog(SharedStore, PointCtx);
					}),
					FCanExecuteAction::CreateLambda([bTrackVisible]()
					{
						return bTrackVisible;
					})
				)
			);
		}
		bAddedEntries = true;

		// "Add Range Annotation..." (only enabled when there is a selection + track visible)
		{
			FAnnotationContext RangeCtx;
			if (bHasSelection)
			{
				RangeCtx = BuildRangeContext(TimingView, SelStart, SelEnd);
			}

			InMenuBuilder.AddMenuEntry(
				LOCTEXT("AddRangeAnnotation", "Add Range Annotation..."),
				!bTrackVisible
					? TrackHiddenReason
					: (bHasSelection
						? LOCTEXT("AddRangeAnnotation_Desc", "Add a highlighted range annotation over the current selection")
						: LOCTEXT("AddRangeAnnotation_NoSel", "Select a time range first to add a range annotation")),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, SharedStore, RangeCtx]()
					{
						ShowAnnotationDialog(SharedStore, RangeCtx);
					}),
					FCanExecuteAction::CreateLambda([bHasSelection, bTrackVisible]()
					{
						return bHasSelection && bTrackVisible;
					})
				)
			);
		}

		// "Annotate This Event..." (only when hovering a thread timing event)
		{
			FAnnotationContext EventCtx;
			if (bHoveredTimingEvent)
			{
				EventCtx = BuildEventContext(TimingView, *HoveredEvent);
			}

			const FText EventMenuLabel = bHoveredTimingEvent && !EventCtx.TimerName.IsEmpty()
				? FText::Format(LOCTEXT("AddEventAnnotationFmt", "Add Event Annotation \"{0}\"..."), FText::FromString(EventCtx.TimerName))
				: LOCTEXT("AddEventAnnotation", "Add Event Annotation...");

			InMenuBuilder.AddMenuEntry(
				EventMenuLabel,
				!bTrackVisible
					? TrackHiddenReason
					: (bHoveredTimingEvent
						? LOCTEXT("AddEventAnnotation_Desc", "Add an annotation anchored to this function call")
						: LOCTEXT("AddEventAnnotation_NoEvent", "Hover over a timing event to annotate it")),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, SharedStore, EventCtx]()
					{
						ShowAnnotationDialog(SharedStore, EventCtx);
					}),
					FCanExecuteAction::CreateLambda([bHoveredTimingEvent, bTrackVisible]()
					{
						return bHoveredTimingEvent && bTrackVisible;
					})
				)
			);
		}

		// If an annotation is hovered, add edit/delete/copy entries
		if (HoveredEvent.IsValid() && HoveredEvent->Is<FUserAnnotationTimingEvent>())
		{
			const FUserAnnotationTimingEvent& AnnotationEvent = HoveredEvent->As<FUserAnnotationTimingEvent>();
			const FGuid AnnotationId = AnnotationEvent.GetAnnotationId();

			InMenuBuilder.AddMenuEntry(
				LOCTEXT("EditAnnotation", "Edit Annotation..."),
				LOCTEXT("EditAnnotation_Desc", "Edit the text of this annotation"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this, SharedStore, AnnotationId]()
				{
					const FUserAnnotation* Annotation = SharedStore->FindAnnotation(AnnotationId);
					if (Annotation)
					{
						FAnnotationContext EditCtx = FAnnotationContext::FromAnnotation(*Annotation);
						ShowAnnotationDialog(SharedStore, EditCtx, &AnnotationId);
					}
				}))
			);

			// Capture the session POINTER (not a reference) and re-validate against the
			// PerSessionDataMap when the lambda fires. Slate menu actions can execute after
			// the menu frame returns; if the trace closed in between, the &TimingView
			// reference would dangle.
			UE::Insights::Timing::ITimingViewSession* SessionPtr = &InSession;

			InMenuBuilder.AddMenuEntry(
				LOCTEXT("DeleteAnnotation", "Delete Annotation"),
				LOCTEXT("DeleteAnnotation_Desc", "Remove this annotation"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this, SharedStore, AnnotationId, SessionPtr]()
				{
					// Snapshot at execute time so a span edit between menu-open and click can't stale-coord the highlight.
					double RemovedSelStart = 0.0;
					double RemovedSelEnd = 0.0;
					if (const FUserAnnotation* ToRemove = SharedStore->FindAnnotation(AnnotationId))
					{
						if (ToRemove->HasEventAnchor())
						{
							RemovedSelStart = ToRemove->EventStartTime;
							RemovedSelEnd = ToRemove->EventEndTime;
						}
						else if (ToRemove->IsRange())
						{
							RemovedSelStart = ToRemove->Time;
							RemovedSelEnd = ToRemove->EndTime;
						}
					}
					if (!SharedStore->RemoveAnnotation(AnnotationId))
					{
						FNotificationInfo Info(LOCTEXT("AnnotationDeleteFailed",
							"Failed to delete annotation. The sidecar .ini file may be read-only."));
						Info.ExpireDuration = 5.0f;
						TSharedPtr<SNotificationItem> Item = FSlateNotificationManager::Get().AddNotification(Info);
						if (Item.IsValid())
						{
							Item->SetCompletionState(SNotificationItem::CS_Fail);
						}
						return;
					}
					// Clear the ruler highlight only if it matches the deleted annotation's span.
					if (PerSessionDataMap.Contains(SessionPtr) && RemovedSelEnd > RemovedSelStart)
					{
						STimingView* TV = static_cast<STimingView*>(SessionPtr);
						const double Epsilon = 1e-6;
						if (FMath::IsNearlyEqual(TV->GetSelectionStartTime(), RemovedSelStart, Epsilon)
							&& FMath::IsNearlyEqual(TV->GetSelectionEndTime(), RemovedSelEnd, Epsilon))
						{
							TV->SelectTimeInterval(0.0, 0.0);
						}
					}
				}))
			);

			InMenuBuilder.AddMenuEntry(
				LOCTEXT("CopyAnnotationText", "Copy Text"),
				LOCTEXT("CopyAnnotationText_Desc", "Copy the annotation text to the clipboard"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([SharedStore, AnnotationId]()
				{
					const FUserAnnotation* Annotation = SharedStore->FindAnnotation(AnnotationId);
					if (Annotation)
					{
						FPlatformApplicationMisc::ClipboardCopy(*Annotation->Text);
					}
				}))
			);

			InMenuBuilder.AddMenuEntry(
				LOCTEXT("CopyAnnotationDescription", "Copy Description"),
				LOCTEXT("CopyAnnotationDescription_Desc", "Copy the annotation description to the clipboard"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([SharedStore, AnnotationId]()
					{
						const FUserAnnotation* Annotation = SharedStore->FindAnnotation(AnnotationId);
						if (Annotation)
						{
							FPlatformApplicationMisc::ClipboardCopy(*Annotation->Description);
						}
					}),
					FCanExecuteAction::CreateLambda([SharedStore, AnnotationId]()
					{
						const FUserAnnotation* Annotation = SharedStore->FindAnnotation(AnnotationId);
						return Annotation && !Annotation->Description.IsEmpty();
					})
				)
			);

			InMenuBuilder.AddMenuEntry(
				LOCTEXT("CopyAnnotationAll", "Copy Text + Description"),
				LOCTEXT("CopyAnnotationAll_Desc", "Copy both the annotation text and description to the clipboard"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([SharedStore, AnnotationId]()
				{
					const FUserAnnotation* Annotation = SharedStore->FindAnnotation(AnnotationId);
					if (Annotation)
					{
						FString Combined = Annotation->Text;
						if (!Annotation->Description.IsEmpty())
						{
							Combined += TEXT("\n") + Annotation->Description;
						}
						FPlatformApplicationMisc::ClipboardCopy(*Combined);
					}
				}))
			);
		}
		// Visibility toggles (Annotations Track / Floating Callouts / Annotation Markers),
		// shared with the Other Tracks filter menu via AddAnnotationVisibilityToggles.
		AddAnnotationVisibilityToggles(InMenuBuilder, InSession, *Data);

		// Top-level toggle: show/hide all annotations in this session.
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("CtxShowAllAnnotations", "Show All Annotations"),
			LOCTEXT("CtxShowAllAnnotations_Desc",
				"Show or hide every annotation in this trace"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, SharedStore]()
				{
					ToggleAllAnnotationVisibility(SharedStore);
				}),
				FCanExecuteAction::CreateLambda([SharedStore]()
				{
					return SharedStore.IsValid() && SharedStore->GetAllAnnotations().Num() > 0;
				}),
				FIsActionChecked::CreateLambda([SharedStore]()
				{
					if (!SharedStore.IsValid())
					{
						return false;
					}
					for (const FUserAnnotation& A : SharedStore->GetAllAnnotations())
					{
						if (A.bVisible) return true;
					}
					return false;
				})
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	InMenuBuilder.EndSection();

	return bAddedEntries;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FUserAnnotationStore> FUserAnnotationsTimingViewExtender::GetAnnotationStoreForCurrentSession() const
{
	// Return the store from the most recently added session (there is typically only one).
	for (const auto& Pair : PerSessionDataMap)
	{
		if (Pair.Value.Store.IsValid())
		{
			return Pair.Value.Store;
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FUserAnnotationsTimingViewExtender::IsAnyCurrentSessionReadOnly() const
{
	for (const auto& Pair : PerSessionDataMap)
	{
		if (Pair.Value.bLoaded && !Pair.Value.SidecarFilePath.IsEmpty() && !Pair.Value.bCanPersist)
		{
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingViewExtender::ExtendOtherTracksFilterMenu(
	UE::Insights::Timing::ITimingViewSession& InSession,
	FMenuBuilder& InMenuBuilder)
{
	FPerSessionData* Data = PerSessionDataMap.Find(&InSession);
	if (!Data || !Data->Track.IsValid())
	{
		return;
	}

	InMenuBuilder.BeginSection("Annotations", LOCTEXT("ContextMenu_Section_Annotations", "Annotations"));
	{
		AddAnnotationVisibilityToggles(InMenuBuilder, InSession, *Data);
	}
	InMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double FUserAnnotationsTimingViewExtender::Snap(
	UE::Insights::Timing::ITimingViewSession& InSession,
	double Time, double SnapTolerance) const
{
	const FPerSessionData* Data = PerSessionDataMap.Find(&InSession);
	if (Data && Data->Track.IsValid() && Data->Track->IsVisible())
	{
		return Data->Track->Snap(Time, SnapTolerance);
	}
	return Time;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingViewExtender::ShowAnnotationDialog(
	TSharedPtr<FUserAnnotationStore> InStore,
	const FAnnotationContext& Context,
	const FGuid* ExistingAnnotationId)
{
	if (!InStore.IsValid())
	{
		return;
	}

	const bool bIsEditing = (ExistingAnnotationId != nullptr);
	const FUserAnnotation* ExistingAnnotation = nullptr;

	if (bIsEditing)
	{
		ExistingAnnotation = InStore->FindAnnotation(*ExistingAnnotationId);
		if (!ExistingAnnotation)
		{
			return;
		}
	}

	const bool bIsRange = (Context.EndTime > Context.Time);
	const bool bHasEventAnchor = !Context.TimerName.IsEmpty() && Context.EventEndTime > Context.EventStartTime;

	const FText DialogTitle = bIsEditing
		? (bHasEventAnchor ? LOCTEXT("EditEventAnnotationTitle", "Edit Event Annotation")
			: bIsRange ? LOCTEXT("EditRangeAnnotationTitle", "Edit Range Annotation") : LOCTEXT("EditAnnotationTitle", "Edit Time Annotation"))
		: (bHasEventAnchor ? LOCTEXT("AddEventAnnotationTitle", "Add Event Annotation")
			: bIsRange ? LOCTEXT("AddRangeAnnotationTitle", "Add Range Annotation") : LOCTEXT("AddAnnotationTitle", "Add Time Annotation"));

	// Generate a sensible default text for new annotations
	FString DefaultText;
	if (!bIsEditing)
	{
		if (bHasEventAnchor)
		{
			DefaultText = Context.TimerName;
		}
		else if (bIsRange)
		{
			DefaultText = FString::Printf(TEXT("Range G:%u\u2014G:%u"), Context.GameFrameNumber, Context.GameFrameNumberEnd);
		}
		else
		{
			DefaultText = FString::Printf(TEXT("Annotation @ G:%u"), Context.GameFrameNumber);
		}
	}

	// Per-axis range checks drive both display and dialog height; a frame line is "range" iff start != end.
	const bool bGameFrameIsRange = Context.GameFrameNumber != Context.GameFrameNumberEnd;
	const bool bRenderFrameIsRange = Context.RenderFrameNumber != Context.RenderFrameNumberEnd;
	const bool bHasTrackLine = !Context.ThreadName.IsEmpty();

	// Sized tight for the normal case; the dialog grows by ErrorBannerExtraHeight when the error banner appears.
	// FrameInfoLine is now newline-separated (game frame line, render frame line, optional track line) so
	// reserve ~16px per line beyond the first.
	int32 DialogHeight = 225;
	if (bHasEventAnchor)
	{
		DialogHeight += 20;
	}
	else
	{
		DialogHeight += bIsRange ? 40 : 10;
	}
	const int32 FrameInfoLineHeight = 16;
	DialogHeight += FrameInfoLineHeight; // second frame line (render) is always shown
	if (bHasTrackLine)
	{
		DialogHeight += FrameInfoLineHeight;
	}
	const int32 ErrorBannerExtraHeight = 28;
	TSharedRef<SWindow> DialogWindow = SNew(SWindow)
		.Title(DialogTitle)
		.ClientSize(FVector2D(600, DialogHeight))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.SizingRule(ESizingRule::FixedSize);

	TSharedPtr<SEditableTextBox> TextInput;
	TSharedPtr<SMultiLineEditableTextBox> DescriptionInput;
	TSharedPtr<SButton> OkButton;
	TSharedPtr<STextBlock> ErrorLabel;
	// Heap-allocate SelectedColor so the dialog lambdas can capture it by shared pointer instead
	// of by reference to a stack local. AddModalWindow keeps the stack frame alive today, but
	// relying on that is fragile for any future non-modal refactor.
	// For new annotations, default to Context.SuggestedColor (derived from the anchored event's
	// color if it's an event annotation, otherwise White).
	TSharedRef<FLinearColor> SelectedColor = MakeShared<FLinearColor>(bIsEditing && ExistingAnnotation
		? ExistingAnnotation->Color
		: Context.SuggestedColor);
	// Edited timestamps for point + range annotations. Event annotations keep
	// Context.Time/EndTime (driven by the anchored event). Initialized from the
	// existing annotation when editing, from Context.Time/EndTime when creating.
	TSharedRef<double> EditedTime = MakeShared<double>(
		bIsEditing && ExistingAnnotation ? ExistingAnnotation->Time : Context.Time);
	TSharedRef<double> EditedEndTime = MakeShared<double>(
		bIsEditing && ExistingAnnotation ? ExistingAnnotation->EndTime : Context.EndTime);
	FGuid EditAnnotationId = bIsEditing ? *ExistingAnnotationId : FGuid();
	TWeakPtr<FUserAnnotationStore> WeakStore = InStore;
	TWeakPtr<SWindow> WeakWindow = DialogWindow;

	// Shared accept handler — invoked by both the OK button and the Enter-to-commit shortcut.
	// Heap-allocated so call sites capture by-value (TSharedRef) instead of by-reference to a
	// stack local, matching the policy used for SelectedColor/EditedTime above. Populated below
	// after SetContent has assigned TextInput/DescriptionInput/ErrorLabel.
	TSharedRef<TFunction<FReply()>> AcceptCallback = MakeShared<TFunction<FReply()>>();

	// Frame-numbers + thread line stays read-only; only the time/range values are editable
	// (and only for non-event annotations — event-anchored timestamps are tied to the source
	// event so editing them would desync the annotation).
	TStringBuilder<1024> FrameInfoBuilder;
	if (bGameFrameIsRange)
	{
		FrameInfoBuilder.Appendf(TEXT("Game Frames: %u \u2014 %u"), Context.GameFrameNumber, Context.GameFrameNumberEnd);
	}
	else
	{
		FrameInfoBuilder.Appendf(TEXT("Game Frame: %u"), Context.GameFrameNumber);
	}
	FrameInfoBuilder.Append(TEXT("\n"));
	if (bRenderFrameIsRange)
	{
		FrameInfoBuilder.Appendf(TEXT("Render Frames: %u \u2014 %u"), Context.RenderFrameNumber, Context.RenderFrameNumberEnd);
	}
	else
	{
		FrameInfoBuilder.Appendf(TEXT("Render Frame: %u"), Context.RenderFrameNumber);
	}
	if (bHasTrackLine)
	{
		FrameInfoBuilder.Append(TEXT("\nTrack: "));
		FrameInfoBuilder.Append(Context.ThreadName);
	}
	const FString FrameInfoLine(FrameInfoBuilder);

	TSharedRef<SVerticalBox> TimeRow = SNew(SVerticalBox);
	if (bHasEventAnchor)
	{
		// Tint the timer-name row with the per-event color so the dialog visually echoes the track.
		const double EventDuration = Context.EventEndTime - Context.EventStartTime;
		const FString EventHeaderSuffix = FString::Printf(TEXT("  (depth: %u)"), Context.EventDepth);
		const FString EventTimeLine = FString::Printf(TEXT("%.6f s — %.6f s  (duration: %.6f s)"),
			Context.EventStartTime, Context.EventEndTime, EventDuration);
		TimeRow->AddSlot().AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(LOCTEXT("EventNamePrefix", "Event: "))
			]
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Context.TimerName))
				.ColorAndOpacity_Lambda([SelectedColor]() { return FSlateColor(*SelectedColor); })
			]
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(FText::FromString(EventHeaderSuffix))
			]
		];
		TimeRow->AddSlot().AutoHeight().Padding(0.0f, 2.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock).Text(FText::FromString(EventTimeLine))
		];
	}
	else if (bIsRange)
	{
		// Editable Start + End in human-readable form ("1h 5m 30.5s"). Display is
		// regenerated from the underlying double on every frame so pasting in plain
		// seconds rounds to the formatted form after commit.
		TimeRow->AddSlot().AutoHeight().Padding(0.0f, 0.0f, 0.0f, 2.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center).Padding(0.0f, 0.0f, 6.0f, 0.0f)
			[
				SNew(STextBlock).Text(LOCTEXT("RangeStartLabel", "Start:"))
			]
			+ SHorizontalBox::Slot().FillWidth(1.0f)
			[
				SNew(SEditableTextBox)
				.Text_Lambda([EditedTime]() { return FText::FromString(FormatAnnotationTime(*EditedTime)); })
				.OnTextCommitted_Lambda([EditedTime](const FText& InText, ETextCommit::Type)
				{
					double Parsed = 0.0;
					if (TryParseAnnotationTime(InText.ToString(), Parsed))
					{
						*EditedTime = Parsed;
					}
				})
			]
		];
		TimeRow->AddSlot().AutoHeight().Padding(0.0f, 2.0f, 0.0f, 2.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center).Padding(0.0f, 0.0f, 6.0f, 0.0f)
			[
				SNew(STextBlock).Text(LOCTEXT("RangeEndLabel", "End:"))
			]
			+ SHorizontalBox::Slot().FillWidth(1.0f)
			[
				SNew(SEditableTextBox)
				.Text_Lambda([EditedEndTime]() { return FText::FromString(FormatAnnotationTime(*EditedEndTime)); })
				.OnTextCommitted_Lambda([EditedEndTime](const FText& InText, ETextCommit::Type)
				{
					double Parsed = 0.0;
					if (TryParseAnnotationTime(InText.ToString(), Parsed))
					{
						*EditedEndTime = Parsed;
					}
				})
			]
		];
	}
	else
	{
		// Point: single editable Time in human-readable form.
		TimeRow->AddSlot().AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center).Padding(0.0f, 0.0f, 6.0f, 0.0f)
			[
				SNew(STextBlock).Text(LOCTEXT("TimeLabel", "Time:"))
			]
			+ SHorizontalBox::Slot().FillWidth(1.0f)
			[
				SNew(SEditableTextBox)
				.Text_Lambda([EditedTime]() { return FText::FromString(FormatAnnotationTime(*EditedTime)); })
				.OnTextCommitted_Lambda([EditedTime](const FText& InText, ETextCommit::Type)
				{
					double Parsed = 0.0;
					if (TryParseAnnotationTime(InText.ToString(), Parsed))
					{
						*EditedTime = Parsed;
					}
				})
			]
		];
	}
	TimeRow->AddSlot().AutoHeight().Padding(0.0f, 4.0f, 0.0f, 0.0f)
	[
		SNew(STextBlock).Text(FText::FromString(FrameInfoLine))
	];

	DialogWindow->SetContent(
		SNew(SVerticalBox)

		// Time / range editor + frame numbers + thread
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.0f, 8.0f, 8.0f, 4.0f)
		[
			TimeRow
		]

		// Text input
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.0f, 4.0f, 8.0f, 4.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(bHasEventAnchor ? LOCTEXT("EventAnnotationTextLabel", "Event Annotation Text:")
						: bIsRange ? LOCTEXT("RangeAnnotationTextLabel", "Range Annotation Text:")
						: LOCTEXT("AnnotationTextLabel", "Time Annotation Text:"))
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 2.0f, 0.0f, 0.0f)
			[
				SAssignNew(TextInput, SEditableTextBox)
				.Text(FText::FromString(bIsEditing && ExistingAnnotation ? ExistingAnnotation->Text : DefaultText))
				.SelectAllTextWhenFocused(true)
				.OnTextCommitted_Lambda([AcceptCallback](const FText& /*Text*/, ETextCommit::Type CommitType)
				{
					if (CommitType == ETextCommit::OnEnter && (*AcceptCallback))
					{
						(*AcceptCallback)();
					}
				})
			]
		]

		// Description input (optional multi-line)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.0f, 2.0f, 8.0f, 2.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DescriptionLabel", "Description (optional):"))
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 2.0f, 0.0f, 0.0f)
			[
				SNew(SBox)
				.MinDesiredHeight(36.0f)
				.MaxDesiredHeight(80.0f)
				[
					SAssignNew(DescriptionInput, SMultiLineEditableTextBox)
					.Text(FText::FromString(bIsEditing && ExistingAnnotation ? ExistingAnnotation->Description : FString()))
					.AutoWrapText(true)
					.AllowMultiLine(true)
				]
			]
		]

		// Color picker
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.0f, 4.0f, 8.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ColorLabel", "Color"))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(24.0f)
				.HeightOverride(24.0f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
					.BorderBackgroundColor_Lambda([SelectedColor]() { return FSlateColor(*SelectedColor); })
					.OnMouseButtonDown_Lambda([SelectedColor, WeakWindow](const FGeometry&, const FPointerEvent&) -> FReply
					{
						FColorPickerArgs Args;
						Args.InitialColor = *SelectedColor;
						Args.bUseAlpha = false;
						Args.bIsModal = true;
						Args.ParentWidget = WeakWindow.Pin();
						Args.OnColorCommitted = FOnLinearColorValueChanged::CreateLambda(
							[SelectedColor](FLinearColor NewColor) { *SelectedColor = NewColor; });
						OpenColorPicker(Args);
						return FReply::Handled();
					})
				]
			]
		]

		// Error label (hidden by default)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.0f, 8.0f, 8.0f, 0.0f)
		[
			SAssignNew(ErrorLabel, STextBlock)
			.Text(LOCTEXT("AnnotationSaveError", "Failed to save annotation. The file may be read-only."))
			.ColorAndOpacity(FLinearColor::Red)
			.Visibility(EVisibility::Collapsed)
		]

		// OK / Cancel buttons
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(8.0f, 12.0f, 8.0f, 16.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SAssignNew(OkButton, SButton)
				.Text(LOCTEXT("OK", "OK"))
				.OnClicked_Lambda([AcceptCallback]() -> FReply { return (*AcceptCallback) ? (*AcceptCallback)() : FReply::Handled(); })
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("Cancel", "Cancel"))
				.OnClicked_Lambda([WeakWindow]() -> FReply
				{
					TSharedPtr<SWindow> Window = WeakWindow.Pin();
					if (Window.IsValid())
					{
						FSlateApplication::Get().RequestDestroyWindow(Window.ToSharedRef());
					}
					return FReply::Handled();
				})
			]
		]
	);

	// SAssignNew has now populated TextInput/DescriptionInput/ErrorLabel — capture by-value.
	*AcceptCallback = [WeakWindow, TextInput, DescriptionInput, ErrorLabel, WeakStore, Context, bIsEditing, EditAnnotationId, SelectedColor, EditedTime, EditedEndTime, bIsRange, bHasEventAnchor, DialogHeight, ErrorBannerExtraHeight]() -> FReply
	{
		TSharedPtr<FUserAnnotationStore> Store = WeakStore.Pin();
		if (Store.IsValid() && TextInput.IsValid())
		{
			const FString Text = TextInput->GetText().ToString();
			const FString Description = DescriptionInput.IsValid() ? DescriptionInput->GetText().ToString() : FString();
			if (!Text.IsEmpty())
			{
				// Event annotations stay locked to their source-event span; others use edited values.
				double EffectiveTime = bHasEventAnchor ? Context.Time : *EditedTime;
				double EffectiveEndTime = bHasEventAnchor ? Context.EndTime : *EditedEndTime;
				if (bIsRange && !bHasEventAnchor && EffectiveEndTime <= EffectiveTime)
				{
					if (ErrorLabel.IsValid())
					{
						ErrorLabel->SetText(LOCTEXT("RangeInvalidEndBeforeStart",
							"Range end time must be greater than start time."));
						if (ErrorLabel->GetVisibility() == EVisibility::Collapsed)
						{
							ErrorLabel->SetVisibility(EVisibility::Visible);
							TSharedPtr<SWindow> ErrorWindow = WeakWindow.Pin();
							if (ErrorWindow.IsValid())
							{
								ErrorWindow->Resize(FVector2D(600, DialogHeight + ErrorBannerExtraHeight));
							}
						}
					}
					return FReply::Handled();
				}

				// Recompute frame numbers from edited time values (skip for event
				// annotations — their frame numbers still match the source event).
				uint32 GameFrameStart = Context.GameFrameNumber;
				uint32 RenderFrameStart = Context.RenderFrameNumber;
				uint32 GameFrameEnd = Context.GameFrameNumberEnd;
				uint32 RenderFrameEnd = Context.RenderFrameNumberEnd;
				if (!bHasEventAnchor)
				{
					PopulateFrameNumbers(EffectiveTime, GameFrameStart, RenderFrameStart);
					if (bIsRange)
					{
						PopulateFrameNumbers(EffectiveEndTime, GameFrameEnd, RenderFrameEnd);
					}
				}

				bool bSuccess = false;
				if (bIsEditing)
				{
					const FUserAnnotation* Existing = Store->FindAnnotation(EditAnnotationId);
					if (Existing)
					{
						FUserAnnotation Updated = *Existing;
						Updated.Text = Text;
						Updated.Description = Description;
						Updated.Color = *SelectedColor;
						if (!bHasEventAnchor)
						{
							Updated.Time = EffectiveTime;
							Updated.EndTime = EffectiveEndTime;
							Updated.GameFrameNumber = GameFrameStart;
							Updated.RenderFrameNumber = RenderFrameStart;
							Updated.GameFrameNumberEnd = GameFrameEnd;
							Updated.RenderFrameNumberEnd = RenderFrameEnd;
						}
						Updated.ModifiedAt = FDateTime::UtcNow();
						bSuccess = Store->UpdateAnnotation(Updated);
					}
				}
				else
				{
					FUserAnnotation NewAnnotation;
					NewAnnotation.Id = FGuid::NewGuid();
					NewAnnotation.Time = EffectiveTime;
					NewAnnotation.EndTime = EffectiveEndTime;
					NewAnnotation.GameFrameNumber = GameFrameStart;
					NewAnnotation.RenderFrameNumber = RenderFrameStart;
					NewAnnotation.GameFrameNumberEnd = GameFrameEnd;
					NewAnnotation.RenderFrameNumberEnd = RenderFrameEnd;
					NewAnnotation.ThreadName = Context.ThreadName;
					NewAnnotation.TimerName = Context.TimerName;
					NewAnnotation.EventStartTime = Context.EventStartTime;
					NewAnnotation.EventEndTime = Context.EventEndTime;
					NewAnnotation.EventDepth = Context.EventDepth;
					NewAnnotation.Text = Text;
					NewAnnotation.Description = Description;
					NewAnnotation.Color = *SelectedColor;
					NewAnnotation.Author = FPlatformProcess::UserName();
					NewAnnotation.Source = Context.SourceLabel;
					NewAnnotation.CreatedAt = FDateTime::UtcNow();
					NewAnnotation.ModifiedAt = FDateTime::UtcNow();
					bSuccess = Store->AddAnnotation(NewAnnotation);
				}

				if (!bSuccess)
				{
					if (ErrorLabel.IsValid())
					{
						// Reset text in case the previous failure was a validation error.
						ErrorLabel->SetText(LOCTEXT("AnnotationSaveErrorReset",
							"Failed to save annotation. The file may be read-only."));
						if (ErrorLabel->GetVisibility() == EVisibility::Collapsed)
						{
							ErrorLabel->SetVisibility(EVisibility::Visible);
							TSharedPtr<SWindow> ErrorWindow = WeakWindow.Pin();
							if (ErrorWindow.IsValid())
							{
								ErrorWindow->Resize(FVector2D(600, DialogHeight + ErrorBannerExtraHeight));
							}
						}
					}
					return FReply::Handled();
				}
			}
		}

		TSharedPtr<SWindow> Window = WeakWindow.Pin();
		if (Window.IsValid())
		{
			FSlateApplication::Get().RequestDestroyWindow(Window.ToSharedRef());
		}
		return FReply::Handled();
	};

	// One-shot focus + select-all on first activation (SetWidgetToFocusOnActivate alone
	// doesn't reliably trigger SelectAllTextWhenFocused).
	TSharedPtr<bool> bFirstActivation = MakeShared<bool>(true);
	DialogWindow->GetOnWindowActivatedEvent().AddLambda(
		[TextInput, bFirstActivation]()
		{
			if (*bFirstActivation && TextInput.IsValid())
			{
				*bFirstActivation = false;
				FSlateApplication::Get().SetKeyboardFocus(TextInput);
				TextInput->SelectAllText();
			}
		});
	FSlateApplication::Get().AddModalWindow(DialogWindow, FSlateApplication::Get().GetActiveTopLevelWindow());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingViewExtender::PopulateFrameNumbers(
	double InTime, uint32& OutGameFrame, uint32& OutRenderFrame)
{
	TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();
	if (InsightsManager.IsValid())
	{
		TSharedPtr<const TraceServices::IAnalysisSession> Session = InsightsManager->GetSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*Session.Get());
			OutGameFrame = FrameProvider.GetFrameNumberForTimestamp(ETraceFrameType::TraceFrameType_Game, InTime);
			OutRenderFrame = FrameProvider.GetFrameNumberForTimestamp(ETraceFrameType::TraceFrameType_Rendering, InTime);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FUserAnnotationsTimingViewExtender::FAnnotationContext FUserAnnotationsTimingViewExtender::FAnnotationContext::FromAnnotation(
	const FUserAnnotation& InAnnotation)
{
	FAnnotationContext Ctx;
	Ctx.Time = InAnnotation.Time;
	Ctx.EndTime = InAnnotation.EndTime;
	Ctx.GameFrameNumber = InAnnotation.GameFrameNumber;
	Ctx.RenderFrameNumber = InAnnotation.RenderFrameNumber;
	Ctx.GameFrameNumberEnd = InAnnotation.GameFrameNumberEnd;
	Ctx.RenderFrameNumberEnd = InAnnotation.RenderFrameNumberEnd;
	Ctx.ThreadName = InAnnotation.ThreadName;
	Ctx.TimerName = InAnnotation.TimerName;
	Ctx.EventStartTime = InAnnotation.EventStartTime;
	Ctx.EventEndTime = InAnnotation.EventEndTime;
	Ctx.EventDepth = InAnnotation.EventDepth;
	// Match BuildEventContext so the Edit dialog tints the event-name row.
	if (!Ctx.TimerName.IsEmpty())
	{
		const uint32 Packed = FTimingEvent::ComputeEventColor(*Ctx.TimerName);
		Ctx.SuggestedColor = FLinearColor(
			static_cast<float>((Packed >> 16) & 0xFF) / 255.0f,
			static_cast<float>((Packed >> 8) & 0xFF) / 255.0f,
			static_cast<float>(Packed & 0xFF) / 255.0f,
			1.0f);
	}
	return Ctx;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FUserAnnotationsTimingViewExtender::FAnnotationContext FUserAnnotationsTimingViewExtender::BuildPointContext(
	STimingView& TimingView, double Time)
{
	FAnnotationContext Ctx;
	Ctx.Time = Time;
	Ctx.SourceLabel = GetSourceLabelFromViewName(TimingView.GetName());

	PopulateFrameNumbers(Time, Ctx.GameFrameNumber, Ctx.RenderFrameNumber);

	const TSharedPtr<FBaseTimingTrack> HoveredTrack = TimingView.GetHoveredTrack();
	if (HoveredTrack.IsValid() && HoveredTrack->Is<FThreadTimingTrack>())
	{
		Ctx.ThreadName = HoveredTrack->GetName();
	}

	return Ctx;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FUserAnnotationsTimingViewExtender::FAnnotationContext FUserAnnotationsTimingViewExtender::BuildRangeContext(
	STimingView& TimingView, double StartTime, double EndTime)
{
	FAnnotationContext Ctx;
	Ctx.Time = StartTime;
	Ctx.EndTime = EndTime;
	Ctx.SourceLabel = GetSourceLabelFromViewName(TimingView.GetName());

	PopulateFrameNumbers(StartTime, Ctx.GameFrameNumber, Ctx.RenderFrameNumber);
	PopulateFrameNumbers(EndTime, Ctx.GameFrameNumberEnd, Ctx.RenderFrameNumberEnd);

	const TSharedPtr<FBaseTimingTrack> HoveredTrack = TimingView.GetHoveredTrack();
	if (HoveredTrack.IsValid() && HoveredTrack->Is<FThreadTimingTrack>())
	{
		Ctx.ThreadName = HoveredTrack->GetName();
	}

	return Ctx;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FUserAnnotationsTimingViewExtender::FAnnotationContext FUserAnnotationsTimingViewExtender::BuildEventContext(
	STimingView& TimingView, const ITimingEvent& Event)
{
	FAnnotationContext Ctx;
	Ctx.SourceLabel = GetSourceLabelFromViewName(TimingView.GetName());

	// Accept any ITimingEvent — CPU thread events from Timing Insights, load-time events
	// from Asset Loading Insights, allocation events from Memory Insights, etc. Base
	// ITimingEvent exposes enough to anchor the annotation (start/end/depth/track).
	Ctx.Time = Event.GetStartTime();
	Ctx.EndTime = Event.GetEndTime();
	Ctx.EventStartTime = Event.GetStartTime();
	Ctx.EventEndTime = Event.GetEndTime();
	Ctx.EventDepth = Event.GetDepth();

	// Track name is the source track (thread name for CPU events, generic track name
	// for other event types). This doubles as the event's display context.
	Ctx.ThreadName = Event.GetTrack()->GetName();

	// Resolve timer name and frame numbers. TimerIndex is specific to FThreadTrackEvent;
	// for other event classes fall back to the track name so the dialog and tooltip
	// still show something meaningful.
	TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();
	if (InsightsManager.IsValid())
	{
		TSharedPtr<const TraceServices::IAnalysisSession> Session = InsightsManager->GetSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			if (Event.Is<FThreadTrackEvent>())
			{
				const FThreadTrackEvent& ThreadEvent = Event.As<FThreadTrackEvent>();
				const TraceServices::ITimingProfilerProvider* TimingProfilerProvider =
					TraceServices::ReadTimingProfilerProvider(*Session.Get());
				if (TimingProfilerProvider)
				{
					const TraceServices::ITimingProfilerTimerReader& TimerReader =
						TimingProfilerProvider->GetTimerReader();
					const TraceServices::FTimingProfilerTimer* Timer =
						TimerReader.GetTimer(ThreadEvent.GetTimerIndex());
					if (Timer && Timer->Name)
					{
						Ctx.TimerName = Timer->Name;
					}
				}
			}
			if (Ctx.TimerName.IsEmpty())
			{
				Ctx.TimerName = Ctx.ThreadName;
			}

			const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*Session.Get());
			Ctx.GameFrameNumber = FrameProvider.GetFrameNumberForTimestamp(
				ETraceFrameType::TraceFrameType_Game, Ctx.Time);
			Ctx.RenderFrameNumber = FrameProvider.GetFrameNumberForTimestamp(
				ETraceFrameType::TraceFrameType_Rendering, Ctx.Time);
			Ctx.GameFrameNumberEnd = FrameProvider.GetFrameNumberForTimestamp(
				ETraceFrameType::TraceFrameType_Game, Ctx.EndTime);
			Ctx.RenderFrameNumberEnd = FrameProvider.GetFrameNumberForTimestamp(
				ETraceFrameType::TraceFrameType_Rendering, Ctx.EndTime);
		}
	}

	// Derive a suggested color from the anchored event's timer name so the annotation's
	// default color matches the event. Uses FTimingEvent's ComputeEventColor hash — the
	// same routine that produces the "By Timer Name" color the user sees on the track.
	if (!Ctx.TimerName.IsEmpty())
	{
		const uint32 Packed = FTimingEvent::ComputeEventColor(*Ctx.TimerName);
		Ctx.SuggestedColor = FLinearColor(
			static_cast<float>((Packed >> 16) & 0xFF) / 255.0f,
			static_cast<float>((Packed >> 8) & 0xFF) / 255.0f,
			static_cast<float>(Packed & 0xFF) / 255.0f,
			1.0f);
	}

	return Ctx;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingViewExtender::ToggleAllAnnotationVisibility()
{
	ToggleAllAnnotationVisibility(GetAnnotationStoreForCurrentSession());
}

void FUserAnnotationsTimingViewExtender::ToggleAllAnnotationVisibility(TSharedPtr<FUserAnnotationStore> InStore)
{
	if (!InStore.IsValid())
	{
		return;
	}

	// Collect IDs first to avoid iterator invalidation during UpdateAnnotation.
	bool bAnyVisible = false;
	TArray<FGuid> AnnotationIds;
	for (const FUserAnnotation& Annotation : InStore->GetAllAnnotations())
	{
		AnnotationIds.Add(Annotation.Id);
		bAnyVisible |= Annotation.bVisible;
	}

	const bool bNewVisible = !bAnyVisible;
	bool bAnyFailed = false;
	for (const FGuid& Id : AnnotationIds)
	{
		const FUserAnnotation* Annotation = InStore->FindAnnotation(Id);
		if (Annotation && Annotation->bVisible != bNewVisible)
		{
			FUserAnnotation Updated = *Annotation;
			Updated.bVisible = bNewVisible;
			Updated.ModifiedAt = FDateTime::UtcNow();
			if (!InStore->UpdateAnnotation(Updated))
			{
				bAnyFailed = true;
			}
		}
	}
	if (bAnyFailed)
	{
		FNotificationInfo Info(LOCTEXT("AnnotationBatchSaveFailed",
			"Failed to update some annotations. The sidecar .ini file may be read-only."));
		Info.ExpireDuration = 5.0f;
		TSharedPtr<SNotificationItem> Item = FSlateNotificationManager::Get().AddNotification(Info);
		if (Item.IsValid())
		{
			Item->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingViewExtender::NavigateToNextAnnotation(STimingView& TimingView, bool bForward)
{
	TSharedPtr<FUserAnnotationStore> Store = GetAnnotationStoreForCurrentSession();
	if (!Store.IsValid())
	{
		return;
	}

	// Build a sorted list of visible annotation times
	TArray<double> Times;
	for (const FUserAnnotation& Annotation : Store->GetAllAnnotations())
	{
		if (Annotation.bVisible)
		{
			Times.Add(Annotation.Time);
		}
	}

	if (Times.Num() == 0)
	{
		return;
	}

	Times.Sort();

	// Use the last navigated annotation time as anchor if available,
	// otherwise fall back to the viewport center. This avoids getting
	// stuck when CenterOnTimeInterval pixel-aligns to a position that
	// doesn't advance past closely spaced annotations.
	constexpr double NavigationEpsilon = 0.0001;
	const FTimingTrackViewport& Viewport = TimingView.GetViewport();
	const double ViewCenter = Viewport.GetStartTime() + Viewport.GetDuration() * 0.5;

	// Use the last navigated time as the cycling anchor if it's still within the visible viewport
	// (i.e. user hasn't panned/scrolled away). If they have, fall back to the viewport center.
	double ReferenceTime;
	if (LastNavigatedTime >= 0.0 && FMath::Abs(LastNavigatedTime - ViewCenter) < Viewport.GetDuration() * 0.5)
	{
		ReferenceTime = LastNavigatedTime;
	}
	else
	{
		ReferenceTime = ViewCenter;
	}

	double TargetTime;
	if (bForward)
	{
		// First annotation strictly after reference, or wrap to first.
		TargetTime = Times[0]; // wrap default
		for (int32 i = 0; i < Times.Num(); ++i)
		{
			if (Times[i] > ReferenceTime + NavigationEpsilon)
			{
				TargetTime = Times[i];
				break;
			}
		}
	}
	else
	{
		// Last annotation strictly before reference, or wrap to last.
		TargetTime = Times.Last(); // wrap default
		for (int32 i = Times.Num() - 1; i >= 0; --i)
		{
			if (Times[i] < ReferenceTime - NavigationEpsilon)
			{
				TargetTime = Times[i];
				break;
			}
		}
	}

	LastNavigatedTime = TargetTime;
	// Find the full annotation for zoom and scroll.
	const FUserAnnotation* NavAnnotation = nullptr;
	for (const FUserAnnotation& Annotation : Store->GetAllAnnotations())
	{
		if (Annotation.bVisible && FMath::IsNearlyEqual(Annotation.Time, TargetTime, NavigationEpsilon))
		{
			NavAnnotation = &Annotation;
			break;
		}
	}

	// Zoom/center to show the annotation. Match the panel's zoom behavior.
	if (NavAnnotation && NavAnnotation->HasEventAnchor())
	{
		const double Duration = NavAnnotation->EventEndTime - NavAnnotation->EventStartTime;
		TimingView.ZoomOnTimeInterval(NavAnnotation->EventStartTime - Duration * 0.1, Duration * 1.2);
	}
	else if (NavAnnotation && NavAnnotation->IsRange())
	{
		const double Duration = NavAnnotation->EndTime - NavAnnotation->Time;
		TimingView.ZoomOnTimeInterval(NavAnnotation->Time - Duration * 0.1, Duration * 1.2);
	}
	else
	{
		// For point annotations, zoom out to a reasonable level if currently
		// zoomed in very tight (e.g., after navigating from an event annotation).
		const double CurrentDuration = TimingView.GetViewport().GetDuration();
		if (CurrentDuration < 0.5)
		{
			TimingView.ZoomOnTimeInterval(TargetTime - 0.5, 1.0);
		}
		else
		{
			TimingView.CenterOnTimeInterval(TargetTime, 0.0);
		}
	}

	// Scroll the target track into view so the annotation highlight is visible.
	if (NavAnnotation && !NavAnnotation->ThreadName.IsEmpty())
	{
		TSharedPtr<FBaseTimingTrack> TargetTrack;
		TimingView.EnumerateTracks([&TargetTrack, &NavAnnotation](TSharedPtr<FBaseTimingTrack> Track)
		{
			if (!TargetTrack.IsValid() && Track.IsValid() && Track->GetName() == NavAnnotation->ThreadName)
			{
				TargetTrack = Track;
			}
		});

		if (TargetTrack.IsValid())
		{
			// Auto-reveal the target track if it was hidden — user pressed N/P to navigate
			// to this specific annotation, so silently staying on a blank viewport would
			// feel broken. They can hide the track again afterward.
			if (!TargetTrack->IsVisible())
			{
				TargetTrack->SetVisibilityFlag(true);
			}
			const int32 LaneDepth = NavAnnotation->HasEventAnchor()
				? static_cast<int32>(NavAnnotation->EventDepth) : -1;
			TimingView.BringScrollableTrackIntoViewDelayed(TargetTrack, LaneDepth);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingViewExtender::ToggleFloatingAnnotations()
{
	bool bNewState = !GetShowFloatingAnnotations();
	for (auto& Pair : PerSessionDataMap)
	{
		if (Pair.Value.Track.IsValid())
		{
			Pair.Value.Track->SetShowFloatingAnnotations(bNewState);
		}
	}
	SaveAnnotationsBool(TEXT("bShowFloatingCallouts"), bNewState);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingViewExtender::ToggleAnnotationMarkers()
{
	const bool bNewState = !GetShowAnnotationMarkers();
	for (auto& Pair : PerSessionDataMap)
	{
		if (Pair.Value.Track.IsValid())
		{
			Pair.Value.Track->SetShowAnnotationMarkers(bNewState);
		}
	}
	SaveAnnotationsBool(TEXT("bShowAnnotationMarkers"), bNewState);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FUserAnnotationsTimingViewExtender::GetShowAnnotationMarkers() const
{
	for (const auto& Pair : PerSessionDataMap)
	{
		if (Pair.Value.Track.IsValid())
		{
			return Pair.Value.Track->GetShowAnnotationMarkers();
		}
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FUserAnnotationsTimingViewExtender::IsAnnotationsTrackVisible(const UE::Insights::Timing::ITimingViewSession& InSession) const
{
	// const_cast is strictly for the TMap lookup: the map key type is the non-const pointer
	// used at insertion time in OnBeginSession. We never mutate *InSession through this pointer.
	const FPerSessionData* Data = PerSessionDataMap.Find(const_cast<UE::Insights::Timing::ITimingViewSession*>(&InSession));
	return Data && Data->Track.IsValid() && Data->Track->IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FUserAnnotationsTimingViewExtender::CanPersistAnnotations(const UE::Insights::Timing::ITimingViewSession& InSession) const
{
	const FPerSessionData* Data = PerSessionDataMap.Find(const_cast<UE::Insights::Timing::ITimingViewSession*>(&InSession));
	return Data && Data->bLoaded && !Data->SidecarFilePath.IsEmpty() && Data->bCanPersist;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingViewExtender::AppendEventAnnotationsToTooltip(FTooltipDrawState& InOutTooltip,
	const FString& ThreadName, const FString& TimerName,
	double EventStartTime, double EventEndTime) const
{
	if (ThreadName.IsEmpty() || TimerName.IsEmpty())
	{
		return;
	}
	for (const auto& Pair : PerSessionDataMap)
	{
		const FPerSessionData& Data = Pair.Value;
		if (!Data.Store.IsValid())
		{
			continue;
		}
		for (const FUserAnnotation& Annotation : Data.Store->GetAllAnnotations())
		{
			if (!Annotation.HasEventAnchor())
			{
				continue;
			}
			if (Annotation.ThreadName != ThreadName || Annotation.TimerName != TimerName)
			{
				continue;
			}
			// Match on event time (tolerance 1ns) — guards against annotating the same timer
			// multiple times in the same thread.
			if (FMath::Abs(Annotation.EventStartTime - EventStartTime) > 1e-9 ||
				FMath::Abs(Annotation.EventEndTime - EventEndTime) > 1e-9)
			{
				continue;
			}
			InOutTooltip.AddTitle(TEXT("Annotation"), FLinearColor(1.0f, 0.85f, 0.3f, 1.0f));
			InOutTooltip.AddNameValueTextLine(TEXT("Text"), Annotation.Text);
			if (!Annotation.Description.IsEmpty())
			{
				// Flatten newlines for single-line tooltip layout.
				FString FlatDesc = Annotation.Description;
				FlatDesc.ReplaceInline(TEXT("\r\n"), TEXT(" "));
				FlatDesc.ReplaceInline(TEXT("\n"), TEXT(" "));
				FlatDesc.ReplaceInline(TEXT("\r"), TEXT(" "));
				InOutTooltip.AddNameValueTextLine(TEXT("Description"), FlatDesc);
			}
			return; // One annotation per event suffices.
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FUserAnnotationsTimingViewExtender::ToggleAnnotationsTrackVisible(UE::Insights::Timing::ITimingViewSession& InSession)
{
	FPerSessionData* Data = PerSessionDataMap.Find(&InSession);
	if (!Data || !Data->Track.IsValid())
	{
		return;
	}
	// Flip the invoking session's track, then mirror the new state to every other session's
	// track so the Other menu checkbox stays in sync across Timing / Memory / Asset Loading
	// Insights when multiple windows are open on the same trace.
	Data->Track->ToggleVisibility();
	const bool bNewState = Data->Track->IsVisible();
	for (auto& Pair : PerSessionDataMap)
	{
		if (&Pair.Value == Data)
		{
			continue;
		}
		if (Pair.Value.Track.IsValid())
		{
			Pair.Value.Track->SetVisibilityFlag(bNewState);
		}
	}
	// Mirror onto the store. Shared across windows on the same trace, so one toggle covers all.
	if (Data->Store.IsValid())
	{
		Data->Store->SetAnnotationsVisible(bNewState);
	}
	SaveAnnotationsBool(TEXT("bTrackVisible"), bNewState);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FUserAnnotationsTimingViewExtender::IsTargetThreadTrackVisible(const UE::Insights::Timing::ITimingViewSession* InSession, const FString& ThreadName) const
{
	if (ThreadName.IsEmpty())
	{
		return true; // No anchor — not applicable; treat as "visible" so we don't warn.
	}
	// Only flag "hidden" when we can POSITIVELY identify a matching track and see that
	// it is hidden. If no matching track can be found (child/sub-tracks not reachable
	// via EnumerateTracks, or the name has drifted since capture), stay quiet rather
	// than false-positive the warning.
	bool bFoundMatch = false;
	bool bFoundVisible = false;
	for (const auto& Pair : PerSessionDataMap)
	{
		UE::Insights::Timing::ITimingViewSession* Session = Pair.Key;
		if (!Session)
		{
			continue;
		}
		// Scope to the host session when supplied — each window has independent track visibility.
		if (InSession && Session != InSession)
		{
			continue;
		}
		Session->EnumerateTracks([&](TSharedPtr<FBaseTimingTrack> Track)
		{
			if (bFoundVisible || !Track.IsValid())
			{
				return;
			}
			const FString TrackName = Track->GetName();
			// Track names can drift between sessions — a thread that was "Foreground Worker"
			// at annotation time may appear as "Foreground Worker #2" later. Try exact match,
			// then case-insensitive, then a substring match in either direction (min 3 chars).
			// NOTE: substring matching is intentionally loose — short prefixes like "RHI" may
			// match multiple tracks ("GRHIThread", "PrefetchRHI"). Only used to gate the
			// "(hidden)" warning in the panel, so false positives are preferred over false
			// negatives (warning shown when it might not apply vs warning hidden when it should).
			const bool bExact = TrackName.Equals(ThreadName, ESearchCase::CaseSensitive);
			const bool bIgnoreCase = !bExact && TrackName.Equals(ThreadName, ESearchCase::IgnoreCase);
			const bool bSubstring = !bExact && !bIgnoreCase &&
				ThreadName.Len() >= 3 && TrackName.Len() >= 3 &&
				(TrackName.Contains(ThreadName, ESearchCase::IgnoreCase) ||
				 ThreadName.Contains(TrackName, ESearchCase::IgnoreCase));
			if (bExact || bIgnoreCase || bSubstring)
			{
				bFoundMatch = true;
				if (Track->IsVisible())
				{
					bFoundVisible = true;
				}
			}
		});
		if (bFoundVisible)
		{
			return true;
		}
	}
	return !bFoundMatch; // No match found: assume visible (avoid false warnings).
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FUserAnnotationsTimingViewExtender::GetShowFloatingAnnotations() const
{
	for (const auto& Pair : PerSessionDataMap)
	{
		if (Pair.Value.Track.IsValid())
		{
			return Pair.Value.Track->GetShowFloatingAnnotations();
		}
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
