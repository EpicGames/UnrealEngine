// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"

// TraceInsights
#include "Insights/ITimingViewExtender.h"
#include "Insights/ViewModels/TimingEvent.h"

class FTooltipDrawState;

namespace UE::Insights { class FTraceMetadataFile; }

namespace UE::Insights::TimingProfiler
{

struct FUserAnnotation;
class FUserAnnotationStore;
class FUserAnnotationsTimingTrack;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * FUserAnnotationsTimingViewExtender
 *
 * Integrates user annotations into the Timing Insights view via the ITimingViewExtender
 * modular feature interface. Follows the FGameplayTimingViewExtender per-session-data pattern.
 *
 * Responsibilities:
 * - Creates FUserAnnotationStore + FUserAnnotationsTimingTrack per session
 * - Derives the sidecar INI path from the trace name and loads metadata
 * - Adds "Add Time Annotation Here..." etc. to the global context menu
 * - Manages the annotation add/edit dialog
 */
class STimingView;

class FUserAnnotationsTimingViewExtender : public UE::Insights::Timing::ITimingViewExtender
{
public:
	FUserAnnotationsTimingViewExtender() = default;
	virtual ~FUserAnnotationsTimingViewExtender() = default;

	/** Returns the annotation store for the current (most recently begun) session, or nullptr. */
	TSharedPtr<FUserAnnotationStore> GetAnnotationStoreForCurrentSession() const;

	/** Fires on session begin / end / store-swap. Subscribers re-query GetAnnotationStoreForCurrentSession. */
	FSimpleMulticastDelegate& GetOnAnnotationStoreChanged() { return OnAnnotationStoreChanged; }

	/**
	 * Drive the sidecar load for every registered session that hasn't loaded yet. Called
	 * by the annotations panel each frame so panels in Memory/Asset Loading Insights
	 * populate even when their STimingView's extender Tick gate (`if (Session)`) hasn't
	 * fired yet.
	 */
	void EnsureAllSessionsLoaded();

	// ITimingViewExtender
	virtual void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void Tick(const UE::Insights::Timing::FTimingViewExtenderTickParams& Params) override;
	virtual bool ExtendGlobalContextMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;
	virtual void ExtendOtherTracksFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;

	/** Context for creating or editing an annotation. */
	struct FAnnotationContext
	{
		double Time = 0.0;
		double EndTime = 0.0;
		uint32 GameFrameNumber = 0;
		uint32 RenderFrameNumber = 0;
		uint32 GameFrameNumberEnd = 0;
		uint32 RenderFrameNumberEnd = 0;
		FString ThreadName;

		/** Event anchor fields (populated when annotating a specific timing event). */
		FString TimerName;
		double EventStartTime = 0.0;
		double EventEndTime = 0.0;
		uint32 EventDepth = 0;

		/** Suggested initial color for new annotations. For event annotations this is
		 *  derived from the anchored event's color so it visually matches. User can override. */
		FLinearColor SuggestedColor = FLinearColor::White;

		/** Friendly label for the source Insights window ("Timing", "Memory", "Asset Loading").
		 *  Baked in when building the context so the dialog's save path can persist it. */
		FString SourceLabel;

		/** Creates a context from an existing annotation (for editing). */
		static FAnnotationContext FromAnnotation(const FUserAnnotation& Annotation);
	};

	/** Shows the annotation add/edit dialog. */
	void ShowAnnotationDialog(TSharedPtr<FUserAnnotationStore> InStore,
		const FAnnotationContext& Context, const FGuid* ExistingAnnotationId = nullptr);

	/** Build a FAnnotationContext for a point annotation at the given time. */
	static FAnnotationContext BuildPointContext(STimingView& TimingView, double Time);

	/** Build a FAnnotationContext for a range annotation over the given selection. */
	static FAnnotationContext BuildRangeContext(STimingView& TimingView, double StartTime, double EndTime);

	/** Build a FAnnotationContext for an event annotation on the hovered event. */
	static FAnnotationContext BuildEventContext(STimingView& TimingView, const ITimingEvent& Event);

	/** Snap a time value to the nearest visible user annotation for the given session. */
	double Snap(UE::Insights::Timing::ITimingViewSession& InSession, double Time, double SnapTolerance) const;

	/** Toggle visibility of all annotations in the current session store. */
	void ToggleAllAnnotationVisibility();

	/** Toggle visibility of all annotations in the given store (multi-window safe). */
	void ToggleAllAnnotationVisibility(TSharedPtr<FUserAnnotationStore> InStore);

	/** Navigate to the next or previous annotation relative to the viewport center time. */
	void NavigateToNextAnnotation(STimingView& TimingView, bool bForward);

	/** Toggle floating annotation callout visibility. Event highlights remain visible. */
	void ToggleFloatingAnnotations();

	/** Returns true if floating annotation callouts are currently shown. */
	bool GetShowFloatingAnnotations() const;

	/** Toggle full-viewport vertical indicator lines and selected-area event highlights (persists). */
	void ToggleAnnotationMarkers();

	/** Returns true if annotation markers (vertical lines and event highlights) are currently shown. */
	bool GetShowAnnotationMarkers() const;

	/** Returns true if the Annotations track is currently visible for the given session. */
	bool IsAnnotationsTrackVisible(const UE::Insights::Timing::ITimingViewSession& InSession) const;

	/**
	 * Returns true if annotations can be created/edited for the given session. False when the
	 * trace is a live/direct stream (no file backing), no sidecar path can be derived, or the
	 * sidecar .ini file exists but is read-only.
	 */
	bool CanPersistAnnotations(const UE::Insights::Timing::ITimingViewSession& InSession) const;

	/**
	 * Returns true if any currently-tracked session has a readable store but cannot be written
	 * to (read-only sidecar file or unwritable folder). Used by the panel to show a banner.
	 */
	bool IsAnyCurrentSessionReadOnly() const;

	/**
	 * Appends annotation info (name + description) to an event's tooltip when the event has
	 * at least one matching annotation in the active session's store. Called from the thread
	 * timing track's InitTooltip.
	 */
	void AppendEventAnnotationsToTooltip(FTooltipDrawState& InOutTooltip,
		const FString& ThreadName, const FString& TimerName,
		double EventStartTime, double EventEndTime) const;

	/**
	 * Returns true if a thread/CPU track matching ThreadName is currently visible in the given
	 * session. Used by the annotations panel to warn when an event annotation's target track
	 * has been hidden. When InSession is nullptr (unknown host), falls back to iterating every
	 * active session — useful for non-panel callers but inaccurate across multi-window layouts
	 * where each window has independent track visibility.
	 */
	bool IsTargetThreadTrackVisible(const UE::Insights::Timing::ITimingViewSession* InSession, const FString& ThreadName) const;

	/** Toggle the Annotations track visibility for the given session (persists). */
	void ToggleAnnotationsTrackVisible(UE::Insights::Timing::ITimingViewSession& InSession);

private:
	struct FPerSessionData;

	/** Run the sidecar-derivation + load path for a single session. Idempotent. */
	void EnsureSessionLoaded(FPerSessionData& InData);

	/** Adds the three annotation visibility toggles (Track / Floating Callouts / Markers) to the
	 *  given menu builder. Single source of truth for both the global context menu and the Other
	 *  Tracks filter menu so the two entry points stay in sync. */
	void AddAnnotationVisibilityToggles(
		FMenuBuilder& InMenuBuilder,
		UE::Insights::Timing::ITimingViewSession& InSession,
		FPerSessionData& InData);

	struct FPerSessionData
	{
		TSharedPtr<FUserAnnotationsTimingTrack> Track;
		TSharedPtr<FUserAnnotationStore> Store;
		TSharedPtr<FTraceMetadataFile> MetadataFile;
		FString SidecarFilePath;
		bool bLoaded = false;
		/**
		 * True when the sidecar .ini can be written — i.e. the trace is file-backed and the
		 * sidecar path is either writable or doesn't yet exist (so it can be created).
		 * False for live/direct traces, remote UTS traces without a local file, and read-only
		 * .ini files. Gates all annotation-creation UI.
		 */
		bool bCanPersist = false;
		/** One-shot flag: track was auto-hidden once because the sidecar can't be created/written. */
		bool bAutoHiddenForReadOnly = false;
	};

	TMap<UE::Insights::Timing::ITimingViewSession*, FPerSessionData> PerSessionDataMap;

	FSimpleMulticastDelegate OnAnnotationStoreChanged;

	/** Last annotation time navigated to, used as anchor for next N/P navigation. */
	double LastNavigatedTime = -1.0;

	/** Populates frame numbers for a given time. */
	static void PopulateFrameNumbers(double InTime,
		uint32& OutGameFrame, uint32& OutRenderFrame);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
