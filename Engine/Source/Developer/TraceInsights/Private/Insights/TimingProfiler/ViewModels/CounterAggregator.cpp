// Copyright Epic Games, Inc. All Rights Reserved.

#include "CounterAggregator.h"

// TraceServices
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Counters.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// TAggregatedStatsEx
////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename Type>
struct TAggregatedStatsEx
{
	static constexpr int32 HistogramLen = 100; // number of buckets per histogram

	uint64 Count;
	TAggregatedStats<Type> BaseStats;

	// Histogram for computing median and lower/upper quartiles.
	int32 Histogram[HistogramLen];
	Type DT; // bucket size

	TAggregatedStatsEx()
	{
		FMemory::Memzero(Histogram, sizeof(int32) * HistogramLen);
		DT = Type(1);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// TCounterAggregationHelper
////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename Type>
class TCounterAggregationHelper
{
public:
	TCounterAggregationHelper<Type>() = default;

	void Reset();

	void SetTimeInterval(double InIntervalStartTime, double InIntervalEndTime)
	{
		IntervalStartTime = InIntervalStartTime;
		IntervalEndTime = InIntervalEndTime;
	}

	void Update(uint32 CounterId, const TraceServices::ICounter& Counter)
	{
		EnumerateValues(CounterId, Counter, UpdateMinMax);
	}

	void PrecomputeHistograms();

	void UpdateHistograms(uint32 CounterId, const TraceServices::ICounter& Counter)
	{
		EnumerateValues(CounterId, Counter, UpdateHistogram);
	}

	void PostProcess(bool bComputeMedian);

	void ApplyResultsTo(const TMap<uint32, FStatsNodePtr>& StatsNodesIdMap) const;

private:
	template<typename CallbackType>
	void EnumerateValues(uint32 CounterId, const TraceServices::ICounter& Counter, CallbackType Callback);

	static void UpdateMinMax(TAggregatedStatsEx<Type>& Stats, Type Value);
	static void UpdateHistogram(TAggregatedStatsEx<Type>& StatsEx, Type Value);
	static void PostProcess(TAggregatedStatsEx<Type>& StatsEx, bool bComputeMedian);

	double IntervalStartTime = 0.0;
	double IntervalEndTime = 0.0;
	TMap<uint32, TAggregatedStatsEx<Type>> StatsMap;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename Type>
void TCounterAggregationHelper<Type>::Reset()
{
	IntervalStartTime = 0.0;
	IntervalEndTime = 0.0;
	StatsMap.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// specialization for Type = double

template<>
template<typename CallbackType>
void TCounterAggregationHelper<double>::EnumerateValues(uint32 CounterId, const TraceServices::ICounter& Counter, CallbackType Callback)
{
	TAggregatedStatsEx<double>* StatsExPtr = StatsMap.Find(CounterId);
	if (!StatsExPtr)
	{
		StatsExPtr = &StatsMap.Add(CounterId);
		StatsExPtr->Count = 0;
		StatsExPtr->BaseStats.Min = +MAX_dbl;
		StatsExPtr->BaseStats.Max = -MAX_dbl;
	}
	TAggregatedStatsEx<double>& StatsEx = *StatsExPtr;

	Counter.EnumerateFloatValues(IntervalStartTime, IntervalEndTime, false, [this, &StatsEx, Callback](double Time, double Value)
		{
			Callback(StatsEx, Value);
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// specialization for Type = int64

template<>
template<typename CallbackType>
void TCounterAggregationHelper<int64>::EnumerateValues(uint32 CounterId, const TraceServices::ICounter& Counter, CallbackType Callback)
{
	TAggregatedStatsEx<int64>* StatsExPtr = StatsMap.Find(CounterId);
	if (!StatsExPtr)
	{
		StatsExPtr = &StatsMap.Add(CounterId);
		StatsExPtr->Count = 0;
		StatsExPtr->BaseStats.Min = MAX_int64;
		StatsExPtr->BaseStats.Max = MIN_int64;
	}
	TAggregatedStatsEx<int64>& StatsEx = *StatsExPtr;

	Counter.EnumerateValues(IntervalStartTime, IntervalEndTime, false, [this, &StatsEx, Callback](double Time, int64 Value)
		{
			Callback(StatsEx, Value);
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename Type>
void TCounterAggregationHelper<Type>::UpdateMinMax(TAggregatedStatsEx<Type>& StatsEx, Type Value)
{
	TAggregatedStats<Type>& Stats = StatsEx.BaseStats;

	Stats.Sum += Value;

	if (Value < Stats.Min)
	{
		Stats.Min = Value;
	}

	if (Value > Stats.Max)
	{
		Stats.Max = Value;
	}

	StatsEx.Count++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// specialization for Type = double

template<>
void TCounterAggregationHelper<double>::PrecomputeHistograms()
{
	for (auto& KV : StatsMap)
	{
		TAggregatedStatsEx<double>& StatsEx = KV.Value;
		const TAggregatedStats<double>& Stats = StatsEx.BaseStats;

		// Each bucket (Histogram[i]) will be centered on a value.
		// I.e. First bucket (bucket 0) is centered on Min value: [Min-DT/2, Min+DT/2)
		// and last bucket (bucket N-1) is centered on Max value: [Max-DT/2, Max+DT/2).

		if (Stats.Max == Stats.Min)
		{
			StatsEx.DT = 1.0; // single large bucket
		}
		else
		{
			StatsEx.DT = (Stats.Max - Stats.Min) / (TAggregatedStatsEx<double>::HistogramLen - 1);
			if (StatsEx.DT == 0.0)
			{
				StatsEx.DT = 1.0;
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// specialization for Type = int64

template<>
void TCounterAggregationHelper<int64>::PrecomputeHistograms()
{
	for (auto& KV : StatsMap)
	{
		TAggregatedStatsEx<int64>& StatsEx = KV.Value;
		const TAggregatedStats<int64>& Stats = StatsEx.BaseStats;

		// Each bucket (Histogram[i]) will be centered on a value.
		// I.e. First bucket (bucket 0) is centered on Min value: [Min-DT/2, Min+DT/2)
		// and last bucket (bucket N-1) is centered on Max value: [Max-DT/2, Max+DT/2).

		if (Stats.Max == Stats.Min)
		{
			StatsEx.DT = 1; // single bucket
		}
		else
		{
			// DT = Ceil[(Max - Min) / (N - 1)]
			StatsEx.DT = (Stats.Max - Stats.Min + TAggregatedStatsEx<int64>::HistogramLen - 2) / (TAggregatedStatsEx<int64>::HistogramLen - 1);
			if (StatsEx.DT == 0)
			{
				StatsEx.DT = 1;
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename Type>
void TCounterAggregationHelper<Type>::UpdateHistogram(TAggregatedStatsEx<Type>& StatsEx, Type Value)
{
	const TAggregatedStats<Type>& Stats = StatsEx.BaseStats;

	// Index = (Value - Min + DT/2) / DT
	int32 Index = static_cast<int32>((Value - Stats.Min + StatsEx.DT / 2) / StatsEx.DT);
	ensure(Index >= 0);
	if (Index < 0)
	{
		Index = 0;
	}
	ensure(Index < TAggregatedStatsEx<Type>::HistogramLen);
	if (Index >= TAggregatedStatsEx<Type>::HistogramLen)
	{
		Index = TAggregatedStatsEx<Type>::HistogramLen - 1;
	}
	StatsEx.Histogram[Index]++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename Type>
void TCounterAggregationHelper<Type>::PostProcess(TAggregatedStatsEx<Type>& StatsEx, bool bComputeMedian)
{
	TAggregatedStats<Type>& Stats = StatsEx.BaseStats;

	// Compute average value.
	if (StatsEx.Count > 0)
	{
		Stats.Average = Stats.Sum / static_cast<Type>(StatsEx.Count);

		if (bComputeMedian)
		{
			const int32 HalfCount = IntCastChecked<int32>(StatsEx.Count / 2);

			// Compute median value.
			int32 Count = 0;
			for (int32 HistogramIndex = 0; HistogramIndex < TAggregatedStatsEx<Type>::HistogramLen; HistogramIndex++)
			{
				Count += StatsEx.Histogram[HistogramIndex];
				if (Count > HalfCount)
				{
					Stats.Median = Stats.Min + HistogramIndex * StatsEx.DT;

					if (HistogramIndex > 0 &&
						StatsEx.Count % 2 == 0 &&
						Count - StatsEx.Histogram[HistogramIndex] == HalfCount)
					{
						const Type PrevMedian = Stats.Min + (HistogramIndex - 1) * StatsEx.DT;
						Stats.Median = (Stats.Median + PrevMedian) / 2;
					}

					break;
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// specialization for Type = double

template<>
void TCounterAggregationHelper<double>::PostProcess(bool bComputeMedian)
{
	for (auto& KV : StatsMap)
	{
		PostProcess(KV.Value, bComputeMedian);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// specialization for Type = int64

template<>
void TCounterAggregationHelper<int64>::PostProcess(bool bComputeMedian)
{
	for (auto& KV : StatsMap)
	{
		PostProcess(KV.Value, bComputeMedian);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// specialization for Type = double

template<>
void TCounterAggregationHelper<double>::ApplyResultsTo(const TMap<uint32, FStatsNodePtr>& StatsNodesIdMap) const
{
	for (const auto& KV : StatsMap)
	{
		// Update the stats node.
		FStatsNodePtr NodePtr = StatsNodesIdMap.FindRef(KV.Key);
		if (NodePtr != nullptr)
		{
			NodePtr->SetAggregatedStatsDouble(KV.Value.Count, KV.Value.BaseStats);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// specialization for Type = int64

template<>
void TCounterAggregationHelper<int64>::ApplyResultsTo(const TMap<uint32, FStatsNodePtr>& StatsNodesIdMap) const
{
	for (const auto& KV : StatsMap)
	{
		// Update the stats node.
		FStatsNodePtr NodePtr = StatsNodesIdMap.FindRef(KV.Key);
		if (NodePtr != nullptr)
		{
			NodePtr->SetAggregatedStatsInt64(KV.Value.Count, KV.Value.BaseStats);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FCounterAggregationWorker
////////////////////////////////////////////////////////////////////////////////////////////////////

class FCounterAggregationWorker : public IStatsAggregationWorker
{
public:
	FCounterAggregationWorker(TSharedPtr<const TraceServices::IAnalysisSession> InSession, double InStartTime, double InEndTime)
		: Session(InSession)
		, StartTime(InStartTime)
		, EndTime(InEndTime)
	{
	}

	virtual ~FCounterAggregationWorker() {}

	virtual void DoWork(TSharedPtr<TraceServices::FCancellationToken> CancellationToken) override;

	void ApplyResultsTo(const TMap<uint32, FStatsNodePtr>& StatsNodesIdMap) const;
	void ResetResults();

private:
	TSharedPtr<const TraceServices::IAnalysisSession> Session;
	double StartTime;
	double EndTime;
	const bool bComputeMedian = true;
	TCounterAggregationHelper<double> CalculationHelperDbl;
	TCounterAggregationHelper<int64>  CalculationHelperInt;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCounterAggregationWorker::DoWork(TSharedPtr<TraceServices::FCancellationToken> CancellationToken)
{
	CalculationHelperDbl.SetTimeInterval(StartTime, EndTime);
	CalculationHelperInt.SetTimeInterval(StartTime, EndTime);

	if (Session.IsValid())
	{
		// Suspend analysis in order to avoid write locks (ones blocked by the read lock below) to further block other read locks.
		//TraceServices::FAnalysisSessionSuspensionScope SessionPauseScope(*Session.Get());

		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::ICounterProvider& CountersProvider = TraceServices::ReadCounterProvider(*Session.Get());

		// Compute instance count and total/min/max inclusive/exclusive times for each counter.
		// Iterate through all counters.
		CountersProvider.EnumerateCounters([this](uint32 CounterId, const TraceServices::ICounter& Counter)
			{
				if (Counter.IsFloatingPoint())
				{
					CalculationHelperDbl.Update(CounterId, Counter);
				}
				else
				{
					CalculationHelperInt.Update(CounterId, Counter);
				}
			});

		// Now, as we know min/max inclusive/exclusive times for counter, we can compute histogram and median values.
		if (bComputeMedian)
		{
			// Update bucket size (DT) for computing histogram.
			CalculationHelperDbl.PrecomputeHistograms();
			CalculationHelperInt.PrecomputeHistograms();

			// Compute histogram.
			// Iterate again through all counters.
			CountersProvider.EnumerateCounters([this](uint32 CounterId, const TraceServices::ICounter& Counter)
				{
					if (Counter.IsFloatingPoint())
					{
						CalculationHelperDbl.UpdateHistograms(CounterId, Counter);
					}
					else
					{
						CalculationHelperInt.UpdateHistograms(CounterId, Counter);
					}
				});
		}
	}

	// Compute average and median inclusive/exclusive times.
	CalculationHelperDbl.PostProcess(bComputeMedian);
	CalculationHelperInt.PostProcess(bComputeMedian);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCounterAggregationWorker::ApplyResultsTo(const TMap<uint32, FStatsNodePtr>& StatsNodesIdMap) const
{
	CalculationHelperDbl.ApplyResultsTo(StatsNodesIdMap);
	CalculationHelperInt.ApplyResultsTo(StatsNodesIdMap);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCounterAggregationWorker::ResetResults()
{
	CalculationHelperDbl.Reset();
	CalculationHelperInt.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FCounterAggregator
////////////////////////////////////////////////////////////////////////////////////////////////////

IStatsAggregationWorker* FCounterAggregator::CreateWorker(TSharedPtr<const TraceServices::IAnalysisSession> InSession)
{
	return new FCounterAggregationWorker(InSession, GetIntervalStartTime(), GetIntervalEndTime());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCounterAggregator::ApplyResultsTo(const TMap<uint32, FStatsNodePtr>& StatsNodesIdMap) const
{
	// It can only be called from OnFinishedCallback.
	check(IsFinished());

	FCounterAggregationWorker* Worker = (FCounterAggregationWorker*)GetWorker();
	check(Worker != nullptr);

	// Apply aggregation stats to tree nodes.
	Worker->ApplyResultsTo(StatsNodesIdMap);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCounterAggregator::ResetResults()
{
	// It can only be called from OnFinishedCallback.
	check(IsFinished());

	FCounterAggregationWorker* Worker = (FCounterAggregationWorker*)GetWorker();
	check(Worker != nullptr);

	Worker->ResetResults();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
