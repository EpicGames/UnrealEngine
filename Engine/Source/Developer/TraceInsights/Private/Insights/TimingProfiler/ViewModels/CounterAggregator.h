// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Insights/TimingProfiler/ViewModels/StatsAggregator.h"
#include "Insights/TimingProfiler/ViewModels/StatsNode.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FCounterAggregator : public FStatsAggregator
{
public:
	FCounterAggregator() : FStatsAggregator(TEXT("Counters")) {}
	virtual ~FCounterAggregator() = default;

	void ApplyResultsTo(const TMap<uint32, FStatsNodePtr>& StatsNodesIdMap) const;
	void ResetResults();

protected:
	virtual IStatsAggregationWorker* CreateWorker(TSharedPtr<const TraceServices::IAnalysisSession> InSession) override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
