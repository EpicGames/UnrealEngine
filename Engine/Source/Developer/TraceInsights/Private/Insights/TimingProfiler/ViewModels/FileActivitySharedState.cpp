// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileActivitySharedState.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

// TraceServices
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/LoadTimeProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/Common/InsightsMenuBuilder.h"
#include "Insights/InsightsStyle.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/TimingProfiler/Tracks/FileActivityTimingTrack.h"
#include "Insights/TimingProfilerCommon.h"
#include "Insights/Widgets/STimingView.h"

#include <limits>

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::FileActivity"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FFileActivityTimingViewCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

class FFileActivityTimingViewCommands : public TCommands<FFileActivityTimingViewCommands>
{
public:
	FFileActivityTimingViewCommands();
	virtual ~FFileActivityTimingViewCommands() {}
	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> ShowHideAllIoTracks;
	TSharedPtr<FUICommandInfo> ShowHideIoOverviewTrack;
	TSharedPtr<FUICommandInfo> ToggleOnlyErrors;
	TSharedPtr<FUICommandInfo> ShowHideIoActivityTrack;
	TSharedPtr<FUICommandInfo> ToggleBackgroundEvents;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

FFileActivityTimingViewCommands::FFileActivityTimingViewCommands()
	: TCommands<FFileActivityTimingViewCommands>(
		TEXT("FileActivityTimingViewCommands"),
		NSLOCTEXT("Contexts", "FileActivityTimingViewCommands", "Insights - Timing View - File Activity"),
		NAME_None,
		FInsightsStyle::GetStyleSetName())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// UI_COMMAND takes long for the compiler to optimize
UE_DISABLE_OPTIMIZATION_SHIP
void FFileActivityTimingViewCommands::RegisterCommands()
{
	// This command is used only for its key binding (to toggle both ShowHideIoOverviewTrack and ShowHideIoActivityTrack in the same time).
	UI_COMMAND(ShowHideAllIoTracks,
		"File Activity Tracks",
		"Shows/hides the File Activity tracks.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::I));

	UI_COMMAND(ShowHideIoOverviewTrack,
		"I/O Overview Track",
		"Shows/hides the I/O Overview track.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord()); // EKeys::I

	UI_COMMAND(ToggleOnlyErrors,
		"Only Errors (I/O Overview Track)",
		"Shows only the events with errors, in the I/O Overview track.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ShowHideIoActivityTrack,
		"I/O Activity Track",
		"Shows/hides the I/O Activity track.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord()); // EKeys::I

	UI_COMMAND(ToggleBackgroundEvents,
		"Background Events (I/O Activity Track)",
		"Shows/hides background events for file activities, in the I/O Activity track.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::O));
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////
// FFileActivitySharedState
////////////////////////////////////////////////////////////////////////////////////////////////////

const uint32 FFileActivitySharedState::MaxLanes = 10000;

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileActivitySharedState::OnBeginSession(Timing::ITimingViewSession& InSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	IoOverviewTrack.Reset();
	IoActivityTrack.Reset();

	bShowHideAllIoTracks = false;
	bForceIoEventsUpdate = false;

	FileActivities.Reset();
	FileActivityMap.Reset();
	AllIoEvents.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileActivitySharedState::OnEndSession(Timing::ITimingViewSession& InSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	IoOverviewTrack.Reset();
	IoActivityTrack.Reset();

	bShowHideAllIoTracks = false;
	bForceIoEventsUpdate = false;

	FileActivities.Reset();
	FileActivityMap.Reset();
	AllIoEvents.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileActivitySharedState::Tick(Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	if (!TraceServices::ReadFileActivityProvider(InAnalysisSession))
	{
		return;
	}

	if (!IoOverviewTrack.IsValid())
	{
		IoOverviewTrack = MakeShared<FOverviewFileActivityTimingTrack>(*this);
		IoOverviewTrack->SetOrder(FTimingTrackOrder::First + 200);
		IoOverviewTrack->SetVisibilityFlag(bShowHideAllIoTracks);
		InSession.AddScrollableTrack(IoOverviewTrack);
	}

	if (!IoActivityTrack.IsValid())
	{
		IoActivityTrack = MakeShared<FDetailedFileActivityTimingTrack>(*this);
		IoActivityTrack->SetOrder(FTimingTrackOrder::Last);
		IoActivityTrack->SetVisibilityFlag(bShowHideAllIoTracks);
		InSession.AddScrollableTrack(IoActivityTrack);
	}

	if (bForceIoEventsUpdate)
	{
		bForceIoEventsUpdate = false;

		FileActivities.Reset();
		FileActivityMap.Reset();
		AllIoEvents.Reset();

		FStopwatch Stopwatch;
		Stopwatch.Start();

		// Enumerate all IO events and cache them.
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(InAnalysisSession);
			const TraceServices::IFileActivityProvider& FileActivityProvider = *TraceServices::ReadFileActivityProvider(InAnalysisSession);
			FileActivityProvider.EnumerateFileActivity([this](const TraceServices::FFileInfo& FileInfo, const TraceServices::IFileActivityProvider::Timeline& Timeline)
				{
					TSharedPtr<FIoFileActivity> Activity = MakeShared<FIoFileActivity>();

					Activity->Id = FileInfo.Id;
					Activity->Path = FileInfo.Path;
					Activity->StartTime = +std::numeric_limits<double>::infinity();
					Activity->EndTime = -std::numeric_limits<double>::infinity();
					Activity->CloseStartTime = +std::numeric_limits<double>::infinity();
					Activity->CloseEndTime = +std::numeric_limits<double>::infinity();
					Activity->EventCount = 0;
					Activity->Index = -1;
					Activity->MaxConcurrentEvents = 0;
					Activity->StartingDepth = 0;

					const int32 ActivityIndex = FileActivities.Num();
					FileActivities.Add(Activity);
					FileActivityMap.Add(FileInfo.Id, Activity);

					TArray<double> ConcurrentEvents;
					Timeline.EnumerateEvents(-std::numeric_limits<double>::infinity(), +std::numeric_limits<double>::infinity(),
						[this, &Activity, ActivityIndex, &FileInfo, &Timeline, &ConcurrentEvents](double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FFileActivity* FileActivity)
						{
							if (FileActivity->ActivityType != TraceServices::FileActivityType_Close)
							{
								// events should be ordered by start time, but Activity->StartTime may not be initialized
								ensure(Activity->StartTime == +std::numeric_limits<double>::infinity() || EventStartTime >= Activity->StartTime);
								if (EventStartTime < Activity->StartTime)
								{
									Activity->StartTime = EventStartTime;
								}

								if (EventEndTime > Activity->EndTime)
								{
									Activity->EndTime = EventEndTime;
								}
							}
							else
							{
								// The time range for the Close event is stored separated;
								// this allows us to insert lanes into the idle time between the last read from a file and when the file is actually closed
								Activity->CloseStartTime = EventStartTime;
								Activity->CloseEndTime = EventEndTime;
							}

							Activity->EventCount++;

							uint32 LocalDepth = MAX_uint32;
							for (int32 i = 0; i < ConcurrentEvents.Num(); ++i)
							{
								if (EventStartTime >= ConcurrentEvents[i])
								{
									LocalDepth = i;
									ConcurrentEvents[i] = EventEndTime;
									break;
								}
							}

							if (LocalDepth == MAX_uint32)
							{
								LocalDepth = ConcurrentEvents.Num();
								ConcurrentEvents.Add(EventEndTime);
								Activity->MaxConcurrentEvents = ConcurrentEvents.Num();
							}

							uint32 Type = ((uint32)FileActivity->ActivityType & 0x0F) | (FileActivity->Failed ? 0x80 : 0);
							AllIoEvents.Add(FIoTimingEvent{ EventStartTime, EventEndTime, LocalDepth, Type, FileActivity->Offset, FileActivity->Size, FileActivity->ActualSize, ActivityIndex, FileActivity->FileHandle, FileActivity->ReadWriteHandle });
							return TraceServices::EEventEnumerate::Continue;
						});

					return true;
				});
		}

		Stopwatch.Stop();
		UE_LOGF(LogTimingProfiler, Log, "[IO] Enumerated %ls events (%ls file activities) in %ls.",
			*FText::AsNumber(AllIoEvents.Num()).ToString(),
			*FText::AsNumber(FileActivities.Num()).ToString(),
			*FormatTimeAuto(Stopwatch.GetAccumulatedTime()));
		Stopwatch.Restart();

		// Sort cached IO file activities by Start Time.
		FileActivities.Sort([](const TSharedPtr<FIoFileActivity>& A, const TSharedPtr<FIoFileActivity>& B) { return A->StartTime < B->StartTime; });

		// Sort cached IO events by Start Time.
		AllIoEvents.Sort([](const FIoTimingEvent& A, const FIoTimingEvent& B) { return A.StartTime < B.StartTime; });

		Stopwatch.Stop();
		UE_LOGF(LogTimingProfiler, Log, "[IO] Sorted file activities and events in %ls.", *FormatTimeAuto(Stopwatch.GetAccumulatedTime()));

		if (FileActivities.Num() > 0)
		{
			//////////////////////////////////////////////////
			// Compute depth for file activities (avoids overlaps).

			Stopwatch.Restart();

			struct FLane
			{
				double EndTime = 0.0f;
				double CloseStartTime;
				double CloseEndTime;
			};

			TArray<FLane> Lanes; // one lane per event depth, a file activity occupies multiple lanes

			for (const TSharedPtr<FIoFileActivity>& FileActivityPtr : FileActivities)
			{
				FIoFileActivity& Activity = *FileActivityPtr;

				// Find lane (avoiding overlaps with other file activities).
				int32 Depth = 0;
				while (Depth < Lanes.Num())
				{
					bool bOverlap = false;
					for (int32 LocalDepth = 0; LocalDepth < Activity.MaxConcurrentEvents; ++LocalDepth)
					{
						if (Depth + LocalDepth >= Lanes.Num())
						{
							break;
						}
						const FLane& Lane = Lanes[Depth + LocalDepth];
						if (Activity.StartTime < Lane.EndTime ||
							(Activity.StartTime < Lane.CloseEndTime && Activity.EndTime > Lane.CloseStartTime)) // overlaps with a Close event
						{
							bOverlap = true;
							Depth += LocalDepth;
							break;
						}
					}
					if (!bOverlap)
					{
						break;
					}
					++Depth;
				}

				int32 NewLaneNum = Depth + Activity.MaxConcurrentEvents;

				if (NewLaneNum > MaxLanes)
				{
					// Snap to the bottom; allows overlaps in this case.
					Activity.StartingDepth = MaxLanes - Activity.MaxConcurrentEvents;
				}
				else
				{
					if (NewLaneNum > Lanes.Num())
					{
						Lanes.AddDefaulted(NewLaneNum - Lanes.Num());
					}

					Activity.StartingDepth = Depth;

					// Set close event only for first lane of the activity.
					Lanes[Depth].CloseStartTime = Activity.CloseStartTime;
					Lanes[Depth].CloseEndTime = Activity.CloseEndTime;

					for (int32 LocalDepth = 0; LocalDepth < Activity.MaxConcurrentEvents; ++LocalDepth)
					{
						Lanes[Depth + LocalDepth].EndTime = Activity.EndTime;
					}
				}
			}

			Stopwatch.Stop();
			UE_LOGF(LogTimingProfiler, Log, "[IO] Computed layout for file activities in %ls.", *FormatTimeAuto(Stopwatch.GetAccumulatedTime()));

			//////////////////////////////////////////////////

			Stopwatch.Restart();

			for (FIoTimingEvent& Event : AllIoEvents)
			{
				Event.Depth += FileActivities[Event.FileActivityIndex]->StartingDepth;
				ensure(Event.Depth < MaxLanes);
			}

			Stopwatch.Stop();
			UE_LOGF(LogTimingProfiler, Log, "[IO] Updated depth for events in %ls.", *FormatTimeAuto(Stopwatch.GetAccumulatedTime()));
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileActivitySharedState::ExtendOtherTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InOutMenuBuilder)
{
	if (&InSession != TimingView)
	{
		return;
	}

	BuildSubMenu(InOutMenuBuilder);

	//InOutMenuBuilder.BeginSection("File Activity");
	//{
	//	InOutMenuBuilder.AddSubMenu(
	//		LOCTEXT("FileActivity_SubMenu", "File Activity"),
	//		LOCTEXT("FileActivity_SubMenu_Desc", "File Activity track options"),
	//		FNewMenuDelegate::CreateSP(this, &FFileActivitySharedState::BuildSubMenu),
	//		false,
	//		FSlateIcon()
	//	);
	//}
	//InOutMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileActivitySharedState::BindCommands()
{
	FFileActivityTimingViewCommands::Register();

	TSharedPtr<FUICommandList> CommandList = TimingView->GetCommandList();
	ensure(CommandList.IsValid());

	// This command is used only for its key binding (to toggle both ShowHideIoOverviewTrack and ShowHideIoActivityTrack in the same time).
	CommandList->MapAction(
		FFileActivityTimingViewCommands::Get().ShowHideAllIoTracks,
		FExecuteAction::CreateSP(this, &FFileActivitySharedState::ShowHideAllIoTracks),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FFileActivitySharedState::IsAllIoTracksToggleOn));

	CommandList->MapAction(
		FFileActivityTimingViewCommands::Get().ShowHideIoOverviewTrack,
		FExecuteAction::CreateSP(this, &FFileActivitySharedState::ShowHideIoOverviewTrack),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FFileActivitySharedState::IsIoOverviewTrackVisible));

	CommandList->MapAction(
		FFileActivityTimingViewCommands::Get().ToggleOnlyErrors,
		FExecuteAction::CreateSP(this, &FFileActivitySharedState::ToggleOnlyErrors),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FFileActivitySharedState::IsOnlyErrorsToggleOn));

	CommandList->MapAction(
		FFileActivityTimingViewCommands::Get().ShowHideIoActivityTrack,
		FExecuteAction::CreateSP(this, &FFileActivitySharedState::ShowHideIoActivityTrack),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FFileActivitySharedState::IsIoActivityTrackVisible));

	CommandList->MapAction(
		FFileActivityTimingViewCommands::Get().ToggleBackgroundEvents,
		FExecuteAction::CreateSP(this, &FFileActivitySharedState::ToggleBackgroundEvents),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FFileActivitySharedState::AreBackgroundEventsVisible));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileActivitySharedState::BuildSubMenu(FMenuBuilder& InOutMenuBuilder)
{
	InOutMenuBuilder.BeginSection("File Activity", LOCTEXT("ContextMenu_Section_FileActivity", "File Activity"));
	{
		// Note: We use the custom AddMenuEntry in order to set the same key binding text for multiple menu items.

		//InOutMenuBuilder.AddMenuEntry(FFileActivityTimingViewCommands::Get().ShowHideIoOverviewTrack);
		FInsightsMenuBuilder::AddMenuEntry(InOutMenuBuilder,
			FUIAction(
				FExecuteAction::CreateSP(this, &FFileActivitySharedState::ShowHideIoOverviewTrack),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FFileActivitySharedState::IsIoOverviewTrackVisible)),
			FFileActivityTimingViewCommands::Get().ShowHideIoOverviewTrack->GetLabel(),
			FFileActivityTimingViewCommands::Get().ShowHideIoOverviewTrack->GetDescription(),
			FFileActivityTimingViewCommands::Get().ShowHideAllIoTracks->GetInputText().ToUpper(), // use same key binding
			EUserInterfaceActionType::ToggleButton);

		InOutMenuBuilder.AddMenuEntry(FFileActivityTimingViewCommands::Get().ToggleOnlyErrors);

		//InOutMenuBuilder.AddMenuEntry(FFileActivityTimingViewCommands::Get().ShowHideIoActivityTrack);
		FInsightsMenuBuilder::AddMenuEntry(InOutMenuBuilder,
			FUIAction(
				FExecuteAction::CreateSP(this, &FFileActivitySharedState::ShowHideIoActivityTrack),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FFileActivitySharedState::IsIoActivityTrackVisible)),
			FFileActivityTimingViewCommands::Get().ShowHideIoActivityTrack->GetLabel(),
			FFileActivityTimingViewCommands::Get().ShowHideIoActivityTrack->GetDescription(),
			FFileActivityTimingViewCommands::Get().ShowHideAllIoTracks->GetInputText().ToUpper(), // use same key binding
			EUserInterfaceActionType::ToggleButton);

		InOutMenuBuilder.AddMenuEntry(FFileActivityTimingViewCommands::Get().ToggleBackgroundEvents);
	}
	InOutMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileActivitySharedState::SetAllIoTracksToggle(bool bOnOff)
{
	bShowHideAllIoTracks = bOnOff;

	if (IoOverviewTrack.IsValid())
	{
		IoOverviewTrack->SetVisibilityFlag(bShowHideAllIoTracks);
	}
	if (IoActivityTrack.IsValid())
	{
		IoActivityTrack->SetVisibilityFlag(bShowHideAllIoTracks);
	}

	if (TimingView)
	{
		TimingView->HandleTrackVisibilityChanged();
	}

	if (bShowHideAllIoTracks)
	{
		RequestUpdate();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFileActivitySharedState::IsIoOverviewTrackVisible() const
{
	return IoOverviewTrack && IoOverviewTrack->IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileActivitySharedState::ShowHideIoOverviewTrack()
{
	if (IoOverviewTrack.IsValid())
	{
		IoOverviewTrack->ToggleVisibility();
	}

	if (TimingView)
	{
		TimingView->HandleTrackVisibilityChanged();
	}

	const bool bIsOverviewTrackVisible = IsIoOverviewTrackVisible();
	const bool bIsActivityTrackVisible = IsIoActivityTrackVisible();

	if (bIsOverviewTrackVisible == bIsActivityTrackVisible)
	{
		bShowHideAllIoTracks = bIsOverviewTrackVisible;
	}

	if (bIsOverviewTrackVisible)
	{
		RequestUpdate();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFileActivitySharedState::IsIoActivityTrackVisible() const
{
	return IoActivityTrack && IoActivityTrack->IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileActivitySharedState::ShowHideIoActivityTrack()
{
	if (IoActivityTrack.IsValid())
	{
		IoActivityTrack->ToggleVisibility();
	}

	if (TimingView)
	{
		TimingView->HandleTrackVisibilityChanged();
	}

	const bool bIsOverviewTrackVisible = IsIoOverviewTrackVisible();
	const bool bIsActivityTrackVisible = IsIoActivityTrackVisible();

	if (bIsOverviewTrackVisible == bIsActivityTrackVisible)
	{
		bShowHideAllIoTracks = bIsOverviewTrackVisible;
	}

	if (bIsActivityTrackVisible)
	{
		RequestUpdate();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFileActivitySharedState::IsOnlyErrorsToggleOn() const
{
	return IoOverviewTrack && IoOverviewTrack->IsOnlyErrorsToggleOn();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileActivitySharedState::ToggleOnlyErrors()
{
	if (IoOverviewTrack)
	{
		IoOverviewTrack->ToggleOnlyErrors();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFileActivitySharedState::AreBackgroundEventsVisible() const
{
	return IoActivityTrack && IoActivityTrack->AreBackgroundEventsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFileActivitySharedState::ToggleBackgroundEvents()
{
	if (IoActivityTrack)
	{
		IoActivityTrack->ToggleBackgroundEvents();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
