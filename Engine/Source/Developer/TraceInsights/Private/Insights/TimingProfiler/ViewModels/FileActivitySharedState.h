// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Templates/SharedPointer.h"

// TraceInsights
#include "Insights/ITimingViewExtender.h"

namespace TraceServices
{
	class IAnalysisSession;
}

namespace UE::Insights::TimingProfiler
{

class FOverviewFileActivityTimingTrack;
class FDetailedFileActivityTimingTrack;
class FFileActivityTimingTrack;
class STimingView;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FIoFileActivity
{
	uint64 Id;
	const TCHAR* Path;
	double StartTime;
	double EndTime;
	double CloseStartTime;
	double CloseEndTime;
	int32 EventCount;
	int32 Index;	// Different FIoFileActivity may have the same Index if their operations don't overlap in time
	int32 MaxConcurrentEvents; // e.g. overlapped IO reads
	uint32 StartingDepth; // Depth of first event on this file
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FIoTimingEvent
{
	double StartTime;
	double EndTime;
	uint32 Depth; // During update, this is local within a track - then it's set to a global depth
	uint32 Type; // TraceServices::EFileActivityType + "Failed" flag
	uint64 Offset;
	uint64 Size;
	uint64 ActualSize;
	int32 FileActivityIndex;
	uint64 FileHandle; // file handle
	uint64 ReadWriteHandle; // for Read/Write operations
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFileActivitySharedState : public Timing::ITimingViewExtender, public TSharedFromThis<FFileActivitySharedState>
{
	friend class FOverviewFileActivityTimingTrack;
	friend class FDetailedFileActivityTimingTrack;
	friend class FFileActivityTimingTrack;

public:
	explicit FFileActivitySharedState(STimingView* InTimingView) : TimingView(InTimingView) {}
	virtual ~FFileActivitySharedState() override = default;

	//////////////////////////////////////////////////
	// ITimingViewExtender interface

	virtual void OnBeginSession(Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(Timing::ITimingViewSession& InSession) override;
	virtual void Tick(Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;
	virtual void ExtendOtherTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InOutMenuBuilder) override;

	//////////////////////////////////////////////////

	void BindCommands();

	const TArray<FIoTimingEvent>& GetAllEvents() const { return AllIoEvents; }

	void RequestUpdate() { bForceIoEventsUpdate = true; }

	bool IsAllIoTracksToggleOn() const { return bShowHideAllIoTracks; }
	void SetAllIoTracksToggle(bool bOnOff);
	void ShowAllIoTracks() { SetAllIoTracksToggle(true); }
	void HideAllIoTracks() { SetAllIoTracksToggle(false); }
	void ShowHideAllIoTracks() { SetAllIoTracksToggle(!IsAllIoTracksToggleOn()); }

	bool IsIoOverviewTrackVisible() const;
	void ShowHideIoOverviewTrack();

	bool IsIoActivityTrackVisible() const;
	void ShowHideIoActivityTrack();

	bool IsOnlyErrorsToggleOn() const;
	void ToggleOnlyErrors();

	bool AreBackgroundEventsVisible() const;
	void ToggleBackgroundEvents();

	static const uint32 MaxLanes;

private:
	void BuildSubMenu(FMenuBuilder& InOutMenuBuilder);

private:
	STimingView* TimingView;

	TSharedPtr<FOverviewFileActivityTimingTrack> IoOverviewTrack;
	TSharedPtr<FDetailedFileActivityTimingTrack> IoActivityTrack;

	bool bShowHideAllIoTracks;
	bool bForceIoEventsUpdate;

	TArray<TSharedPtr<FIoFileActivity>> FileActivities;
	TMap<uint64, TSharedPtr<FIoFileActivity>> FileActivityMap;

	/** All IO events, cached. */
	TArray<FIoTimingEvent> AllIoEvents;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
