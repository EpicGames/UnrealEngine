// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameStatsHelper.h"

// TraceServices
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsights
#include "Insights/InsightsManager.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameStatsHelper::ComputeFrameStatsForTimer(TArray<FFrameStatsCachedEvent>& FrameStatsEvents, uint32 TimerId, const TSet<uint32>& Timelines)
{
	if (FrameStatsEvents.Num() == 0)
	{
		return;
	}

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			Session->ReadAccessCheck();
			const double SessionDuration = Session->GetDurationSeconds();

			for (uint32 TimelineIndex : Timelines)
			{
				TimingProfilerProvider->ReadTimeline(TimelineIndex,
					[&FrameStatsEvents, TimerId, SessionDuration, TimingProfilerProvider]
					(const TraceServices::ITimingProfilerTimeline& Timeline)
					{
						const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();
						ProcessTimeline(FrameStatsEvents, TimerId, 0.0, SessionDuration, Timeline, TimerReader);
					});
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameStatsHelper::ComputeFrameStatsForTimer(TArray<FFrameStatsCachedEvent>& FrameStatsEvents, uint32 TimerId)
{
	if (FrameStatsEvents.Num() == 0)
	{
		return;
	}

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			Session->ReadAccessCheck();
			const double SessionDuration = Session->GetDurationSeconds();

			TimingProfilerProvider->EnumerateTimelines(
				[&FrameStatsEvents, TimerId, SessionDuration, TimingProfilerProvider]
				(const TraceServices::ITimingProfilerTimeline& Timeline)
				{
					const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();
					ProcessTimeline(FrameStatsEvents, TimerId, 0.0, SessionDuration, Timeline, TimerReader);
				});
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameStatsHelper::ComputeFrameStatsForTimer(TArray<FFrameStatsCachedEvent>& FrameStatsEvents, uint32 TimerId, const TraceServices::ITimingProfilerTimeline& Timeline)
{
	if (FrameStatsEvents.Num() == 0)
	{
		return;
	}

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			Session->ReadAccessCheck();
			const double SessionDuration = Session->GetDurationSeconds();

			const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();
			ProcessTimeline(FrameStatsEvents, TimerId, 0.0, SessionDuration, Timeline, TimerReader);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameStatsHelper::ProcessTimeline(
	TArray<FFrameStatsCachedEvent>& FrameStatsEvents,
	uint32 TimerId,
	double StartTime,
	double EndTime,
	const TraceServices::ITimingProfilerTimeline& Timeline,
	const TraceServices::ITimingProfilerTimerReader& TimerReader)
{
	struct FEnumerateAsyncTaskData
	{
		double StartTime;
		int32 NestedDepth = 0;
	};

	TArray<FEnumerateAsyncTaskData> DataArray;

	TraceServices::ITimingProfilerTimeline::EnumerateAsyncParams Params;
	Params.IntervalStart = StartTime;
	Params.IntervalEnd = EndTime;
	Params.Resolution = 0.0;
	Params.SetupCallback =
		[&DataArray]
		(uint32 NumTasks)
		{
			DataArray.AddDefaulted(NumTasks);
		};
	Params.EventCallback =
		[&TimerReader, &FrameStatsEvents, TimerId, &DataArray]
		(bool bIsEnter, double Time, const TraceServices::FTimingProfilerEvent& Event, uint32 TaskIndex)
		{
			const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(Event.TimerIndex);
			if (ensure(Timer != nullptr))
			{
				if (Timer->Id == TimerId)
				{
					FEnumerateAsyncTaskData& CurrentTaskData = DataArray[TaskIndex];
					if (bIsEnter)
					{
						if (CurrentTaskData.NestedDepth == 0)
						{
							CurrentTaskData.StartTime = Time;
						}

						++CurrentTaskData.NestedDepth;
					}
					else
					{
						check(CurrentTaskData.NestedDepth > 0);
						if (--CurrentTaskData.NestedDepth > 0)
						{
							return TraceServices::EEventEnumerate::Continue;
						}

						int32 Index = Algo::UpperBoundBy(FrameStatsEvents, CurrentTaskData.StartTime, &FFrameStatsCachedEvent::FrameStartTime);
						if (Index > 0)
						{
							--Index;
						}

						// This can happen when the event is between frames.
						if (CurrentTaskData.StartTime > FrameStatsEvents[Index].FrameEndTime)
						{
							Index++;
							if (Index >= FrameStatsEvents.Num())
							{
								return TraceServices::EEventEnumerate::Continue;
							}
						}

						double EndTime = Time;
						do
						{
							FFrameStatsCachedEvent& Entry = FrameStatsEvents[Index];

							if (EndTime < Entry.FrameStartTime)
							{
								return TraceServices::EEventEnumerate::Continue;
							}

							if (CurrentTaskData.StartTime < Entry.FrameStartTime)
							{
								CurrentTaskData.StartTime = Entry.FrameStartTime;
							}

							const double Duration = FMath::Min(EndTime, Entry.FrameEndTime) - CurrentTaskData.StartTime;
							ensure(Duration >= 0.0f);
							for (double Value = Entry.Duration.load(); !Entry.Duration.compare_exchange_strong(Value, Value + Duration););

							Index++;
						}
						while (Index < FrameStatsEvents.Num());
					}
				}
			}
			return TraceServices::EEventEnumerate::Continue;
		};

	Timeline.EnumerateEventsDownSampledAsync(Params);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
