// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"

// TraceServices
#include "TraceServices/Model/TimingProfiler.h"

#include <atomic>

namespace UE::Insights::TimingProfiler
{

struct FFrameStatsCachedEvent
{
	FFrameStatsCachedEvent()
		: FrameStartTime(0.0)
		, FrameEndTime(0.0)
	{
		Duration.store(0.0);
	}

	FFrameStatsCachedEvent(const FFrameStatsCachedEvent& Other)
		: FrameStartTime(Other.FrameStartTime)
		, FrameEndTime(Other.FrameEndTime)
	{
		Duration.store(Other.Duration.load());
	}

	double FrameStartTime;
	double FrameEndTime;

	// The sum of the all the instances of a timing event in a frame.
	std::atomic<double> Duration;
};

class FFrameStatsHelper
{
public:
	static void ComputeFrameStatsForTimer(TArray<FFrameStatsCachedEvent>& FrameStatsEvents, uint32 TimerId, const TSet<uint32>& Timelines);
	static void ComputeFrameStatsForTimer(TArray<FFrameStatsCachedEvent>& FrameStatsEvents, uint32 TimerId);
	static void ComputeFrameStatsForTimer(TArray<FFrameStatsCachedEvent>& FrameStatsEvents, uint32 TimerId, const TraceServices::ITimingProfilerTimeline& Timeline);

private:
	static void ProcessTimeline(TArray<FFrameStatsCachedEvent>& FrameStatsEvents,
								uint32 TimerId,
								double StartTime,
								double EndTime,
								const TraceServices::ITimingProfilerTimeline& Timeline,
								const TraceServices::ITimingProfilerTimerReader& TimerReader);
};

} // namespace UE::Insights::TimingProfiler
