// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameTimingTrack.h"

#include "Fonts/FontMeasure.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Styling/SlateBrush.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/TimingProfiler/ViewModels/FrameTrackHelper.h"
#include "Insights/TimingProfiler/ViewModels/TimerNode.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::FrameTimingTrack"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FFrameTimingViewCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

FFrameTimingViewCommands::FFrameTimingViewCommands()
: TCommands<FFrameTimingViewCommands>(
	TEXT("FrameTimingViewCommands"),
	NSLOCTEXT("Contexts", "FrameTimingViewCommands", "Insights - Timing View - Frames"),
	NAME_None,
	FInsightsStyle::GetStyleSetName())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FFrameTimingViewCommands::~FFrameTimingViewCommands()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// UI_COMMAND takes long for the compiler to optimize
UE_DISABLE_OPTIMIZATION_SHIP
void FFrameTimingViewCommands::RegisterCommands()
{
	UI_COMMAND(ShowHideAllFrameTracks,
		"Frame Tracks",
		"Shows/hides all Frame tracks.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::R));
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////
// FFrameSharedState
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FFrameTimingTrack> FFrameSharedState::GetFrameTrack(uint32 InFrameType)
{
	TSharedPtr<FFrameTimingTrack>*const TrackPtrPtr = FrameTracks.Find(InFrameType);
	return TrackPtrPtr ? *TrackPtrPtr : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFrameSharedState::IsFrameTrackVisible(uint32 InFrameType) const
{
	const TSharedPtr<FFrameTimingTrack>*const TrackPtrPtr = FrameTracks.Find(InFrameType);
	return TrackPtrPtr && (*TrackPtrPtr)->IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameSharedState::OnBeginSession(Timing::ITimingViewSession& InSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	bShowHideAllFrameTracks = false;
	FrameTracks.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameSharedState::OnEndSession(Timing::ITimingViewSession& InSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	bShowHideAllFrameTracks = false;
	FrameTracks.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameSharedState::Tick(Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	bool bTracksOrderChanged = false;
	int32 Order = FTimingTrackOrder::First + 300;
	constexpr int32 OrderIncrement = 10;
	static_assert(FTimingTrackOrder::GroupRange > 100 + OrderIncrement * TraceFrameType_Count, "Order group range too small");

	// Create a track for each available frame type.
	for (uint32 FrameType = 0; FrameType < TraceFrameType_Count; ++FrameType)
	{
		TSharedPtr<FFrameTimingTrack>* TrackPtrPtr = FrameTracks.Find(FrameType);
		if (!TrackPtrPtr)
		{
			const FString TrackName = FFrameTimingTrack::GetFrameTrackName(FrameType);
			TSharedPtr<FFrameTimingTrack> Track = MakeShared<FFrameTimingTrack>(*this, TrackName, FrameType);
			Track->SetOrder(Order);
			FrameTracks.Add(FrameType, Track);
			Track->SetVisibilityFlag(bShowHideAllFrameTracks);
			InSession.AddScrollableTrack(Track);
		}
		else
		{
			TSharedPtr<FFrameTimingTrack> Track = *TrackPtrPtr;
			if (Track->GetOrder() != Order)
			{
				Track->SetOrder(Order);
				bTracksOrderChanged = true;
			}
		}

		Order += OrderIncrement;
	}

	if (bTracksOrderChanged)
	{
		InSession.InvalidateScrollableTracksOrder();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameSharedState::ExtendOtherTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InOutMenuBuilder)
{
	if (&InSession != TimingView)
	{
		return;
	}

	InOutMenuBuilder.BeginSection("FrameTracks", LOCTEXT("ContextMenu_Section_FrameTracks", "Frames"));
	{
		InOutMenuBuilder.AddMenuEntry(FFrameTimingViewCommands::Get().ShowHideAllFrameTracks);
	}
	InOutMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameSharedState::BindCommands()
{
	FFrameTimingViewCommands::Register();

	TSharedPtr<FUICommandList> CommandList = TimingView->GetCommandList();
	ensure(CommandList.IsValid());

	CommandList->MapAction(
		FFrameTimingViewCommands::Get().ShowHideAllFrameTracks,
		FExecuteAction::CreateSP(this, &FFrameSharedState::ShowHideAllFrameTracks),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FFrameSharedState::IsAllFrameTracksToggleOn));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameSharedState::SetAllFrameTracksToggle(bool bOnOff)
{
	bShowHideAllFrameTracks = bOnOff;

	for (const auto& KV : FrameTracks)
	{
		FFrameTimingTrack& Track = *KV.Value;
		Track.SetVisibilityFlag(bShowHideAllFrameTracks);
	}

	if (TimingView)
	{
		TimingView->HandleTrackVisibilityChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FFrameTimingTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FFrameTimingTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTimingTrack::Reset()
{
	FTimingEventsTrack::Reset();

	Header.Reset();
	Header.SetCanBeCollapsed(true);
	Header.SetIsCollapsed(true);
	Header.UpdateSize();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTimingTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(*Session.Get());

		const FTimingTrackViewport& Viewport = Context.GetViewport();

		const TArray64<double>& FrameStartTimes = FramesProvider.GetFrameStartTimes(static_cast<ETraceFrameType>(FrameType));

		const int64 StartLowerBound = Algo::LowerBound(FrameStartTimes, Viewport.GetStartTime());
		const uint64 StartIndex = (StartLowerBound > 0) ? StartLowerBound - 1 : 0;

		const int64 EndLowerBound = Algo::LowerBound(FrameStartTimes, Viewport.GetEndTime());
		const uint64 EndIndex = EndLowerBound;

		const uint32 Color = FFrameTrackDrawHelper::GetColor32ByFrameType(FrameType);

		FramesProvider.EnumerateFrames(static_cast<ETraceFrameType>(FrameType), StartIndex, EndIndex,
			[this, Color, &Builder]
			(const TraceServices::FFrame& Frame)
			{
				constexpr uint32 Depth = 0;
				const FString FrameName = GetShortFrameName(Frame.Index);
				const uint64 IndexAsFrameType = Frame.Index;
				Builder.AddEvent(Frame.StartTime, Frame.EndTime, Depth, *FrameName, IndexAsFrameType, Color);
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTimingTrack::BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	const TSharedPtr<ITimingEventFilter> EventFilterPtr = Context.GetEventFilter();
	if (EventFilterPtr.IsValid() && EventFilterPtr->FilterTrack(*this))
	{
		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(*Session.Get());

			const FTimingTrackViewport& Viewport = Context.GetViewport();

			const TArray64<double>& FrameStartTimes = FramesProvider.GetFrameStartTimes(static_cast<ETraceFrameType>(FrameType));

			const int64 StartLowerBound = Algo::LowerBound(FrameStartTimes, Viewport.GetStartTime());
			const uint64 StartIndex = (StartLowerBound > 0) ? StartLowerBound - 1 : 0;

			const int64 EndLowerBound = Algo::LowerBound(FrameStartTimes, Viewport.GetEndTime());
			const uint64 EndIndex = EndLowerBound;

			const uint32 Color = FFrameTrackDrawHelper::GetColor32ByFrameType(FrameType);

			FramesProvider.EnumerateFrames(static_cast<ETraceFrameType>(FrameType), StartIndex, EndIndex,
				[this, Color, &Builder, &EventFilterPtr]
				(const TraceServices::FFrame& Frame)
				{
					constexpr uint32 Depth = 0;
					const uint64 IndexAsEventType = Frame.Index;

					FTimingEvent TimingEvent(SharedThis(this), Frame.StartTime, Frame.EndTime, Depth, IndexAsEventType);

					if (EventFilterPtr->FilterEvent(TimingEvent))
					{
						const FString FrameName = GetShortFrameName(Frame.Index);
						Builder.AddEvent(Frame.StartTime, Frame.EndTime, Depth, *FrameName, IndexAsEventType, Color);
					}
				});
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTimingTrack::Update(const ITimingTrackUpdateContext& Context)
{
	FTimingEventsTrack::Update(Context);

	Header.SetFontScale(Context.GetGeometry().Scale);
	Header.UpdateSize();
	Header.Update(Context);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTimingTrack::PostUpdate(const ITimingTrackUpdateContext& Context)
{
	FTimingEventsTrack::PostUpdate(Context);

	Header.PostUpdate(Context);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTimingTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	if (GetHeight() > 0)
	{
		DrawEvents(Context, 1.0f);

		Header.Draw(Context);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTimingTrack::PostDraw(const ITimingTrackDrawContext& Context) const
{
	if (!Header.IsCollapsed())
	{
		const FTimingTrackViewport& Viewport = Context.GetViewport();
		DrawMarkers(Context, Viewport.GetPosY(), Viewport.GetHeight());
	}

	const TSharedPtr<const ITimingEvent> SelectedEventPtr = Context.GetSelectedEvent();
	if (SelectedEventPtr.IsValid() &&
		SelectedEventPtr->CheckTrack(this) &&
		SelectedEventPtr->Is<FTimingEvent>())
	{
		const FTimingEvent& SelectedEvent = SelectedEventPtr->As<FTimingEvent>();
		const ITimingViewDrawHelper& Helper = Context.GetHelper();
		const float FontScale = Context.GetDrawContext().Geometry.Scale;
		DrawSelectedEventInfo(SelectedEvent, Context.GetViewport(), Context.GetDrawContext(), Helper.GetWhiteBrush(), Helper.GetEventFont());
	}

	Header.PostDraw(Context);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTimingTrack::DrawSelectedEventInfo(const FTimingEvent& SelectedEvent, const FTimingTrackViewport& Viewport, const FDrawContext& DrawContext, const FSlateBrush* WhiteBrush, const FSlateFontInfo& Font) const
{
	FindFrame(SelectedEvent,
		[this, &SelectedEvent, &Font, &Viewport, &DrawContext, &WhiteBrush]
		(double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const TraceServices::FFrame& InFoundFrame)
		{
			//const double Duration = SelectedEvent.GetDuration();
			const double Duration = InFoundFrame.EndTime - InFoundFrame.StartTime;
			const FString Str = GetCompleteFrameName(InFoundFrame.Index, Duration);

			const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			const float FontScale = DrawContext.Geometry.Scale;
			const FVector2D Size = FontMeasureService->Measure(Str, Font, FontScale) / FontScale;
			const float W = static_cast<float>(Size.X);
			const float H = static_cast<float>(Size.Y);
			const float X = Viewport.GetWidth() - W - 23.0f;
			const float Y = Viewport.GetPosY() + Viewport.GetHeight() - H - 18.0f;

			const FLinearColor BackgroundColor(0.05f, 0.05f, 0.05f, 1.0f);
			const FLinearColor TextColor(0.7f, 0.7f, 0.7f, 1.0f);

			DrawContext.DrawBox(X - 8.0f, Y - 2.0f, W + 16.0f, H + 4.0f, WhiteBrush, BackgroundColor);
			DrawContext.LayerId++;

			DrawContext.DrawText(X, Y, Str, Font, TextColor);
			DrawContext.LayerId++;
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply FFrameTimingTrack::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (IsVisible() && IsHeaderHovered())
		{
			ToggleCollapsed();
			Reply = FReply::Handled();
		}
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply FFrameTimingTrack::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return OnMouseButtonDown(MyGeometry, MouseEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTimingTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	if (InTooltipEvent.CheckTrack(this) && InTooltipEvent.Is<FTimingEvent>())
	{
		const FTimingEvent& TooltipEvent = InTooltipEvent.As<FTimingEvent>();

		FindFrame(TooltipEvent,
			[this, &InOutTooltip, &TooltipEvent]
			(double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const TraceServices::FFrame& InFoundFrame)
			{
				InOutTooltip.ResetContent();

				const FString FrameName = GetFrameName(FrameType, InFoundFrame.Index);
				InOutTooltip.AddTitle(FrameName);

				//const double Duration = TooltipEvent.GetDuration();
				const double Duration = InFoundFrame.EndTime - InFoundFrame.StartTime;
				InOutTooltip.AddNameValueTextLine(TEXT("Duration:"), FormatTimeAuto(Duration));

				InOutTooltip.UpdateLayout();
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FFrameTimingTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
{
	TSharedPtr<const ITimingEvent> FoundEvent;

	FindFrame(InSearchParameters,
		[this, &FoundEvent]
		(double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const TraceServices::FFrame& InFoundFrame)
		{
			const uint64 IndexAsEventType = InFoundFrame.Index; // storing the frame index as event type
			FoundEvent = MakeShared<FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth, IndexAsEventType);
		});

	return FoundEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTimingTrack::OnEventSelected(const ITimingEvent& InSelectedEvent) const
{
	//if (InSelectedEvent.CheckTrack(this) && InSelectedEvent.Is<FTimingEvent>())
	//{
	//	const FTimingEvent& TrackEvent = InSelectedEvent.As<FTimingEvent>();
	//	const uint64 FrameIndex = TrackEvent.GetType(); // the frame index was stored as event type
	//}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTimingTrack::OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const
{
	if (InSelectedEvent.CheckTrack(this) && InSelectedEvent.Is<FTimingEvent>())
	{
		const FTimingEvent& TrackEvent = InSelectedEvent.As<FTimingEvent>();

		const uint64 FrameIndex = TrackEvent.GetType(); // using event type as frame index
		const FString FrameName = GetFrameName(FrameType, FrameIndex);

		// Copy name of selected frame to clipboard.
		FPlatformApplicationMisc::ClipboardCopy(*FrameName);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTimingTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection(TEXT("Misc"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ContextMenu_ToggleCollapsed", "Collapsed"),
			LOCTEXT("ContextMenu_ToggleCollapsed_Desc", "Whether the vertical marker lines (the start and the end of each frame) are collapsed or expanded."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FFrameTimingTrack::ToggleCollapsed),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FFrameTimingTrack::IsCollapsed)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFrameTimingTrack::FindFrame(const FTimingEvent& InTimingEvent, TFunctionRef<void(double, double, uint32, const TraceServices::FFrame&)> InFoundPredicate) const
{
	auto MatchEvent = [&InTimingEvent](double InStartTime, double InEndTime, uint32 InDepth)
	{
		return InDepth == InTimingEvent.GetDepth()
			&& InStartTime == InTimingEvent.GetStartTime()
			&& InEndTime == InTimingEvent.GetEndTime();
	};

	FTimingEventSearchParameters SearchParameters(InTimingEvent.GetStartTime(), InTimingEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch, MatchEvent);
	SearchParameters.SearchHandle = &InTimingEvent.GetSearchHandle();
	return FindFrame(SearchParameters, InFoundPredicate);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FFrameTimingTrack::FindFrame(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const TraceServices::FFrame&)> InFoundPredicate) const
{
	return TTimingEventSearch<TraceServices::FFrame>::Search(

		InParameters,

		// Search Predicate
		[this]
		(TTimingEventSearch<TraceServices::FFrame>::FContext& InContext)
		{
			TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
			if (Session.IsValid())
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
				const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(*Session.Get());

				const TArray64<double>& FrameStartTimes = FramesProvider.GetFrameStartTimes(static_cast<ETraceFrameType>(FrameType));

				const int64 StartLowerBound = Algo::LowerBound(FrameStartTimes, InContext.GetParameters().StartTime);
				const uint64 StartIndex = (StartLowerBound > 0) ? StartLowerBound - 1 : 0;

				const int64 EndLowerBound = Algo::LowerBound(FrameStartTimes, InContext.GetParameters().EndTime);
				const uint64 EndIndex = EndLowerBound;

				FramesProvider.EnumerateFrames(static_cast<ETraceFrameType>(FrameType), StartIndex, EndIndex,
					[&InContext]
					(const TraceServices::FFrame& Frame)
					{
						if (Frame.StartTime < InContext.GetParameters().EndTime &&
							Frame.EndTime > InContext.GetParameters().StartTime)
						{
							constexpr uint32 Depth = 0;
							InContext.Check(Frame.StartTime, Frame.EndTime, Depth, Frame);
						}
					});
			}
		},

		// No Filter Predicate

		// Found Predicate
		InFoundPredicate,

		// No Payload Matched Predicate

		// Cache
		SearchCache

	); // Search
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FString FFrameTimingTrack::GetShortFrameName(const uint64 InFrameIndex) const
{
	return FString::Printf(TEXT("%" UINT64_FMT), InFrameIndex);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FFrameTimingTrack::GetFrameName(const uint32 FrameType, const uint64 InFrameIndex)
{
	return FString::Printf(TEXT("%s Frame %" UINT64_FMT), FFrameTrackDrawHelper::FrameTypeToString(FrameType), InFrameIndex);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FString FFrameTimingTrack::GetCompleteFrameName(const uint64 InFrameIndex, const double InFrameDuration) const
{
	return FString::Printf(TEXT("%s Frame %" UINT64_FMT " (%s)"), FFrameTrackDrawHelper::FrameTypeToString(FrameType), InFrameIndex, *FormatTimeAuto(InFrameDuration));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FFrameTimingTrack::GetFrameTrackName(uint32 FrameType)
{
	return FString::Printf(TEXT("%s Frames"), FFrameTrackDrawHelper::FrameTypeToString(FrameType));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
