// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/Commands.h"

// TraceServices
#include "TraceServices/Model/Frames.h"

// TraceInsights
#include "Insights/ITimingViewExtender.h"
#include "Insights/ViewModels/TimingEventSearch.h" // for TTimingEventSearchCache
#include "Insights/ViewModels/TimingEventsTrack.h"
#include "Insights/ViewModels/TrackHeader.h"

struct FSlateBrush;

class FTimingEvent;
class FTimingEventSearchParameters;

namespace UE::Insights::TimingProfiler
{

class FFrameTimingTrack;
class STimingView;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFrameTimingViewCommands : public TCommands<FFrameTimingViewCommands>
{
public:
	FFrameTimingViewCommands();
	virtual ~FFrameTimingViewCommands();
	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> ShowHideAllFrameTracks;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFrameSharedState : public Timing::ITimingViewExtender, public TSharedFromThis<FFrameSharedState>
{
public:
	explicit FFrameSharedState(STimingView* InTimingView) : TimingView(InTimingView) {}
	virtual ~FFrameSharedState() = default;

	TSharedPtr<FFrameTimingTrack> GetFrameTrack(uint32 InFrameType);

	bool IsFrameTrackVisible(uint32 InFrameType) const;

	//////////////////////////////////////////////////
	// ITimingViewExtender interface

	virtual void OnBeginSession(Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(Timing::ITimingViewSession& InSession) override;
	virtual void Tick(Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;
	virtual void ExtendOtherTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;

	//////////////////////////////////////////////////

	void BindCommands();

	bool IsAllFrameTracksToggleOn() const { return bShowHideAllFrameTracks; }
	void SetAllFrameTracksToggle(bool bOnOff);
	void ShowAllFrameTracks() { SetAllFrameTracksToggle(true); }
	void HideAllFrameTracks() { SetAllFrameTracksToggle(false); }
	void ShowHideAllFrameTracks() { SetAllFrameTracksToggle(!IsAllFrameTracksToggleOn()); }

private:
	STimingView* TimingView;

	bool bShowHideAllFrameTracks;

	/** Maps frame type to track pointer. */
	TMap<uint32, TSharedPtr<FFrameTimingTrack>> FrameTracks;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFrameTimingTrack : public FTimingEventsTrack
{
	INSIGHTS_DECLARE_RTTI(FFrameTimingTrack, FTimingEventsTrack)

public:
	explicit FFrameTimingTrack(FFrameSharedState& InSharedState, const FString& InName, uint32 InFrameType)
		: FTimingEventsTrack(InName)
		, FrameType(InFrameType)
		, Header(*this)
	{
		Reset();
	}

	virtual ~FFrameTimingTrack() = default;

	//////////////////////////////////////////////////

	uint32 GetFrameType() const { return FrameType; }
	//void SetFrameType(uint32 InFrameType) { FrameType = InFrameType; }

	bool IsCollapsed() const { return Header.IsCollapsed(); }
	void Expand() { Header.SetIsCollapsed(false); }
	void Collapse() { Header.SetIsCollapsed(true); }
	void ToggleCollapsed() { Header.ToggleCollapsed(); }

	const FString GetShortFrameName(const uint64 FrameIndex) const;
	static FString GetFrameName(const uint32 FrameType, const uint64 FrameIndex);
	const FString GetCompleteFrameName(const uint64 FrameIndex, const double FrameDuration) const;

	static FString GetFrameTrackName(uint32 FrameType);

	//////////////////////////////////////////////////
	// FBaseTimingTrack/FTimingEventsTrack overrides

	virtual void Reset() override final;

	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;
	virtual void BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;

	virtual void Update(const ITimingTrackUpdateContext& Context) override;
	virtual void PostUpdate(const ITimingTrackUpdateContext& Context) override;

	virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	virtual void PostDraw(const ITimingTrackDrawContext& Context) const override;

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;

	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;

	virtual void OnEventSelected(const ITimingEvent& InSelectedEvent) const override;
	virtual void OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const override;
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;

	//////////////////////////////////////////////////

private:
	void DrawSelectedEventInfo(const FTimingEvent& SelectedEvent, const FTimingTrackViewport& Viewport, const FDrawContext& DrawContext, const FSlateBrush* WhiteBrush, const FSlateFontInfo& Font) const;

	bool FindFrame(const FTimingEvent& InTimingEvent, TFunctionRef<void(double, double, uint32, const TraceServices::FFrame&)> InFoundPredicate) const;
	bool FindFrame(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const TraceServices::FFrame&)> InFoundPredicate) const;

private:
	uint32 FrameType; // ETraceFrameType

	FTrackHeader Header;

	// Search cache
	mutable TTimingEventSearchCache<TraceServices::FFrame> SearchCache;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
