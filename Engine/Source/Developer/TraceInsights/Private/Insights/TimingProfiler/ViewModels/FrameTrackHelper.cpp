// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameTrackHelper.h"

#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/DrawElements.h"

// TraceServices
#include "TraceServices/Model/Frames.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/TimingProfiler/ViewModels/FrameTrackViewport.h"
#include "Insights/ViewModels/DrawHelpers.h"

#include <limits>

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::FrameTrack"

namespace UE::Insights::TimingProfiler
{

///////////////////////////////////////////////////////////////////////////////////////////////////
// FFrameTrackSeries / FTimerFrameStatsTrackSeries
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FFrameTrackSeries)
INSIGHTS_IMPLEMENT_RTTI(FTimerFrameStatsTrackSeries)
	
////////////////////////////////////////////////////////////////////////////////////////////////////
	
////////////////////////////////////////////////////////////////////////////////////////////////////
// FFrameTrackSeriesBuilder
////////////////////////////////////////////////////////////////////////////////////////////////////

FFrameTrackSeriesBuilder::FFrameTrackSeriesBuilder(FFrameTrackSeries& InSeries, const FFrameTrackViewport& InViewport)
	: Series(InSeries)
	, Viewport(InViewport)
	, NumAddedFrames(0)
{
	SampleW = Viewport.GetSampleWidth();
	FramesPerSample = Viewport.GetNumFramesPerSample();
	NumSamples = FMath::Max(0, FMath::CeilToInt(Viewport.GetWidth() / SampleW));
	FirstFrameIndex = Viewport.GetFirstFrameIndex();

	Series.NumAggregatedFrames = 0;
	Series.Samples.Reset();
	Series.Samples.AddDefaulted(NumSamples);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTrackSeriesBuilder::AddFrame(const TraceServices::FFrame& Frame)
{
	NumAddedFrames++;

	const int32 FrameIndex = IntCastChecked<int32>(Frame.Index);

	int32 SampleIndex = (FrameIndex - FirstFrameIndex) / FramesPerSample;
	if (SampleIndex >= 0 && SampleIndex < NumSamples)
	{
		FFrameTrackSample& Sample = Series.Samples[SampleIndex];
		Sample.NumFrames++;

		double Duration = Frame.EndTime - Frame.StartTime;
		Sample.TotalDuration += Duration;
		if (Frame.StartTime < Sample.StartTime)
		{
			Sample.StartTime = Frame.StartTime;
		}
		if (Frame.EndTime > Sample.EndTime)
		{
			Sample.EndTime = Frame.EndTime;
		}
		if (Duration > Sample.LargestFrameDuration)
		{
			Sample.LargestFrameIndex = FrameIndex;
			Sample.LargestFrameStartTime = Frame.StartTime;
			Sample.LargestFrameDuration = Duration;
		}

		Series.NumAggregatedFrames++;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FFrameTrackDrawHelper
////////////////////////////////////////////////////////////////////////////////////////////////////

FFrameTrackDrawHelper::FFrameTrackDrawHelper(const FDrawContext& InDrawContext, const FFrameTrackViewport& InViewport)
	: DrawContext(InDrawContext)
	, Viewport(InViewport)
	, WhiteBrush(FAppStyle::Get().GetBrush("WhiteBrush"))
	, HoveredFrameBorderBrush(FInsightsStyle::Get().GetBrush("HoveredEventBorder"))
	, NumFrames(0)
	, NumDrawSamples(0)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTrackDrawHelper::DrawBackground() const
{
	const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

	const float X0 = 0.0f;
	const float X1 = ViewportX.GetMinPos() - ViewportX.GetPos();
	const float X2 = ViewportX.GetMaxPos() - ViewportX.GetPos();
	const float X3 = FMath::CeilToFloat(Viewport.GetWidth());

	const float Y = 0.0f;
	const float H = FMath::CeilToFloat(Viewport.GetHeight());

	FDrawHelpers::DrawBackground(DrawContext, WhiteBrush, X0, X1, X2, X3, Y, H);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* FFrameTrackDrawHelper::FrameTypeToString(int32 FrameType)
{
	switch (FrameType)
	{
		case TraceFrameType_Game:      return TEXT("Game");
		case TraceFrameType_Rendering: return TEXT("Rendering");
		default:                       return TEXT("Unknown");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FFrameTrackDrawHelper::FrameTypeToText(int32 FrameType)
{
	switch (FrameType)
	{
		case TraceFrameType_Game:      return LOCTEXT("GameFrame", "Game Frame");
		case TraceFrameType_Rendering: return LOCTEXT("RenderingFrame", "Rendering Frame");
		default:                       return LOCTEXT("UnknownFrame", "Unknown Frame");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FFrameTrackDrawHelper::GetColor32ByFrameType(int32 FrameType)
{
	switch (FrameType)
	{
		case TraceFrameType_Game:      return 0xFF5555FF;
		case TraceFrameType_Rendering: return 0xFFFF5555;
		default:                       return 0xFF666666;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FFrameTrackDrawHelper::GetColorByFrameType(int32 FrameType)
{
	constexpr float Alpha = 0.9f;
	switch (FrameType)
	{
		case TraceFrameType_Game:      return FLinearColor(0.75f, 1.0f, 1.0f, Alpha);
		case TraceFrameType_Rendering: return FLinearColor(1.0f, 0.75f, 0.75f, Alpha);
		default:                       return FLinearColor(1.0f, 1.0f, 1.0f, Alpha);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTrackDrawHelper::DrawCached(const FFrameTrackSeries& Series) const
{
	if (Series.GetNumAggregatedFrames() == 0)
	{
		return;
	}

	NumFrames += Series.GetNumAggregatedFrames();

	FLinearColor SeriesColor = Series.GetColor();

	const float SampleW = Viewport.GetSampleWidth();
	const int32 NumSamples = Series.GetNumSamples();

	const FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();

	const float ViewHeight = FMath::RoundToFloat(Viewport.GetHeight());
	const float BaselineY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(0.0));

	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
	{
		const FFrameTrackSample& Sample = Series.GetSample(SampleIndex);
		if (Sample.NumFrames == 0)
		{
			continue;
		}

		NumDrawSamples++;

		const float X = static_cast<float>(SampleIndex) * SampleW;
		float ValueY;

		FLinearColor ColorFill = SeriesColor;

		if (Sample.LargestFrameDuration == std::numeric_limits<double>::infinity())
		{
			ValueY = ViewHeight;
			ColorFill.R = 0.0f;
			ColorFill.G = 0.0f;
			ColorFill.B = 0.0f;
		}
		else
		{
			ValueY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(Sample.LargestFrameDuration));
			if (Sample.LargestFrameDuration > UpperThresholdTime)
			{
				ColorFill.G *= 0.5f;
				ColorFill.B *= 0.5f;
			}
			else if (Sample.LargestFrameDuration > LowerThresholdTime)
			{
				ColorFill.B *= 0.5f;
			}
		}

		if (ValueY > BaselineY + ViewHeight)
		{
			ValueY = BaselineY + ViewHeight;
		}

		const float H = ValueY - BaselineY;
		const float Y = ViewHeight - H;

		const FLinearColor ColorBorder(ColorFill.R * 0.75f, ColorFill.G * 0.75f, ColorFill.B * 0.75f, 1.0);

		if (SampleW > 2.0f)
		{
			DrawContext.DrawBox(X + 1.0f, Y + 1.0f, SampleW - 2.0f, H - 2.0f, WhiteBrush, ColorFill);

			// Draw border.
			DrawContext.DrawBox(X, Y, 1.0, H, WhiteBrush, ColorBorder);
			DrawContext.DrawBox(X + SampleW - 1.0f, Y, 1.0, H, WhiteBrush, ColorBorder);
			DrawContext.DrawBox(X + 1.0f, Y, SampleW - 2.0f, 1.0f, WhiteBrush, ColorBorder);
			DrawContext.DrawBox(X + 1.0f, Y + H - 1.0f, SampleW - 2.0f, 1.0f, WhiteBrush, ColorBorder);
		}
		else
		{
			DrawContext.DrawBox(X, Y, SampleW, H, WhiteBrush, ColorBorder);
		}
	}

	DrawContext.LayerId++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTrackDrawHelper::DrawHoveredSample(const FFrameTrackSample& Sample) const
{
	const float SampleW = Viewport.GetSampleWidth();
	const int32 FramesPerSample = Viewport.GetNumFramesPerSample();
	const int32 FirstFrameIndex = Viewport.GetFirstFrameIndex();
	const int32 SampleIndex = (Sample.LargestFrameIndex - FirstFrameIndex) / FramesPerSample;
	const float X = static_cast<float>(SampleIndex) * SampleW;

	const FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();

	const float ViewHeight = FMath::RoundToFloat(Viewport.GetHeight());
	const float BaselineY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(0.0));

	float ValueY;
	if (Sample.LargestFrameDuration == std::numeric_limits<double>::infinity())
	{
		ValueY = ViewHeight;
	}
	else
	{
		ValueY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(Sample.LargestFrameDuration));
	}
	const float H = ValueY - BaselineY;
	const float Y = ViewHeight - H;

	const FLinearColor ColorBorder(1.0f, 1.0f, 0.0f, 1.0);
	DrawContext.DrawBox(X - 1.0f, Y - 1.0f, SampleW + 2.0f, H + 2.0f, HoveredFrameBorderBrush, ColorBorder);
	DrawContext.LayerId++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FFrameTrackDrawHelper::DrawHighlightedInterval(const FFrameTrackSeries& Series, const double StartTime, const double EndTime) const
{
	const int32 NumSamples = Series.GetNumSamples();

	//TODO: binary search
	int32 Index1 = 0;
	int32 Index2 = NumSamples - 1;
	while (Index1 < NumSamples && Series.GetSample(Index1).EndTime < StartTime)
	{
		Index1++;
	}
	while (Index2 >= Index1 && Series.GetSample(Index2).StartTime > EndTime)
	{
		Index2--;
	}

	if (Index1 <= Index2)
	{
		const float SampleW = Viewport.GetSampleWidth();
		float X1 = static_cast<float>(Index1) * SampleW;
		float X2 = static_cast<float>(Index2 + 1) * SampleW;

		constexpr float Y1 = 0.0f; // allows 12px for the horizontal scrollbar (one displayed on top of the track)
		const float Y2 = Viewport.GetHeight();
		constexpr float D = 2.0f; // line thickness (for both horizontal and vertical lines)
		constexpr float H = 10.0f; // height of corner lines

		const FLinearColor Color(1.0f, 1.0f, 1.0f, 1.0f);

		if (X1 >= 0.0f && X1 < Viewport.GetWidth() - 2.0f)
		{
			// Draw left side vertical lines.
			DrawContext.DrawBox(X1 - D, Y1, D, H, WhiteBrush, Color);
			DrawContext.DrawBox(X1 - D, Y2 - H, D, H, WhiteBrush, Color);
		}

		if (X2 >= -2.0f && X2 < Viewport.GetWidth())
		{
			// Draw right side vertical lines.
			DrawContext.DrawBox(X2, Y1, D, H, WhiteBrush, Color);
			DrawContext.DrawBox(X2, Y2 - H, D, H, WhiteBrush, Color);
		}

		if (X1 < 0)
		{
			X1 = 0.0f;
		}
		if (X2 > Viewport.GetWidth())
		{
			X2 = Viewport.GetWidth();
		}
		if (X1 < X2)
		{
			// Draw horizontal lines.
			DrawContext.DrawBox(X1, Y1, X2 - X1, D, WhiteBrush, Color);
			DrawContext.DrawBox(X1, Y2 - D, X2 - X1, D, WhiteBrush, Color);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
