// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsightsCore
#include "InsightsCore/Common/SimpleRtti.h"

#include "Styling/WidgetStyle.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class ESlateDrawEffect : uint8;
class FSlateWindowElementList;
struct FGeometry;
struct FSlateBrush;

namespace TraceServices
{
	struct FFrame;
}

namespace UE::Insights { class FDrawContext; }

namespace UE::Insights::TimingProfiler
{

class FFrameTrackViewport;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FFrameTrackSample
{
	int32 NumFrames;
	double TotalDuration; // sum of durations of all frames in this sample
	double StartTime; // min start time of all frames in this sample
	double EndTime; // max end time of all frames in this sample
	int32 LargestFrameIndex; // index of the largest frame
	double LargestFrameStartTime; // start time of the largest frame
	double LargestFrameDuration; // duration of the largest frame

	FFrameTrackSample()
		: NumFrames(0)
		, TotalDuration(0.0)
		, StartTime(DBL_MAX)
		, EndTime(-DBL_MAX)
		, LargestFrameIndex(0)
		, LargestFrameStartTime(0.0)
		, LargestFrameDuration(0.0)
	{}

	FFrameTrackSample(const FFrameTrackSample&) = default;
	FFrameTrackSample& operator=(const FFrameTrackSample&) = default;

	FFrameTrackSample(FFrameTrackSample&&) = default;
	FFrameTrackSample& operator=(FFrameTrackSample&&) = default;

	bool Equals(const FFrameTrackSample& Other) const
	{
		return NumFrames == Other.NumFrames
			&& TotalDuration == Other.TotalDuration
			&& StartTime == Other.StartTime
			&& EndTime == Other.EndTime
			&& LargestFrameIndex == Other.LargestFrameIndex
			&& LargestFrameStartTime == Other.LargestFrameStartTime
			&& LargestFrameDuration == Other.LargestFrameDuration;
	}

	static bool AreEquals(const FFrameTrackSample& A, const FFrameTrackSample& B)
	{
		return A.Equals(B);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFrameTrackSeries
{
	friend class FFrameTrackSeriesBuilder;
	
	INSIGHTS_DECLARE_RTTI_BASE(FFrameTrackSeries)
	
public:
	explicit FFrameTrackSeries(int32 InFrameType)
		: FrameType(InFrameType)
		, bIsVisible(true)
		, NumAggregatedFrames(0)
		, Samples()
	{
	}
	virtual ~FFrameTrackSeries() = default;
	
	int32 GetFrameType() const { return FrameType; }
	
	bool IsVisible() const { return bIsVisible; }
	void SetVisibility(bool bOnOff) { bIsVisible = bOnOff; }
	
	/** Get the number of aggregated frames */
	int32 GetNumAggregatedFrames() const { return NumAggregatedFrames; }
	/** Set the number of aggregated frames */
	void SetNumAggregatedFrames(int32 InNumAggregatedFrames) { NumAggregatedFrames = InNumAggregatedFrames; }

	/** Get the number of Samples */
	int32 GetNumSamples() const { return Samples.Num(); }
	/** Access a single Sample by an index */
	FFrameTrackSample& GetSample(int32 InIndex) { return Samples[InIndex]; }
	const FFrameTrackSample& GetSample(int32 InIndex) const { return Samples[InIndex]; }
	/** Returns the Samples Array by Reference */
	TArray<FFrameTrackSample>& GetSamples() { return Samples; }

	FLinearColor GetColor() const { return Color; }
	void SetColor(const FLinearColor& InColor) { Color = InColor; }

	FText GetName() const { return Name; }
	void SetName(const FText& InName) { Name = InName; }

	void Reset() { NumAggregatedFrames = 0; Samples.Reset(); }

private:
	int32 FrameType;
	bool bIsVisible;
	
	/** Total number of frames aggregated in samples; i.e. sum of all Sample.NumFrames */
	int32 NumAggregatedFrames;
	/** The aggregated samples */
	TArray<FFrameTrackSample> Samples;
	
	FLinearColor Color;
	FText Name;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimerFrameStatsTrackSeries : public FFrameTrackSeries
{
	INSIGHTS_DECLARE_RTTI(FTimerFrameStatsTrackSeries, FFrameTrackSeries)
	
public:
	FTimerFrameStatsTrackSeries(int32 InFrameType, uint32 InTimerId)
		: FFrameTrackSeries(InFrameType)
		, TimerId(InTimerId)
	{
	}
	virtual ~FTimerFrameStatsTrackSeries() = default;

	uint32 GetTimerId() const { return TimerId; }

private:
	uint32 TimerId;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFrameTrackSeriesBuilder
{
public:
	explicit FFrameTrackSeriesBuilder(FFrameTrackSeries& InSeries, const FFrameTrackViewport& InViewport);

	/**
	 * Non-copyable
	 */
	FFrameTrackSeriesBuilder(const FFrameTrackSeriesBuilder&) = delete;
	FFrameTrackSeriesBuilder& operator=(const FFrameTrackSeriesBuilder&) = delete;

	void AddFrame(const TraceServices::FFrame& Frame);

	int32 GetNumAddedFrames() const { return NumAddedFrames; }

private:
	FFrameTrackSeries& Series; // series to update
	const FFrameTrackViewport& Viewport;

	float SampleW; // width of a sample, in Slate units
	int32 FramesPerSample; // number of frames in a sample
	int32 FirstFrameIndex; // index of first frame in first sample; can be negative
	int32 NumSamples; // total number of samples

	// Debug stats.
	int32 NumAddedFrames; // counts total number of added frame events
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FFrameTrackDrawHelper
{
public:
	explicit FFrameTrackDrawHelper(const FDrawContext& InDrawContext, const FFrameTrackViewport& InViewport);

	/**
	 * Non-copyable
	 */
	FFrameTrackDrawHelper(const FFrameTrackDrawHelper&) = delete;
	FFrameTrackDrawHelper& operator=(const FFrameTrackDrawHelper&) = delete;

	void SetThresholds(double InUpperThresholdTime, double InLowerThresholdTime)
	{
		UpperThresholdTime = InUpperThresholdTime;
		LowerThresholdTime = InLowerThresholdTime;
	}

	void DrawBackground() const;
	void DrawCached(const FFrameTrackSeries& Series) const;
	void DrawHoveredSample(const FFrameTrackSample& Sample) const;
	void DrawHighlightedInterval(const FFrameTrackSeries& Series, const double StartTime, const double EndTime) const;

	static const TCHAR* FrameTypeToString(int32 FrameType);
	static FText FrameTypeToText(int32 FrameType);
	static uint32 GetColor32ByFrameType(int32 FrameType);
	static FLinearColor GetColorByFrameType(int32 FrameType);

	int32 GetNumFrames() const { return NumFrames; }
	int32 GetNumDrawSamples() const { return NumDrawSamples; }

private:
	const FDrawContext& DrawContext;
	const FFrameTrackViewport& Viewport;

	const FSlateBrush* WhiteBrush;
	const FSlateBrush* HoveredFrameBorderBrush;

	double UpperThresholdTime = 1.0 / 30.0;
	double LowerThresholdTime = 1.0 / 60.0;

	// Debug stats.
	mutable int32 NumFrames;
	mutable int32 NumDrawSamples;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
