// Copyright Epic Games, Inc. All Rights Reserved.

#include "GpuFenceRelation.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"

// TraceInsights
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingViewDrawHelper.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FGpuFenceRelation)

////////////////////////////////////////////////////////////////////////////////////////////////////

FGpuFenceRelation::FGpuFenceRelation(double InSourceTime, int32 InSourceQueueId, double InTargetTime, int32 InTargetQueueId)
{
	SourceTime = InSourceTime;
	SourceQueueId = InSourceQueueId;
	TargetTime = InTargetTime;
	TargetQueueId = InTargetQueueId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGpuFenceRelation::Draw(const FDrawContext& DrawContext, const FTimingTrackViewport& Viewport, const ITimingViewDrawHelper& Helper, const ITimingEventRelation::EDrawFilter Filter)
{
	int32 LayerId = Helper.GetRelationLayerId();

	TSharedPtr<const FBaseTimingTrack> SourceTrackShared = SourceTrack.Pin();
	TSharedPtr<const FBaseTimingTrack> TargetTrackShared = TargetTrack.Pin();

	if (!SourceTrackShared.IsValid() || !TargetTrackShared.IsValid())
	{
		return;
	}

	if (Filter == ITimingEventRelation::EDrawFilter::BetweenScrollableTracks)
	{
		if (SourceTrackShared->GetLocation() != ETimingTrackLocation::Scrollable ||
			TargetTrackShared->GetLocation() != ETimingTrackLocation::Scrollable)
		{
			return;
		}
	}

	if (Filter == ITimingEventRelation::EDrawFilter::BetweenDockedTracks)
	{
		if (SourceTrackShared->GetLocation() == ETimingTrackLocation::Scrollable &&
			TargetTrackShared->GetLocation() == ETimingTrackLocation::Scrollable)
		{
			return;
		}

		LayerId = DrawContext.LayerId;
	}

	const int32 OutlineLayerId = LayerId - 1;

	float X1 = Viewport.TimeToSlateUnitsRounded(SourceTime);
	float X2 = Viewport.TimeToSlateUnitsRounded(TargetTime);
	if (FMath::Max(X1, X2) < 0.0f || FMath::Min(X1, X2) > Viewport.GetWidth())
	{
		return;
	}

	if (!SourceTrackShared->IsVisible() && !TargetTrackShared->IsVisible())
	{
		return;
	}

	const int32 MaxEventDepth = (int32)TimingProfiler::FTimingProfilerManager::Get()->GetEventDepthLimit() - 1;

	float Y1 = 0.0f;
	float Y2 = 0.0f;
	if (SourceTrackShared->IsVisible())
	{
		int32 ActualSourceDepth = FMath::Min(SourceDepth, MaxEventDepth);
		Y1 = SourceTrackShared->GetPosY();
		float ChildTracksHeight = SourceTrackShared->GetChildTracksTopHeight(Viewport.GetLayout());
		if (ActualSourceDepth >= 0)
		{
			Y1 += Viewport.GetLayout().GetLaneY(ActualSourceDepth) + Viewport.GetLayout().EventH / 2.0f;
		}
		else if(ChildTracksHeight > 0.0f)
		{
			Y1 -= Viewport.GetLayout().EventH / 2.0f;
		}
		Y1 += ChildTracksHeight;
	}
	else
	{
		Y1 = TargetTrackShared->GetPosY();
	}

	if (TargetTrackShared->IsVisible())
	{
		int32 ActualTargetDepth = FMath::Min(TargetDepth, MaxEventDepth);
		Y2 = TargetTrackShared->GetPosY();
		float ChildTracksHeight = TargetTrackShared->GetChildTracksTopHeight(Viewport.GetLayout());
		if (ActualTargetDepth >= 0)
		{
			Y2 += Viewport.GetLayout().GetLaneY(ActualTargetDepth) + Viewport.GetLayout().EventH / 2.0f;
		}
		else if (ChildTracksHeight > 0.0f)
		{
			Y2 -= Viewport.GetLayout().EventH / 2.0f;
		}
		Y2 += ChildTracksHeight;
	}
	else
	{
		Y2 = SourceTrackShared->GetPosY();
	}

	const FVector2D StartPoint = FVector2D(X1, Y1);
	const FVector2D EndPoint = FVector2D(X2, Y2);
	const double Distance = FVector2D::Distance(StartPoint, EndPoint);

	constexpr double LineHeightAtStart = 4.0;
	constexpr double LineLengthAtStart = 4.0;
	constexpr double LineLengthAtEnd = 12.0;

	const FVector2D StartDir(FMath::Max(static_cast<double>(X2 - X1), 4.0 * (LineLengthAtStart + LineLengthAtEnd)), 0.0);

	constexpr float OutlineThickness = 5.0f;
	constexpr float LineThickness = 3.0f;

	constexpr double ArrowDirectionLen = 10.0;
	constexpr double ArrowRotationAngle = 20.0;
	FVector2D ArrowDirection(-ArrowDirectionLen, 0.0);

	auto ToLiniarColorNoAlpha = [](uint32 Value)
		{
			const float R = static_cast<float>((Value & 0xFF000000) >> 24) / 255.0f;
			const float G = static_cast<float>((Value & 0x00FF0000) >> 16) / 255.0f;
			const float B = static_cast<float>((Value & 0x0000FF00) >> 8) / 255.0f;
			return FLinearColor(R, G, B);
		};

	const FLinearColor OutlineColor(0.0f, 0.0f, 0.0f, 1.0f);
	const FLinearColor Color = ToLiniarColorNoAlpha(0xFF0000FF); // Red

	TArray<FVector2D> LinePoints;
	LinePoints.Add(StartPoint + FVector2D(0.0, -LineHeightAtStart / 2.0));
	LinePoints.Add(StartPoint + FVector2D(0.0, +LineHeightAtStart / 2.0));
	DrawContext.DrawLines(OutlineLayerId, 0.0f, 0.0f, LinePoints, OutlineColor, /*bAntialias=*/ true, OutlineThickness);
	DrawContext.DrawLines(LayerId, 0.0f, 0.0f, LinePoints, Color, /*bAntialias=*/ true, LineThickness);

	constexpr double MinDistance = 1.5 * (LineLengthAtStart + LineLengthAtEnd);
	constexpr double MaxDistance = 10000.0; // arbitrary limit to avoid stack overflow in recursive FLineBuilder::Subdivide when rendering splines
	if (Distance > MinDistance && Distance < MaxDistance && !FMath::IsNearlyEqual(StartPoint.Y, EndPoint.Y))
	{
		FVector2D SplineStart(StartPoint.X + LineLengthAtStart, StartPoint.Y);
		FVector2D SplineEnd(EndPoint.X - LineLengthAtEnd, EndPoint.Y);
		DrawContext.DrawSpline(OutlineLayerId, 0.0f, 0.0f, SplineStart, StartDir, SplineEnd, StartDir, OutlineThickness, OutlineColor);
		DrawContext.DrawSpline(LayerId, 0.0f, 0.0f, SplineStart, StartDir, SplineEnd, StartDir, LineThickness, Color);

		LinePoints.Empty();
		LinePoints.Add(StartPoint);
		LinePoints.Add(SplineStart);
		DrawContext.DrawLines(OutlineLayerId, 0.0f, 0.0f, LinePoints, OutlineColor, /*bAntialias=*/ true, OutlineThickness);
		DrawContext.DrawLines(LayerId, 0.0f, 0.0f, LinePoints, Color, /*bAntialias=*/ true, LineThickness);

		LinePoints.Empty();
		LinePoints.Add(SplineEnd);
		LinePoints.Add(EndPoint);
		DrawContext.DrawLines(OutlineLayerId, 0.0f, 0.0f, LinePoints, OutlineColor, /*bAntialias=*/ true, OutlineThickness);
		DrawContext.DrawLines(LayerId, 0.0f, 0.0f, LinePoints, Color, /*bAntialias=*/ true, LineThickness);
	}
	else
	{
		LinePoints.Empty();
		LinePoints.Add(StartPoint);
		LinePoints.Add(EndPoint);
		DrawContext.DrawLines(OutlineLayerId, 0.0f, 0.0f, LinePoints, OutlineColor, /*bAntialias=*/ true, OutlineThickness);
		DrawContext.DrawLines(LayerId, 0.0f, 0.0f, LinePoints, Color, /*bAntialias=*/ true, LineThickness);

		ArrowDirection = StartPoint - EndPoint;
		ArrowDirection.Normalize();
		ArrowDirection *= ArrowDirectionLen;
	}

	FVector2D ArrowOrigin = EndPoint;

	LinePoints.Empty();
	LinePoints.Add(ArrowOrigin);
	LinePoints.Add(ArrowOrigin + ArrowDirection.GetRotated(-ArrowRotationAngle));
	DrawContext.DrawLines(OutlineLayerId, 0.0f, 0.0f, LinePoints, OutlineColor, /*bAntialias=*/ true, OutlineThickness);
	DrawContext.DrawLines(LayerId, 0.0f, 0.0f, LinePoints, Color, /*bAntialias=*/ true, LineThickness);

	LinePoints.Empty();
	LinePoints.Add(ArrowOrigin);
	LinePoints.Add(ArrowOrigin + ArrowDirection.GetRotated(ArrowRotationAngle));
	DrawContext.DrawLines(OutlineLayerId, 0.0f, 0.0f, LinePoints, OutlineColor, /*bAntialias=*/ true, OutlineThickness);
	DrawContext.DrawLines(LayerId, 0.0f, 0.0f, LinePoints, Color, /*bAntialias=*/ true, LineThickness);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
