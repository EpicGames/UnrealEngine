// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Insights/ViewModels/ITimingEvent.h"

namespace UE::Insights::TimingProfiler
{

class FGpuFenceRelation : public ITimingEventRelation
{
	INSIGHTS_DECLARE_RTTI(FGpuFenceRelation, ITimingEventRelation)

public:
	FGpuFenceRelation(double InSourceTime, int32 InSourceQueueId, double InTargetTime, int32 InTargetQueueId);
	virtual ~FGpuFenceRelation() = default;

	virtual void Draw(const FDrawContext& DrawContext, const FTimingTrackViewport& Viewport, const ITimingViewDrawHelper& Helper, const ITimingEventRelation::EDrawFilter Filter) override;

	void SetSourceTrack(TSharedPtr<const FBaseTimingTrack> InSourceTrack) { SourceTrack = InSourceTrack; }
	TSharedPtr<const FBaseTimingTrack> GetSourceTrack() { return SourceTrack.Pin(); }

	void SetTargetTrack(TSharedPtr<const FBaseTimingTrack> InTargetTrack) { TargetTrack = InTargetTrack; }
	TSharedPtr<const FBaseTimingTrack> GetTargetTrack() { return TargetTrack.Pin(); }

	double GetSourceTime() { return SourceTime; }
	int32 GetSourceQueueId() { return SourceQueueId; }
	void SetSourceDepth(int32 InDepth) { SourceDepth = InDepth; }
	int32 GetSourceDepth() { return SourceDepth; }

	double GetTargetTime() { return TargetTime; }
	int32 GetTargetQueueId() { return TargetQueueId; }
	void SetTargetDepth(int32 InDepth) { TargetDepth = InDepth; }
	int32 GetTargetDepth() { return TargetDepth; }

private:
	double SourceTime;
	int32 SourceQueueId;
	int32 SourceDepth = -1;
	double TargetTime;
	int32 TargetQueueId;
	int32 TargetDepth = -1;

	TWeakPtr<const FBaseTimingTrack> SourceTrack;
	TWeakPtr<const FBaseTimingTrack> TargetTrack;
};

} // namespace UE::Insights::TimingProfiler
