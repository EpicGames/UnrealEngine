// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsAggregator.h"

// TraceInsights
#include "Insights/TimingProfilerCommon.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::FStatsAggregator"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FStatsAggregationTask
////////////////////////////////////////////////////////////////////////////////////////////////////

class FStatsAggregationTask : public FNonAbandonableTask
{
public:
	FStatsAggregationTask(IStatsAggregationWorker* InWorker, TSharedPtr<TraceServices::FCancellationToken> InCancellationToken)
		: CancellationToken(InCancellationToken)
		, Worker(InWorker)
	{
		check(Worker != nullptr);
	}

	virtual ~FStatsAggregationTask() { delete Worker; }

	IStatsAggregationWorker* GetWorker() { return Worker; }

	void DoWork() { Worker->DoWork(CancellationToken); }

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FStatsAggregationTask, STATGROUP_ThreadPoolAsyncTasks);
	}

private:
	TSharedPtr<TraceServices::FCancellationToken> CancellationToken;
	IStatsAggregationWorker* Worker;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FStatsAggregator
////////////////////////////////////////////////////////////////////////////////////////////////////

FStatsAggregator::FStatsAggregator(const FString InLogName)
	: LogName(InLogName)
	, IntervalStartTime(0.0)
	, IntervalEndTime(0.0)
	, AsyncTask(nullptr)
	, bIsStartRequested(false)
	, bIsFinished(false)
	, AllOpsStopwatch()
	, CurrentOpStopwatch()
	, OperationCount(0)
{
	CancellationToken = MakeShared<TraceServices::FCancellationToken>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FStatsAggregator::~FStatsAggregator()
{
	ResetAsyncTask();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStatsAggregator::ResetAsyncTask()
{
	// Clean up our async task if we're deleted before it is completed.
	if (AsyncTask)
	{
		if (!AsyncTask->Cancel())
		{
			CancellationToken->Cancel();
			AsyncTask->EnsureCompletion();
		}

		delete AsyncTask;
		AsyncTask = nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStatsAggregator::Start()
{
	if (IntervalStartTime < IntervalEndTime)
	{
		if (!AsyncTask)
		{
			AllOpsStopwatch.Restart();
			OperationCount = 0;
		}

		UE_LOGF(LogTimingProfiler, Log, "[%ls] Request async aggregation (op %d) [%fs to %fs] (%fs)...",
			*LogName, OperationCount + 1, IntervalStartTime, IntervalEndTime, IntervalEndTime - IntervalStartTime);
		bIsStartRequested = true;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStatsAggregator::Cancel()
{
	if (AsyncTask)
	{
		UE_LOGF(LogTimingProfiler, Log, "[%ls] Cancel requested for async aggregation (op %d)...", *LogName, OperationCount);
		CancellationToken->Cancel();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStatsAggregator::Tick(TSharedPtr<const TraceServices::IAnalysisSession> InSession,
							const double InCurrentTime,
							const float InDeltaTime,
							TFunctionRef<void()> OnFinishedCallback)
{
	if (AsyncTask && AsyncTask->IsDone())
	{
		double FinishedDuration = 0.0;

		if (!bIsStartRequested && !CancellationToken->ShouldCancel())
		{
			FStopwatch FinishedStopwatch;
			FinishedStopwatch.Start();

			bIsFinished = true;
			OnFinishedCallback();
			bIsFinished = false;

			FinishedStopwatch.Stop();
			FinishedDuration = FinishedStopwatch.GetAccumulatedTime();

			AllOpsStopwatch.Stop();
		}

		delete AsyncTask;
		AsyncTask = nullptr;

		CurrentOpStopwatch.Stop();
		const double CurrentOpDuration = CurrentOpStopwatch.GetAccumulatedTime();

		AllOpsStopwatch.Update();
		const double AllOpsDuration = AllOpsStopwatch.GetAccumulatedTime();

		UE_LOGF(LogTimingProfiler, Log, "[%ls] Aggregated stats computed in %.4fs (%.4fs + %.4fs)%ls%ls - total: %.4fs (%d ops)",
			*LogName,
			CurrentOpDuration, CurrentOpDuration - FinishedDuration, FinishedDuration,
			CancellationToken->ShouldCancel() ? TEXT(" - CANCELED") : TEXT(""),
			bIsStartRequested ? TEXT(" - IGNORED") : TEXT(""),
			AllOpsDuration, OperationCount);
	}

	if (bIsStartRequested)
	{
		if (AsyncTask)
		{
			if (!CancellationToken->ShouldCancel())
			{
				// Cancel and wait for the previous async task to finish.
				UE_LOGF(LogTimingProfiler, Log, "[%ls] Cancel previous async aggregation (op %d)...", *LogName, OperationCount);
				CancellationToken->Cancel();
			}
		}
		else
		{
			CurrentOpStopwatch.Restart();
			++OperationCount;

			bIsStartRequested = false;
			CancellationToken->Reset();

			UE_LOGF(LogTimingProfiler, Log, "[%ls] Start async aggregation (op %d)...", *LogName, OperationCount);

			IStatsAggregationWorker* Worker = CreateWorker(InSession);
			check(Worker);

			AsyncTask = new FStatsAggregationAsyncTask(Worker, CancellationToken);
			AsyncTask->StartBackgroundTask();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FStatsAggregator::GetCurrentOperationName() const
{
	return LOCTEXT("OperationName", "Computing aggregated stats");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

IStatsAggregationWorker* FStatsAggregator::GetWorker() const
{
	// It can only be called from OnFinishedCallback.
	check(bIsFinished && AsyncTask && AsyncTask->IsDone());

	return AsyncTask->GetTask().GetWorker();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
