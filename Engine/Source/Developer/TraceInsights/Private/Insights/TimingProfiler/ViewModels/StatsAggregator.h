// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Async/AsyncWork.h"

// TraceServices
#include "TraceServices/Common/CancellationToken.h"

// TraceInsightsCore
#include "InsightsCore/Common/IAsyncOperationStatusProvider.h"
#include "InsightsCore/Common/Stopwatch.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace TraceServices
{
	class IAnalysisSession;
}

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class IStatsAggregator : public IAsyncOperationStatusProvider
{
public:
	virtual void Start() = 0;
	virtual void Cancel() = 0;

	virtual bool IsCancelRequested() const = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class IStatsAggregationWorker
{
public:
	virtual ~IStatsAggregationWorker() = default;
	virtual void DoWork(TSharedPtr<TraceServices::FCancellationToken> CancellationToken) = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FStatsAggregationTask;
typedef FAsyncTask<FStatsAggregationTask> FStatsAggregationAsyncTask;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FStatsAggregator : public IStatsAggregator
{
public:
	explicit FStatsAggregator(const FString InLogName);
	virtual ~FStatsAggregator();

	double GetIntervalStartTime() const { return IntervalStartTime; }
	double GetIntervalEndTime() const { return IntervalEndTime; }
	bool IsEmptyTimeInterval() const { return IntervalStartTime >= IntervalEndTime; }

	void SetTimeInterval(double InStartTime, double InEndTime)
	{
		IntervalStartTime = InStartTime;
		IntervalEndTime = InEndTime;
	}

	void Tick(TSharedPtr<const TraceServices::IAnalysisSession> InSession, const double InCurrentTime, const float InDeltaTime, TFunctionRef<void()> OnFinishedCallback);

	//////////////////////////////////////////////////
	// IStatsAggregator

	virtual void Start() override;
	virtual void Cancel() override;

	virtual bool IsCancelRequested() const override { return CancellationToken->ShouldCancel(); }
	virtual bool IsRunning() const override { return AsyncTask != nullptr; }

	virtual double GetAllOperationsDuration() override { AllOpsStopwatch.Update(); return AllOpsStopwatch.GetAccumulatedTime(); }
	virtual double GetCurrentOperationDuration() override { CurrentOpStopwatch.Update(); return CurrentOpStopwatch.GetAccumulatedTime(); }
	virtual uint32 GetOperationCount() const override { return OperationCount; }

	virtual FText GetCurrentOperationName() const;

	//////////////////////////////////////////////////

protected:
	virtual IStatsAggregationWorker* CreateWorker(TSharedPtr<const TraceServices::IAnalysisSession> InSession) = 0;

	// Returns true only when it is called from OnFinishedCallback.
	bool IsFinished() const { return bIsFinished; }

	// Gets the worker object. It can only be called from OnFinishedCallback.
	IStatsAggregationWorker* GetWorker() const;

private:
	void ResetAsyncTask();

private:
	FString LogName;

	double IntervalStartTime;
	double IntervalEndTime;

	FStatsAggregationAsyncTask* AsyncTask;

	TSharedPtr<TraceServices::FCancellationToken> CancellationToken;
	bool bIsStartRequested;
	bool bIsFinished;

	mutable FStopwatch AllOpsStopwatch;
	mutable FStopwatch CurrentOpStopwatch;
	uint32 OperationCount;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
