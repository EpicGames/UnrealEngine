// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsGroupingAndSorting.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/StatsNodeHelper.h"

//#define INSIGHTS_ENSURE ensure
#define INSIGHTS_ENSURE(...)

// Sort by name (ascending).
#define INSIGHTS_DEFAULT_SORTING_NODES(A, B) return (A->GetDisplayName().CompareTo(B->GetDisplayName()) < 0);
//#define INSIGHTS_DEFAULT_SORTING_NODES(A, B) return A->GetDefaultSortOrder() < B->GetDefaultSortOrder();

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::StatsNode"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorting by Node Type
////////////////////////////////////////////////////////////////////////////////////////////////////

FStatsNodeSortingByStatsType::FStatsNodeSortingByStatsType(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByStatsType")),
		LOCTEXT("Sorting_ByStatsType_Name", "By Type"),
		LOCTEXT("Sorting_ByStatsType_Title", "Sort By Type"),
		LOCTEXT("Sorting_ByStatsType_Desc", "Sort by counter type."),
		InColumnRef)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStatsNodeSortingByStatsType::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->GetTypeName() == FStatsNode::TypeName);
			const EStatsNodeType ValueA = reinterpret_cast<FStatsNode*>(A.Get())->GetType();

			INSIGHTS_ENSURE(B.IsValid() && B->GetTypeName() == FStatsNode::TypeName);
			const EStatsNodeType ValueB = reinterpret_cast<FStatsNode*>(B.Get())->GetType();

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by node type (ascending).
				return ValueA < ValueB;
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->GetTypeName() == FStatsNode::TypeName);
			const EStatsNodeType ValueA = reinterpret_cast<FStatsNode*>(A.Get())->GetType();

			INSIGHTS_ENSURE(B.IsValid() && B->GetTypeName() == FStatsNode::TypeName);
			const EStatsNodeType ValueB = reinterpret_cast<FStatsNode*>(B.Get())->GetType();

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by node type (descending).
				return ValueB < ValueA;
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorting by Data Type
////////////////////////////////////////////////////////////////////////////////////////////////////

FStatsNodeSortingByDataType::FStatsNodeSortingByDataType(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByDataType")),
		LOCTEXT("Sorting_ByDataType_Name", "By Data Type"),
		LOCTEXT("Sorting_ByDataType_Title", "Sort By Data Type"),
		LOCTEXT("Sorting_ByDataType_Desc", "Sort by data type of counter values."),
		InColumnRef)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStatsNodeSortingByDataType::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->GetTypeName() == FStatsNode::TypeName);
			const EStatsNodeDataType ValueA = reinterpret_cast<FStatsNode*>(A.Get())->GetDataType();

			INSIGHTS_ENSURE(B.IsValid() && B->GetTypeName() == FStatsNode::TypeName);
			const EStatsNodeDataType ValueB = reinterpret_cast<FStatsNode*>(B.Get())->GetDataType();

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by data type (ascending).
				return ValueA < ValueB;
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->GetTypeName() == FStatsNode::TypeName);
			const EStatsNodeDataType ValueA = reinterpret_cast<FStatsNode*>(A.Get())->GetDataType();

			INSIGHTS_ENSURE(B.IsValid() && B->GetTypeName() == FStatsNode::TypeName);
			const EStatsNodeDataType ValueB = reinterpret_cast<FStatsNode*>(B.Get())->GetDataType();

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by data type (descending).
				return ValueB < ValueA;
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sort by Count (Aggregated Statistic)
////////////////////////////////////////////////////////////////////////////////////////////////////

FStatsNodeSortingByCount::FStatsNodeSortingByCount(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByCount")),
		LOCTEXT("Sorting_ByCount_Name", "By Count"),
		LOCTEXT("Sorting_ByCount_Title", "Sort By Count"),
		LOCTEXT("Sorting_ByCount_Desc", "Sort by aggregated value count."),
		InColumnRef)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStatsNodeSortingByCount::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->GetTypeName() == FStatsNode::TypeName);
			const uint64 ValueA = reinterpret_cast<FStatsNode*>(A.Get())->GetAggregatedStats().Count;

			INSIGHTS_ENSURE(B.IsValid() && B->GetTypeName() == FStatsNode::TypeName);
			const uint64 ValueB = reinterpret_cast<FStatsNode*>(B.Get())->GetAggregatedStats().Count;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by instance count (ascending).
				return ValueA < ValueB;
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->GetTypeName() == FStatsNode::TypeName);
			const uint64 ValueA = reinterpret_cast<FStatsNode*>(A.Get())->GetAggregatedStats().Count;

			INSIGHTS_ENSURE(B.IsValid() && B->GetTypeName() == FStatsNode::TypeName);
			const uint64 ValueB = reinterpret_cast<FStatsNode*>(B.Get())->GetAggregatedStats().Count;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by instance count (descending).
				return ValueB < ValueA;
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
#undef INSIGHTS_DEFAULT_SORTING_NODES
#undef INSIGHTS_ENSURE
