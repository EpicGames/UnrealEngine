// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/StatsNode.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorters
////////////////////////////////////////////////////////////////////////////////////////////////////

class FStatsNodeSortingByStatsType : public FTableCellValueSorter
{
public:
	FStatsNodeSortingByStatsType(TSharedRef<FTableColumn> InColumnRef);

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FStatsNodeSortingByDataType : public FTableCellValueSorter
{
public:
	FStatsNodeSortingByDataType(TSharedRef<FTableColumn> InColumnRef);

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FStatsNodeSortingByCount : public FTableCellValueSorter
{
public:
	FStatsNodeSortingByCount(TSharedRef<FTableColumn> InColumnRef);

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// Organizers
////////////////////////////////////////////////////////////////////////////////////////////////////

/** Enumerates types of grouping or sorting for the stats nodes. */
enum class EStatsGroupingMode
{
	/** Creates a single group for all timers. */
	Flat,

	/** Creates one group for one letter. */
	ByName,

	/** Creates groups based on stats metadata group names. */
	ByMetaGroupName,

	/** Creates one group for each node type. */
	ByType,

	/** Creates one group for each data type. */
	ByDataType,

	/** Creates one group for each logarithmic range ie. 0, [1 .. 10), [10 .. 100), [100 .. 1K), etc. */
	ByCount,

	/** Invalid enum type, may be used as a number of enumerations. */
	InvalidOrMax,
};

/** Type definition for shared pointers to instances of EStatsGroupingMode. */
typedef TSharedPtr<EStatsGroupingMode> EStatsGroupingModePtr;

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
