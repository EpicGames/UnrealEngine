// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsNode.h"

#include "Internationalization/Internationalization.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::StatsNode"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FStatsNode)

////////////////////////////////////////////////////////////////////////////////////////////////////

FStatsNode::FStatsNode(uint32 InCounterId, const FText& InDisplayName, const FName InMetaGroupName, EStatsNodeType InType, EStatsNodeDataType InDataType)
	: FBaseTreeNode(InType == EStatsNodeType::Group)
	, DisplayName(InDisplayName)
	, CounterId(InCounterId)
	, MetaGroupName(InMetaGroupName)
	, Type(InType)
	, DataType(InDataType)
	, bIsAddedToGraph(false)
{
	const uint32 HashColor = GetCounterId() * 0x2c2c57ed;
	Color.R = ((HashColor >> 16) & 0xFF) / 255.0f;
	Color.G = ((HashColor >>  8) & 0xFF) / 255.0f;
	Color.B = ((HashColor      ) & 0xFF) / 255.0f;
	Color.A = 1.0f;

	ResetAggregatedStats();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FStatsNode::FStatsNode(const FText& InDisplayName)
	: FBaseTreeNode(true)
	, DisplayName(InDisplayName)
	, CounterId(InvalidCounterId)
	, Type(EStatsNodeType::Group)
	, DataType(EStatsNodeDataType::InvalidOrMax)
	, Color(0.0f, 0.0f, 0.0f, 1.0f)
	, bIsAddedToGraph(false)
{
	ResetAggregatedStats();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FStatsNode::~FStatsNode()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStatsNode::ResetAggregatedStats()
{
	AggregatedStats.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStatsNode::SetAggregatedStatsDouble(uint64 InCount, const TAggregatedStats<double>& InAggregatedStats)
{
	AggregatedStats.Count = InCount;
	AggregatedStats.DoubleStats = InAggregatedStats;
	UpdateAggregatedStatsInt64FromDouble();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStatsNode::SetAggregatedStatsInt64(uint64 InCount, const TAggregatedStats<int64>& InAggregatedStats)
{
	AggregatedStats.Count = InCount;
	AggregatedStats.Int64Stats = InAggregatedStats;
	UpdateAggregatedStatsDoubleFromInt64();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStatsNode::UpdateAggregatedStatsInt64FromDouble()
{
	TAggregatedStats<int64>& Int64Stats = AggregatedStats.Int64Stats;
	TAggregatedStats<double>& DoubleStats = AggregatedStats.DoubleStats;

	Int64Stats.Sum           = static_cast<int64>(DoubleStats.Sum);
	Int64Stats.Min           = static_cast<int64>(DoubleStats.Min);
	Int64Stats.Max           = static_cast<int64>(DoubleStats.Max);
	Int64Stats.Average       = static_cast<int64>(DoubleStats.Average);
	Int64Stats.Median        = static_cast<int64>(DoubleStats.Median);
	Int64Stats.LowerQuartile = static_cast<int64>(DoubleStats.LowerQuartile);
	Int64Stats.UpperQuartile = static_cast<int64>(DoubleStats.UpperQuartile);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStatsNode::UpdateAggregatedStatsDoubleFromInt64()
{
	TAggregatedStats<double>& DoubleStats = AggregatedStats.DoubleStats;
	TAggregatedStats<int64>& Int64Stats = AggregatedStats.Int64Stats;

	DoubleStats.Sum           = static_cast<double>(Int64Stats.Sum);
	DoubleStats.Min           = static_cast<double>(Int64Stats.Min);
	DoubleStats.Max           = static_cast<double>(Int64Stats.Max);
	DoubleStats.Average       = static_cast<double>(Int64Stats.Average);
	DoubleStats.Median        = static_cast<double>(Int64Stats.Median);
	DoubleStats.LowerQuartile = static_cast<double>(Int64Stats.LowerQuartile);
	DoubleStats.UpperQuartile = static_cast<double>(Int64Stats.UpperQuartile);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FStatsNodeDisplayHint
{
	static const FName Seconds;
	static const FName Bytes;
};

const FName FStatsNodeDisplayHint::Seconds(TEXT("Seconds"));
const FName FStatsNodeDisplayHint::Bytes(TEXT("Bytes"));

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FStatsNodeMetaGroupName
{
	static const FName Time;
	static const FName Memory;
	static const FName Bandwidth;
	static const FName Percent;
};

const FName FStatsNodeMetaGroupName::Time(TEXT("Time"));
const FName FStatsNodeMetaGroupName::Memory(TEXT("Memory"));
const FName FStatsNodeMetaGroupName::Bandwidth(TEXT("Bandwidth"));
const FName FStatsNodeMetaGroupName::Percent(TEXT("Percent"));
static const FText NAText = LOCTEXT("AggregatedStatsNA", "N/A");

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FStatsNode::FormatAggregatedStatsValue(double ValueDbl, int64 ValueInt, bool bForTooltip) const
{
	if (AggregatedStats.Count == 0)
	{
		return NAText;
	}

	if (GetDataType() == EStatsNodeDataType::Double)
	{
		//TODO: if (GetDisplayHint() == FStatsNodeDisplayHint::Seconds)
		if (GetMetaGroupName() == FStatsNodeMetaGroupName::Time)
		{
			if (bForTooltip)
			{
				return FText::FromString(FormatTimeAuto(ValueDbl, 2));
			}
			else
			{
				return FText::FromString(FormatTimeAuto(ValueDbl, 1));
			}
		}
		else if (GetMetaGroupName() == FStatsNodeMetaGroupName::Percent)
		{
			FNumberFormattingOptions FormattingOptions;
			FormattingOptions.MaximumFractionalDigits = 2;
			return FText::Format(LOCTEXT("Counter_FractionValueFmt", "{0}"), FText::AsPercent(ValueDbl, &FormattingOptions));
		}
		else if (GetMetaGroupName() == FStatsNodeMetaGroupName::Bandwidth)
		{
			FNumberFormattingOptions FormattingOptions;
			FormattingOptions.MaximumFractionalDigits = 2;
			if (ValueInt < 1024)
			{
				return FText::Format(LOCTEXT("Counter_BwValueFmt1", "{0} B/s"), FText::AsNumber(ValueDbl, &FormattingOptions));
			}
			else
			{
				// @note AsMemory only support integer values
				return FText::Format(LOCTEXT("Counter_BwValueFmt2", "{0}/s"), FText::AsMemory(ValueInt, &FormattingOptions));
			}
		}
		else
		{
			FNumberFormattingOptions FormattingOptions;
			FormattingOptions.MaximumFractionalDigits = bForTooltip ? 12 : 6;
			return FText::AsNumber(ValueDbl, &FormattingOptions);
		}
	} // EStatsNodeDataType::Double
	else if (GetDataType() == EStatsNodeDataType::Int64)
	{
		//TODO: if (GetDisplayHint() == FStatsNodeDisplayHint::Bytes)
		if (GetMetaGroupName() == FStatsNodeMetaGroupName::Memory)
		{
			if (ValueInt > 0)
			{
				if (bForTooltip)
				{
					if (ValueInt < 1024)
					{
						return FText::Format(LOCTEXT("Counter_MemValueFmt1", "{0} bytes"), FText::AsNumber(ValueInt));
					}
					else
					{
						FNumberFormattingOptions FormattingOptions;
						FormattingOptions.MaximumFractionalDigits = 2;
						return FText::Format(LOCTEXT("Counter_MemValueFmt2", "{0} ({1} bytes)"), FText::AsMemory(ValueInt, &FormattingOptions), FText::AsNumber(ValueInt));
					}
				}
				else
				{
					FNumberFormattingOptions FormattingOptions;
					FormattingOptions.MaximumFractionalDigits = 1;
					return FText::AsMemory(ValueInt, &FormattingOptions);
				}
			}
			else if (ValueInt == 0)
			{
				return FText::FromString(TEXT("0"));
			}
			else
			{
				if (bForTooltip)
				{
					if (-ValueInt < 1024)
					{
						return FText::Format(LOCTEXT("Counter_NegMemValueFmt1", "-{0} bytes"), FText::AsNumber(-ValueInt));
					}
					else
					{
						FNumberFormattingOptions FormattingOptions;
						FormattingOptions.MaximumFractionalDigits = 2;
						return FText::Format(LOCTEXT("Counter_NegMemValueFmt2", "-{0} (-{1} bytes)"), FText::AsMemory(-ValueInt, &FormattingOptions), FText::AsNumber(-ValueInt));
					}
				}
				else
				{
					FNumberFormattingOptions FormattingOptions;
					FormattingOptions.MaximumFractionalDigits = 1;
					return FText::Format(LOCTEXT("Counter_NegMemValueFmt3", "-{0}"), FText::AsMemory(-ValueInt, &FormattingOptions));
				}
			}
		}
		else if (GetMetaGroupName() == FStatsNodeMetaGroupName::Bandwidth)
		{
			FNumberFormattingOptions FormattingOptions;
			FormattingOptions.MaximumFractionalDigits = 2;
			if (ValueInt < 1024)
			{
				return FText::Format(LOCTEXT("Counter_BwValueFmt1", "{0} B/s"), FText::AsNumber(ValueInt));
			}
			else
			{
				return FText::Format(LOCTEXT("Counter_BwValueFmt2", "{0}/s"), FText::AsMemory(ValueInt, &FormattingOptions));
			}
		}
		else
		{
			return FText::AsNumber(ValueInt);
		}
	} // EStatsNodeDataType::Int64

	return NAText;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FStatsNode::GetTextForAggregatedStatsSum(bool bForTooltip) const
{
	// It doesn't make sense to print sum values for bandwidth or fractional values
	if (GetMetaGroupName() == FStatsNodeMetaGroupName::Bandwidth || GetMetaGroupName() == FStatsNodeMetaGroupName::Percent)
	{
		return NAText;
	}
	return FormatAggregatedStatsValue(AggregatedStats.DoubleStats.Sum, AggregatedStats.Int64Stats.Sum, bForTooltip);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FStatsNode::GetTextForAggregatedStatsMin(bool bForTooltip) const
{
	return FormatAggregatedStatsValue(AggregatedStats.DoubleStats.Min, AggregatedStats.Int64Stats.Min, bForTooltip);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FStatsNode::GetTextForAggregatedStatsMax(bool bForTooltip) const
{
	return FormatAggregatedStatsValue(AggregatedStats.DoubleStats.Max, AggregatedStats.Int64Stats.Max, bForTooltip);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FStatsNode::GetTextForAggregatedStatsAverage(bool bForTooltip) const
{
	return FormatAggregatedStatsValue(AggregatedStats.DoubleStats.Average, AggregatedStats.Int64Stats.Average, bForTooltip);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FStatsNode::GetTextForAggregatedStatsMedian(bool bForTooltip) const
{
	return FormatAggregatedStatsValue(AggregatedStats.DoubleStats.Median, AggregatedStats.Int64Stats.Median, bForTooltip);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FStatsNode::GetTextForAggregatedStatsLowerQuartile(bool bForTooltip) const
{
	return FormatAggregatedStatsValue(AggregatedStats.DoubleStats.LowerQuartile, AggregatedStats.Int64Stats.LowerQuartile, bForTooltip);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FStatsNode::GetTextForAggregatedStatsUpperQuartile(bool bForTooltip) const
{
	return FormatAggregatedStatsValue(AggregatedStats.DoubleStats.UpperQuartile, AggregatedStats.Int64Stats.UpperQuartile, bForTooltip);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FStatsNode::GetTextForAggregatedStatsDiff(bool bForTooltip) const
{
	return FormatAggregatedStatsValue(AggregatedStats.DoubleStats.Max - AggregatedStats.DoubleStats.Min, AggregatedStats.Int64Stats.Max - AggregatedStats.Int64Stats.Min, bForTooltip);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
