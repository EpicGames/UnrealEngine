// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class EStatsNodeType
{
	/** A Counter node. */
	Counter,

	/** A "Stat" node. */
	Stat,

	/** A group node. */
	Group,

	/** Invalid enum type, may be used as a number of enumerations. */
	InvalidOrMax,
};

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class EStatsNodeDataType
{
	Double, // all values are double
	Int64,  // all values are int64

	Undefined, // a mix of double and int64 values (ex.: for a group node)

	/** Invalid enum type, may be used as a number of enumerations. */
	InvalidOrMax,
};

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class EStasNodeAggregatedStats
{
	Count,
	Sum,
	Min,
	Max,
	Average,
	Median,
	LowerQuartile,
	UpperQuartile,
};

////////////////////////////////////////////////////////////////////////////////////////////////////

union FStatsNodeAggregatedStatsValue
{
	double Value;
	int64 IntValue;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename Type>
struct TAggregatedStats
{
	Type Sum; /** Sum of all values. */
	Type Min; /** Min value. */
	Type Max; /** Max value. */
	Type Average; /** Average value. */
	Type Median; /** Median value. */
	Type LowerQuartile; /** Lower Quartile value. */
	Type UpperQuartile; /** Upper Quartile value. */

	TAggregatedStats()
		: Sum(0)
		, Min(0)
		, Max(0)
		, Average(0)
		, Median(0)
		, LowerQuartile(0)
		, UpperQuartile(0)
	{
	}

	void Reset()
	{
		Sum = 0;
		Min = 0;
		Max = 0;
		Average = 0;
		Median = 0;
		LowerQuartile = 0;
		UpperQuartile = 0;
	}
};

struct FAggregatedStats
{
	uint64 Count = 0; /** Number of values. */
	TAggregatedStats<double> DoubleStats;
	TAggregatedStats<int64> Int64Stats;

	void Reset()
	{
		Count = 0;
		DoubleStats.Reset();
		Int64Stats.Reset();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FStatsNode;

/** Type definition for shared pointers to instances of FStatsNode. */
typedef TSharedPtr<class FStatsNode> FStatsNodePtr;

/** Type definition for shared references to instances of FStatsNode. */
typedef TSharedRef<class FStatsNode> FStatsNodeRef;

/** Type definition for shared references to const instances of FStatsNode. */
typedef TSharedRef<const class FStatsNode> FStatsNodeRefConst;

/** Type definition for weak references to instances of FStatsNode. */
typedef TWeakPtr<class FStatsNode> FStatsNodeWeak;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class used to store information about a stats counter node (used in SStatsView).
 */
class FStatsNode : public FBaseTreeNode
{
	INSIGHTS_DECLARE_RTTI(FStatsNode, FBaseTreeNode)

public:
	static constexpr uint32 InvalidCounterId = uint32(-1);

public:
	/** Initialization constructor for the stats node. */
	FStatsNode(uint32 InCounterId, const FText& InDisplayName, const FName InMetaGroupName, EStatsNodeType InType, EStatsNodeDataType InDataType);

	/** Initialization constructor for the group node. */
	explicit FStatsNode(const FText& InDisplayName);

	virtual ~FStatsNode();

	virtual const FText GetDisplayName() const override
	{
		return DisplayName;
	}

	/**
	 * @return the counter id as provided by analyzer.
	 */
	uint32 GetCounterId() const
	{
		return CounterId;
	}

	/**
	 * @return the name of the meta group that this stats node belongs to, taken from the metadata.
	 */
	const FName& GetMetaGroupName() const
	{
		return MetaGroupName;
	}

	/**
	 * @return the type of this node or EStatsNodeType::Group for group nodes.
	 */
	EStatsNodeType GetType() const
	{
		return Type;
	}

	/**
	 * @return the data type of node values
	 */
	const EStatsNodeDataType GetDataType() const
	{
		return DataType;
	}
	/**
	 * Sets the data type of node values.
	 */
	void SetDataType(EStatsNodeDataType InDataType)
	{
		DataType = InDataType;
	}

	/**
	 * @return color of the node. Used when showing a graph series for a stats counter.
	 */
	virtual FLinearColor GetColor() const override
	{
		return Color;
	}

	void SetColor(const FLinearColor& InColor)
	{
		Color = InColor;
	}

	virtual uint32 GetDefaultSortOrder() const override
	{
		return CounterId;
	}

	bool IsAddedToGraph() const
	{
		return bIsAddedToGraph;
	}

	void SetAddedToGraphFlag(bool bOnOff)
	{
		bIsAddedToGraph = bOnOff;
	}

	/**
	 * @return the aggregated stats of this stats counter.
	 */
	const FAggregatedStats& GetAggregatedStats() const { return AggregatedStats; }
	FAggregatedStats& GetAggregatedStats() { return AggregatedStats; }

	void ResetAggregatedStats();

	void SetAggregatedStatsDouble(uint64 InCount, const TAggregatedStats<double>& InAggregatedStats);
	void SetAggregatedStatsInt64(uint64 InCount, const TAggregatedStats<int64>& InAggregatedStats);

	void UpdateAggregatedStatsInt64FromDouble();
	void UpdateAggregatedStatsDoubleFromInt64();

	const FText FormatAggregatedStatsValue(double ValueDbl, int64 ValueInt, bool bForTooltip = false) const;

	const FText GetTextForAggregatedStatsSum(bool bForTooltip = false) const;
	const FText GetTextForAggregatedStatsMin(bool bForTooltip = false) const;
	const FText GetTextForAggregatedStatsMax(bool bForTooltip = false) const;
	const FText GetTextForAggregatedStatsAverage(bool bForTooltip = false) const;
	const FText GetTextForAggregatedStatsMedian(bool bForTooltip = false) const;
	const FText GetTextForAggregatedStatsLowerQuartile(bool bForTooltip = false) const;
	const FText GetTextForAggregatedStatsUpperQuartile(bool bForTooltip = false) const;
	const FText GetTextForAggregatedStatsDiff(bool bForTooltip = false) const;

private:
	/** The display name of the node. */
	FText DisplayName;

	/** The counter id provided by the analyzer. */
	uint32 CounterId;

	/** The name of the meta group that this stats counter belongs to, based on the stats' metadata; only valid for stats counter nodes. */
	const FName MetaGroupName;

	/** The type of this node. */
	const EStatsNodeType Type;

	/** The data type of counter values. */
	EStatsNodeDataType DataType;

	/** Color of the node. */
	FLinearColor Color;

	/** If a graph series was added to the Main Graph for this counter. */
	bool bIsAddedToGraph;

	/** Aggregated stats. */
	FAggregatedStats AggregatedStats;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
