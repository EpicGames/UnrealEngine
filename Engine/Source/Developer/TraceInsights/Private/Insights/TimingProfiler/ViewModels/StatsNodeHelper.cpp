// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsNodeHelper.h"

// TraceInsights
#include "Insights/InsightsStyle.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::StatsNode"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// StatsNode Type Helper
////////////////////////////////////////////////////////////////////////////////////////////////////

FText StatsNodeTypeHelper::ToText(const EStatsNodeType NodeType)
{
	static_assert(static_cast<int>(EStatsNodeType::InvalidOrMax) == 3, "Not all cases are handled in switch below!?");
	switch (NodeType)
	{
		case EStatsNodeType::Counter:	return LOCTEXT("Stats_Name_Counter", "Counter");
		case EStatsNodeType::Stat:		return LOCTEXT("Stats_Name_Stat", "Stat");
		case EStatsNodeType::Group:		return LOCTEXT("Stats_Name_Group", "Group");
		default:						return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText StatsNodeTypeHelper::ToDescription(const EStatsNodeType NodeType)
{
	static_assert(static_cast<int>(EStatsNodeType::InvalidOrMax) == 3, "Not all cases are handled in switch below!?");
	switch (NodeType)
	{
		case EStatsNodeType::Counter:	return LOCTEXT("Stats_Desc_Counter", "Counter node");
		case EStatsNodeType::Stat:		return LOCTEXT("Stats_Desc_Stat", "Stat node");
		case EStatsNodeType::Group:		return LOCTEXT("Stats_Desc_Group", "Group node");
		default:						return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* StatsNodeTypeHelper::GetIconForGroup()
{
	return FInsightsStyle::GetBrush("Icons.Group.TreeItem");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* StatsNodeTypeHelper::GetIcon(const EStatsNodeType NodeType)
{
	static_assert(static_cast<int>(EStatsNodeType::InvalidOrMax) == 3, "Not all cases are handled in switch below!?");
	switch (NodeType)
	{
		case EStatsNodeType::Counter:	return FInsightsStyle::GetBrush("Icons.Counter.TreeItem");
		case EStatsNodeType::Stat:		return FInsightsStyle::GetBrush("Icons.StatCounter.TreeItem");
		case EStatsNodeType::Group:		return FInsightsStyle::GetBrush("Icons.Group.TreeItem");
		default:						return nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// StatsNode DataType Helper
////////////////////////////////////////////////////////////////////////////////////////////////////

FText StatsNodeDataTypeHelper::ToText(const EStatsNodeDataType DataType)
{
	static_assert(static_cast<int>(EStatsNodeDataType::InvalidOrMax) == 3, "Not all cases are handled in switch below!?");
	switch (DataType)
	{
	case EStatsNodeDataType::Double:	return LOCTEXT("Stats_Name_Double", "Double");
	case EStatsNodeDataType::Int64:		return LOCTEXT("Stats_Name_Int64", "Int64");
	case EStatsNodeDataType::Undefined:	return LOCTEXT("Stats_Name_Undefined", "Undefined");
	default:							return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText StatsNodeDataTypeHelper::ToDescription(const EStatsNodeDataType DataType)
{
	static_assert(static_cast<int>(EStatsNodeDataType::InvalidOrMax) == 3, "Not all cases are handled in switch below!?");
	switch (DataType)
	{
	case EStatsNodeDataType::Double:	return LOCTEXT("Stats_Desc_Double", "Data type for counter values is Double.");
	case EStatsNodeDataType::Int64:		return LOCTEXT("Stats_Desc_Int64", "Data type for counter values is Int64.");
	case EStatsNodeDataType::Undefined:	return LOCTEXT("Stats_Desc_Undefined", "Data type for counter values is undefined (ex.: a mix of Double and Int64).");
	default:							return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* StatsNodeDataTypeHelper::GetIcon(const EStatsNodeDataType DataType)
{
	static_assert(static_cast<int>(EStatsNodeDataType::InvalidOrMax) == 3, "Not all cases are handled in switch below!?");
	switch (DataType)
	{
	case EStatsNodeDataType::Double:	return FInsightsStyle::GetBrush("Icons.DataTypeDouble.TreeItem");
	case EStatsNodeDataType::Int64:		return FInsightsStyle::GetBrush("Icons.DataTypeInt64.TreeItem");
	case EStatsNodeDataType::Undefined:	return FInsightsStyle::GetBrush("Icons.Leaf.TreeItem");
	default:							return nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// StatsNode Grouping Helper
////////////////////////////////////////////////////////////////////////////////////////////////////

FText StatsNodeGroupingHelper::ToText(const EStatsGroupingMode GroupingMode)
{
	static_assert(static_cast<int>(EStatsGroupingMode::InvalidOrMax) == 6, "Not all cases are handled in switch below!?");
	switch (GroupingMode)
	{
		case EStatsGroupingMode::Flat:				return LOCTEXT("Grouping_Name_Flat",			"Flat");
		case EStatsGroupingMode::ByName:			return LOCTEXT("Grouping_Name_ByName",			"Stats Name");
		case EStatsGroupingMode::ByMetaGroupName:	return LOCTEXT("Grouping_Name_MetaGroupName",	"Meta Group Name");
		case EStatsGroupingMode::ByType:			return LOCTEXT("Grouping_Name_Type",			"Counter Type");
		case EStatsGroupingMode::ByDataType:		return LOCTEXT("Grouping_Name_DataType",		"Data Type");
		case EStatsGroupingMode::ByCount:			return LOCTEXT("Grouping_Name_Count",			"Count");
		default:									return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText StatsNodeGroupingHelper::ToDescription(const EStatsGroupingMode GroupingMode)
{
	static_assert(static_cast<int>(EStatsGroupingMode::InvalidOrMax) == 6, "Not all cases are handled in switch below!?");
	switch (GroupingMode)
	{
		case EStatsGroupingMode::Flat:				return LOCTEXT("Grouping_Desc_Flat",			"Creates a single group. Includes all counters.");
		case EStatsGroupingMode::ByName:			return LOCTEXT("Grouping_Desc_ByName",			"Creates one group for one letter.");
		case EStatsGroupingMode::ByMetaGroupName:	return LOCTEXT("Grouping_Desc_MetaGroupName",	"Creates groups based on metadata group names of counters.");
		case EStatsGroupingMode::ByType:			return LOCTEXT("Grouping_Desc_Type",			"Creates one group for each counter type.");
		case EStatsGroupingMode::ByDataType:		return LOCTEXT("Grouping_Desc_DataType",		"Creates one group for each data type.");
		case EStatsGroupingMode::ByCount:			return LOCTEXT("Grouping_Desc_Count",			"Creates one group for each logarithmic range ie. 0, [1 .. 10), [10 .. 100), [100 .. 1K), etc.");
		default:									return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
