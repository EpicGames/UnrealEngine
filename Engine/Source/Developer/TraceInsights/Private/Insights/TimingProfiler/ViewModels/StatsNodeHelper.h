// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Styling/SlateBrush.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/StatsNode.h"
#include "Insights/TimingProfiler/ViewModels/StatsGroupingAndSorting.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Helper struct that contains helper functions and classes for EStatsNodeType enum. */
struct StatsNodeTypeHelper
{
	/**
	 * @param Type - The value to get the text for.
	 *
	 * @return text representation of the specified EStatsNodeType value.
	 */
	static FText ToText(const EStatsNodeType Type);

	/**
	 * @param Type - The value to get the text for.
	 *
	 * @return text representation with more detailed explanation of the specified EStatsNodeType value.
	 */
	static FText ToDescription(const EStatsNodeType Type);

	static const FSlateBrush* GetIconForGroup();
	static const FSlateBrush* GetIcon(const EStatsNodeType Type);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Helper struct that contains helper functions and classes for EStatsNodeDataType enum. */
struct StatsNodeDataTypeHelper
{
	/**
	 * @param Type - The value to get the text for.
	 *
	 * @return text representation of the specified EStatsNodeDataType value.
	 */
	static FText ToText(const EStatsNodeDataType Type);

	/**
	 * @param Type - The value to get the text for.
	 *
	 * @return text representation with more detailed explanation of the specified EStatsNodeDataType value.
	 */
	static FText ToDescription(const EStatsNodeDataType Type);

	static const FSlateBrush* GetIcon(const EStatsNodeDataType Type);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Helper struct that contains grouping static functions and classes. */
struct StatsNodeGroupingHelper
{
	/**
	 * @param StatsGroupingMode - The value to get the text for.
	 *
	 * @return text representation of the specified EStatsGroupingMode value.
	 */
	static FText ToText(const EStatsGroupingMode StatsGroupingMode);

	/**
	 * @param StatsGroupingMode - The value to get the text for.
	 *
	 * @return text representation with more detailed explanation of the specified EStatsGroupingMode value.
	 */
	static FText ToDescription(const EStatsGroupingMode StatsGroupingMode);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
