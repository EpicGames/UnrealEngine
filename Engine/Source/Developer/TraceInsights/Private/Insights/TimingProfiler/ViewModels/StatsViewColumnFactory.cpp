// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsViewColumnFactory.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/StatsGroupingAndSorting.h"
#include "Insights/TimingProfiler/ViewModels/StatsNodeHelper.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::SStatsView"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

// Column identifiers
const FName FStatsViewColumns::NameColumnID(TEXT("Name"));
const FName FStatsViewColumns::MetaGroupNameColumnID(TEXT("MetaGroupName"));
const FName FStatsViewColumns::TypeColumnID(TEXT("Type"));
const FName FStatsViewColumns::DataTypeColumnID(TEXT("DataType"));
const FName FStatsViewColumns::CountColumnID(TEXT("Count"));
const FName FStatsViewColumns::SumColumnID(TEXT("Sum"));
const FName FStatsViewColumns::MaxColumnID(TEXT("Max"));
const FName FStatsViewColumns::UpperQuartileColumnID(TEXT("UpperQuartile"));
const FName FStatsViewColumns::AverageColumnID(TEXT("Average"));
const FName FStatsViewColumns::MedianColumnID(TEXT("Median"));
const FName FStatsViewColumns::LowerQuartileColumnID(TEXT("LowerQuartile"));
const FName FStatsViewColumns::MinColumnID(TEXT("Min"));
const FName FStatsViewColumns::DiffColumnID(TEXT("Diff"));

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStatsViewColumnFactory::CreateStatsViewColumns(TArray<TSharedRef<FTableColumn>>& Columns)
{
	Columns.Reset();

	Columns.Add(CreateNameColumn());
	Columns.Add(CreateMetaGroupNameColumn());
	Columns.Add(CreateTypeColumn());
	Columns.Add(CreateDataTypeColumn());
	Columns.Add(CreateCountColumn());
	Columns.Add(CreateSumColumn());
	Columns.Add(CreateMaxColumn());
	//Columns.Add(CreateUpperQuartileColumn());
	Columns.Add(CreateAverageColumn());
	Columns.Add(CreateMedianColumn());
	//Columns.Add(CreateLowerQuartileColumn());
	Columns.Add(CreateMinColumn());
	Columns.Add(CreateDiffColumn());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FStatsViewColumnFactory::CreateNameColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FStatsViewColumns::NameColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Name_ColumnName", "Name"));
	Column.SetTitleName(LOCTEXT("Name_ColumnTitle", "Counter or Group Name"));
	Column.SetDescription(LOCTEXT("Name_ColumnDesc", "Name of counter or group"));

	Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered |
					ETableColumnFlags::IsHierarchy);

	Column.SetHorizontalAlignment(HAlign_Left);
	Column.SetInitialWidth(206.0f);
	Column.SetMinWidth(42.0f);

	Column.SetDataType(ETableCellDataType::Text);

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FDisplayNameValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByName>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FStatsViewColumnFactory::CreateMetaGroupNameColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FStatsViewColumns::MetaGroupNameColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("MetaGroupName_ColumnName", "Meta Group"));
	Column.SetTitleName(LOCTEXT("MetaGroupName_ColumnTitle", "Meta Group Name"));
	Column.SetDescription(LOCTEXT("MetaGroupName_ColumnDesc", "Name of meta group"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Left);
	Column.SetInitialWidth(100.0f);

	Column.SetDataType(ETableCellDataType::Text);

	class FMetaGroupNameValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FStatsViewColumns::MetaGroupNameColumnID);
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(FText::FromName(StatsNode.GetMetaGroupName())));
		}
	};
	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMetaGroupNameValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByTextValue>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FStatsViewColumnFactory::CreateTypeColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FStatsViewColumns::TypeColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Type_ColumnName", "Type"));
	Column.SetTitleName(LOCTEXT("Type_ColumnTitle", "Type"));
	Column.SetDescription(LOCTEXT("Type_ColumnDesc", "Type of counter or group"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Left);
	Column.SetInitialWidth(60.0f);

	Column.SetDataType(ETableCellDataType::Text);

	class FStatsTypeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FStatsViewColumns::TypeColumnID);
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(StatsNodeTypeHelper::ToText(StatsNode.GetType())));
		}
	};
	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FStatsTypeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByTextValue>(ColumnRef);
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FStatsNodeSortingByStatsType>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FStatsViewColumnFactory::CreateDataTypeColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FStatsViewColumns::DataTypeColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("DataType_ColumnName", "DataType"));
	Column.SetTitleName(LOCTEXT("DataType_ColumnTitle", "Data Type"));
	Column.SetDescription(LOCTEXT("DataType_ColumnDesc", "Data type of counter values"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
		//ETableColumnFlags::ShouldBeVisible |
		ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Left);
	Column.SetInitialWidth(60.0f);

	Column.SetDataType(ETableCellDataType::Text);

	class FStatsTypeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FStatsViewColumns::DataTypeColumnID);
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(StatsNodeDataTypeHelper::ToText(StatsNode.GetDataType())));
		}
	};
	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FStatsTypeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByTextValue>(ColumnRef);
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FStatsNodeSortingByDataType>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FStatsViewColumnFactory::CreateCountColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FStatsViewColumns::CountColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Count_ColumnName", "Count"));
	Column.SetTitleName(LOCTEXT("Count_ColumnTitle", "Sample Count"));
	Column.SetDescription(LOCTEXT("Count_ColumnDesc", "Number of selected samples"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(60.0f);

	Column.SetDataType(ETableCellDataType::Int64);
	Column.SetAggregation(ETableColumnAggregation::Sum);

	class FCountValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FStatsViewColumns::CountColumnID);
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(StatsNode.GetAggregatedStats().Count)));
		}
	};
	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FCountValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FStatsNodeSortingByCount>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FStatsViewColumnFactory::CreateSumColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FStatsViewColumns::SumColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Sum_ColumnName", "Sum"));
	Column.SetTitleName(LOCTEXT("Sum_ColumnTitle", "Sum"));
	Column.SetDescription(LOCTEXT("Sum_ColumnDesc", "Sum of selected samples"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(AggregatedStatsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);
	Column.SetAggregation(ETableColumnAggregation::Sum);

	class FSumValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FStatsViewColumns::SumColumnID);
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(StatsNode.GetAggregatedStats().DoubleStats.Sum));
		}
	};
	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FSumValueGetter>();
	Column.SetValueGetter(Getter);

	class FSumValueFormatter : public FTableCellValueFormatter
	{
	public:
		virtual FText FormatValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			ensure(Column.GetId() == FStatsViewColumns::SumColumnID);
			//check(Node.Is<FStatsNode>());
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return StatsNode.GetTextForAggregatedStatsSum();
		}
	};
	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FSumValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FStatsNodeSortingBySum>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FStatsViewColumnFactory::CreateMaxColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FStatsViewColumns::MaxColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Max_ColumnName", "Max"));
	Column.SetTitleName(LOCTEXT("Max_ColumnTitle", "Maximum"));
	Column.SetDescription(LOCTEXT("Max_ColumnDesc", "Maximum of selected values"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(AggregatedStatsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);
	//Column.SetAggregation(ETableColumnAggregation::Max);

	class FMaxValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FStatsViewColumns::MaxColumnID);
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(StatsNode.GetAggregatedStats().DoubleStats.Max));
		}
	};
	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMaxValueGetter>();
	Column.SetValueGetter(Getter);

	class FMaxValueFormatter : public FTableCellValueFormatter
	{
	public:
		virtual FText FormatValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			ensure(Column.GetId() == FStatsViewColumns::MaxColumnID);
			//check(Node.Is<FStatsNode>());
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return StatsNode.GetTextForAggregatedStatsMax();
		}
	};
	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FMaxValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FStatsNodeSortingByMax>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FStatsViewColumnFactory::CreateUpperQuartileColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FStatsViewColumns::UpperQuartileColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("UpperQuartile_ColumnName", "Upper"));
	Column.SetTitleName(LOCTEXT("UpperQuartile_ColumnTitle", "Upper Quartile"));
	Column.SetDescription(LOCTEXT("UpperQuartile_ColumnDesc", "Upper quartile (Q3; third quartile; 75th percentile) of selected values"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(AggregatedStatsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);
	//Column.SetAggregation(ETableColumnAggregation::Max);

	class FUpperQuartileValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FStatsViewColumns::UpperQuartileColumnID);
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(StatsNode.GetAggregatedStats().DoubleStats.UpperQuartile));
		}
	};
	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FUpperQuartileValueGetter>();
	Column.SetValueGetter(Getter);

	class FUpperQuartileValueFormatter : public FTableCellValueFormatter
	{
	public:
		virtual FText FormatValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			ensure(Column.GetId() == FStatsViewColumns::UpperQuartileColumnID);
			//check(Node.Is<FStatsNode>());
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return StatsNode.GetTextForAggregatedStatsUpperQuartile();
		}
	};
	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FUpperQuartileValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FStatsNodeSortingByUpperQuartile>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FStatsViewColumnFactory::CreateAverageColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FStatsViewColumns::AverageColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Average_ColumnName", "Avg."));
	Column.SetTitleName(LOCTEXT("Average_ColumnTitle", "Average"));
	Column.SetDescription(LOCTEXT("Average_ColumnDesc", "Average (arithmetic mean) of selected values"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(AggregatedStatsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);

	class FAverageValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FStatsViewColumns::AverageColumnID);
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(StatsNode.GetAggregatedStats().DoubleStats.Average));
		}
	};
	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAverageValueGetter>();
	Column.SetValueGetter(Getter);

	class FAverageValueFormatter : public FTableCellValueFormatter
	{
	public:
		virtual FText FormatValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			ensure(Column.GetId() == FStatsViewColumns::AverageColumnID);
			//check(Node.Is<FStatsNode>());
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return StatsNode.GetTextForAggregatedStatsAverage();
		}
	};
	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FAverageValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FStatsNodeSortingByAverage>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FStatsViewColumnFactory::CreateMedianColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FStatsViewColumns::MedianColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Median_ColumnName", "Med."));
	Column.SetTitleName(LOCTEXT("Median_ColumnTitle", "Median"));
	Column.SetDescription(LOCTEXT("Median_ColumnDesc", "Median (Q2; second quartile; 50th percentile) of selected values"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(AggregatedStatsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);

	class FMedianValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FStatsViewColumns::MedianColumnID);
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(StatsNode.GetAggregatedStats().DoubleStats.Median));
		}
	};
	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMedianValueGetter>();
	Column.SetValueGetter(Getter);

	class FMedianValueFormatter : public FTableCellValueFormatter
	{
	public:
		virtual FText FormatValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			ensure(Column.GetId() == FStatsViewColumns::MedianColumnID);
			//check(Node.Is<FStatsNode>());
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return StatsNode.GetTextForAggregatedStatsMedian();
		}
	};
	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FMedianValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FStatsNodeSortingByMedian>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FStatsViewColumnFactory::CreateLowerQuartileColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FStatsViewColumns::LowerQuartileColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("LowerQuartile_ColumnName", "Lower"));
	Column.SetTitleName(LOCTEXT("LowerQuartile_ColumnTitle", "Lower Quartile"));
	Column.SetDescription(LOCTEXT("LowerQuartile_ColumnDesc", "Lower quartile (Q1; first quartile; 25th percentile) of selected values"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(AggregatedStatsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);
	//Column.SetAggregation(ETableColumnAggregation::Min);

	class FLowerQuartileValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FStatsViewColumns::LowerQuartileColumnID);
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(StatsNode.GetAggregatedStats().DoubleStats.LowerQuartile));
		}
	};
	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FLowerQuartileValueGetter>();
	Column.SetValueGetter(Getter);

	class FLowerQuartileValueFormatter : public FTableCellValueFormatter
	{
	public:
		virtual FText FormatValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			ensure(Column.GetId() == FStatsViewColumns::LowerQuartileColumnID);
			//check(Node.Is<FStatsNode>());
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return StatsNode.GetTextForAggregatedStatsLowerQuartile();
		}
	};
	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FLowerQuartileValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FStatsNodeSortingByLowerQuartile>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FStatsViewColumnFactory::CreateMinColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FStatsViewColumns::MinColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Min_ColumnName", "Min"));
	Column.SetTitleName(LOCTEXT("Min_ColumnTitle", "Minimum"));
	Column.SetDescription(LOCTEXT("Min_ColumnDesc", "Minimum of selected values"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(AggregatedStatsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);
	//Column.SetAggregation(ETableColumnAggregation::Min);

	class FMinValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FStatsViewColumns::MinColumnID);
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(StatsNode.GetAggregatedStats().DoubleStats.Min));
		}
	};
	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMinValueGetter>();
	Column.SetValueGetter(Getter);

	class FMinValueFormatter : public FTableCellValueFormatter
	{
	public:
		virtual FText FormatValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			ensure(Column.GetId() == FStatsViewColumns::MinColumnID);
			//check(Node.Is<FStatsNode>());
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return StatsNode.GetTextForAggregatedStatsMin();
		}
	};
	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FMinValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FStatsNodeSortingByMin>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FStatsViewColumnFactory::CreateDiffColumn()
{
	TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FStatsViewColumns::DiffColumnID);
	FTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Diff_ColumnName", "Max - Min"));
	Column.SetTitleName(LOCTEXT("Diff_ColumnTitle", "Maximum - Minimum"));
	Column.SetDescription(LOCTEXT("Diff_ColumnDesc", "Maximum - Minimum of selected values"));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
		ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(AggregatedStatsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);

	class FDiffValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FStatsViewColumns::DiffColumnID);
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(StatsNode.GetAggregatedStats().DoubleStats.Max - StatsNode.GetAggregatedStats().DoubleStats.Min));
		}
	};
	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FDiffValueGetter>();
	Column.SetValueGetter(Getter);

	class FDiffValueFormatter : public FTableCellValueFormatter
	{
	public:
		virtual FText FormatValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
		{
			ensure(Column.GetId() == FStatsViewColumns::DiffColumnID);
			const FStatsNode& StatsNode = static_cast<const FStatsNode&>(Node);
			return StatsNode.GetTextForAggregatedStatsDiff();
		}
	};
	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDiffValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
