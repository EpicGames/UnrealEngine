// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

namespace UE::Insights { class FTableColumn; }

namespace UE::Insights::TimingProfiler
{

// Column identifiers
struct FStatsViewColumns
{
	static const FName NameColumnID;
	static const FName MetaGroupNameColumnID;
	static const FName TypeColumnID;
	static const FName DataTypeColumnID;
	static const FName CountColumnID;
	static const FName SumColumnID;
	static const FName MaxColumnID;
	static const FName UpperQuartileColumnID;
	static const FName AverageColumnID;
	static const FName MedianColumnID;
	static const FName LowerQuartileColumnID;
	static const FName MinColumnID;
	static const FName DiffColumnID;
};

struct FStatsViewColumnFactory
{
public:
	static void CreateStatsViewColumns(TArray<TSharedRef<FTableColumn>>& Columns);

	static TSharedRef<FTableColumn> CreateNameColumn();
	static TSharedRef<FTableColumn> CreateMetaGroupNameColumn();
	static TSharedRef<FTableColumn> CreateTypeColumn();
	static TSharedRef<FTableColumn> CreateDataTypeColumn();
	static TSharedRef<FTableColumn> CreateCountColumn();
	static TSharedRef<FTableColumn> CreateSumColumn();
	static TSharedRef<FTableColumn> CreateMaxColumn();
	static TSharedRef<FTableColumn> CreateUpperQuartileColumn();
	static TSharedRef<FTableColumn> CreateAverageColumn();
	static TSharedRef<FTableColumn> CreateMedianColumn();
	static TSharedRef<FTableColumn> CreateLowerQuartileColumn();
	static TSharedRef<FTableColumn> CreateMinColumn();
	static TSharedRef<FTableColumn> CreateDiffColumn();

private:
	static constexpr float AggregatedStatsColumnInitialWidth = 80.0f;
};

} // namespace UE::Insights::TimingProfiler
