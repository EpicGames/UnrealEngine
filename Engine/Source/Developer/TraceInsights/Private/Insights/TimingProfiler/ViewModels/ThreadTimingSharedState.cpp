// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThreadTimingSharedState.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/LowLevelMemTracker.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/LoadTimeProfiler.h"
#include "TraceServices/Model/StackSamples.h"
#include "TraceServices/Model/TimingProfiler.h"
#include "TraceServices/Model/Threads.h"

// TraceInsightsCore
#include "InsightsCore/Common/Log.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/TimingProfiler/Tracks/CpuStackSampleTimingTrack.h"
#include "Insights/TimingProfiler/Tracks/CpuTimingTrack.h"
#include "Insights/TimingProfiler/Tracks/GpuTimingTrack.h"
#include "Insights/TimingProfiler/Tracks/VerseTimingTrack.h"
#include "Insights/TimingProfiler/ViewModels/GpuFenceRelation.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::ThreadTiming"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FThreadTimingViewCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

class FThreadTimingViewCommands : public TCommands<FThreadTimingViewCommands>
{
public:
	FThreadTimingViewCommands();
	virtual ~FThreadTimingViewCommands() {}
	virtual void RegisterCommands() override;

public:
	/** Restores all settings to default state. */
	TSharedPtr<FUICommandInfo> ResetToDefaultValues;

	/** Toggles visibility for GPU thread track(s). */
	TSharedPtr<FUICommandInfo> ShowHideAllGpuTracks;

	/** Toggles visibility for GPU work header tracks. */
	TSharedPtr<FUICommandInfo> Command_ShowWorkTracks;

	/** Extends the visualization of GPU work events over the GPU timing tracks. */
	TSharedPtr<FUICommandInfo> Command_ShowGpuWorkOverlays;

	/** Shows/hides the extended vertical lines at the edges of each GPU work event. */
	TSharedPtr<FUICommandInfo> Command_ShowGpuWorkExtendedLines;

	/** If enabled, relations between Signal and Wait fences will be displayed when selecting a Timing Event in a GPU Queue Track. */
	TSharedPtr<FUICommandInfo> Command_ShowGpuFenceRelations;

	/** Shows/hides the GPU fences child track. */
	TSharedPtr<FUICommandInfo> Command_ShowGpuFencesTrack;

	/** Shows/hides the extended vertical lines at the location of GPU fences. */
	TSharedPtr<FUICommandInfo> Command_ShowGpuFencesExtendedLines;

	/** Toggles visibility for all Verse tracks at once. */
	TSharedPtr<FUICommandInfo> ShowHideAllVerseTracks;

	/** Toggles visibility for all CPU thread tracks at once. */
	TSharedPtr<FUICommandInfo> ShowHideAllCpuTracks;

	/** Toggles visibility for all CPU Sampling thread tracks at once. */
	TSharedPtr<FUICommandInfo> ShowHideAllCpuSamplingTracks;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

FThreadTimingViewCommands::FThreadTimingViewCommands()
	: TCommands<FThreadTimingViewCommands>(
		TEXT("ThreadTimingViewCommands"),
		NSLOCTEXT("Contexts", "ThreadTimingViewCommands", "Insights - Timing View - Threads"),
		NAME_None,
		FInsightsStyle::GetStyleSetName())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// UI_COMMAND takes long for the compiler to optimize
UE_DISABLE_OPTIMIZATION_SHIP
void FThreadTimingViewCommands::RegisterCommands()
{
	UI_COMMAND(ResetToDefaultValues,
		"Reset To Default",
		"Resets all track settings back to their default state.",
		EUserInterfaceActionType::Button,
		FInputChord());

	UI_COMMAND(ShowHideAllGpuTracks,
		"GPU Track(s)",
		"Shows/hides the GPU track(s).",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::Y));

	UI_COMMAND(Command_ShowWorkTracks,
		"Show GPU Work Track(s)",
		"Shows/hides the GPU Work header track(s).",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(Command_ShowGpuWorkOverlays,
		"Show GPU Work Overlays",
		"Extends the visualization of GPU work events over the GPU timing tracks.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(Command_ShowGpuWorkExtendedLines,
		"Show GPU Work Extended Lines",
		"Shows/hides the extended vertical lines at the edges of each GPU work event.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(Command_ShowGpuFenceRelations,
		"Show GPU Fences Relations",
		"If enabled, relations between signal and wait fences will be displayed when selecting a timing event in a GPU timing track.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(Command_ShowGpuFencesTrack,
		"Show GPU Fences Track(s)",
		"Shows/hides the GPU fences header tracks.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(Command_ShowGpuFencesExtendedLines,
		"Show GPU Fences Extended Lines",
		"Shows/hides the extended vertical lines at the location of GPU fences.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(ShowHideAllVerseTracks,
		"Verse Sampling Track",
		"Shows/hides the Verse Sampling track.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::V, EModifierKey::Shift));

	UI_COMMAND(ShowHideAllCpuTracks,
		"CPU Thread Tracks",
		"Shows/hides all CPU tracks (and all CPU thread groups).",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EKeys::U));

	UI_COMMAND(ShowHideAllCpuSamplingTracks,
		"CPU Stack Sampling Tracks",
		"Shows/hides all CPU Stack Sampling thread tracks.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////
// FThreadTimingSharedState
////////////////////////////////////////////////////////////////////////////////////////////////////

FThreadTimingSharedState::FThreadTimingSharedState(STimingView* InTimingView)
	: TimingView(InTimingView)
	, TimingViewSettings(MakeShared<FTimingViewLocalSettings>())
{
	check(TimingView != nullptr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FGpuQueueTimingTrack> FThreadTimingSharedState::GetGpuTrack(uint32 InQueueId)
{
	TSharedPtr<FGpuQueueTimingTrack>* const TrackPtrPtr = GpuTracks.Find(InQueueId);
	return TrackPtrPtr ? *TrackPtrPtr : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingSharedState::IsOldGpu1TrackVisible() const
{
	return OldGpu1Track != nullptr && OldGpu1Track->IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingSharedState::IsOldGpu2TrackVisible() const
{
	return OldGpu2Track != nullptr && OldGpu2Track->IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingSharedState::IsAnyGpuTrackVisible() const
{
	if (IsOldGpu1TrackVisible() || IsOldGpu2TrackVisible())
	{
		return true;
	}
	for (const auto& KV : GpuTracks)
	{
		if (KV.Value->IsVisible())
		{
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingSharedState::IsGpuTrackVisible(uint32 InQueueId) const
{
	const TSharedPtr<FGpuQueueTimingTrack>* const TrackPtrPtr = GpuTracks.Find(InQueueId);
	return TrackPtrPtr && (*TrackPtrPtr)->IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingSharedState::IsVerseSamplingTrackVisible() const
{
	return VerseSamplingTrack != nullptr && VerseSamplingTrack->IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::GetVisibleGpuQueues(TSet<uint32>& OutSet) const
{
	OutSet.Reset();
	for (const auto& KV : GpuTracks)
	{
		const FGpuQueueTimingTrack& Track = *KV.Value;
		if (Track.IsVisible())
		{
			OutSet.Add(KV.Key);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FCpuTimingTrack> FThreadTimingSharedState::GetCpuTrack(uint32 InThreadId)
{
	TSharedPtr<FCpuTimingTrack>* const TrackPtrPtr = CpuTracks.Find(InThreadId);
	return TrackPtrPtr ? *TrackPtrPtr : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingSharedState::IsCpuTrackVisible(uint32 InThreadId) const
{
	const TSharedPtr<FCpuTimingTrack>* const TrackPtrPtr = CpuTracks.Find(InThreadId);
	return TrackPtrPtr && (*TrackPtrPtr)->IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::GetVisibleCpuThreads(TSet<uint32>& OutSet) const
{
	OutSet.Reset();
	for (const auto& KV : CpuTracks)
	{
		const FCpuTimingTrack& Track = *KV.Value;
		if (Track.IsVisible())
		{
			OutSet.Add(KV.Key);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FCpuStackSampleTimingTrack> FThreadTimingSharedState::GetCpuSamplingTrack(uint32 InSystemThreadId)
{
	TSharedPtr<FCpuStackSampleTimingTrack>* const TrackPtrPtr = CpuStackSampleTracks.Find(InSystemThreadId);
	return TrackPtrPtr ? *TrackPtrPtr : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingSharedState::IsCpuSamplingTrackVisible(uint32 InSystemThreadId) const
{
	const TSharedPtr<FCpuStackSampleTimingTrack>* const TrackPtrPtr = CpuStackSampleTracks.Find(InSystemThreadId);
	return TrackPtrPtr && (*TrackPtrPtr)->IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::GetVisibleCpuSamplingThreads(TSet<uint32>& OutSet) const
{
	OutSet.Reset();
	for (const auto& KV : CpuStackSampleTracks)
	{
		const FCpuStackSampleTimingTrack& Track = *KV.Value;
		if (Track.IsVisible())
		{
			OutSet.Add(KV.Key);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::GetVisibleTimelineIndexes(TSet<uint32>& OutSet) const
{
	OutSet.Reset();

	if (OldGpu1Track.IsValid() && OldGpu1Track->IsVisible())
	{
		OutSet.Add(OldGpu1Track->GetTimelineIndex());
	}

	if (OldGpu2Track.IsValid() && OldGpu2Track->IsVisible())
	{
		OutSet.Add(OldGpu2Track->GetTimelineIndex());
	}

	for (const auto& KV : GpuTracks)
	{
		const FGpuQueueTimingTrack& Track = *KV.Value;
		if (Track.IsVisible())
		{
			OutSet.Add(Track.GetTimelineIndex());
		}
	}

	if (VerseSamplingTrack.IsValid() && VerseSamplingTrack->IsVisible())
	{
		OutSet.Add(VerseSamplingTrack->GetTimelineIndex());
	}

	for (const auto& KV : CpuTracks)
	{
		const FCpuTimingTrack& Track = *KV.Value;
		if (Track.IsVisible())
		{
			OutSet.Add(Track.GetTimelineIndex());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::ResetToDefaultValues()
{
	// Note: The default values are set in RegisterDefaultToggleOptions() from InsightsSettings.h
	TimingViewSettings->Reset();

	if (TimingView &&
		TimingView->GetName() == FInsightsManagerTabs::TimingProfilerTabId)
	{
		ShowAllGpuTracks();
		ShowAllVerseTracks();
		ShowAllCpuTracks();
		ShowAllCpuSamplingTracks();
	}
	else
	{
		HideAllGpuTracks();
		HideAllVerseTracks();
		HideAllCpuTracks();
		HideAllCpuSamplingTracks();
	}

	// Update (create or remove) the Work/Fences child tracks based on the default visibility.
	SetGpuWorkTracksVisibility(AreGpuWorkTracksVisible());
	SetGpuFencesTracksVisibility(AreGpuFencesTracksVisible());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingSharedState::AreCpuSamplingTracksAvailable() const
{
	return CpuStackSamplingTimelineCount > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingSharedState::AreVerseTracksAvailable() const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			return TimingProfilerProvider->HasVerseTiming();
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::OnBeginSession(Timing::ITimingViewSession& InSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	if (TimingView &&
		(TimingView->GetName() == FInsightsManagerTabs::TimingProfilerTabId ||
		 TimingView->GetName() == FInsightsManagerTabs::LoadingProfilerTabId ||
		 TimingView->GetName() == FInsightsManagerTabs::MemoryProfilerTabId))
	{
		TimingViewSettings = MakeShared<FTimingViewPersistentSettings>(TimingView->GetName());
	}
	else
	{
		TimingViewSettings = MakeShared<FTimingViewLocalSettings>();

		TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowAllCpuTracks, false);
		TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowAllCpuSamplingTracks, false);
		TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowAllGpuTracks, false);
		TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowAllVerseTracks, false);
	}

	OldGpu1Track = nullptr;
	OldGpu2Track = nullptr;
	GpuTracks.Reset();
	CpuTracks.Reset();
	ThreadGroups.Reset();

	TimingProfilerTimelineCount = 0;
	LoadTimeProfilerTimelineCount = 0;

	if (TimingView)
	{
		TimingView->OnSelectedEventChanged().AddSP(this, &FThreadTimingSharedState::OnTimingEventSelected);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::OnEndSession(Timing::ITimingViewSession& InSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	OldGpu1Track = nullptr;
	OldGpu2Track = nullptr;
	GpuTracks.Reset();
	CpuTracks.Reset();
	ThreadGroups.Reset();

	TimingProfilerTimelineCount = 0;
	LoadTimeProfilerTimelineCount = 0;

	if (TimingView)
	{
		TimingView->OnSelectedEventChanged().RemoveAll(this);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::Tick(Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	bool bTracksOrderChanged = false;

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(InAnalysisSession);
	const TraceServices::ILoadTimeProfilerProvider* LoadTimeProfilerProvider = TraceServices::ReadLoadTimeProfilerProvider(InAnalysisSession);

	if (TimingProfilerProvider)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(InAnalysisSession);

		const uint32 CurrentTimingProfilerTimelineCount = TimingProfilerProvider->GetTimelineCount();
		const uint32 CurrentLoadTimeProfilerTimelineCount = (LoadTimeProfilerProvider) ? LoadTimeProfilerProvider->GetTimelineCount() : 0;

		if (CurrentTimingProfilerTimelineCount != TimingProfilerTimelineCount ||
			CurrentLoadTimeProfilerTimelineCount != LoadTimeProfilerTimelineCount)
		{
			TimingProfilerTimelineCount = CurrentTimingProfilerTimelineCount;
			LoadTimeProfilerTimelineCount = CurrentLoadTimeProfilerTimelineCount;

			LLM_SCOPE_BYTAG(Insights);

			const bool bShowAllGpuTracks = TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowAllGpuTracks);

			// Check if we have the old GPU timelines.
			if (!OldGpu1Track.IsValid())
			{
				uint32 GpuTimelineIndex;
				if (TimingProfilerProvider->GetGpuTimelineIndex(GpuTimelineIndex))
				{
					OldGpu1Track = MakeShared<FGpuTimingTrack>(*this, TEXT("GPU"), nullptr, GpuTimelineIndex, FGpuTimingTrack::Gpu1ThreadId);
					OldGpu1Track->SetOrder(FTimingTrackOrder::Gpu);
					OldGpu1Track->SetVisibilityFlag(bShowAllGpuTracks);
					InSession.AddScrollableTrack(OldGpu1Track);
				}
			}
			if (!OldGpu2Track.IsValid())
			{
				uint32 GpuTimelineIndex;
				if (TimingProfilerProvider->GetGpu2TimelineIndex(GpuTimelineIndex))
				{
					OldGpu2Track = MakeShared<FGpuTimingTrack>(*this, TEXT("GPU2"), nullptr, GpuTimelineIndex, FGpuTimingTrack::Gpu2ThreadId);
					OldGpu2Track->SetOrder(FTimingTrackOrder::Gpu + 1);
					OldGpu2Track->SetVisibilityFlag(bShowAllGpuTracks);
					InSession.AddScrollableTrack(OldGpu2Track);
				}
			}

			int32 GpuTrackOrder = FTimingTrackOrder::Gpu + 100;
			int32 CpuTrackOrder = FTimingTrackOrder::Cpu;

			// Iterate through GPU queues.
			TimingProfilerProvider->EnumerateGpuQueues(
				[this, bShowAllGpuTracks , &InSession, &bTracksOrderChanged, &GpuTrackOrder]
				(const TraceServices::FGpuQueueInfo& QueueInfo)
				{
					// Check if there is an available GPU track for this queue.
					TSharedPtr<FGpuQueueTimingTrack>* TrackPtrPtr = GpuTracks.Find(QueueInfo.Id);
					if (TrackPtrPtr == nullptr)
					{
						// Create new Timing Events track for the GPU Queue.
						TSharedPtr<FGpuQueueTimingTrack> Track = MakeShared<FGpuQueueTimingTrack>(*this, QueueInfo.GetDisplayName(), QueueInfo.TimelineIndex, QueueInfo.Id);
						Track->SetOrder(GpuTrackOrder);
						Track->SetVisibilityFlag(bShowAllGpuTracks);
						GpuTracks.Add(QueueInfo.Id, Track);
						InSession.AddScrollableTrack(Track);

						if (AreGpuWorkTracksVisible())
						{
							// Create the GPU Work track and attach it to the GPU queue track.
							const FString WorkTrackName = FString::Printf(TEXT("GPU%u - %s %u - WORK"), QueueInfo.GPU, QueueInfo.Name, QueueInfo.Index);
							TSharedRef<FGpuQueueWorkTimingTrack> WorkTrack = MakeShared<FGpuQueueWorkTimingTrack>(*this, WorkTrackName, QueueInfo.WorkTimelineIndex, QueueInfo.Id);
							WorkTrack->SetLocation(Track->GetLocation());
							WorkTrack->SetParentTrack(Track);
							Track->AddChildTrack(WorkTrack);
						}

						if (AreGpuFencesTracksVisible())
						{
							const FString FencesTrackName = FString::Printf(TEXT("GPU%u - %s %u - Fences"), QueueInfo.GPU, QueueInfo.Name, QueueInfo.Index);
							TSharedRef<FGpuFencesTimingTrack> FencesTrack = MakeShared<FGpuFencesTimingTrack>(*this, FencesTrackName, QueueInfo.Id);
							FencesTrack->SetLocation(Track->GetLocation());
							FencesTrack->SetParentTrack(Track);
							Track->AddChildTrack(FencesTrack);
						}
					}
					else
					{
						TSharedPtr<FGpuQueueTimingTrack> Track = *TrackPtrPtr;
						if (Track->GetOrder() != GpuTrackOrder)
						{
							Track->SetOrder(GpuTrackOrder);
							bTracksOrderChanged = true;
						}
					}
					GpuTrackOrder += 100;
				});

			// Check if we have the Verse sampling timeline.
			if (!VerseSamplingTrack.IsValid())
			{
				uint32 VerseTimelineIndex;
				if (TimingProfilerProvider->GetVerseTimelineIndex(VerseTimelineIndex))
				{
					VerseSamplingTrack = MakeShared<FVerseTimingTrack>(*this, TEXT("Verse Sampling"), VerseTimelineIndex);
					VerseSamplingTrack->SetOrder(FTimingTrackOrder::Cpu - 100);
					const bool bShowAllVerseTracks = TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowAllVerseTracks);
					VerseSamplingTrack->SetVisibilityFlag(bShowAllVerseTracks);
					InSession.AddScrollableTrack(VerseSamplingTrack);
				}
			}

			// Iterate through threads.
			const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(InAnalysisSession);
			ThreadProvider.EnumerateThreads(
				[this, &InSession, &bTracksOrderChanged, &CpuTrackOrder, TimingProfilerProvider, LoadTimeProfilerProvider]
				(const TraceServices::FThreadInfo& ThreadInfo)
				{
					// Check if this thread is part of a group?
					const TCHAR* GroupName = ThreadInfo.GroupName;
					if (!GroupName || *GroupName == 0)
					{
						GroupName = ThreadInfo.Name;
					}
					if (!GroupName || *GroupName == 0)
					{
						GroupName = TEXT("Other Threads");
					}
					if (!ThreadGroups.Contains(GroupName))
					{
						// Note: The GroupName pointer should be valid for the duration of the session.
						ThreadGroups.Add(GroupName, { GroupName, 0, CpuTrackOrder });
					}
					else
					{
						FThreadGroup& ThreadGroup = ThreadGroups[GroupName];
						ThreadGroup.Order = CpuTrackOrder;
					}

					const bool bIsGroupVisible = TimingViewSettings->HasCpuThreadGroup(GroupName) ?
						TimingViewSettings->IsCpuThreadGroupVisible(GroupName) :
						TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowAllCpuTracks);

					// Check if there is an available Asset Loading track for this thread.
					bool bIsLoadingThread = false;
					uint32 LoadingTimelineIndex;
					if (LoadTimeProfilerProvider && LoadTimeProfilerProvider->GetCpuThreadTimelineIndex(ThreadInfo.Id, LoadingTimelineIndex))
					{
						bIsLoadingThread = true;
					}

					// Check if there is an available CPU track for this thread.
					uint32 CpuTimelineIndex;
					if (TimingProfilerProvider->GetCpuThreadTimelineIndex(ThreadInfo.Id, CpuTimelineIndex))
					{
						TSharedPtr<FCpuTimingTrack>* TrackPtrPtr = CpuTracks.Find(ThreadInfo.Id);
						if (TrackPtrPtr == nullptr)
						{
							FString TrackName(ThreadInfo.Name && *ThreadInfo.Name ? ThreadInfo.Name : FString::Printf(TEXT("Thread %u"), ThreadInfo.Id));

							// Create new Timing Events track for the CPU thread.
							TSharedPtr<FCpuTimingTrack> Track = MakeShared<FCpuTimingTrack>(*this, TrackName, GroupName, CpuTimelineIndex, ThreadInfo.Id);
							Track->SetOrder(CpuTrackOrder);
							CpuTracks.Add(ThreadInfo.Id, Track);

							FThreadGroup& ThreadGroup = ThreadGroups[GroupName];
							ThreadGroup.NumTimelines++;

							if (bIsLoadingThread &&
								TimingView &&
								TimingView->GetName() == FInsightsManagerTabs::LoadingProfilerTabId)
							{
								Track->SetVisibilityFlag(true);
							}
							else
							{
								Track->SetVisibilityFlag(bIsGroupVisible);
							}

							InSession.AddScrollableTrack(Track);
						}
						else
						{
							TSharedPtr<FCpuTimingTrack> Track = *TrackPtrPtr;
							if (Track->GetOrder() != CpuTrackOrder)
							{
								Track->SetOrder(CpuTrackOrder);
								bTracksOrderChanged = true;
							}
						}
					}

					constexpr int32 OrderIncrement = FTimingTrackOrder::GroupRange / 1000; // distribute max 1000 tracks in the order group range
					static_assert(OrderIncrement >= 1, "Order group range too small");
					CpuTrackOrder += OrderIncrement;
				});
		}
	}

	const TraceServices::IStackSamplesProvider* StackSamplesProvider = TraceServices::ReadStackSamplesProvider(InAnalysisSession);
	if (StackSamplesProvider)
	{
		TraceServices::FProviderReadScopeLock _(*StackSamplesProvider);

		const uint32 CurrentTimelineCount = StackSamplesProvider->GetTimelineCount();

		if (CurrentTimelineCount != CpuStackSamplingTimelineCount)
		{
			CpuStackSamplingTimelineCount = CurrentTimelineCount;

			LLM_SCOPE_BYTAG(Insights);

			int32 CpuStackSampleTrackOrder = FTimingTrackOrder::Cpu + FTimingTrackOrder::GroupRange;

			StackSamplesProvider->EnumerateTimelines(
				[this, &InSession, &bTracksOrderChanged, &CpuStackSampleTrackOrder]
				(const TraceServices::FStackSampleTimeline& Timeline, const TraceServices::FStackSampleThread& Thread)
				{
					const bool bIsAllCpuSamplingTracksEnabled = IsAllCpuSamplingTracksToggleOn();
					TSharedPtr<FCpuStackSampleTimingTrack>* TrackPtrPtr = CpuStackSampleTracks.Find(Thread.SystemThreadId);
					if (TrackPtrPtr == nullptr)
					{
						FString TrackName(Thread.Name && *Thread.Name ? Thread.Name : FString::Printf(TEXT("Sampling Thread 0x%X"), Thread.SystemThreadId));
						TSharedPtr<FCpuStackSampleTimingTrack> Track = MakeShared<FCpuStackSampleTimingTrack>(*this, TrackName, Thread.SystemThreadId);
						Track->SetOrder(CpuStackSampleTrackOrder);
						CpuStackSampleTracks.Add(Thread.SystemThreadId, Track);
						Track->SetVisibilityFlag(bIsAllCpuSamplingTracksEnabled);
						InSession.AddScrollableTrack(Track);
					}
					else
					{
						TSharedPtr<FCpuStackSampleTimingTrack> Track = *TrackPtrPtr;
						Track->SetVisibilityFlag(bIsAllCpuSamplingTracksEnabled);

						if (Track->GetOrder() != CpuStackSampleTrackOrder)
						{
							Track->SetOrder(CpuStackSampleTrackOrder);
							bTracksOrderChanged = true;
						}
					}

					constexpr int32 OrderIncrement = FTimingTrackOrder::GroupRange / 1000; // distribute max 1000 tracks in the order group range
					static_assert(OrderIncrement >= 1, "Order group range too small");
					CpuStackSampleTrackOrder += OrderIncrement;
				});
		}
	}

	if (bTracksOrderChanged)
	{
		InSession.InvalidateScrollableTracksOrder();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::ExtendGpuTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InOutMenuBuilder)
{
	if (&InSession != TimingView)
	{
		return;
	}

	InOutMenuBuilder.BeginSection("GpuTracks", LOCTEXT("ContextMenu_Section_GpuTracks", "GPU Tracks"));
	{
		InOutMenuBuilder.AddMenuEntry(FThreadTimingViewCommands::Get().ShowHideAllGpuTracks);
		InOutMenuBuilder.AddMenuEntry(FThreadTimingViewCommands::Get().Command_ShowWorkTracks);
		InOutMenuBuilder.AddMenuEntry(FThreadTimingViewCommands::Get().Command_ShowGpuWorkOverlays);
		InOutMenuBuilder.AddMenuEntry(FThreadTimingViewCommands::Get().Command_ShowGpuWorkExtendedLines);
		InOutMenuBuilder.AddMenuEntry(FThreadTimingViewCommands::Get().Command_ShowGpuFenceRelations);
		InOutMenuBuilder.AddMenuEntry(FThreadTimingViewCommands::Get().Command_ShowGpuFencesTrack);
		InOutMenuBuilder.AddMenuEntry(FThreadTimingViewCommands::Get().Command_ShowGpuFencesExtendedLines);
	}
	InOutMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::ExtendCpuTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InOutMenuBuilder)
{
	if (&InSession != TimingView)
	{
		return;
	}

	InOutMenuBuilder.BeginSection("CpuSamplingTracks", LOCTEXT("ContextMenu_Section_CpuSamplingTracks", "CPU Sampling Tracks"));
	{
		InOutMenuBuilder.AddMenuEntry(FThreadTimingViewCommands::Get().ShowHideAllCpuSamplingTracks);
	}
	InOutMenuBuilder.EndSection();

	InOutMenuBuilder.BeginSection("VerseTracks", LOCTEXT("ContextMenu_Section_VerseTracks", "Verse Tracks"));
	{
		InOutMenuBuilder.AddMenuEntry(FThreadTimingViewCommands::Get().ShowHideAllVerseTracks);
	}
	InOutMenuBuilder.EndSection();

	InOutMenuBuilder.BeginSection("CpuTracks", LOCTEXT("ContextMenu_Section_CpuTracks", "CPU Tracks"));
	{
		InOutMenuBuilder.AddMenuEntry(FThreadTimingViewCommands::Get().ShowHideAllCpuTracks);
	}
	InOutMenuBuilder.EndSection();

	InOutMenuBuilder.BeginSection("CpuThreadGroups", LOCTEXT("ContextMenu_Section_CpuThreadGroups", "CPU Thread Groups"));
	CreateThreadGroupsMenu(InOutMenuBuilder);
	InOutMenuBuilder.EndSection();

	InOutMenuBuilder.BeginSection("Misc", LOCTEXT("ContextMenu_Section_Misc", "Misc Settings"));
	{
		InOutMenuBuilder.AddMenuEntry(FThreadTimingViewCommands::Get().ResetToDefaultValues);
	}
	InOutMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::BindCommands()
{
	FThreadTimingViewCommands::Register();

	if (!TimingView)
	{
		return;
	}

	TSharedPtr<FUICommandList> CommandList = TimingView->GetCommandList();
	ensure(CommandList.IsValid());

	CommandList->MapAction(
		FThreadTimingViewCommands::Get().ResetToDefaultValues,
		FExecuteAction::CreateSP(this, &FThreadTimingSharedState::ResetToDefaultValues));

	CommandList->MapAction(
		FThreadTimingViewCommands::Get().ShowHideAllGpuTracks,
		FExecuteAction::CreateSP(this, &FThreadTimingSharedState::ShowHideAllGpuTracks),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FThreadTimingSharedState::IsAllGpuTracksToggleOn));

	CommandList->MapAction(
		FThreadTimingViewCommands::Get().Command_ShowWorkTracks,
		FExecuteAction::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuWorkTracks_Execute),
		FCanExecuteAction::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuWorkTracks_CanExecute),
		FIsActionChecked::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuWorkTracks_IsChecked));

	CommandList->MapAction(
		FThreadTimingViewCommands::Get().Command_ShowGpuWorkOverlays,
		FExecuteAction::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuWorkOverlays_Execute),
		FCanExecuteAction::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuWorkOverlays_CanExecute),
		FIsActionChecked::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuWorkOverlays_IsChecked));

	CommandList->MapAction(
		FThreadTimingViewCommands::Get().Command_ShowGpuWorkExtendedLines,
		FExecuteAction::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuWorkExtendedLines_Execute),
		FCanExecuteAction::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuWorkExtendedLines_CanExecute),
		FIsActionChecked::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuWorkExtendedLines_IsChecked));

	CommandList->MapAction(
		FThreadTimingViewCommands::Get().Command_ShowGpuFencesTrack,
		FExecuteAction::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuFencesTracks_Execute),
		FCanExecuteAction::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuFencesTracks_CanExecute),
		FIsActionChecked::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuFencesTracks_IsChecked));

	CommandList->MapAction(
		FThreadTimingViewCommands::Get().Command_ShowGpuFencesExtendedLines,
		FExecuteAction::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuFencesExtendedLines_Execute),
		FCanExecuteAction::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuFencesExtendedLines_CanExecute),
		FIsActionChecked::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuFencesExtendedLines_IsChecked));

	CommandList->MapAction(
		FThreadTimingViewCommands::Get().Command_ShowGpuFenceRelations,
		FExecuteAction::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuFencesRelations_Execute),
		FCanExecuteAction::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuFencesRelations_CanExecute),
		FIsActionChecked::CreateSP(this, &FThreadTimingSharedState::Command_ShowGpuFencesRelations_IsChecked));

	CommandList->MapAction(
		FThreadTimingViewCommands::Get().ShowHideAllVerseTracks,
		FExecuteAction::CreateSP(this, &FThreadTimingSharedState::ShowHideAllVerseTracks),
		FCanExecuteAction::CreateSP(this, &FThreadTimingSharedState::AreVerseTracksAvailable),
		FIsActionChecked::CreateSP(this, &FThreadTimingSharedState::IsAllVerseTracksToggleOn));

	CommandList->MapAction(
		FThreadTimingViewCommands::Get().ShowHideAllCpuTracks,
		FExecuteAction::CreateSP(this, &FThreadTimingSharedState::ShowHideAllCpuTracks),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FThreadTimingSharedState::IsAllCpuTracksToggleOn));

	CommandList->MapAction(
		FThreadTimingViewCommands::Get().ShowHideAllCpuSamplingTracks,
		FExecuteAction::CreateSP(this, &FThreadTimingSharedState::ShowHideAllCpuSamplingTracks),
		FCanExecuteAction::CreateSP(this, &FThreadTimingSharedState::AreCpuSamplingTracksAvailable),
		FIsActionChecked::CreateSP(this, &FThreadTimingSharedState::IsAllCpuSamplingTracksToggleOn));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::CreateThreadGroupsMenu(FMenuBuilder& InOutMenuBuilder)
{
	// Sort the list of thread groups.
	TArray<const FThreadGroup*> SortedThreadGroups;
	SortedThreadGroups.Reserve(ThreadGroups.Num());
	for (const auto& KV : ThreadGroups)
	{
		SortedThreadGroups.Add(&KV.Value);
	}
	Algo::SortBy(SortedThreadGroups, &FThreadGroup::GetOrder);

	for (const FThreadGroup* ThreadGroupPtr : SortedThreadGroups)
	{
		const FThreadGroup& ThreadGroup = *ThreadGroupPtr;
		if (ThreadGroup.NumTimelines > 0)
		{
			InOutMenuBuilder.AddMenuEntry(
				//FText::FromString(ThreadGroup.Name),
				FText::Format(LOCTEXT("ThreadGroupFmt", "{0} ({1})"), FText::FromString(ThreadGroup.Name), ThreadGroup.NumTimelines),
				TAttribute<FText>(), // no tooltip
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FThreadTimingSharedState::ToggleTrackVisibilityByGroup_Execute, ThreadGroup.Name),
						  FCanExecuteAction::CreateLambda([] { return true; }),
						  FIsActionChecked::CreateSP(this, &FThreadTimingSharedState::ToggleTrackVisibilityByGroup_IsChecked, ThreadGroup.Name)),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::SetAllVerseTracksToggle(bool bOnOff)
{
	TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowAllVerseTracks, bOnOff);

	if (VerseSamplingTrack.IsValid())
	{
		VerseSamplingTrack->SetVisibilityFlag(bOnOff);
	}

	if (TimingView)
	{
		TimingView->HandleTrackVisibilityChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::SetAllCpuTracksToggle(bool bOnOff)
{
	TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowAllCpuTracks, bOnOff);

	for (const auto& KV : CpuTracks)
	{
		FCpuTimingTrack& Track = *KV.Value;
		Track.SetVisibilityFlag(bOnOff);
	}

	TimingViewSettings->ResetCpuThreadGroups();

	if (TimingView)
	{
		TimingView->HandleTrackVisibilityChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::SetAllCpuSamplingTracksToggle(bool bOnOff)
{
	TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowAllCpuSamplingTracks, bOnOff);

	for (const auto& KV : CpuStackSampleTracks)
	{
		FCpuStackSampleTimingTrack& Track = *KV.Value;
		Track.SetVisibilityFlag(bOnOff);
	}

	if (TimingView)
	{
		TimingView->HandleTrackVisibilityChanged();
	}

}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::SetAllGpuTracksToggle(bool bOnOff)
{
	TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowAllGpuTracks, bOnOff);

	if (OldGpu1Track.IsValid())
	{
		OldGpu1Track->SetVisibilityFlag(bOnOff);
	}
	if (OldGpu2Track.IsValid())
	{
		OldGpu2Track->SetVisibilityFlag(bOnOff);
	}
	for (const auto& KV : GpuTracks)
	{
		FGpuQueueTimingTrack& Track = *KV.Value;
		Track.SetVisibilityFlag(bOnOff);
	}

	if (TimingView)
	{
		TimingView->HandleTrackVisibilityChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FThreadTimingSharedState::ToggleTrackVisibilityByGroup_IsChecked(const TCHAR* InGroupName) const
{
	if (ThreadGroups.Contains(InGroupName))
	{
		return TimingViewSettings->HasCpuThreadGroup(InGroupName) ?
				TimingViewSettings->IsCpuThreadGroupVisible(InGroupName) :
				TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowAllCpuTracks);
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::ToggleTrackVisibilityByGroup_Execute(const TCHAR* InGroupName)
{
	if (ThreadGroups.Contains(InGroupName))
	{
		bool bShowAllCpuTracks = TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowAllCpuTracks);
		bool bNewVisibility = TimingViewSettings->HasCpuThreadGroup(InGroupName) ?
			!TimingViewSettings->IsCpuThreadGroupVisible(InGroupName) :
			!bShowAllCpuTracks;

		if (bNewVisibility != bShowAllCpuTracks)
		{
			TimingViewSettings->AddCpuThreadGroup(InGroupName, bNewVisibility);
		}
		else
		{
			TimingViewSettings->RemoveCpuThreadGroup(InGroupName);
		}

		for (const auto& KV : CpuTracks)
		{
			FCpuTimingTrack& Track = *KV.Value;
			if (Track.GetGroupName() == InGroupName)
			{
				Track.SetVisibilityFlag(bNewVisibility);
			}
		}

		if (TimingView)
		{
			TimingView->HandleTrackVisibilityChanged();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::ToggleOptionState(const TCHAR* CommandName)
{
	const bool bIsEnabled = TimingViewSettings->IsToggleOptionEnabled(CommandName);
	TimingViewSettings->SetToggleOption(CommandName, !bIsEnabled);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<const ITimingEvent> FThreadTimingSharedState::FindMaxEventInstance(uint32 TimerId, double StartTime, double EndTime, double MaxValidTime)
{
	auto CompareAndAssignEvent =
		[](TSharedPtr<const ITimingEvent>& TimingEvent, TSharedPtr<const ITimingEvent>& TrackEvent)
		{
			if (!TrackEvent.IsValid())
			{
				return;
			}

			if (!TimingEvent.IsValid() || TrackEvent->GetDuration() > TimingEvent->GetDuration())
			{
				TimingEvent = TrackEvent;
			}
		};

	TSharedPtr<const ITimingEvent> TimingEvent;
	TSharedPtr<const ITimingEvent> TrackEvent;

	for (const auto& KV : CpuTracks)
	{
		const FCpuTimingTrack& Track = *KV.Value;
		if (Track.IsVisible())
		{
			TrackEvent = Track.FindMaxEventInstance(TimerId, StartTime, EndTime, MaxValidTime);
			CompareAndAssignEvent(TimingEvent, TrackEvent);
		}
	}

	for (const auto& KV : CpuStackSampleTracks)
	{
		const FCpuStackSampleTimingTrack& Track = *KV.Value;
		if (Track.IsVisible())
		{
			TrackEvent = Track.FindMaxEventInstance(TimerId, StartTime, EndTime, MaxValidTime);
			CompareAndAssignEvent(TimingEvent, TrackEvent);
		}
	}

	if (OldGpu1Track.IsValid() && OldGpu1Track->IsVisible())
	{
		TrackEvent = OldGpu1Track->FindMaxEventInstance(TimerId, StartTime, EndTime, MaxValidTime);
		CompareAndAssignEvent(TimingEvent, TrackEvent);
	}

	if (OldGpu2Track.IsValid() && OldGpu2Track->IsVisible())
	{
		TrackEvent = OldGpu2Track->FindMaxEventInstance(TimerId, StartTime, EndTime, MaxValidTime);
		CompareAndAssignEvent(TimingEvent, TrackEvent);
	}

	for (const auto& KV : GpuTracks)
	{
		const FGpuQueueTimingTrack& Track = *KV.Value;
		if (Track.IsVisible())
		{
			TrackEvent = Track.FindMaxEventInstance(TimerId, StartTime, EndTime, MaxValidTime);
			CompareAndAssignEvent(TimingEvent, TrackEvent);
		}
	}

	return TimingEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<const ITimingEvent> FThreadTimingSharedState::FindMinEventInstance(uint32 TimerId, double StartTime, double EndTime, double MaxValidTime)
{
	auto CompareAndAssignEvent =
		[](TSharedPtr<const ITimingEvent>& TimingEvent, TSharedPtr<const ITimingEvent>& TrackEvent)
		{
			if (!TrackEvent.IsValid())
			{
				return;
			}

			if (!TimingEvent.IsValid() || TrackEvent->GetDuration() < TimingEvent->GetDuration())
			{
				TimingEvent = TrackEvent;
			}
		};

	TSharedPtr<const ITimingEvent> TimingEvent;
	TSharedPtr<const ITimingEvent> TrackEvent;

	for (const auto& KV : CpuTracks)
	{
		const FCpuTimingTrack& Track = *KV.Value;
		if (Track.IsVisible())
		{
			TrackEvent = Track.FindMinEventInstance(TimerId, StartTime, EndTime, MaxValidTime);
			CompareAndAssignEvent(TimingEvent, TrackEvent);
		}
	}

	for (const auto& KV : CpuStackSampleTracks)
	{
		const FCpuStackSampleTimingTrack& Track = *KV.Value;
		if (Track.IsVisible())
		{
			TrackEvent = Track.FindMinEventInstance(TimerId, StartTime, EndTime, MaxValidTime);
			CompareAndAssignEvent(TimingEvent, TrackEvent);
		}
	}

	if (OldGpu1Track.IsValid() && OldGpu1Track->IsVisible())
	{
		TrackEvent = OldGpu1Track->FindMinEventInstance(TimerId, StartTime, EndTime, MaxValidTime);
		CompareAndAssignEvent(TimingEvent, TrackEvent);
	}

	if (OldGpu2Track.IsValid() && OldGpu2Track->IsVisible())
	{
		TrackEvent = OldGpu2Track->FindMinEventInstance(TimerId, StartTime, EndTime, MaxValidTime);
		CompareAndAssignEvent(TimingEvent, TrackEvent);
	}

	for (const auto& KV : GpuTracks)
	{
		const FGpuQueueTimingTrack& Track = *KV.Value;
		if (Track.IsVisible())
		{
			TrackEvent = Track.FindMinEventInstance(TimerId, StartTime, EndTime, MaxValidTime);
			CompareAndAssignEvent(TimingEvent, TrackEvent);
		}
	}

	return TimingEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::SetGpuWorkTracksVisibility(bool bOnOff)
{
	TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowGpuWorkTracks, bOnOff);

	if (bOnOff)
	{
		AddGpuWorkChildTracks();
	}
	else
	{
		RemoveGpuWorkChildTracks();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::AddGpuWorkChildTracks()
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session);
	if (TimingProfilerProvider)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session);

		TimingProfilerProvider->EnumerateGpuQueues(
			[this](const TraceServices::FGpuQueueInfo& QueueInfo)
			{
				// Check if there is an available GPU track for this queue.
				TSharedPtr<FGpuQueueTimingTrack>* TrackPtrPtr = GpuTracks.Find(QueueInfo.Id);
				if (TrackPtrPtr != nullptr)
				{
					const TSharedPtr<FGpuQueueTimingTrack>& QueueTrack = *TrackPtrPtr;
					check(QueueTrack.IsValid());

					// Check if the Work child track is not already created.
					if (!QueueTrack->FindChildTrackOfType<FGpuQueueWorkTimingTrack>())
					{
						// Create the GPU Work track and attach it to the GPU queue track.
						const FString WorkTrackName = FString::Printf(TEXT("GPU%u - %s %u - Work"), QueueInfo.GPU, QueueInfo.Name, QueueInfo.Index);
						TSharedRef<FGpuQueueWorkTimingTrack> WorkTrack = MakeShared<FGpuQueueWorkTimingTrack>(*this, WorkTrackName, QueueInfo.WorkTimelineIndex, QueueInfo.Id);
						WorkTrack->SetLocation(QueueTrack->GetLocation());
						QueueTrack->AddChildTrack(WorkTrack, 0);
						WorkTrack->SetParentTrack(QueueTrack);
					}
				}
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::RemoveGpuWorkChildTracks()
{
	for (auto& Pair : GpuTracks)
	{
		const TSharedPtr<FGpuQueueTimingTrack>& QueueTrack = Pair.Value;
		check(QueueTrack.IsValid());
		if (TSharedPtr<FGpuQueueWorkTimingTrack> WorkTrack = QueueTrack->FindChildTrackOfType<FGpuQueueWorkTimingTrack>())
		{
			QueueTrack->RemoveChildTrack(WorkTrack.ToSharedRef());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::SetGpuFencesTracksVisibility(bool bOnOff)
{
	TimingViewSettings->SetToggleOption(FTimingViewSettings::Option_ShowGpuFencesTracks, bOnOff);

	if (bOnOff)
	{
		AddGpuFencesChildTracks();
	}
	else
	{
		RemoveGpuFencesChildTracks();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::AddGpuFencesChildTracks()
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session);
	if (TimingProfilerProvider)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session);

		TimingProfilerProvider->EnumerateGpuQueues(
			[this](const TraceServices::FGpuQueueInfo& QueueInfo)
			{
				// Check if there is an available GPU track for this queue.
				TSharedPtr<FGpuQueueTimingTrack>* TrackPtrPtr = GpuTracks.Find(QueueInfo.Id);
				if (TrackPtrPtr != nullptr)
				{
					const TSharedPtr<FGpuQueueTimingTrack>& QueueTrack = *TrackPtrPtr;
					check(QueueTrack.IsValid());

					// Check if the Fences child track is not already created.
					if (!QueueTrack->FindChildTrackOfType<FGpuFencesTimingTrack>())
					{
						// Create the GPU Fences track and attach it to the GPU queue track.
						const FString FencesTrackName = FString::Printf(TEXT("GPU%u - %s %u - Fences"), QueueInfo.GPU, QueueInfo.Name, QueueInfo.Index);
						TSharedRef<FGpuFencesTimingTrack> FencesTrack = MakeShared<FGpuFencesTimingTrack>(*this, FencesTrackName, QueueInfo.Id);
						FencesTrack->SetLocation(QueueTrack->GetLocation());
						QueueTrack->AddChildTrack(FencesTrack, QueueTrack->GetChildTracks().Num());
						FencesTrack->SetParentTrack(QueueTrack);
					}
				}
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::RemoveGpuFencesChildTracks()
{
	for (auto& Pair : GpuTracks)
	{
		const TSharedPtr<FGpuQueueTimingTrack>& QueueTrack = Pair.Value;
		check(QueueTrack.IsValid());
		if (TSharedPtr<FGpuFencesTimingTrack> FencesTrack = QueueTrack->FindChildTrackOfType<FGpuFencesTimingTrack>())
		{
			QueueTrack->RemoveChildTrack(FencesTrack.ToSharedRef());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::Command_ShowGpuFencesRelations_Execute()
{
	ToggleOptionState(FTimingViewSettings::Option_ShowGpuFencesRelations);

	bool bVisibility = TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowGpuFencesRelations);
	if (bVisibility == false && TimingView)
	{
		TimingView->EditCurrentRelations().RemoveAll(
			[](TUniquePtr<ITimingEventRelation>& Item)
			{
				return Item->Is<FGpuFenceRelation>();
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTimingSharedState::OnTimingEventSelected(TSharedPtr<const ITimingEvent> InSelectedEvent)
{
	if (!AreGpuFenceRelationsVisible())
	{
		return;
	}

	if (TimingView)
	{
		TimingView->EditCurrentRelations().RemoveAll(
			[](TUniquePtr<ITimingEventRelation>& Item)
			{
				return Item->Is<FGpuFenceRelation>();
			});
	}

	if (!InSelectedEvent.IsValid())
	{
		return;
	}

	TSharedRef<const FBaseTimingTrack> BaseTrack = InSelectedEvent->GetTrack();

	uint32 QueueId = 0;
	if (BaseTrack->Is<FGpuQueueTimingTrack>())
	{
		const FGpuQueueTimingTrack& GpuQueueTrack = BaseTrack->As<const FGpuQueueTimingTrack>();
		QueueId = GpuQueueTrack.GetThreadId();
	}
	else if (BaseTrack->Is<FGpuQueueWorkTimingTrack>())
	{
		const FGpuQueueWorkTimingTrack& GpuQueueWorkTrack = BaseTrack->As<const FGpuQueueWorkTimingTrack>();
		QueueId = GpuQueueWorkTrack.GetThreadId();
	}
	else
	{
		return;
	}

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session);
	if (!TimingProfilerProvider)
	{
		return;
	}

	using namespace TraceServices;

	auto AddFenceRelation =
		[this, TimingProfilerProvider]
		(const FGpuSignalFence& SignalFence, const FGpuWaitFence& WaitFence, uint32 WaitFenceQueueId)
		{
			uint32 Index = 0;
			uint32 SignalFenceQueueId = WaitFence.QueueToWaitForId;
			bool Result = TimingProfilerProvider->GetGpuQueueTimelineIndex(SignalFenceQueueId, Index);
			if (!Result)
			{
				return;
			}

			int32 SourceDepth = 0;
			TimingProfilerProvider->ReadTimeline(Index,
				[SignalFence, &SourceDepth]
				(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
				{
					SourceDepth = Timeline.GetDepthAt(SignalFence.Timestamp) - 1;
				});

			Result = TimingProfilerProvider->GetGpuQueueTimelineIndex(WaitFenceQueueId, Index);
			if (!Result)
			{
				return;
			}

			int32 TargetDepth = 0;
			TimingProfilerProvider->ReadTimeline(Index,
				[WaitFence, &TargetDepth]
				(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
				{
					TargetDepth = Timeline.GetDepthAt(WaitFence.Timestamp) - 1;
				});

			TUniquePtr<ITimingEventRelation> RelationBase = MakeUnique<FGpuFenceRelation>(SignalFence.Timestamp, SignalFenceQueueId, WaitFence.Timestamp, WaitFenceQueueId);
			FGpuFenceRelation& Relation = RelationBase->As<FGpuFenceRelation>();

			Relation.SetSourceDepth(SourceDepth);
			Relation.SetTargetDepth(TargetDepth);

			TSharedPtr<FGpuQueueTimingTrack>* Track = GpuTracks.Find(SignalFenceQueueId);
			if (Track == nullptr)
			{
				return;
			}
			Relation.SetSourceTrack(*Track);

			Track = GpuTracks.Find(WaitFenceQueueId);
			if (Track == nullptr)
			{
				return;
			}
			Relation.SetTargetTrack(*Track);

			if (TimingView)
			{
				TimingView->AddRelation(RelationBase);
			}
		};

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session);

	TimingProfilerProvider->EnumerateResolvedGpuFences(QueueId, InSelectedEvent->GetStartTime(), InSelectedEvent->GetEndTime(),
		[AddFenceRelation, TimingProfilerProvider, QueueId]
		(uint32 SignalFenceQueueId, const FGpuSignalFence& SignalFence, uint32 WaitFenceQueueId, const FGpuWaitFence& WaitFence)
		{
			AddFenceRelation(SignalFence, WaitFence, WaitFenceQueueId);
			return EEnumerateResult::Continue;
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
