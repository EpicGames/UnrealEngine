// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/ContainerAllocationPolicies.h" // for FDefaultSetAllocator
#include "Containers/Map.h"
#include "Misc/Crc.h" // for TStringPointerMapKeyFuncs_DEPRECATED
#include "Templates/SharedPointer.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsSettings.h"
#include "Insights/ITimingViewExtender.h"

namespace TraceServices
{
	class IAnalysisSession;
}

namespace UE::Insights::TimingProfiler
{

class FCpuTimingTrack;
class FCpuStackSampleTimingTrack;
class FGpuTimingTrack;
class FGpuQueueTimingTrack;
class FGpuQueueWorkTimingTrack;
class FVerseTimingTrack;
class STimingView;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FThreadTimingSharedState : public Timing::ITimingViewExtender, public TSharedFromThis<FThreadTimingSharedState>
{
private:
	struct FThreadGroup
	{
		const TCHAR* Name; /**< The thread group name; pointer to string owned by ThreadProvider. */
		uint32 NumTimelines; /**< Number of thread timelines associated with this group. */
		int32 Order; //**< Order index used for sorting. Inherited from last thread timeline associated with this group. **/

		int32 GetOrder() const { return Order; }
	};

	class ITimingViewSettings
	{
	public:
		virtual ~ITimingViewSettings() = default;

		virtual bool IsToggleOptionEnabled(const TCHAR* Key) const = 0;
		virtual void SetToggleOption(const TCHAR* Key, bool Value) = 0;

		virtual bool IsCpuThreadGroupVisible(const TCHAR* GroupName) const = 0;
		virtual bool HasCpuThreadGroup(const TCHAR* GroupName) const = 0;
		virtual void AddCpuThreadGroup(const TCHAR* GroupName, bool bIsVisible) = 0;
		virtual void RemoveCpuThreadGroup(const TCHAR* GroupName) = 0;
		virtual void ResetCpuThreadGroups() = 0;

		virtual void Reset() = 0;
	};

	class FTimingViewPersistentSettings : public ITimingViewSettings
	{
	public:
		FTimingViewPersistentSettings(const FName& InTimingViewName)
			: Settings(GetInsightsSettings().GetOrCreateTimingViewSettings(InTimingViewName))
		{
		}

		virtual ~FTimingViewPersistentSettings() = default;

		const FName& GetName() const
		{
			return Settings.GetName();
		}

		//////////////////////////////////////////////////

		virtual bool IsToggleOptionEnabled(const TCHAR* Key) const override
		{
			return Settings.IsToggleOptionEnabled(Key);
		}
		virtual void SetToggleOption(const TCHAR* Key, bool Value) override
		{
			GetInsightsSettings().SetAndSaveTimingViewToggleOption(GetName(), Key, Value);
		}

		//////////////////////////////////////////////////

		virtual bool IsCpuThreadGroupVisible(const TCHAR* GroupName) const override
		{
			return Settings.IsCpuThreadGroupVisible(GroupName);
		}
		virtual bool HasCpuThreadGroup(const TCHAR* GroupName) const override
		{
			return Settings.HasCpuThreadGroup(GroupName);
		}
		virtual void AddCpuThreadGroup(const TCHAR* GroupName, bool Value) override
		{
			GetInsightsSettings().AddAndSaveTimingViewCpuThreadGroup(GetName(), GroupName, Value);
		}
		virtual void RemoveCpuThreadGroup(const TCHAR* GroupName) override
		{
			GetInsightsSettings().RemoveAndSaveTimingViewCpuThreadGroup(GetName(), GroupName);
		}
		virtual void ResetCpuThreadGroups() override
		{
			GetInsightsSettings().ResetAndSaveTimingViewCpuThreadGroups(GetName());
		}

		//////////////////////////////////////////////////

		virtual void Reset() override
		{
			Settings.Reset();
		}

	private:
		FInsightsSettings& GetInsightsSettings()
		{
			return UE::Insights::FInsightsManager::Get()->GetSettings();
		}
		const FInsightsSettings& GetInsightsSettings() const
		{
			return UE::Insights::FInsightsManager::Get()->GetSettings();
		}

	private:
		FTimingViewSettings& Settings;
	};

	class FTimingViewLocalSettings : public ITimingViewSettings
	{
	public:
		FTimingViewLocalSettings() {}
		virtual ~FTimingViewLocalSettings() = default;

		virtual bool IsToggleOptionEnabled(const TCHAR* Key) const override              { return Settings.IsToggleOptionEnabled(Key); }
		virtual void SetToggleOption(const TCHAR* Key, bool Value) override              { Settings.SetToggleOption(Key, Value); }
		virtual bool IsCpuThreadGroupVisible(const TCHAR* GroupName) const override      { return Settings.IsCpuThreadGroupVisible(GroupName); }
		virtual bool HasCpuThreadGroup(const TCHAR* GroupName) const override            { return Settings.HasCpuThreadGroup(GroupName); }
		virtual void AddCpuThreadGroup(const TCHAR* GroupName, bool bIsVisible) override { Settings.AddCpuThreadGroup(GroupName, bIsVisible); }
		virtual void RemoveCpuThreadGroup(const TCHAR* GroupName) override               { Settings.RemoveCpuThreadGroup(GroupName); }
		virtual void ResetCpuThreadGroups() override                                     { Settings.ResetCpuThreadGroups(); }
		virtual void Reset() override                                                    { Settings.Reset(); }

	private:
		FTimingViewSettings Settings;
	};

public:
	explicit FThreadTimingSharedState(STimingView* InTimingView);
	virtual ~FThreadTimingSharedState() override = default;

	//////////////////////////////////////////////////
	// GPU

	TSharedPtr<FGpuTimingTrack> GetOldGpu1Track() { return OldGpu1Track; }
	TSharedPtr<FGpuTimingTrack> GetOldGpu2Track() { return OldGpu2Track; }
	bool IsOldGpu1TrackVisible() const;
	bool IsOldGpu2TrackVisible() const;

	TSharedPtr<FGpuQueueTimingTrack> GetGpuTrack(uint32 InQueueId);
	bool IsAnyGpuTrackVisible() const;
	bool IsGpuTrackVisible(uint32 InQueueId) const;
	void GetVisibleGpuQueues(TSet<uint32>& OutSet) const;

	bool IsAllGpuTracksToggleOn() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowAllGpuTracks); }
	void SetAllGpuTracksToggle(bool bOnOff);
	void ShowAllGpuTracks() { SetAllGpuTracksToggle(true); }
	void HideAllGpuTracks() { SetAllGpuTracksToggle(false); }
	void ShowHideAllGpuTracks() { SetAllGpuTracksToggle(!IsAllGpuTracksToggleOn()); }

	bool AreOverlaysVisibleInGpuQueueTracks() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowGpuWorkOverlays); }
	bool AreExtendedLinesVisibleInGpuQueueTracks() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowGpuWorkExtendedLines); }

	bool AreGpuWorkTracksVisible() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowGpuWorkTracks); }
	void SetGpuWorkTracksVisibility(bool bOnOff);

	bool AreGpuFencesTracksVisible() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowGpuFencesTracks); }
	void SetGpuFencesTracksVisibility(bool bOnOff);

	bool AreGpuFencesExtendedLinesVisible() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowGpuFencesExtendedLines); }
	bool AreGpuFenceRelationsVisible() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowGpuFencesRelations); }

	//////////////////////////////////////////////////
	// Verse Sampling

	bool AreVerseTracksAvailable() const;

	TSharedPtr<FVerseTimingTrack> GetVerseSamplingTrack() { return VerseSamplingTrack; }
	bool IsVerseSamplingTrackVisible() const;

	bool IsAllVerseTracksToggleOn() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowAllVerseTracks); }
	void SetAllVerseTracksToggle(bool bOnOff);
	void ShowAllVerseTracks() { SetAllVerseTracksToggle(true); }
	void HideAllVerseTracks() { SetAllVerseTracksToggle(false); }
	void ShowHideAllVerseTracks() { SetAllVerseTracksToggle(!IsAllVerseTracksToggleOn()); }

	//////////////////////////////////////////////////
	// CPU

	TSharedPtr<FCpuTimingTrack> GetCpuTrack(uint32 InThreadId);
	const TMap<uint32, TSharedPtr<FCpuTimingTrack>>& GetAllCpuTracks() { return CpuTracks; }
	bool IsCpuTrackVisible(uint32 InThreadId) const;
	void GetVisibleCpuThreads(TSet<uint32>& OutSet) const;

	bool IsAllCpuTracksToggleOn() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowAllCpuTracks); }
	void SetAllCpuTracksToggle(bool bOnOff);
	void ShowAllCpuTracks() { SetAllCpuTracksToggle(true); }
	void HideAllCpuTracks() { SetAllCpuTracksToggle(false); }
	void ShowHideAllCpuTracks() { SetAllCpuTracksToggle(!IsAllCpuTracksToggleOn()); }

	//////////////////////////////////////////////////
	// CPU Sampling

	bool AreCpuSamplingTracksAvailable() const;

	TSharedPtr<FCpuStackSampleTimingTrack> GetCpuSamplingTrack(uint32 InSystemThreadId);
	const TMap<uint32, TSharedPtr<FCpuStackSampleTimingTrack >>& GetAllCpuSamplingTracks() { return CpuStackSampleTracks; }
	bool IsCpuSamplingTrackVisible(uint32 InSystemThreadId) const;
	void GetVisibleCpuSamplingThreads(TSet<uint32>& OutSet) const;

	bool IsAllCpuSamplingTracksToggleOn() const { return TimingViewSettings->IsToggleOptionEnabled(FTimingViewSettings::Option_ShowAllCpuSamplingTracks); }
	void SetAllCpuSamplingTracksToggle(bool bOnOff);
	void ShowAllCpuSamplingTracks() { SetAllCpuSamplingTracksToggle(true); }
	void HideAllCpuSamplingTracks() { SetAllCpuSamplingTracksToggle(false); }
	void ShowHideAllCpuSamplingTracks() { SetAllCpuSamplingTracksToggle(!IsAllCpuSamplingTracksToggleOn()); }

	//////////////////////////////////////////////////
	// ITimingViewExtender interface

	virtual void OnBeginSession(Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(Timing::ITimingViewSession& InSession) override;
	virtual void Tick(Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;
	virtual void ExtendGpuTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;
	virtual void ExtendCpuTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;

	//////////////////////////////////////////////////

	void BindCommands();

	void ResetToDefaultValues();

	void GetVisibleTimelineIndexes(TSet<uint32>& OutSet) const;

	TSharedPtr<const ITimingEvent> FindMaxEventInstance(uint32 TimerId, double StartTime, double EndTime, double MaxValidTime);
	TSharedPtr<const ITimingEvent> FindMinEventInstance(uint32 TimerId, double StartTime, double EndTime, double MaxValidTime);

private:
	void CreateThreadGroupsMenu(FMenuBuilder& MenuBuilder);

	bool ToggleTrackVisibilityByGroup_IsChecked(const TCHAR* InGroupName) const;
	void ToggleTrackVisibilityByGroup_Execute(const TCHAR* InGroupName);
	void ToggleOptionState(const TCHAR* CommandName);

	//////////////////////////////////////////////////
	// GPU

	void Command_ShowGpuWorkTracks_Execute() { SetGpuWorkTracksVisibility(!AreGpuWorkTracksVisible()); }
	bool Command_ShowGpuWorkTracks_CanExecute() { return GpuTracks.Num() > 0; }
	bool Command_ShowGpuWorkTracks_IsChecked() { return AreGpuWorkTracksVisible(); }

	void Command_ShowGpuWorkOverlays_Execute() { ToggleOptionState(FTimingViewSettings::Option_ShowGpuWorkOverlays); }
	bool Command_ShowGpuWorkOverlays_CanExecute() { return AreGpuWorkTracksVisible() && GpuTracks.Num() > 0; }
	bool Command_ShowGpuWorkOverlays_IsChecked() { return AreOverlaysVisibleInGpuQueueTracks(); }

	void Command_ShowGpuWorkExtendedLines_Execute() { ToggleOptionState(FTimingViewSettings::Option_ShowGpuWorkExtendedLines); }
	bool Command_ShowGpuWorkExtendedLines_CanExecute() { return AreGpuWorkTracksVisible() && GpuTracks.Num() > 0; }
	bool Command_ShowGpuWorkExtendedLines_IsChecked() { return AreExtendedLinesVisibleInGpuQueueTracks(); }

	void Command_ShowGpuFencesExtendedLines_Execute() { ToggleOptionState(FTimingViewSettings::Option_ShowGpuFencesExtendedLines); }
	bool Command_ShowGpuFencesExtendedLines_CanExecute() { return AreGpuFencesTracksVisible() && GpuTracks.Num() > 0; }
	bool Command_ShowGpuFencesExtendedLines_IsChecked() { return AreGpuFencesExtendedLinesVisible(); }

	void Command_ShowGpuFencesRelations_Execute();
	bool Command_ShowGpuFencesRelations_CanExecute() { return GpuTracks.Num() > 0; }
	bool Command_ShowGpuFencesRelations_IsChecked() { return AreGpuFenceRelationsVisible(); }

	void Command_ShowGpuFencesTracks_Execute() { SetGpuFencesTracksVisibility(!AreGpuFencesTracksVisible()); }
	bool Command_ShowGpuFencesTracks_CanExecute() { return GpuTracks.Num() > 0; }
	bool Command_ShowGpuFencesTracks_IsChecked() { return AreGpuFencesTracksVisible(); }

	void AddGpuWorkChildTracks();
	void RemoveGpuWorkChildTracks();

	void AddGpuFencesChildTracks();
	void RemoveGpuFencesChildTracks();

	//////////////////////////////////////////////////

	void OnTimingEventSelected(TSharedPtr<const ITimingEvent> InSelectedEvent);

private:
	STimingView* TimingView = nullptr;

	TSharedPtr<FGpuTimingTrack> OldGpu1Track;
	TSharedPtr<FGpuTimingTrack> OldGpu2Track;

	/** Maps GPU queue id to track pointer (GPU Queue timing tracks). */
	TMap<uint32, TSharedPtr<FGpuQueueTimingTrack>> GpuTracks;

	TSharedPtr<FVerseTimingTrack> VerseSamplingTrack;

	/** Maps CPU thread id to track pointer (CPU timing tracks). */
	TMap<uint32, TSharedPtr<FCpuTimingTrack>> CpuTracks;

	/** Maps CPU thread group name to thread group info. */
	TMap<const TCHAR*, FThreadGroup, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<const TCHAR*, FThreadGroup>> ThreadGroups;

	/** Maps system thread id to track pointer (CPU Stack Sample timing tracks). */
	TMap<uint32, TSharedPtr<FCpuStackSampleTimingTrack>> CpuStackSampleTracks;

	uint32 TimingProfilerTimelineCount = 0;
	uint32 LoadTimeProfilerTimelineCount = 0;
	uint32 CpuStackSamplingTimelineCount = 0;

	TSharedRef<ITimingViewSettings> TimingViewSettings;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
