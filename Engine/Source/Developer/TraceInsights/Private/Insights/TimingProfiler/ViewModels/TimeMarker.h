// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/UnrealString.h"
#include "Math/Color.h"

// TraceInsights
#include "Insights/ITimingViewSession.h" // for ITimeMarker


////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimeMarker : public Timing::ITimeMarker
{
public:
	FTimeMarker() {}
	virtual ~FTimeMarker() = default;

	double GetTime() const override { return Time; }
	void SetTime(const double  InTime) override { Time = InTime; }

	const FString& GetName() const { return Name; }
	void SetName(const FString& InName) { Name = InName; }

	const FLinearColor& GetColor() const { return Color; }
	void SetColor(const FLinearColor& InColor) { Color = InColor; }

	bool IsVisible() const { return bIsVisible; }
	void SetVisibility(bool bOnOff) { bIsVisible = bOnOff; }

	bool IsHighlighted() const { return bIsHighlighted; }
	void SetHighlighted(bool bOnOff) { bIsHighlighted = bOnOff; }

	bool IsDragging() const { return bIsDragging; }
	void StartDragging() { bIsDragging = true; }
	void StopDragging() { bIsDragging = false; }

	float GetCrtTextWidth() const { return CrtTextWidth; }
	void SetCrtTextWidthAnimated(const float InTextWidth) const { CrtTextWidth = CrtTextWidth * 0.6f + InTextWidth * 0.4f; }

private:
	double Time = 0.0;
	FString Name = TEXT("T");
	FLinearColor Color = FLinearColor(1.0f, 1.0f, 1.0f, 0.5f);
	bool bIsVisible = true;
	bool bIsHighlighted = false;
	bool bIsDragging = false;

	// Smoothed time marker text width to avoid flickering
	mutable float CrtTextWidth = 0.0f;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
