// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimerAggregator.h"

#include "ProfilingDebugging/MiscTrace.h" // for ETraceFrameType

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/StackSamples.h"

// TraceInsights
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/ViewModels/ThreadTimingSharedState.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/Widgets/STimingView.h"

namespace UE::Insights::TimingProfiler
{

struct FTimerAggregationParams
{
	double StartTime = 0.0;
	double EndTime = -1.0;

	TSet<uint32> GpuQueues;
	bool bIncludeOldGpu1 = false;
	bool bIncludeOldGpu2 = false;

	bool bIncludeVerseSampling = false;

	TSet<uint32> CpuThreads;  // [Thread Id]
	TSet<uint32> CpuSamplingThreads; // [System Thread Id]

	ETimerAggregationMode AggregationMode = ETimerAggregationMode::Instance;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimerAggregationWorker
////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimerAggregationWorker : public IStatsAggregationWorker
{
public:
	FTimerAggregationWorker(TSharedPtr<const TraceServices::IAnalysisSession> InSession, const FTimerAggregationParams& InParams)
		: Session(InSession)
		, StartTime(InParams.StartTime)
		, EndTime(InParams.EndTime)
		, GpuQueues(InParams.GpuQueues)
		, bIncludeOldGpu1(InParams.bIncludeOldGpu1)
		, bIncludeOldGpu2(InParams.bIncludeOldGpu2)
		, bIncludeVerseSampling(InParams.bIncludeVerseSampling)
		, CpuThreads(InParams.CpuThreads)
		, CpuSamplingThreads(InParams.CpuSamplingThreads)
		, AggregationMode(InParams.AggregationMode)
		, ResultTable()
	{
	}

	virtual ~FTimerAggregationWorker()
	{
	}

	virtual void DoWork(TSharedPtr<TraceServices::FCancellationToken> CancellationToken) override;

	TraceServices::ITable<TraceServices::FTimingProfilerAggregatedStats>* GetResultTable() const
	{
		return ResultTable.Get();
	}

	void ResetResults()
	{
		ResultTable.Reset();
	}

	ETraceFrameType GetFrameType() const
	{
		switch (AggregationMode)
		{
		case ETimerAggregationMode::Instance:
			return ETraceFrameType::TraceFrameType_Count;
		case ETimerAggregationMode::GameFrame:
			return ETraceFrameType::TraceFrameType_Game;
		case ETimerAggregationMode::RenderingFrame:
			return ETraceFrameType::TraceFrameType_Rendering;
		default:
			return ETraceFrameType::TraceFrameType_Count;
		}
	}

private:
	TSharedPtr<const TraceServices::IAnalysisSession> Session;
	double StartTime;
	double EndTime;
	TSet<uint32> GpuQueues;
	bool bIncludeOldGpu1;
	bool bIncludeOldGpu2;
	bool bIncludeVerseSampling;
	TSet<uint32> CpuThreads; // [Thread Id]
	TSet<uint32> CpuSamplingThreads; // [System Thread Id]
	ETimerAggregationMode AggregationMode;
	TUniquePtr<TraceServices::ITable<TraceServices::FTimingProfilerAggregatedStats>> ResultTable;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimerAggregationWorker::DoWork(TSharedPtr<TraceServices::FCancellationToken> CancellationToken)
{
	if (!Session.IsValid())
	{
		return;
	}

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
	if (!TimingProfilerProvider)
	{
		return;
	}

	TraceServices::FCreateAggregationParams Params;
	Params.IntervalStart = StartTime;
	Params.IntervalEnd = EndTime;
	Params.GpuQueueFilter = [this](uint32 QueueId) { return GpuQueues.Contains(QueueId); };
	Params.bIncludeOldGpu1 = bIncludeOldGpu1;
	Params.bIncludeOldGpu2 = bIncludeOldGpu2;
	Params.bIncludeVerseSampling = bIncludeVerseSampling;
	Params.CpuThreadFilter = [this](uint32 ThreadId) { return CpuThreads.Contains(ThreadId); };
	Params.FrameType = GetFrameType();
	Params.CancellationToken = CancellationToken;

	bool bHasCpuSamplingTimelines = false;
	if (CpuSamplingThreads.Num() > 0)
	{
		const TraceServices::IStackSamplesProvider* StackSamplesProvider = TraceServices::ReadStackSamplesProvider(*Session.Get());
		if (StackSamplesProvider)
		{
			TraceServices::FProviderReadScopeLock _(*StackSamplesProvider);
			for (uint32 SystemThreadId : CpuSamplingThreads)
			{
				const TraceServices::FStackSampleTimeline* Timeline = StackSamplesProvider->GetTimeline(SystemThreadId);
				if (Timeline)
				{
					Params.CustomTimelines.Add(Timeline);
					bHasCpuSamplingTimelines = true;
				}
			}
			// Aggregation needs to execute under the read locks of both Session and StackSamplesProvider.
			if (bHasCpuSamplingTimelines)
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
				ResultTable.Reset(TimingProfilerProvider->CreateAggregation(Params));
			}
		}
	}
	if (!bHasCpuSamplingTimelines)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		ResultTable.Reset(TimingProfilerProvider->CreateAggregation(Params));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimerAggregator
////////////////////////////////////////////////////////////////////////////////////////////////////

IStatsAggregationWorker* FTimerAggregator::CreateWorker(TSharedPtr<const TraceServices::IAnalysisSession> InSession)
{
	FTimerAggregationParams Params;

	Params.StartTime = GetIntervalStartTime();
	Params.EndTime = GetIntervalEndTime();
	Params.AggregationMode = GetAggregationMode();

	TSharedPtr<STimingProfilerWindow> Wnd = FTimingProfilerManager::Get()->GetProfilerWindow();
	if (Wnd.IsValid())
	{
		TSharedPtr<STimingView> TimingView = Wnd->GetTimingView();
		if (TimingView.IsValid())
		{
			TSharedPtr<FThreadTimingSharedState> ThreadTimingSharedState = TimingView->GetThreadTimingSharedState();
			if (ThreadTimingSharedState.IsValid())
			{
				ThreadTimingSharedState->GetVisibleGpuQueues(Params.GpuQueues);
				Params.bIncludeOldGpu1 = ThreadTimingSharedState->IsOldGpu1TrackVisible();
				Params.bIncludeOldGpu2 = ThreadTimingSharedState->IsOldGpu2TrackVisible();
				Params.bIncludeVerseSampling = ThreadTimingSharedState->IsVerseSamplingTrackVisible();
				ThreadTimingSharedState->GetVisibleCpuThreads(Params.CpuThreads);
				ThreadTimingSharedState->GetVisibleCpuSamplingThreads(Params.CpuSamplingThreads);
			}
		}
	}

	return new FTimerAggregationWorker(InSession, Params);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TraceServices::ITable<TraceServices::FTimingProfilerAggregatedStats>* FTimerAggregator::GetResultTable() const
{
	// It can only be called from OnFinishedCallback.
	check(IsFinished());

	FTimerAggregationWorker* Worker = (FTimerAggregationWorker*)GetWorker();
	check(Worker != nullptr);

	return Worker->GetResultTable();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimerAggregator::ResetResults()
{
	// It can only be called from OnFinishedCallback.
	check(IsFinished());

	FTimerAggregationWorker* Worker = (FTimerAggregationWorker*)GetWorker();
	check(Worker != nullptr);

	Worker->ResetResults();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
