// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceServices
#include "TraceServices/Containers/Tables.h"
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/StatsAggregator.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class ETimerAggregationMode
{
	Instance,
	GameFrame,
	RenderingFrame,

	Count
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimerAggregator : public FStatsAggregator
{
public:
	FTimerAggregator() : FStatsAggregator(TEXT("Timers")) {}
	virtual ~FTimerAggregator() = default;

	TraceServices::ITable<TraceServices::FTimingProfilerAggregatedStats>* GetResultTable() const;
	void ResetResults();

	ETimerAggregationMode GetAggregationMode() const { return AggregationMode; }
	void SetAggregationMode(ETimerAggregationMode InMode) { AggregationMode = InMode; }

protected:
	virtual IStatsAggregationWorker* CreateWorker(TSharedPtr<const TraceServices::IAnalysisSession> InSession) override;

private:
	ETimerAggregationMode AggregationMode = ETimerAggregationMode::Instance;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
