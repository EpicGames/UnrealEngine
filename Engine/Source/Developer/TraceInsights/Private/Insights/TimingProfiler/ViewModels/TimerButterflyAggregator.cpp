// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimerButterflyAggregator.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/StackSamples.h"
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsights
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Tracks/ThreadTimingTrack.h"
#include "Insights/TimingProfiler/ViewModels/ThreadTimingSharedState.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/Widgets/STimingView.h"

namespace UE::Insights::TimingProfiler
{

struct FTimerButterflyAggregationParams
{
	double StartTime = 0.0;
	double EndTime = -1.0;

	TSet<uint32> GpuQueues;
	bool bIncludeOldGpu1 = false;
	bool bIncludeOldGpu2 = false;

	bool bIncludeVerseSampling = false;

	TSet<uint32> CpuThreads; // [Thread Id]
	TSet<uint32> CpuSamplingThreads; // [System Thread Id]
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimerButterflyAggregationWorker
////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimerButterflyAggregationWorker : public IStatsAggregationWorker
{
public:
	FTimerButterflyAggregationWorker(TSharedPtr<const TraceServices::IAnalysisSession> InSession, const FTimerButterflyAggregationParams& InParams)
		: Session(InSession)
		, StartTime(InParams.StartTime)
		, EndTime(InParams.EndTime)
		, GpuQueues(InParams.GpuQueues)
		, bIncludeOldGpu1(InParams.bIncludeOldGpu1)
		, bIncludeOldGpu2(InParams.bIncludeOldGpu2)
		, bIncludeVerseSampling(InParams.bIncludeVerseSampling)
		, CpuThreads(InParams.CpuThreads)
		, CpuSamplingThreads(InParams.CpuSamplingThreads)
		, ResultButterfly()
	{
	}

	virtual ~FTimerButterflyAggregationWorker() {}

	virtual void DoWork(TSharedPtr<TraceServices::FCancellationToken> CancellationToken) override;

	TraceServices::ITimingProfilerButterfly* GetResultButterfly() const { return ResultButterfly.Get(); }
	void ResetResults() { ResultButterfly.Reset(); }

private:
	TSharedPtr<const TraceServices::IAnalysisSession> Session;
	double StartTime;
	double EndTime;
	TSet<uint32> GpuQueues;
	bool bIncludeOldGpu1;
	bool bIncludeOldGpu2;
	bool bIncludeVerseSampling;
	TSet<uint32> CpuThreads; // [Thread Id]
	TSet<uint32> CpuSamplingThreads; // [System Thread Id]
	TUniquePtr<TraceServices::ITimingProfilerButterfly> ResultButterfly;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimerButterflyAggregationWorker::DoWork(TSharedPtr<TraceServices::FCancellationToken> CancellationToken)
{
	if (!Session.IsValid())
	{
		return;
	}

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
	if (!TimingProfilerProvider)
	{
		return;
	}

	double SessionDuration = 0.0;
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		SessionDuration = Session->GetDurationSeconds();
	}
	const double AdjustedEndTime = FMath::Min(SessionDuration, EndTime);
	if (StartTime <= AdjustedEndTime)
	{
		TraceServices::FCreateButterflyParams Params;
		Params.IntervalStart = StartTime;
		Params.IntervalEnd = AdjustedEndTime;
		Params.GpuQueueFilter = [this](uint32 QueueId) { return GpuQueues.Contains(QueueId); };
		Params.bIncludeOldGpu1 = bIncludeOldGpu1;
		Params.bIncludeOldGpu2 = bIncludeOldGpu2;
		Params.bIncludeVerseSampling = bIncludeVerseSampling;
		Params.CpuThreadFilter = [this](uint32 ThreadId) { return CpuThreads.Contains(ThreadId); };

		bool bHasCpuSamplingTimelines = false;
		if (CpuSamplingThreads.Num() > 0)
		{
			const TraceServices::IStackSamplesProvider* StackSamplesProvider = TraceServices::ReadStackSamplesProvider(*Session.Get());
			if (StackSamplesProvider)
			{
				TraceServices::FProviderReadScopeLock _(*StackSamplesProvider);
				for (uint32 SystemThreadId : CpuSamplingThreads)
				{
					const TraceServices::FStackSampleTimeline* Timeline = StackSamplesProvider->GetTimeline(SystemThreadId);
					if (Timeline)
					{
						Params.CustomTimelines.Add(Timeline);
						bHasCpuSamplingTimelines = true;
					}
				}
				// Aggregation needs to execute under the read locks of both Session and StackSamplesProvider.
				if (bHasCpuSamplingTimelines)
				{
					TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
					ResultButterfly.Reset(TimingProfilerProvider->CreateButterfly(Params));
				}
			}
		}
		if (!bHasCpuSamplingTimelines)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			ResultButterfly.Reset(TimingProfilerProvider->CreateButterfly(Params));
		}
	}
	else
	{
		// Generate empty results (invalid time range selection).
		TraceServices::FCreateButterflyParams Params;
		Params.IntervalStart = 0.0;
		Params.IntervalEnd = -1.0;

		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		ResultButterfly.Reset(TimingProfilerProvider->CreateButterfly(Params));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimerButterflyAggregator
////////////////////////////////////////////////////////////////////////////////////////////////////

IStatsAggregationWorker* FTimerButterflyAggregator::CreateWorker(TSharedPtr<const TraceServices::IAnalysisSession> InSession)
{
	FTimerButterflyAggregationParams Params;

	Params.StartTime = GetIntervalStartTime();
	Params.EndTime = GetIntervalEndTime();

	TSharedPtr<STimingProfilerWindow> Wnd = FTimingProfilerManager::Get()->GetProfilerWindow();
	if (Wnd.IsValid())
	{
		TSharedPtr<STimingView> TimingView = Wnd->GetTimingView();
		if (TimingView.IsValid())
		{
			TSharedPtr<FThreadTimingSharedState> ThreadTimingSharedState = TimingView->GetThreadTimingSharedState();
			if (ThreadTimingSharedState.IsValid())
			{
				ThreadTimingSharedState->GetVisibleGpuQueues(Params.GpuQueues);
				Params.bIncludeOldGpu1 = ThreadTimingSharedState->IsOldGpu1TrackVisible();
				Params.bIncludeOldGpu2 = ThreadTimingSharedState->IsOldGpu2TrackVisible();
				Params.bIncludeVerseSampling = ThreadTimingSharedState->IsVerseSamplingTrackVisible();
				ThreadTimingSharedState->GetVisibleCpuThreads(Params.CpuThreads);
				ThreadTimingSharedState->GetVisibleCpuSamplingThreads(Params.CpuSamplingThreads);
			}
		}
	}

	return new FTimerButterflyAggregationWorker(InSession, Params);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TraceServices::ITimingProfilerButterfly* FTimerButterflyAggregator::GetResultButterfly() const
{
	// It can only be called from OnFinishedCallback.
	check(IsFinished());

	FTimerButterflyAggregationWorker* Worker = (FTimerButterflyAggregationWorker*)GetWorker();
	check(Worker != nullptr);

	return Worker->GetResultButterfly();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimerButterflyAggregator::ResetResults()
{
	// It can only be called from OnFinishedCallback.
	check(IsFinished());

	FTimerButterflyAggregationWorker* Worker = (FTimerButterflyAggregationWorker*)GetWorker();
	check(Worker != nullptr);

	Worker->ResetResults();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
