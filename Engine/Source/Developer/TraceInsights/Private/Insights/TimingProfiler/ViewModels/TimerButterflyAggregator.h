// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceServices
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/StatsAggregator.h"

namespace TraceServices
{
	class IAnalysisSession;
	class ITimingProfilerButterfly;
}

namespace UE::Insights::TimingProfiler
{

class FTimerButterflyAggregator : public FStatsAggregator
{
public:
	FTimerButterflyAggregator() : FStatsAggregator(TEXT("Butterfly")) {}
	virtual ~FTimerButterflyAggregator() = default;

	TraceServices::ITimingProfilerButterfly* GetResultButterfly() const;
	void ResetResults();

protected:
	virtual IStatsAggregationWorker* CreateWorker(TSharedPtr<const TraceServices::IAnalysisSession> InSession) override;
};

} // namespace UE::Insights::TimingProfiler
