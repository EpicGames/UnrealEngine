// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimerFilters.h"

#include "CborReader.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"

// TraceServices
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Tracks/ThreadTimingTrack.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::TimerFilters"

namespace UE::Insights::TimingProfiler
{

INSIGHTS_IMPLEMENT_RTTI(FTimerNameFilterState)
INSIGHTS_IMPLEMENT_RTTI(FTimerNameFilter)

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimerNameFilterState
////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimerNameFilterState::Update()
{
	if (FilterValue.IsEmpty() || !SelectedOperator.IsValid())
	{
		return;
	}

	TimerIds.Empty();

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
	if (TimingProfilerProvider)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();

		uint32 TimerCount = TimerReader.GetTimerCount();
		for (uint32 TimerIndex = 0; TimerIndex < TimerCount; ++TimerIndex)
		{
			const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(TimerIndex);
			if (Timer && Timer->Name)
			{
				if (SelectedOperator->GetKey() == EFilterOperator::Eq)
				{
					if (FCString::Stricmp(Timer->Name, *FilterValue) == 0)
					{
						TimerIds.Add(Timer->Id);
					}
				}
				else if (SelectedOperator->GetKey() == EFilterOperator::Contains)
				{
					if (FCString::Stristr(Timer->Name, *FilterValue))
					{
						TimerIds.Add(Timer->Id);
					}
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimerNameFilterState::ApplyFilter(const FBaseFilterContext& Context) const
{
	if (!Context.HasData(Filter->GetKey()))
	{
		return Context.GetDefaultReturnValue();
	}

	uint32 TimerId = static_cast<uint32>(Context.GetDataAsInt64(Filter->GetKey()));
	return TimerIds.Contains(TimerId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimerNameFilterState::Equals(const FFilterState& Other) const
{
	if (this->GetTypeName() != Other.GetTypeName())
	{
		return false;
	}

	const FTimerNameFilterState& OtherTimerNameFilter = StaticCast<const FTimerNameFilterState&>(Other);
	return FilterValue == OtherTimerNameFilter.FilterValue;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FFilterState> FTimerNameFilterState::DeepCopy() const
{
	TSharedRef<FTimerNameFilterState> Copy = MakeShared<FTimerNameFilterState>(*this);
	return Copy;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimerNameFilter
////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNameFilter::FTimerNameFilter()
	: FFilterWithSuggestions(
		static_cast<int32>(EFilterField::TimerName),
		LOCTEXT("TimerName", "Timer Name"),
		LOCTEXT("TimerName", "Timer Name"),
		EFilterDataType::Custom,
		nullptr,
		nullptr)
{
	SupportedOperators = MakeShared<TArray<TSharedPtr<IFilterOperator>>>();
	SupportedOperators->Add(StaticCastSharedRef<IFilterOperator>(MakeShared<FFilterOperator<int64>>(
		EFilterOperator::Eq, TEXT("IS"), [](int64 lhs, int64 rhs) { return lhs == rhs; })));
	SupportedOperators->Add(StaticCastSharedRef<IFilterOperator>(MakeShared<FFilterOperator<int64>>(
		EFilterOperator::Contains, TEXT("CONTAINS"), [](int64 lhs, int64 rhs) { return lhs == rhs; })));

	SetCallback([this](const FString& Text, TArray<FString>& OutSuggestions)
		{
			this->PopulateTimerNameSuggestionList(Text, OutSuggestions);
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimerNameFilter::PopulateTimerNameSuggestionList(const FString& Text, TArray<FString>& OutSuggestions)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
	if (TimingProfilerProvider)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();

		uint32 TimerCount = TimerReader.GetTimerCount();
		for (uint32 TimerIndex = 0; TimerIndex < TimerCount; ++TimerIndex)
		{
			const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(TimerIndex);
			if (Timer && Timer->Name)
			{
				if (Text.IsEmpty())
				{
					OutSuggestions.Add(Timer->Name);
					continue;
				}
				const TCHAR* FoundString = FCString::Stristr(Timer->Name, *Text);
				if (FoundString)
				{
					OutSuggestions.Add(Timer->Name);
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMetadataFilterState
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FMetadataFilterState)

FMetadataFilterState::FMetadataFilterState(TSharedRef<FFilter> InFilter)
	: FFilterState(InFilter)
{
	AvailableDataTypes.Add(MakeShared<FMetadataFilterDataTypeEntry>(EMetadataFilterDataType::Int, LOCTEXT("IntDataType", "Int")));
	AvailableDataTypes.Add(MakeShared<FMetadataFilterDataTypeEntry>(EMetadataFilterDataType::Double, LOCTEXT("DoubleDataType", "Double")));
	AvailableDataTypes.Add(MakeShared<FMetadataFilterDataTypeEntry>(EMetadataFilterDataType::Bool, LOCTEXT("BoolDataType", "Bool")));
	AvailableDataTypes.Add(MakeShared<FMetadataFilterDataTypeEntry>(EMetadataFilterDataType::String, LOCTEXT("StringDataType", "String")));

	SelectedDataType = AvailableDataTypes[0];
	DataType_OnSelectionChanged(SelectedDataType, ESelectInfo::Type::Direct);

	BoolOperators.Add((MakeShared<FFilterOperator<bool>>(EFilterOperator::Eq, TEXT("IS"), [](bool lhs, bool rhs) { return lhs == rhs; })));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataFilterState::AddCustomUI(TSharedRef<SHorizontalBox> Box)
{
	Box->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(4.0f, 0.0f, 4.0f, 0.0f))
		[
			SNew(SEditableTextBox)
			.MinDesiredWidth(50.0f)
			.OnTextCommitted(this, &FMetadataFilterState::OnKeyTextBoxValueCommitted)
			.Text(this, &FMetadataFilterState::GetKeyTextBoxValue)
			.ToolTipText(LOCTEXT("MetadataKeyTooltipText", "The key value for the metadata."))
			.HintText(LOCTEXT("MetadataKeyHint", "The metadata field name"))
		];

	Box->AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(FMargin(0.0f, 2.0f))
		[
			SNew(SComboBox<TSharedPtr<FMetadataFilterDataTypeEntry>>)
			.OptionsSource(&AvailableDataTypes)
			.OnSelectionChanged(this, &FMetadataFilterState::DataType_OnSelectionChanged)
			.OnGenerateWidget(this, &FMetadataFilterState::DataType_OnGenerateWidget)
			[
				SNew(STextBlock)
				.Text(this, &FMetadataFilterState::DataType_GetSelectionText)
			]
		];

		Box->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.FillWidth(1.0)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.AutoWidth()
			[
				SAssignNew(OperatorComboBox, SComboBox<TSharedPtr<IFilterOperator>>)
				.OptionsSource(&AvailableOperators)
				.OnSelectionChanged(this, &FMetadataFilterState::AvailableOperators_OnSelectionChanged)
				.OnGenerateWidget(this, &FMetadataFilterState::AvailableOperators_OnGenerateWidget)
				[
					SNew(STextBlock)
					.Text(this, &FMetadataFilterState::AvailableOperators_GetSelectionText)
				]
			];

		Box->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.FillWidth(1.0)
		.Padding(FMargin(4.0f, 0.0f, 4.0f, 0.0f))
		[
			SNew(SEditableTextBox)
			.MinDesiredWidth(200.0f)
			.OnTextCommitted(this, &FMetadataFilterState::OnTermTextBoxValueCommitted)
			.Text(this, &FMetadataFilterState::GetTermTextBoxValue)
			.ToolTipText(LOCTEXT("MetadataValueTooltipText", "The value of the metadata field."))
			.HintText(LOCTEXT("MetadataValueHint", "The value for the metadata field."))
		];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataFilterState::Update()
{
	bKeyAllowsAll = false;
	bTermAllowsAll = false;

	if (Key.Equals(TEXT("*")))
	{
		bKeyAllowsAll = true;
	}
	if(Term.Equals(TEXT("*")))
	{
		bTermAllowsAll = true;
	}

	ConvertedData = ConvertedDataVariant();

	ensure(SelectedDataType.IsValid());
	switch (SelectedDataType->Type)
	{
		case EMetadataFilterDataType::Bool:
		{
			ConvertedData.Set<bool>(FCString::ToBool(*Term));
			break;
		}
		case EMetadataFilterDataType::Int:
		{
			ConvertedData.Set<int64>(FCString::Atoi64(*Term));
			break;
		}
		case EMetadataFilterDataType::Double:
		{
			ConvertedData.Set<double>(FCString::Atod(*Term));
			break;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMetadataFilterState::ApplyFilter(const FBaseFilterContext& Context) const
{
	if (!Context.HasData(Filter->GetKey()))
	{
		return Context.GetDefaultReturnValue();
	}

	uint32 TimerIndex = static_cast<uint32>(Context.GetDataAsInt64(Filter->GetKey()));
	if (static_cast<int32>(TimerIndex) >= 0)
	{
		return false;
	}
	else if (ShouldShowAllMetadataEvents())
	{
		return true;
	}

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return false;
	}

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
	if (TimingProfilerProvider == nullptr)
	{
		return false;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
	const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();

	const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(TimerIndex);
	TArrayView<const uint8> Metadata = TimerReader.GetMetadata(TimerIndex);
	if (Timer->HasValidMetadataSpecId())
	{
		const TraceServices::FMetadataSpec* MetadataSpec = nullptr;
		MetadataSpec = TimingProfilerProvider->GetMetadataSpec(Timer->MetadataSpecId);
		if (MetadataSpec)
		{
			return ApplyFilterToMetadata(MetadataSpec, Metadata);
		}
	}

	if (Metadata.Num() > 0)
	{
		return ApplyFilterToMetadata(nullptr, Metadata);
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMetadataFilterState::ApplyFilterToMetadata(const TraceServices::FMetadataSpec* MetadataSpec, TArrayView<const uint8>& Metadata) const
{
	FMemoryReaderView MemoryReader(Metadata);
	FCborReader CborReader(&MemoryReader, ECborEndianness::StandardCompliant);
	FCborContext Context;

	ensure(SelectedDataType);

	// We have 3 cases:
	// 1. InMetadataSpec in not null and InMetadata is a list of values (dynamic metadata)
	// 2. InMetadataSpec is null and InMetadata is a list of values (dynamic metadata without a spec emited or saved)
	// 3. InMetadataSpec is null and InMetadata is a map of key value pairs. (static metadata)
	bool bIsMap = false;
	bool bHasReadFirstValue = false;
	if (MetadataSpec == nullptr)
	{
		if (!CborReader.ReadNext(Context))
		{
			return false;
		}

		if (Context.MajorType() == ECborCode::Map)
		{
			bIsMap = true;
		}
		else
		{
			bHasReadFirstValue = true;
		}
	}

	for (uint32 Index = 0; true; ++Index)
	{
		bool bKeyMatched = false;
		if (MetadataSpec)
		{
			TStringView<const TCHAR> CurrentKey;
			if (Index < static_cast<uint32>(MetadataSpec->FieldNames.Num()))
			{
				CurrentKey = MetadataSpec->FieldNames[Index];
			}

			bKeyMatched = CurrentKey.Equals(Key, ESearchCase::IgnoreCase);
		}
		else if (bIsMap)
		{
			// Read key
			if (!CborReader.ReadNext(Context) || !Context.IsString())
			{
				break;
			}

			TStringView<const ANSICHAR> CurrentKey(Context.AsCString(), static_cast<int32>(Context.AsLength()));
			bKeyMatched = CurrentKey.Equals(Key, ESearchCase::IgnoreCase);
		}
		else
		{
			bKeyMatched = Key.IsEmpty();
		}

		if (Index > 0 || !bHasReadFirstValue)
		{
			if (!CborReader.ReadNext(Context))
			{
				break;
			}
		}

		if (Context.MajorType() == ECborCode::Array)
		{
			CborReader.SkipContainer(ECborCode::Array);
		}

		if (!bKeyMatched && !bKeyAllowsAll)
		{
			continue;
		}

		if (bTermAllowsAll)
		{
			return true;
		}

		switch (Context.MajorType())
		{
		case ECborCode::Int:
		case ECborCode::Uint:
		{
			if (SelectedDataType->Type != EMetadataFilterDataType::Int)
			{
				continue;
			}
			ensure(ConvertedData.IsType<int64>());

			int64 MetadataValue = Context.AsInt();
			int64 InputValue = ConvertedData.Get<int64>();

			using FFilterOperatorInt64 = FFilterOperator<int64>;
			FFilterOperatorInt64* Operator = (FFilterOperatorInt64*)SelectedOperator.Get();
			if (Operator->Apply(MetadataValue, InputValue))
			{
				return true;
			}

			continue;
		}

		case ECborCode::TextString:
		{
			if (SelectedDataType->Type != EMetadataFilterDataType::String)
			{
				continue;
			}

			FString Value = Context.AsString();

			using FFilterOperatorString = FFilterOperator<FStringView>;
			FFilterOperatorString* Operator = (FFilterOperatorString*)SelectedOperator.Get();
			if (Operator->Apply(Value, Term))
			{
				return true;
			}

			continue;
		}

		case ECborCode::ByteString:
		{
			if (SelectedDataType->Type != EMetadataFilterDataType::String)
			{
				continue;
			}

			FAnsiStringView Value(Context.AsCString(), static_cast<int32>(Context.AsLength()));
			FString ValueStr(Value);

			using FFilterOperatorString = FFilterOperator<FStringView>;
			FFilterOperatorString* Operator = (FFilterOperatorString*)SelectedOperator.Get();
			if (Operator->Apply(ValueStr, Term))
			{
				return true;
			}

			continue;
		}
		}

		if (Context.RawCode() == (ECborCode::Prim | ECborCode::Value_4Bytes))
		{
			if (SelectedDataType->Type != EMetadataFilterDataType::Double)
			{
				continue;
			}
			ensure(ConvertedData.IsType<double>());

			double Value = static_cast<double>(Context.AsFloat());
			double InputValue = ConvertedData.Get<double>();

			using FFilterOperatorDouble = FFilterOperator<double>;
			FFilterOperatorDouble* Operator = (FFilterOperatorDouble*)SelectedOperator.Get();
			if (Operator->Apply(Value, InputValue))
			{
				return true;
			}

			continue;
		}

		if (Context.RawCode() == (ECborCode::Prim | ECborCode::Value_8Bytes))
		{
			if (SelectedDataType->Type != EMetadataFilterDataType::Double)
			{
				continue;
			}
			ensure(ConvertedData.IsType<double>());

			double Value = Context.AsDouble();
			double InputValue = ConvertedData.Get<double>();

			using FFilterOperatorDouble = FFilterOperator<double>;
			FFilterOperatorDouble* Operator = (FFilterOperatorDouble*)SelectedOperator.Get();
			if (Operator->Apply(Value, InputValue))
			{
				return true;
			}

			continue;
		}

		if (Context.RawCode() == (ECborCode::Prim | ECborCode::False))
		{
			if (SelectedDataType->Type != EMetadataFilterDataType::Bool)
			{
				continue;
			}
			ensure(ConvertedData.IsType<bool>());

			using FFilterOperatorBool = FFilterOperator<bool>;
			FFilterOperatorBool* Operator = (FFilterOperatorBool*)SelectedOperator.Get();
			if (Operator->Apply(ConvertedData.Get<bool>(), false))
			{
				return true;
			}

			continue;
		}

		if (Context.RawCode() == (ECborCode::Prim | ECborCode::True))
		{
			if (SelectedDataType->Type != EMetadataFilterDataType::Bool)
			{
				continue;
			}
			ensure(ConvertedData.IsType<bool>());

			using FFilterOperatorBool = FFilterOperator<bool>;
			FFilterOperatorBool* Operator = (FFilterOperatorBool*)SelectedOperator.Get();
			if (Operator->Apply(ConvertedData.Get<bool>(), true))
			{
				return true;
			}

			continue;
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMetadataFilterState::GetKeyTextBoxValue() const
{
	return FText::FromString(Key);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataFilterState::OnKeyTextBoxValueCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	Key = InNewText.ToString();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> FMetadataFilterState::DataType_OnGenerateWidget(TSharedPtr<FMetadataFilterDataTypeEntry> InDataType)
{
	TSharedRef<SHorizontalBox> Widget = SNew(SHorizontalBox);
	Widget->AddSlot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(InDataType->Name)
			.Margin(2.0f)
		];

	return Widget;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataFilterState::DataType_OnSelectionChanged(TSharedPtr<FMetadataFilterDataTypeEntry> InDataType, ESelectInfo::Type SelectInfo)
{
	SelectedDataType = InDataType;

	AvailableOperators.Empty();
	SelectedOperator = nullptr;

	switch (SelectedDataType->Type)
	{
	case EMetadataFilterDataType::Bool:
	{
		AvailableOperators.Insert(BoolOperators, 0);
		break;
	}
	case EMetadataFilterDataType::Int:
	{
		AvailableOperators.Insert(*FFilterService::Get()->GetIntegerOperators(), 0);
		break;
	}
	case EMetadataFilterDataType::Double:
	{
		AvailableOperators.Insert(*FFilterService::Get()->GetDoubleOperators(), 0);
		break;
	}
	case EMetadataFilterDataType::String:
	{
		AvailableOperators.Insert(*FFilterService::Get()->GetStringOperators(), 0);
		break;
	}
	}

	if (AvailableOperators.Num() > 0)
	{
		AvailableOperators_OnSelectionChanged(AvailableOperators[0], ESelectInfo::Type::Direct);

		if (OperatorComboBox.IsValid())
		{
			OperatorComboBox->RefreshOptions();
			OperatorComboBox->SetSelectedItem(AvailableOperators[0]);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMetadataFilterState::DataType_GetSelectionText() const
{
	return SelectedDataType->Name;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> FMetadataFilterState::AvailableOperators_OnGenerateWidget(TSharedPtr<IFilterOperator> InOperator)
{
	TSharedRef<SHorizontalBox> Widget = SNew(SHorizontalBox);
	Widget->AddSlot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(FText::FromString(InOperator->GetName()))
			.Margin(2.0f)
		];

	return Widget;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataFilterState::AvailableOperators_OnSelectionChanged(TSharedPtr<IFilterOperator> InOperator, ESelectInfo::Type SelectInfo)
{
	if (InOperator)
	{
		SelectedOperator = InOperator;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMetadataFilterState::AvailableOperators_GetSelectionText() const
{
	return FText::FromString(SelectedOperator.IsValid() ? SelectedOperator->GetName(): TEXT(""));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FMetadataFilterState::GetTermTextBoxValue() const
{
	return FText::FromString(Term);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMetadataFilterState::OnTermTextBoxValueCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	Term = InNewText.ToString();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FMetadataFilterState::Equals(const FFilterState& Other) const
{
	if (this->GetTypeName() != Other.GetTypeName())
	{
		return false;
	}

	bool bIsEqual = true;

	const FMetadataFilterState& OtherMetadataFilter = StaticCast<const FMetadataFilterState&>(Other);

	bIsEqual &= Key.Equals(OtherMetadataFilter.Key);
	bIsEqual &= Term.Equals(OtherMetadataFilter.Term);
	bIsEqual &= SelectedDataType == OtherMetadataFilter.SelectedDataType;

	return bIsEqual;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FFilterState> FMetadataFilterState::DeepCopy() const
{
	TSharedRef<FMetadataFilterState> Copy = MakeShared<FMetadataFilterState>(*this);

	return Copy;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMetadataFilter
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FMetadataFilter)

FMetadataFilter::FMetadataFilter()
	: FFilter(static_cast<int32>(EFilterField::Metadata),
		LOCTEXT("MetadataFilterName", "Metadata"),
		LOCTEXT("MetadataFilterDesc", "A filter for timing event metadata."),
		EFilterDataType::Custom,
		nullptr,
		nullptr)
{
	SupportedOperators = MakeShared<TArray<TSharedPtr<IFilterOperator>>>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
