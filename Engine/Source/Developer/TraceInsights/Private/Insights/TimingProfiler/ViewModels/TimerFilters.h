// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Types/SlateEnums.h"
#include "Widgets/Input/SComboBox.h"

// TraceInsightsCore
#include "InsightsCore/Common/SimpleRtti.h"
#include "InsightsCore/Filter/ViewModels/Filters.h"

namespace TraceServices
{
	struct FMetadataSpec;
}

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimerNameFilterState : public FFilterState
{
	INSIGHTS_DECLARE_RTTI(FTimerNameFilterState, FFilterState)

public:
	FTimerNameFilterState(TSharedRef<FFilter> InFilter)
		: FFilterState(InFilter)
	{}

	FTimerNameFilterState(const FTimerNameFilterState& Other)
		: FFilterState(Other)
	{
		FilterValue = Other.FilterValue;
	}

	virtual ~FTimerNameFilterState() = default;

	virtual void Update() override;

	virtual bool ApplyFilter(const FBaseFilterContext& Context) const override;

	virtual void SetFilterValue(FString InFilterValue) override { FilterValue = InFilterValue; }

	virtual bool Equals(const FFilterState& Other) const;
	virtual TSharedRef<FFilterState> DeepCopy() const;

private:
	FString FilterValue;
	TSet<uint32> TimerIds;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimerNameFilter : public FFilterWithSuggestions
{
	INSIGHTS_DECLARE_RTTI(FTimerNameFilter, FFilterWithSuggestions)

public:
	FTimerNameFilter();

	virtual ~FTimerNameFilter() = default;

	virtual TSharedRef<FFilterState> BuildFilterState()
	{
		return MakeShared<FTimerNameFilterState>(SharedThis(this));
	}

	virtual TSharedRef<FFilterState> BuildFilterState(const FFilterState& Other)
	{
		return MakeShared<FTimerNameFilterState>(static_cast<const FTimerNameFilterState&>(Other));
	}

	void PopulateTimerNameSuggestionList(const FString& Text, TArray<FString>& OutSuggestions);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class EMetadataFilterDataType
{
	Bool = 1,
	Int = 2,
	Double = 3,
	String = 4,
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FMetadataFilterDataTypeEntry
{
	FMetadataFilterDataTypeEntry(EMetadataFilterDataType InType, FText InName)
	{
		Type = InType;
		Name = InName;
	}

	EMetadataFilterDataType Type;
	FText Name;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMetadataFilterState : public FFilterState, public TSharedFromThis<FMetadataFilterState>
{
	INSIGHTS_DECLARE_RTTI(FMetadataFilterState, FFilterState)

public:
	FMetadataFilterState(TSharedRef<FFilter> InFilter);

	virtual ~FMetadataFilterState() = default;

	virtual void Update() override;

	virtual bool ApplyFilter(const FBaseFilterContext& Context) const override;

	virtual void SetFilterValue(FString InFilterValue) override {}

	virtual bool HasCustomUI() const override { return true; }
	virtual void AddCustomUI(TSharedRef<SHorizontalBox> LeftBox) override;

	virtual bool Equals(const FFilterState& Other) const override ;
	virtual TSharedRef<FFilterState> DeepCopy() const override ;

private:
	FText GetKeyTextBoxValue() const;
	void OnKeyTextBoxValueCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

	TSharedRef<SWidget> DataType_OnGenerateWidget(TSharedPtr<FMetadataFilterDataTypeEntry> InDataType);
	void DataType_OnSelectionChanged(TSharedPtr<FMetadataFilterDataTypeEntry> InDataType, ESelectInfo::Type SelectInfo);
	FText DataType_GetSelectionText() const;

	TSharedRef<SWidget> AvailableOperators_OnGenerateWidget(TSharedPtr<IFilterOperator> InOperator);
	void AvailableOperators_OnSelectionChanged(TSharedPtr<IFilterOperator> InOperator, ESelectInfo::Type SelectInfo);
	FText AvailableOperators_GetSelectionText() const;

	FText GetTermTextBoxValue() const;
	void OnTermTextBoxValueCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

	bool ApplyFilterToMetadata(const TraceServices::FMetadataSpec* MetadataSpec, TArrayView<const uint8>& Metadata) const;

	bool ShouldShowAllMetadataEvents() const { return bKeyAllowsAll && bTermAllowsAll; }

private:
	FString Key;
	FString Term;

	typedef TVariant<double, int64, bool> ConvertedDataVariant;
	ConvertedDataVariant ConvertedData;

	bool bKeyAllowsAll = false;
	bool bTermAllowsAll = false;

	TArray<TSharedPtr<FMetadataFilterDataTypeEntry>> AvailableDataTypes;
	TSharedPtr<FMetadataFilterDataTypeEntry> SelectedDataType;

	TArray<TSharedPtr<IFilterOperator>> AvailableOperators;
	TArray<TSharedPtr<IFilterOperator>> BoolOperators;

	TSharedPtr<SComboBox<TSharedPtr<IFilterOperator>>> OperatorComboBox;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMetadataFilter : public FFilter
{
	INSIGHTS_DECLARE_RTTI(FMetadataFilter, FFilter)

public:
	FMetadataFilter();

	virtual ~FMetadataFilter() = default;

	virtual TSharedRef<FFilterState> BuildFilterState()
	{
		return MakeShared<FMetadataFilterState>(SharedThis(this));
	}

	virtual TSharedRef<FFilterState> BuildFilterState(const FFilterState& Other)
	{
		return MakeShared<FMetadataFilterState>(static_cast<const FMetadataFilterState&>(Other));
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
