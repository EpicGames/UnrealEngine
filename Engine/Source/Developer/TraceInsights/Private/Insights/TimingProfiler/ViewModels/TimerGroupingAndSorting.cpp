// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimerGroupingAndSorting.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/TimerNodeHelper.h"

//#define INSIGHTS_ENSURE ensure
#define INSIGHTS_ENSURE(...)

// Sort by name (ascending).
#define INSIGHTS_DEFAULT_SORTING_NODES(A, B) return (A->GetDisplayName().CompareTo(B->GetDisplayName()) < 0);
//#define INSIGHTS_DEFAULT_SORTING_NODES(A, B) return A->GetDefaultSortOrder() < B->GetDefaultSortOrder();

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::TimerNode"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorting by Timer Type
////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNodeSortingByTimerType::FTimerNodeSortingByTimerType(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByTimerType")),
		LOCTEXT("Sorting_ByTimerType_Name", "By Type"),
		LOCTEXT("Sorting_ByTimerType_Title", "Sort By Type"),
		LOCTEXT("Sorting_ByTimerType_Desc", "Sort by timer type."),
		InColumnRef)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimerNodeSortingByTimerType::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->GetTypeName() == FTimerNode::TypeName);
			const ETimerNodeType ValueA = reinterpret_cast<FTimerNode*>(A.Get())->GetType();

			INSIGHTS_ENSURE(B.IsValid() && B->GetTypeName() == FTimerNode::TypeName);
			const ETimerNodeType ValueB = reinterpret_cast<FTimerNode*>(B.Get())->GetType();

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by timer type (ascending).
				return ValueA < ValueB;
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->GetTypeName() == FTimerNode::TypeName);
			const ETimerNodeType ValueA = reinterpret_cast<FTimerNode*>(A.Get())->GetType();

			INSIGHTS_ENSURE(B.IsValid() && B->GetTypeName() == FTimerNode::TypeName);
			const ETimerNodeType ValueB = reinterpret_cast<FTimerNode*>(B.Get())->GetType();

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by timer type (descending).
				return ValueB < ValueA;
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sort by Instance Count
////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNodeSortingByInstanceCount::FTimerNodeSortingByInstanceCount(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByInstanceCount")),
		LOCTEXT("Sorting_ByInstanceCount_Name", "By Instance Count"),
		LOCTEXT("Sorting_ByInstanceCount_Title", "Sort By Instance Count"),
		LOCTEXT("Sorting_ByInstanceCount_Desc", "Sort by aggregated instance count."),
		InColumnRef)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimerNodeSortingByInstanceCount::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->GetTypeName() == FTimerNode::TypeName);
			const uint64 ValueA = reinterpret_cast<FTimerNode*>(A.Get())->GetAggregatedStats().InstanceCount;

			INSIGHTS_ENSURE(B.IsValid() && B->GetTypeName() == FTimerNode::TypeName);
			const uint64 ValueB = reinterpret_cast<FTimerNode*>(B.Get())->GetAggregatedStats().InstanceCount;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by instance count (ascending).
				return ValueA < ValueB;
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->GetTypeName() == FTimerNode::TypeName);
			const uint64 ValueA = reinterpret_cast<FTimerNode*>(A.Get())->GetAggregatedStats().InstanceCount;

			INSIGHTS_ENSURE(B.IsValid() && B->GetTypeName() == FTimerNode::TypeName);
			const uint64 ValueB = reinterpret_cast<FTimerNode*>(B.Get())->GetAggregatedStats().InstanceCount;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by instance count (descending).
				return ValueB < ValueA;
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sort by Total Inclusive Time
////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNodeSortingByTotalInclusiveTime::FTimerNodeSortingByTotalInclusiveTime(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByTotalInclusiveTime")),
		LOCTEXT("Sorting_ByTotalInclusiveTime_Name", "By Total Inclusive Time"),
		LOCTEXT("Sorting_ByTotalInclusiveTime_Title", "Sort By Total Inclusive Time"),
		LOCTEXT("Sorting_ByTotalInclusiveTime_Desc", "Sort by aggregated total inclusive time."),
		InColumnRef)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimerNodeSortingByTotalInclusiveTime::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->GetTypeName() == FTimerNode::TypeName);
			const double ValueA = reinterpret_cast<FTimerNode*>(A.Get())->GetAggregatedStats().TotalInclusiveTime;

			INSIGHTS_ENSURE(B.IsValid() && B->GetTypeName() == FTimerNode::TypeName);
			const double ValueB = reinterpret_cast<FTimerNode*>(B.Get())->GetAggregatedStats().TotalInclusiveTime;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by total inclusive time (ascending).
				return ValueA < ValueB;
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->GetTypeName() == FTimerNode::TypeName);
			const double ValueA = reinterpret_cast<FTimerNode*>(A.Get())->GetAggregatedStats().TotalInclusiveTime;

			INSIGHTS_ENSURE(B.IsValid() && B->GetTypeName() == FTimerNode::TypeName);
			const double ValueB = reinterpret_cast<FTimerNode*>(B.Get())->GetAggregatedStats().TotalInclusiveTime;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by total inclusive time (descending).
				return ValueB < ValueA;
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sort by Total Exclusive Time
////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNodeSortingByTotalExclusiveTime::FTimerNodeSortingByTotalExclusiveTime(TSharedRef<FTableColumn> InColumnRef)
	: FTableCellValueSorter(
		FName(TEXT("ByTotalExclusiveTime")),
		LOCTEXT("Sorting_ByTotalExclusiveTime_Name", "By Total Exclusive Time"),
		LOCTEXT("Sorting_ByTotalExclusiveTime_Title", "Sort By Total Exclusive Time"),
		LOCTEXT("Sorting_ByTotalExclusiveTime_Desc", "Sort by aggregated total exclusive time."),
		InColumnRef)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimerNodeSortingByTotalExclusiveTime::Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const
{
	if (SortMode == ESortMode::Ascending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->GetTypeName() == FTimerNode::TypeName);
			const double ValueA = reinterpret_cast<FTimerNode*>(A.Get())->GetAggregatedStats().TotalExclusiveTime;

			INSIGHTS_ENSURE(B.IsValid() && B->GetTypeName() == FTimerNode::TypeName);
			const double ValueB = reinterpret_cast<FTimerNode*>(B.Get())->GetAggregatedStats().TotalExclusiveTime;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by total exclusive time (ascending).
				return ValueA < ValueB;
			}
		});
	}
	else // if (SortMode == ESortMode::Descending)
	{
		NodesToSort.Sort([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
		{
			INSIGHTS_ENSURE(A.IsValid() && A->GetTypeName() == FTimerNode::TypeName);
			const double ValueA = reinterpret_cast<FTimerNode*>(A.Get())->GetAggregatedStats().TotalExclusiveTime;

			INSIGHTS_ENSURE(B.IsValid() && B->GetTypeName() == FTimerNode::TypeName);
			const double ValueB = reinterpret_cast<FTimerNode*>(B.Get())->GetAggregatedStats().TotalExclusiveTime;

			if (ValueA == ValueB)
			{
				INSIGHTS_DEFAULT_SORTING_NODES(A, B)
			}
			else
			{
				// Sort by total exclusive time (descending).
				return ValueB < ValueA;
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
#undef INSIGHTS_DEFAULT_SORTING_NODES
#undef INSIGHTS_ENSURE
