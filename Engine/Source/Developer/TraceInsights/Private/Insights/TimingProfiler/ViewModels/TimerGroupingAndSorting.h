// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/TimerNode.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorters
////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimerNodeSortingByTimerType: public FTableCellValueSorter
{
public:
	FTimerNodeSortingByTimerType(TSharedRef<FTableColumn> InColumnRef);

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimerNodeSortingByInstanceCount : public FTableCellValueSorter
{
public:
	FTimerNodeSortingByInstanceCount(TSharedRef<FTableColumn> InColumnRef);

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimerNodeSortingByTotalInclusiveTime : public FTableCellValueSorter
{
public:
	FTimerNodeSortingByTotalInclusiveTime(TSharedRef<FTableColumn> InColumnRef);

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimerNodeSortingByTotalExclusiveTime : public FTableCellValueSorter
{
public:
	FTimerNodeSortingByTotalExclusiveTime(TSharedRef<FTableColumn> InColumnRef);

	virtual void Sort(TArray<FBaseTreeNodePtr>& NodesToSort, ESortMode SortMode) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// Organizers
////////////////////////////////////////////////////////////////////////////////////////////////////

/** Enumerates types of grouping or sorting for the timer nodes. */
enum class ETimerGroupingMode
{
	/** Creates a single group for all timers. */
	Flat,

	/** Creates one group for one letter. */
	ByName,

	/** Creates groups based on timer metadata group names. */
	ByMetaGroupName,

	/** Creates one group for each timer type. */
	ByType,

	/** Creates one group for each logarithmic range ie. 0, [1 .. 10), [10 .. 100), [100 .. 1K), etc. */
	ByInstanceCount,

	ByTotalInclusiveTime,

	ByTotalExclusiveTime,

	/** Invalid enum type, may be used as a number of enumerations. */
	InvalidOrMax,
};

/** Type definition for shared pointers to instances of ETimerGroupingMode. */
typedef TSharedPtr<ETimerGroupingMode> ETimerGroupingModePtr;

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
