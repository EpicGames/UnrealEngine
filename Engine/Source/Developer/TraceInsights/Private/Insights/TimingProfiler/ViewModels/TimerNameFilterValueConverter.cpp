// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimerNameFilterValueConverter.h"

// TraceServices
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsights
#include "Insights/InsightsManager.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::FEventNameFilterValueConverter"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimerNameFilterValueConverter::Convert(const FString& Input, int64& Output, FText& OutError) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();

			uint32 TimerCount = TimerReader.GetTimerCount();
			for (uint32 TimerIndex = 0; TimerIndex < TimerCount; ++TimerIndex)
			{
				const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(TimerIndex);
				if (Timer && Timer->Name)
				{
					if (FCString::Strcmp(Timer->Name, *Input) == 0)
					{
						Output = Timer->Id;
						return true;
					}
				}
			}
		}
	}

	OutError = LOCTEXT("NoTimerFound", "No timer with this name was found!");
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTimerNameFilterValueConverter::GetTooltipText() const
{
	return LOCTEXT("TooltipText", "Enter the exact name of the timer.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTimerNameFilterValueConverter::GetHintText() const
{
	// Use the name of a well known event as the hint to show the user what kind of name we are expecting.
	return FText::FromString(TEXT("FEngineLoop"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
