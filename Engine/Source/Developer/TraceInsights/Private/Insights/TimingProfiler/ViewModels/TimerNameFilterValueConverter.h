// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// TraceInsightsCore
#include "InsightsCore/Filter/ViewModels/Filters.h"

namespace UE::Insights::TimingProfiler
{

class FTimerNameFilterValueConverter : public IFilterValueConverter
{
public:
	virtual bool Convert(const FString& Input, int64& Output, FText& OutError) const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetHintText() const override;
};

} // namespace UE::Insights::TimingProfiler
