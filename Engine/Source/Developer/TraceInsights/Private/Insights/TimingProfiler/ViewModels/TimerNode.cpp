// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimerNode.h"

// TraceServices
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/ViewModels/TimingEvent.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::TimerNode"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FTimerNode)

static FName GetGpuScopeGroup()
{
	static FName Group(TEXT("GPU"));
	return Group;
}

static FName GetVerseSamplingGroup()
{
	static FName Group(TEXT("Verse"));
	return Group;
}

static FName GetCpuScopeGroup()
{
	static FName Group(TEXT("CPU"));
	return Group;
}

static FName GetCpuSamplingGroup()
{
	static FName Group(TEXT("CPU Sampling"));
	return Group;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FName MakeSafeFName(const TCHAR* InName)
{
	if (!InName || *InName == TEXT('\0'))
	{
		return FName();
	}

	int32 Len = FCString::Strlen(InName);
	if (Len >= NAME_SIZE)
	{
		return FName(NAME_SIZE - 1, InName + (Len - NAME_SIZE + 1));
	}
	return FName(Len, InName);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNode::FTimerNode(uint32 InTimerId, const TCHAR* InName, ETimerNodeType InType, bool bInIsGroup)
	: FBaseTreeNode(bInIsGroup)
	, DisplayName(FText::FromString(InName))
	, TimerId(InTimerId)
	, MetaGroupName(InType == ETimerNodeType::GpuScope ? GetGpuScopeGroup() :
					InType == ETimerNodeType::VerseSampling ? GetVerseSamplingGroup() :
					InType == ETimerNodeType::CpuScope ? GetCpuScopeGroup() :
					InType == ETimerNodeType::CpuSampling ? GetCpuSamplingGroup() :
					NAME_None)
	, Type(InType)
{
	uint32 Color32 = FTimingEvent::ComputeEventColor(InName);
	Color.R = ((Color32 >> 16) & 0xFF) / 255.0f;
	Color.G = ((Color32 >>  8) & 0xFF) / 255.0f;
	Color.B = ((Color32      ) & 0xFF) / 255.0f;
	Color.A = 1.0f;

	ResetAggregatedStats();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNode::FTimerNode(const FText& InDisplayName)
	: FBaseTreeNode(true)
	, DisplayName(InDisplayName)
	, TimerId(InvalidTimerId)
	, Type(ETimerNodeType::Group)
	, Color(0.0f, 0.0f, 0.0f, 1.0f)
{
	ResetAggregatedStats();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNode::~FTimerNode()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimerNode::ResetAggregatedStats()
{
	AggregatedStats = TraceServices::FTimingProfilerAggregatedStats();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimerNode::SetAggregatedStats(const TraceServices::FTimingProfilerAggregatedStats& InAggregatedStats)
{
	AggregatedStats = InAggregatedStats;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimerNode::GetSourceFileAndLine(FString& OutFile, uint32& OutLine) const
{
	bool bIsSourceFileValid = false;

	if (GetTimerId() != FTimerNode::InvalidTimerId)
	{
		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
			if (TimingProfilerProvider)
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

				const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();

				const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(GetTimerId());
				if (Timer && Timer->File)
				{
					OutFile = FString(Timer->File);
					OutLine = Timer->Line;
					bIsSourceFileValid = true;
				}
			}
		}
	}
	if (!bIsSourceFileValid)
	{
		OutFile.Reset();
		OutLine = 0;
	}
	return bIsSourceFileValid;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
