// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceServices
#include "TraceServices/Containers/Timelines.h"
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class ETimerNodeType
{
	/** The TimerNode is a GPU Scope timer. */
	GpuScope,

	/** The TimerNode is a Verse Sampling timer. */
	VerseSampling,

	/** The TimerNode is a CPU Scope timer. */
	CpuScope,

	/** The TimerNode is a CPU Sampling timer. */
	CpuSampling,

	/** The TimerNode is a group node. */
	Group,

	/** Invalid enum type, may be used as a number of enumerations. */
	InvalidOrMax,
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimerNode;

/** Type definition for shared pointers to instances of FTimerNode. */
typedef TSharedPtr<class FTimerNode> FTimerNodePtr;

/** Type definition for shared references to instances of FTimerNode. */
typedef TSharedRef<class FTimerNode> FTimerNodeRef;

/** Type definition for shared references to const instances of FTimerNode. */
typedef TSharedRef<const class FTimerNode> FTimerNodeRefConst;

/** Type definition for weak references to instances of FTimerNode. */
typedef TWeakPtr<class FTimerNode> FTimerNodeWeak;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Class used to store information about a timer node (used in the STimersView).
 */
class FTimerNode : public FBaseTreeNode
{
	INSIGHTS_DECLARE_RTTI(FTimerNode, FBaseTreeNode)

public:
	static constexpr uint32 InvalidTimerId = uint32(-1);

public:
	/** Initialization constructor for the timer node. */
	FTimerNode(uint32 InTimerId, const TCHAR* InName, ETimerNodeType InType, bool bInIsGroup);

	/** Initialization constructor for the group node. */
	explicit FTimerNode(const FText& InDisplayName);

	virtual ~FTimerNode();

	virtual const FText GetDisplayName() const override
	{
		return DisplayName;
	}

	/**
	 * @return the timer id as provided by analyzer. It can be used as an index.
	 */
	uint32 GetTimerId() const { return TimerId; }

	/**
	 * @return a name of the meta group that this timer node belongs to, taken from the metadata.
	 */
	const FName& GetMetaGroupName() const { return MetaGroupName; }

	/**
	 * @return a type of this timer node or ETimerNodeType::Group for group nodes.
	 */
	ETimerNodeType GetType() const { return Type; }

	/**
	 * @return color of the node. Used when showing a graph series for a stats counter.
	 */
	virtual FLinearColor GetColor() const override
	{
		return Color;
	}

	void SetColor(const FLinearColor& InColor)
	{
		Color = InColor;
	}

	virtual uint32 GetDefaultSortOrder() const override
	{
		return TimerId;
	}

	bool IsAddedToGraph() const
	{
		return bIsAddedToGraphs;
	}

	void SetAddedToGraphsFlag(bool bInIsAddedToGraphs)
	{
		bIsAddedToGraphs = bInIsAddedToGraphs;
	}

	/**
	 * @return the aggregated stats for this timer.
	 */
	const TraceServices::FTimingProfilerAggregatedStats& GetAggregatedStats() const { return AggregatedStats; }
	TraceServices::FTimingProfilerAggregatedStats& GetAggregatedStats() { return AggregatedStats; }

	void ResetAggregatedStats();
	void SetAggregatedStats(const TraceServices::FTimingProfilerAggregatedStats& AggregatedStats);

	bool IsHotPath() const { return bIsHotPath; }
	void SetIsHotPath(bool bOnOff) { bIsHotPath = bOnOff; }

	bool GetSourceFileAndLine(FString& OutFile, uint32& OutLine) const;

private:
	/** The display name of the node. */
	FText DisplayName;

	/** The timer id provided by analyzer. It can also be used as an index. */
	const uint32 TimerId;

	/** The name of the meta group that this timer belongs to, based on the timer's metadata; only valid for timer nodes. */
	const FName MetaGroupName;

	/** Holds the type of this timer. */
	const ETimerNodeType Type;

	/** Color of the node. */
	FLinearColor Color;

	/** True if the timer is added to a frame track or to a graph track. */
	bool bIsAddedToGraphs = false;

	/** True if this tree node is on the hot path. */
	bool bIsHotPath = false;

	/** Aggregated stats. */
	TraceServices::FTimingProfilerAggregatedStats AggregatedStats;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
