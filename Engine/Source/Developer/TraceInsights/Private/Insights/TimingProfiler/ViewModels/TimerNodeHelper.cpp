// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimerNodeHelper.h"

// TraceInsights
#include "Insights/InsightsStyle.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::TimerNode"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// TimerNode Type Helper
////////////////////////////////////////////////////////////////////////////////////////////////////

FText TimerNodeTypeHelper::ToText(const ETimerNodeType NodeType)
{
	static_assert(static_cast<int>(ETimerNodeType::InvalidOrMax) == 5, "Not all cases are handled in switch below!?");
	switch (NodeType)
	{
		case ETimerNodeType::GpuScope:		return LOCTEXT("Timer_Name_GpuScope", "GPU");
		case ETimerNodeType::VerseSampling:	return LOCTEXT("Timer_Name_VerseSampling", "Verse");
		case ETimerNodeType::CpuScope:		return LOCTEXT("Timer_Name_CpuScope", "CPU");
		case ETimerNodeType::CpuSampling:	return LOCTEXT("Timer_Name_CpuSampling", "Sampling");
		case ETimerNodeType::Group:			return LOCTEXT("Timer_Name_Group", "Group");
		default:							return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText TimerNodeTypeHelper::ToDescription(const ETimerNodeType NodeType)
{
	static_assert(static_cast<int>(ETimerNodeType::InvalidOrMax) == 5, "Not all cases are handled in switch below!?");
	switch (NodeType)
	{
		case ETimerNodeType::GpuScope:		return LOCTEXT("Timer_Desc_GpuScope", "GPU Scope timer");
		case ETimerNodeType::VerseSampling:	return LOCTEXT("Timer_Desc_VerseSampling", "Verse Sampling timer");
		case ETimerNodeType::CpuScope:		return LOCTEXT("Timer_Desc_CpuScope", "CPU Scope timer");
		case ETimerNodeType::CpuSampling:	return LOCTEXT("Timer_Desc_CpuSampling", "CPU Sampling timer");
		case ETimerNodeType::Group:			return LOCTEXT("Timer_Desc_Group", "Group timer node");
		default:							return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* TimerNodeTypeHelper::GetIconForGroup()
{
	return FInsightsStyle::GetBrush("Icons.Group.TreeItem");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* TimerNodeTypeHelper::GetIconForTimerNodeType(const ETimerNodeType NodeType)
{
	static_assert(static_cast<int>(ETimerNodeType::InvalidOrMax) == 5, "Not all cases are handled in switch below!?");
	switch (NodeType)
	{
		case ETimerNodeType::GpuScope:		return FInsightsStyle::GetBrush("Icons.GpuTimer.TreeItem");
		case ETimerNodeType::VerseSampling:	return FInsightsStyle::GetBrush("Icons.CpuTimer.TreeItem");
		case ETimerNodeType::CpuScope:		return FInsightsStyle::GetBrush("Icons.CpuTimer.TreeItem");
		case ETimerNodeType::CpuSampling:	return FInsightsStyle::GetBrush("Icons.CpuTimer.TreeItem");
		case ETimerNodeType::Group:			return FInsightsStyle::GetBrush("Icons.Group.TreeItem");
		default:							return nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// TimerNode Grouping Helper
////////////////////////////////////////////////////////////////////////////////////////////////////

FText TimerNodeGroupingHelper::ToText(const ETimerGroupingMode GroupingMode)
{
	static_assert(static_cast<int>(ETimerGroupingMode::InvalidOrMax) == 7, "Not all cases are handled in switch below!?");
	switch (GroupingMode)
	{
		case ETimerGroupingMode::Flat:					return LOCTEXT("Grouping_Name_Flat",				"Flat");
		case ETimerGroupingMode::ByName:				return LOCTEXT("Grouping_Name_ByName",				"Timer Name");
		case ETimerGroupingMode::ByMetaGroupName:		return LOCTEXT("Grouping_Name_MetaGroupName",		"Meta Group Name");
		case ETimerGroupingMode::ByType:				return LOCTEXT("Grouping_Name_Type",				"Timer Type");
		case ETimerGroupingMode::ByInstanceCount:		return LOCTEXT("Grouping_Name_InstanceCount",		"Instance Count");
		case ETimerGroupingMode::ByTotalInclusiveTime:	return LOCTEXT("Grouping_Name_TotalInclusiveTime",	"Total Inclusive Time");
		case ETimerGroupingMode::ByTotalExclusiveTime:	return LOCTEXT("Grouping_Name_TotalExclusiveTime",	"Total Exclusive Time");
		default:										return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText TimerNodeGroupingHelper::ToDescription(const ETimerGroupingMode GroupingMode)
{
	static_assert(static_cast<int>(ETimerGroupingMode::InvalidOrMax) == 7, "Not all cases are handled in switch below!?");
	switch (GroupingMode)
	{
		case ETimerGroupingMode::Flat:					return LOCTEXT("Grouping_Desc_Flat",				"Creates a single group. Includes all timers.");
		case ETimerGroupingMode::ByName:				return LOCTEXT("Grouping_Desc_ByName",				"Creates one group for one letter.");
		case ETimerGroupingMode::ByMetaGroupName:		return LOCTEXT("Grouping_Desc_MetaGroupName",		"Creates groups based on metadata group names of timers.");
		case ETimerGroupingMode::ByType:				return LOCTEXT("Grouping_Desc_Type",				"Creates one group for each timer type.");
		case ETimerGroupingMode::ByInstanceCount:		return LOCTEXT("Grouping_Desc_InstanceCount",		"Creates one group for each logarithmic range ie. 0, [1 .. 10), [10 .. 100), [100 .. 1K), etc.");
		case ETimerGroupingMode::ByTotalInclusiveTime:	return LOCTEXT("Grouping_Desc_TotalInclusiveTime",	"Creates one group for each logarithmic range ie. 0.001 - 0.01, 0.01 - 0.1, 0.1 - 1.0, 1.0 - 10.0 etc");
		case ETimerGroupingMode::ByTotalExclusiveTime:	return LOCTEXT("Grouping_Desc_TotalExclusiveTime",	"Creates one group for each logarithmic range ie. 0.001 - 0.01, 0.01 - 0.1, 0.1 - 1.0, 1.0 - 10.0 etc");
		default:										return LOCTEXT("InvalidOrMax", "InvalidOrMax");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
