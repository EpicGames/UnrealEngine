// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Styling/SlateBrush.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/TimerNode.h"
#include "Insights/TimingProfiler/ViewModels/TimerGroupingAndSorting.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Helper struct that contains helper functions and classes for ETimerNodeType enum. */
struct TimerNodeTypeHelper
{
	/**
	 * @param Type - The value to get the text for.
	 *
	 * @return text representation of the specified ETimerNodeType value.
	 */
	static FText ToText(const ETimerNodeType Type);

	/**
	 * @param Type - The value to get the text for.
	 *
	 * @return text representation with more detailed explanation of the specified ETimerNodeType value.
	 */
	static FText ToDescription(const ETimerNodeType Type);

	static const FSlateBrush* GetIconForGroup();
	static const FSlateBrush* GetIconForTimerNodeType(const ETimerNodeType Type);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Helper struct that contains grouping static functions and classes. */
struct TimerNodeGroupingHelper
{
	/**
	 * @param TimerGroupingMode - The value to get the text for.
	 *
	 * @return text representation of the specified ETimerGroupingMode value.
	 */
	static FText ToText(const ETimerGroupingMode TimerGroupingMode);

	/**
	 * @param TimerGroupingMode - The value to get the text for.
	 *
	 * @return text representation with more detailed explanation of the specified ETimerGroupingMode value.
	 */
	static FText ToDescription(const ETimerGroupingMode TimerGroupingMode);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
