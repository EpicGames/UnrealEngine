// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimersViewColumnFactory.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/TimerGroupingAndSorting.h"
#include "Insights/TimingProfiler/ViewModels/TimerNodeHelper.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::STimersView"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Column identifiers

const FName FTimersViewColumns::NameColumnID(TEXT("Name")); // TEXT("_Hierarchy")
const FName FTimersViewColumns::IdColumnID(TEXT("Id"));
const FName FTimersViewColumns::MetaGroupNameColumnID(TEXT("MetaGroupName"));
const FName FTimersViewColumns::TypeColumnID(TEXT("Type"));
const FName FTimersViewColumns::InstanceCountColumnID(TEXT("Count"));
const FName FTimersViewColumns::ChildInstanceCountColumnID(TEXT("ChildCount"));
const FName FTimersViewColumns::AverageInstanceCountColumnID(TEXT("AverageCount"));

// Inclusive Time columns
const FName FTimersViewColumns::TotalInclusiveTimeColumnID(TEXT("TotalInclTime"));
const FName FTimersViewColumns::ParentPercentageInclusiveTimeColumnID(TEXT("ParentPercentageInclTime"));
const FName FTimersViewColumns::RootPercentageInclusiveTimeColumnID(TEXT("RootPercentageInclTime"));
const FName FTimersViewColumns::MaxInclusiveTimeColumnID(TEXT("MaxInclTime"));
const FName FTimersViewColumns::UpperQuartileInclusiveTimeColumnID(TEXT("UpperQuartileInclTime"));
const FName FTimersViewColumns::AverageInclusiveTimeColumnID(TEXT("AverageInclTime"));
const FName FTimersViewColumns::MedianInclusiveTimeColumnID(TEXT("MedianInclTime"));
const FName FTimersViewColumns::LowerQuartileInclusiveTimeColumnID(TEXT("LowerQuartileInclTime"));
const FName FTimersViewColumns::MinInclusiveTimeColumnID(TEXT("MinInclTime"));

// Exclusive Time columns
const FName FTimersViewColumns::TotalExclusiveTimeColumnID(TEXT("TotalExclTime"));
const FName FTimersViewColumns::PercentageExclusiveInclusiveTimeColumnID(TEXT("PercentageExclInclTime"));
const FName FTimersViewColumns::MaxExclusiveTimeColumnID(TEXT("MaxExclTime"));
const FName FTimersViewColumns::UpperQuartileExclusiveTimeColumnID(TEXT("UpperQuartileExclTime"));
const FName FTimersViewColumns::AverageExclusiveTimeColumnID(TEXT("AverageExclTime"));
const FName FTimersViewColumns::MedianExclusiveTimeColumnID(TEXT("MedianExclTime"));
const FName FTimersViewColumns::LowerQuartileExclusiveTimeColumnID(TEXT("LowerQuartileExclTime"));
const FName FTimersViewColumns::MinExclusiveTimeColumnID(TEXT("MinExclTime"));

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimersViewColumnFactory::CreateTimersViewColumns(TArray<TSharedRef<FTableColumn>>& Columns)
{
	Columns.Reset();

	Columns.Add(CreateNameColumn());
	Columns.Add(CreateIdColumn());
	Columns.Add(CreateMetaGroupNameColumn());
	Columns.Add(CreateTypeColumn());
	Columns.Add(CreateInstanceCountColumn());
	Columns.Add(CreateAverageInstanceCountColumn());

	Columns.Add(CreateTotalInclusiveTimeColumn());
	Columns.Add(CreateMaxInclusiveTimeColumn());
	Columns.Add(CreateAverageInclusiveTimeColumn());
	Columns.Add(CreateMedianInclusiveTimeColumn());
	Columns.Add(CreateMinInclusiveTimeColumn());

	Columns.Add(CreateTotalExclusiveTimeColumn());
	Columns.Add(CreatePercentageExclusiveInclusiveTimeColumn());
	Columns.Add(CreateMaxExclusiveTimeColumn());
	Columns.Add(CreateAverageExclusiveTimeColumn());
	Columns.Add(CreateMedianExclusiveTimeColumn());
	Columns.Add(CreateMinExclusiveTimeColumn());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimersViewColumnFactory::CreateCallersViewColumns(TArray<TSharedRef<FTableColumn>>& Columns)
{
	Columns.Reset();

	Columns.Add(CreateNameColumn());
	Columns.Add(CreateIdColumn());
	Columns.Add(CreateTypeColumn());
	Columns.Add(CreateInstanceCountColumn());
	Columns.Add(CreateChildInstanceCountColumn());
	Columns.Add(CreateTotalInclusiveTimeColumn());
	Columns.Add(CreateParentPercentageInclusiveTimeColumn());
	Columns.Add(CreateRootPercentageInclusiveTimeColumn());
	Columns.Add(CreateAverageInclusiveTimeColumn());
	Columns.Add(CreateTotalExclusiveTimeColumn());
	Columns.Add(CreatePercentageExclusiveInclusiveTimeColumn());
	Columns.Add(CreateAverageExclusiveTimeColumn());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimersViewColumnFactory::CreateCalleesViewColumns(TArray<TSharedRef<FTableColumn>>& Columns)
{
	Columns.Reset();

	Columns.Add(CreateNameColumn());
	Columns.Add(CreateIdColumn());
	Columns.Add(CreateTypeColumn());
	Columns.Add(CreateInstanceCountColumn());
	Columns.Add(CreateChildInstanceCountColumn());
	Columns.Add(CreateTotalInclusiveTimeColumn());
	Columns.Add(CreateParentPercentageInclusiveTimeColumn());
	Columns.Add(CreateRootPercentageInclusiveTimeColumn());
	Columns.Add(CreateAverageInclusiveTimeColumn());
	Columns.Add(CreateTotalExclusiveTimeColumn());
	Columns.Add(CreatePercentageExclusiveInclusiveTimeColumn());
	Columns.Add(CreateAverageExclusiveTimeColumn());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateNameColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::NameColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Name_ColumnName", "Name"));
	Column.SetTitleName(LOCTEXT("Name_ColumnTitle", "Timer or Group Name"));

	FText Description = LOCTEXT("Name_ColumnDesc", "Name of timer or group");
	Column.SetDescription(ETimerAggregationMode::Instance, Description);
	Column.SetDescription(ETimerAggregationMode::GameFrame, Description);
	Column.SetDescription(ETimerAggregationMode::RenderingFrame, Description);

	Column.SetFlags(ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered |
					ETableColumnFlags::IsHierarchy);

	Column.SetHorizontalAlignment(HAlign_Left);
	Column.SetInitialWidth(246.0f);
	Column.SetMinWidth(42.0f);

	Column.SetDataType(ETableCellDataType::Text);

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FDisplayNameValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByName>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateIdColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::IdColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Id_ColumnName", "Id"));
	Column.SetTitleName(LOCTEXT("Id_ColumnTitle", "Timer Id"));

	FText Description = LOCTEXT("Id_ColumnDesc", "Temporary index/id assigned to timers during analysis");
	Column.SetDescription(ETimerAggregationMode::Instance, Description);
	Column.SetDescription(ETimerAggregationMode::GameFrame, Description);
	Column.SetDescription(ETimerAggregationMode::RenderingFrame, Description);

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(60.0f);

	Column.SetDataType(ETableCellDataType::Int64);

	class FIdValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::IdColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			const uint32 TimerId = TimerNode.GetTimerId();
			if (TimerId != FTimerNode::InvalidTimerId)
			{
				return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(TimerId)));
			}
			else
			{
				return TOptional<FTableCellValue>();
			}
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FIdValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Ascending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateMetaGroupNameColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::MetaGroupNameColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("MetaGroupName_ColumnName", "Meta Group"));
	Column.SetTitleName(LOCTEXT("MetaGroupName_ColumnTitle", "Meta Group Name"));

	FText Description = LOCTEXT("MetaGroupName_ColumnDesc", "Name of the meta group");
	Column.SetDescription(ETimerAggregationMode::Instance, Description);
	Column.SetDescription(ETimerAggregationMode::GameFrame, Description);
	Column.SetDescription(ETimerAggregationMode::RenderingFrame, Description);

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Left);
	Column.SetInitialWidth(100.0f);

	Column.SetDataType(ETableCellDataType::Text);

	class FMetaGroupNameValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::MetaGroupNameColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(FText::FromName(TimerNode.GetMetaGroupName())));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMetaGroupNameValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByTextValue>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateTypeColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::TypeColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("Type_ColumnName", "Type"));
	Column.SetTitleName(LOCTEXT("Type_ColumnTitle", "Type"));

	FText Description = LOCTEXT("Type_ColumnDesc", "Type of timer or group");
	Column.SetDescription(ETimerAggregationMode::Instance, Description);
	Column.SetDescription(ETimerAggregationMode::GameFrame, Description);
	Column.SetDescription(ETimerAggregationMode::RenderingFrame, Description);

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Left);
	Column.SetInitialWidth(60.0f);

	Column.SetDataType(ETableCellDataType::Text);

	class FTimerTypeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::TypeColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(TimerNodeTypeHelper::ToText(TimerNode.GetType())));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTimerTypeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FTextValueFormatter>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByTextValue>(ColumnRef);
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FTimerNodeSortingByTimerType>(ColumnRef);
	Column.SetValueSorter(Sorter);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateInstanceCountColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::InstanceCountColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("InstanceCount_ColumnName", "Count"));
	Column.SetTitleName(LOCTEXT("InstanceCount_ColumnTitle", "Instance Count"));

	FText Description = LOCTEXT("InstanceCount_ColumnDesc", "Number of timing event instances of the selected timer");
	Column.SetDescription(ETimerAggregationMode::Instance, Description);
	Column.SetDescription(ETimerAggregationMode::GameFrame, Description);
	Column.SetDescription(ETimerAggregationMode::RenderingFrame, Description);

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(60.0f);

	Column.SetDataType(ETableCellDataType::Int64);

	class FInstanceCountValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::InstanceCountColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(static_cast<int64>(TimerNode.GetAggregatedStats().InstanceCount)));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FInstanceCountValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FTimerNodeSortingByInstanceCount>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateChildInstanceCountColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::ChildInstanceCountColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("ChildInstanceCount_ColumnName", "Child Count"));
	Column.SetTitleName(LOCTEXT("ChildInstanceCount_ColumnTitle", "Child Instance Count"));

	FText Description = LOCTEXT("ChildInstanceCount_ColumnDesc", "Total number of timing event instances of the child timers (callers or callees)");
	Column.SetDescription(ETimerAggregationMode::Instance, Description);
	Column.SetDescription(ETimerAggregationMode::GameFrame, Description);
	Column.SetDescription(ETimerAggregationMode::RenderingFrame, Description);

	Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(60.0f);

	Column.SetDataType(ETableCellDataType::Int64);

	class FInstanceCountValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::ChildInstanceCountColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			int64 TotalChildInstanceCount = 0;
			for (const auto& ChildNode : TimerNode.GetChildren())
			{
				TotalChildInstanceCount += ChildNode->As<FTimerNode>().GetAggregatedStats().InstanceCount;
			}
			return TOptional<FTableCellValue>(FTableCellValue(TotalChildInstanceCount));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FInstanceCountValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FTimerNodeSortingByInstanceCount>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateAverageInstanceCountColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::AverageInstanceCountColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("AverageInstanceCount_ColumnName", "C.Avg"));
	Column.SetTitleName(LOCTEXT("AverageInstanceCount_ColumnTitle", "Average Instance Count"));

	Column.SetDescription(ETimerAggregationMode::Instance,
		LOCTEXT("AverageInstanceCount_ColumnDesc",
			"Total number of timing event instances of the selected timer in the selected range\n"
			"In Instance mode, this column has the same value as the Instance Count column."));
	Column.SetDescription(ETimerAggregationMode::GameFrame,
		LOCTEXT("AverageInstanceCount_GameFrameColumnDesc",
			"Game Frame Average Instance Count\n"
			"Average number of timing event instances of the selected timer per-frame."));
	Column.SetDescription(ETimerAggregationMode::RenderingFrame,
		LOCTEXT("AverageInstanceCount_RenderingFrameColumnDesc",
			"Rendering Frame Average Instance Count\n"
			"Average number of timing event instances of the selected timer per-frame."));

	Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(60.0f);

	Column.SetDataType(ETableCellDataType::Double);

	class FAverageInstanceCountValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::AverageInstanceCountColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(TimerNode.GetAggregatedStats().AverageInstanceCount));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAverageInstanceCountValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<FDoubleValueFormatterAsCustomNumber> Formatter = MakeShared<FDoubleValueFormatterAsCustomNumber>();
	Formatter->GetFormattingOptions().MaximumFractionalDigits = 2;
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Inclusive Time Columns
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateTotalInclusiveTimeColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::TotalInclusiveTimeColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("TotalInclusiveTime_ColumnName", "Incl"));
	Column.SetTitleName(LOCTEXT("TotalInclusiveTime_ColumnTitle", "Total Inclusive Time"));

	FText Description = LOCTEXT("TotalInclusiveTime_ColumnDesc", "Total inclusive duration of selected timer's instances");
	Column.SetDescription(ETimerAggregationMode::Instance, Description);
	Column.SetDescription(ETimerAggregationMode::GameFrame, Description);
	Column.SetDescription(ETimerAggregationMode::RenderingFrame, Description);

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(TotalTimeColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);
	Column.SetAggregation(ETableColumnAggregation::Sum);

	class FTotalInclusiveTimeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::TotalInclusiveTimeColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(TimerNode.GetAggregatedStats().TotalInclusiveTime));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTotalInclusiveTimeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeAuto>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FTimerNodeSortingByTotalInclusiveTime>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateParentPercentageInclusiveTimeColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::ParentPercentageInclusiveTimeColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("ParentPercentageInclusiveTime_ColumnName", "% Parent"));
	Column.SetTitleName(LOCTEXT("ParentPercentageInclusiveTime_ColumnTitle", "Total Inclusive Time / Parent's Total Inclusive Time (%)"));

	FText Description = LOCTEXT("ParentPercentageInclusiveTime_ColumnDesc", "Percentage of total inclusive duration (the selected timer relative to the parent timer)");
	Column.SetDescription(ETimerAggregationMode::Instance, Description);
	Column.SetDescription(ETimerAggregationMode::GameFrame, Description);
	Column.SetDescription(ETimerAggregationMode::RenderingFrame, Description);

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
		ETableColumnFlags::ShouldBeVisible |
		ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(PercentageColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);
	Column.SetAggregation(ETableColumnAggregation::Sum);

	class FParentPercentageInclusiveTimeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::ParentPercentageInclusiveTimeColumnID);
			if (Node.Is<FTimerNode>())
			{
				const FTimerNode& TimerNode = Node.As<FTimerNode>();
				FBaseTreeNodePtr ParentNode = TimerNode.GetParent();
				if (ParentNode.IsValid() && ParentNode->Is<FTimerNode>())
				{
					const double ParentTime = ParentNode->As<FTimerNode>().GetAggregatedStats().TotalInclusiveTime;
					if (ParentTime > 0.0)
					{
						const double Time = TimerNode.GetAggregatedStats().TotalInclusiveTime;
						return TOptional<FTableCellValue>(FTableCellValue(Time / ParentTime));
					}
				}
			}
			return TOptional<FTableCellValue>();
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FParentPercentageInclusiveTimeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsPercent>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FTimerNodeSortingByTotalInclusiveTime>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateRootPercentageInclusiveTimeColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::RootPercentageInclusiveTimeColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("RootPercentageInclusiveTime_ColumnName", "% Root"));
	Column.SetTitleName(LOCTEXT("RootPercentageInclusiveTime_ColumnTitle", "Total Inclusive Time / Root's Total Inclusive Time (%)"));

	FText Description = LOCTEXT("RootPercentageInclusiveTime_ColumnDesc", "Percentage of total inclusive duration (the selected timer relative to the root timer)");
	Column.SetDescription(ETimerAggregationMode::Instance, Description);
	Column.SetDescription(ETimerAggregationMode::GameFrame, Description);
	Column.SetDescription(ETimerAggregationMode::RenderingFrame, Description);

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
		ETableColumnFlags::ShouldBeVisible |
		ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(PercentageColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);
	Column.SetAggregation(ETableColumnAggregation::Sum);

	class FRootPercentageInclusiveTimeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::RootPercentageInclusiveTimeColumnID);
			if (Node.Is<FTimerNode>())
			{
				const FTimerNode& TimerNode = Node.As<FTimerNode>();
				FBaseTreeNodePtr RootNode = TimerNode.GetParent();
				while (RootNode.IsValid() && RootNode->GetParent().IsValid())
				{
					RootNode = RootNode->GetParent();
				}
				if (RootNode.IsValid() && RootNode->Is<FTimerNode>())
				{
					const double RootTime = RootNode->As<FTimerNode>().GetAggregatedStats().TotalInclusiveTime;
					if (RootTime > 0.0)
					{
						const double Time = TimerNode.GetAggregatedStats().TotalInclusiveTime;
						return TOptional<FTableCellValue>(FTableCellValue(Time / RootTime));
					}
				}
			}
			return TOptional<FTableCellValue>();
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FRootPercentageInclusiveTimeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsPercent>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FTimerNodeSortingByTotalInclusiveTime>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateMaxInclusiveTimeColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::MaxInclusiveTimeColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("MaxInclusiveTime_ColumnName", "I.Max"));
	Column.SetTitleName(LOCTEXT("MaxInclusiveTime_ColumnTitle", "Max Inclusive Time (ms)"));

	Column.SetDescription(ETimerAggregationMode::Instance,
		LOCTEXT("MaxInclusiveTime_ColumnDesc",
			"Maximum inclusive duration of selected timer's instances, in milliseconds"));
	Column.SetDescription(ETimerAggregationMode::GameFrame,
		LOCTEXT("MaxInclusiveTime_GameFrameColumnDesc",
			"Game Frame Maximum Inclusive Duration\n"
			"Inclusive duration is computed for a single frame as the sum of inclusive duration of all instances of the timer in the respective frame.\n"
			"The maximum is selected from these per-frame inclusive durations.\n"
			"Unit is milliseconds."));
	Column.SetDescription(ETimerAggregationMode::RenderingFrame,
		LOCTEXT("MaxInclusiveTime_RenderingFrameColumnDesc",
			"Rendering Frame Maximum Inclusive Duration\n"
			"Inclusive duration is computed for a single frame as the sum of inclusive duration of all instances of the timer in the respective frame.\n"
			"The maximum is selected from these per-frame inclusive durations.\n"
			"Unit is milliseconds."));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(TimeMsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);

	class FMaxInclusiveTimeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::MaxInclusiveTimeColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(TimerNode.GetAggregatedStats().MaxInclusiveTime));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMaxInclusiveTimeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeMs>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateAverageInclusiveTimeColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::AverageInclusiveTimeColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("AvgInclusiveTime_ColumnName", "I.Avg"));
	Column.SetTitleName(LOCTEXT("AvgInclusiveTime_ColumnTitle", "Average Inclusive Time (ms)"));

	Column.SetDescription(ETimerAggregationMode::Instance,
		LOCTEXT("AvgInclusiveTime_ColumnDesc",
			"Average inclusive duration of selected timer's instances, in milliseconds"));
	Column.SetDescription(ETimerAggregationMode::GameFrame,
		LOCTEXT("AvgInclusiveTime_GameFrameColumnDesc",
			"Game Frame Average Inclusive Duration\n"
			"Inclusive duration is computed for a single frame as the sum of inclusive duration of all instances of the timer in the respective frame.\n"
			"The average is computed from these per-frame inclusive durations.\n"
			"Unit is milliseconds."));
	Column.SetDescription(ETimerAggregationMode::RenderingFrame,
		LOCTEXT("AvgInclusiveTime_RenderingFrameColumnDesc",
			"Rendering Frame Average Inclusive Duration\n"
			"Inclusive duration is computed for a single frame as the sum of inclusive duration of all instances of the timer in the respective frame.\n"
			"The average is computed from these per-frame inclusive durations.\n"
			"Unit is milliseconds."));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(TimeMsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);

	class FAverageInclusiveTimeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::AverageInclusiveTimeColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(TimerNode.GetAggregatedStats().AverageInclusiveTime));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAverageInclusiveTimeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeMs>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateMedianInclusiveTimeColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::MedianInclusiveTimeColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("MedInclusiveTime_ColumnName", "I.Med"));
	Column.SetTitleName(LOCTEXT("MedInclusiveTime_ColumnTitle", "Median Inclusive Time (ms)"));

	Column.SetDescription(ETimerAggregationMode::Instance,
		LOCTEXT("MedInclusiveTime_ColumnDesc",
			"Median inclusive duration of selected timer's instances, in milliseconds"));
	Column.SetDescription(ETimerAggregationMode::GameFrame,
		LOCTEXT("MedInclusiveTime_GameFrameColumnDesc",
			"Game Frame Median Inclusive Duration\n"
			"Inclusive duration is computed for a single frame as the sum of inclusive duration of all instances of the timer in the respective frame.\n"
			"The median is computed from these per-frame inclusive durations.\n"
			"Unit is milliseconds."));
	Column.SetDescription(ETimerAggregationMode::RenderingFrame,
		LOCTEXT("MedInclusiveTime_RenderingFrameColumnDesc",
			"Rendering Frame Median Inclusive Duration\n"
			"Inclusive duration is computed for a single frame as the sum of inclusive duration of all instances of the timer in the respective frame.\n"
			"The median is computed from these per-frame inclusive durations.\n"
			"Unit is milliseconds."));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(TimeMsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);

	class FMedianInclusiveTimeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::MedianInclusiveTimeColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(TimerNode.GetAggregatedStats().MedianInclusiveTime));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMedianInclusiveTimeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeMs>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateMinInclusiveTimeColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::MinInclusiveTimeColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("MinInclusiveTime_ColumnName", "I.Min"));
	Column.SetTitleName(LOCTEXT("MinInclusiveTime_ColumnTitle", "Min Inclusive Time (ms)"));

	Column.SetDescription(ETimerAggregationMode::Instance,
		LOCTEXT("MinInclusiveTime_ColumnDesc",
			"Minimum inclusive duration of selected timer's instances, in milliseconds"));
	Column.SetDescription(ETimerAggregationMode::GameFrame,
		LOCTEXT("MinInclusiveTime_GameFrameColumnDesc",
			"Game Frame Minimum Inclusive Duration\n"
			"Inclusive duration is computed for a single frame as the sum of inclusive duration of all instances of the timer in the respective frame.\n"
			"The minimum is selected from these per-frame inclusive durations.\n"
			"Unit is milliseconds."));
	Column.SetDescription(ETimerAggregationMode::RenderingFrame,
		LOCTEXT("MinInclusiveTime_RenderingFrameColumnDesc",
			"Rendering Frame Minimum Inclusive Duration\n"
			"Inclusive duration is computed for a single frame as the sum of inclusive duration of all instances of the timer in the respective frame.\n"
			"The minimum is selected from these per-frame inclusive durations.\n"
			"Unit is milliseconds."));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(TimeMsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);

	class FMinInclusiveTimeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::MinInclusiveTimeColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(TimerNode.GetAggregatedStats().MinInclusiveTime));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMinInclusiveTimeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeMs>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Exclusive Time Columns
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateTotalExclusiveTimeColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::TotalExclusiveTimeColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("TotalExclusiveTime_ColumnName", "Excl"));
	Column.SetTitleName(LOCTEXT("TotalExclusiveTime_ColumnTitle", "Total Exclusive Time"));

	FText Description = LOCTEXT("TotalExclusiveTime_ColumnDesc", "Total exclusive duration of selected timer's instances");
	Column.SetDescription(ETimerAggregationMode::Instance, Description);
	Column.SetDescription(ETimerAggregationMode::GameFrame, Description);
	Column.SetDescription(ETimerAggregationMode::RenderingFrame, Description);

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(TotalTimeColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);
	Column.SetAggregation(ETableColumnAggregation::Sum);

	class FTotalExclusiveTimeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::TotalExclusiveTimeColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(TimerNode.GetAggregatedStats().TotalExclusiveTime));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTotalExclusiveTimeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeAuto>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FTimerNodeSortingByTotalExclusiveTime>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreatePercentageExclusiveInclusiveTimeColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::PercentageExclusiveInclusiveTimeColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("PercentageExclusiveInclusiveTime_ColumnName", "Excl/Incl"));
	Column.SetTitleName(LOCTEXT("PercentageExclusiveInclusiveTime_ColumnTitle", "Total Exclusive Time / Total Inclusive Time (%)"));

	FText Description = LOCTEXT("PercentageExclusiveInclusiveTime_ColumnDesc", "Percentage of total exclusive duration from the total inclusive duration");
	Column.SetDescription(ETimerAggregationMode::Instance, Description);
	Column.SetDescription(ETimerAggregationMode::GameFrame, Description);
	Column.SetDescription(ETimerAggregationMode::RenderingFrame, Description);

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
		//ETableColumnFlags::ShouldBeVisible |
		ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(PercentageColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);
	Column.SetAggregation(ETableColumnAggregation::Sum);

	class FPercentageExclusiveInclusiveTimeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::PercentageExclusiveInclusiveTimeColumnID);
			if (Node.Is<FTimerNode>())
			{
				const FTimerNode& TimerNode = Node.As<FTimerNode>();
				const double InclTime = TimerNode.GetAggregatedStats().TotalInclusiveTime;
				if (InclTime > 0.0)
				{
					const double ExclTime = TimerNode.GetAggregatedStats().TotalExclusiveTime;
					return TOptional<FTableCellValue>(FTableCellValue(ExclTime / InclTime));
				}
			}
			return TOptional<FTableCellValue>();
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPercentageExclusiveInclusiveTimeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsPercent>();
	Column.SetValueFormatter(Formatter);

	//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FTimerNodeSortingByTotalExclusiveTime>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateMaxExclusiveTimeColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::MaxExclusiveTimeColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("MaxExclusiveTime_ColumnName", "E.Max"));
	Column.SetTitleName(LOCTEXT("MaxExclusiveTime_ColumnTitle", "Max Exclusive Time (ms)"));

	Column.SetDescription(ETimerAggregationMode::Instance,
		LOCTEXT("MaxExclusiveTime_ColumnDesc",
			"Maximum exclusive duration of selected timer's instances, in milliseconds"));
	Column.SetDescription(ETimerAggregationMode::GameFrame,
		LOCTEXT("MaxExclusiveTime_GameFrameColumnDesc",
			"Game Frame Maximum Exclusive Duration\n"
			"Exclusive duration is computed for a single frame as the sum of exclusive duration of all instances of the timer in the respective frame.\n"
			"The maximum is selected from these per-frame exclusive durations.\n"
			"Unit is milliseconds."));
	Column.SetDescription(ETimerAggregationMode::RenderingFrame,
		LOCTEXT("MaxExclusiveTime_RenderingFrameColumnDesc",
			"Rendering Frame Maximum Exclusive Duration\n"
			"Exclusive duration is computed for a single frame as the sum of exclusive duration of all instances of the timer in the respective frame.\n"
			"The maximum is selected from these per-frame exclusive durations.\n"
			"Unit is milliseconds."));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(TimeMsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);

	class FMaxExclusiveTimeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::MaxExclusiveTimeColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(TimerNode.GetAggregatedStats().MaxExclusiveTime));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMaxExclusiveTimeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeMs>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateAverageExclusiveTimeColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::AverageExclusiveTimeColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("AvgExclusiveTime_ColumnName", "E.Avg"));
	Column.SetTitleName(LOCTEXT("AvgExclusiveTime_ColumnTitle", "Average Exclusive Time (ms)"));

	Column.SetDescription(ETimerAggregationMode::Instance,
		LOCTEXT("AvgExclusiveTime_ColumnDesc",
			"Average exclusive duration of selected timer's instances, in milliseconds"));
	Column.SetDescription(ETimerAggregationMode::GameFrame,
		LOCTEXT("AvgExclusiveTime_GameFrameColumnDesc",
			"Game Frame Average Exclusive Duration\n"
			"Exclusive duration is computed for a single frame as the sum of exclusive duration of all instances of the timer in the respective frame.\n"
			"The average is computed from these per-frame exclusive durations.\n"
			"Unit is milliseconds."));
	Column.SetDescription(ETimerAggregationMode::RenderingFrame,
		LOCTEXT("AvgExclusiveTime_RenderingFrameColumnDesc",
			"Rendering Frame Average Exclusive Duration\n"
			"Exclusive duration is computed for a single frame as the sum of exclusive duration of all instances of the timer in the respective frame.\n"
			"The average is computed from these per-frame exclusive durations.\n"
			"Unit is milliseconds."));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(TimeMsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);

	class FAverageExclusiveTimeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::AverageExclusiveTimeColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(TimerNode.GetAggregatedStats().AverageExclusiveTime));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAverageExclusiveTimeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeMs>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateMedianExclusiveTimeColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::MedianExclusiveTimeColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("MedExclusiveTime_ColumnName", "E.Med"));
	Column.SetTitleName(LOCTEXT("MedExclusiveTime_ColumnTitle", "Median Exclusive Time (ms)"));

	Column.SetDescription(ETimerAggregationMode::Instance,
		LOCTEXT("MedExclusiveTime_ColumnDesc",
			"Median exclusive duration of selected timer's instances, in milliseconds"));
	Column.SetDescription(ETimerAggregationMode::GameFrame,
		LOCTEXT("MedExclusiveTime_GameFrameColumnDesc",
			"Game Frame Median Exclusive Duration\n"
			"Exclusive duration is computed for a single frame as the sum of exclusive duration of all instances of the timer in the respective frame.\n"
			"The median is computed from these per-frame exclusive durations.\n"
			"Unit is milliseconds."));
	Column.SetDescription(ETimerAggregationMode::RenderingFrame,
		LOCTEXT("MedExclusiveTime_RenderingFrameColumnDesc",
			"Rendering Frame Median Exclusive Duration\n"
			"Exclusive duration is computed for a single frame as the sum of exclusive duration of all instances of the timer in the respective frame.\n"
			"The median is computed from these per-frame exclusive durations.\n"
			"Unit is milliseconds."));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(TimeMsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);

	class FMedianExclusiveTimeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::MedianExclusiveTimeColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(TimerNode.GetAggregatedStats().MedianExclusiveTime));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMedianExclusiveTimeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeMs>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FTableColumn> FTimersViewColumnFactory::CreateMinExclusiveTimeColumn()
{
	TSharedRef<FTimersTableColumn> ColumnRef = MakeShared<FTimersTableColumn>(FTimersViewColumns::MinExclusiveTimeColumnID);
	FTimersTableColumn& Column = *ColumnRef;

	Column.SetShortName(LOCTEXT("MinExclusiveTime_ColumnName", "E.Min"));
	Column.SetTitleName(LOCTEXT("MinExclusiveTime_ColumnTitle", "Min Exclusive Time (ms)"));

	Column.SetDescription(ETimerAggregationMode::Instance,
		LOCTEXT("MinExclusiveTime_ColumnDesc",
			"Minimum exclusive duration of selected timer's instances, in milliseconds"));
	Column.SetDescription(ETimerAggregationMode::GameFrame,
		LOCTEXT("MinExclusiveTime_GameFrameColumnDesc",
			"Game Frame Minimum Exclusive Duration\n"
			"Exclusive duration is computed for a single frame as the sum of exclusive duration of all instances of the timer in the respective frame.\n"
			"The minimum is selected from these per-frame exclusive durations.\n"
			"Unit is milliseconds."));
	Column.SetDescription(ETimerAggregationMode::RenderingFrame,
		LOCTEXT("MinExclusiveTime_RenderingFrameColumnDesc",
			"Rendering Frame Minimum Exclusive Duration\n"
			"Exclusive duration is computed for a single frame as the sum of exclusive duration of all instances of the timer in the respective frame.\n"
			"The minimum is selected from these per-frame exclusive durations.\n"
			"Unit is milliseconds."));

	Column.SetFlags(ETableColumnFlags::CanBeHidden |
					//ETableColumnFlags::ShouldBeVisible |
					ETableColumnFlags::CanBeFiltered);

	Column.SetHorizontalAlignment(HAlign_Right);
	Column.SetInitialWidth(TimeMsColumnInitialWidth);

	Column.SetDataType(ETableCellDataType::Double);

	class FMinExclusiveTimeValueGetter : public FTableCellValueGetter
	{
	public:
		virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const
		{
			ensure(Column.GetId() == FTimersViewColumns::MinExclusiveTimeColumnID);
			const FTimerNode& TimerNode = static_cast<const FTimerNode&>(Node);
			return TOptional<FTableCellValue>(FTableCellValue(TimerNode.GetAggregatedStats().MinExclusiveTime));
		}
	};

	TSharedRef<ITableCellValueGetter> Getter = MakeShared<FMinExclusiveTimeValueGetter>();
	Column.SetValueGetter(Getter);

	TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FDoubleValueFormatterAsTimeMs>();
	Column.SetValueFormatter(Formatter);

	TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByDoubleValue>(ColumnRef);
	Column.SetValueSorter(Sorter);
	Column.SetInitialSortMode(EColumnSortMode::Descending);

	return ColumnRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
