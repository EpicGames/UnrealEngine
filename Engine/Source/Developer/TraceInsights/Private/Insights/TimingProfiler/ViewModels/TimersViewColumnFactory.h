// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/TimerAggregator.h"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

// Column identifiers
struct FTimersViewColumns
{
	static const FName NameColumnID;
	static const FName IdColumnID;
	static const FName MetaGroupNameColumnID;
	static const FName TypeColumnID;
	static const FName InstanceCountColumnID;
	static const FName ChildInstanceCountColumnID;
	static const FName AverageInstanceCountColumnID;

	// Inclusive Time columns
	static const FName TotalInclusiveTimeColumnID;
	static const FName ParentPercentageInclusiveTimeColumnID;
	static const FName RootPercentageInclusiveTimeColumnID;
	static const FName MaxInclusiveTimeColumnID;
	static const FName UpperQuartileInclusiveTimeColumnID;
	static const FName AverageInclusiveTimeColumnID;
	static const FName MedianInclusiveTimeColumnID;
	static const FName LowerQuartileInclusiveTimeColumnID;
	static const FName MinInclusiveTimeColumnID;

	// Exclusive Time columns
	static const FName TotalExclusiveTimeColumnID;
	static const FName PercentageExclusiveInclusiveTimeColumnID;
	static const FName MaxExclusiveTimeColumnID;
	static const FName UpperQuartileExclusiveTimeColumnID;
	static const FName AverageExclusiveTimeColumnID;
	static const FName MedianExclusiveTimeColumnID;
	static const FName LowerQuartileExclusiveTimeColumnID;
	static const FName MinExclusiveTimeColumnID;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimersTableColumn : public FTableColumn
{
public:
	FTimersTableColumn(const FName InId)
		: FTableColumn(InId)
	{}

	FText GetDescription(ETimerAggregationMode InAggregationMode) const
	{
		switch (InAggregationMode)
		{
		case ETimerAggregationMode::GameFrame:
			return GameFrame_Description;
		case ETimerAggregationMode::RenderingFrame:
			return RenderingFrame_Description;
		default:
			return FTableColumn::GetDescription();
		}
	}

	void SetDescription(ETimerAggregationMode InAggregationMode, FText InDescription)
	{
		switch (InAggregationMode)
		{
		case ETimerAggregationMode::GameFrame:
			GameFrame_Description = InDescription;
			break;
		case ETimerAggregationMode::RenderingFrame:
			RenderingFrame_Description = InDescription;
			break;
		case ETimerAggregationMode::Instance:
			FTableColumn::SetDescription(InDescription);
			break;
		default:
			ensure(0);
		}
	}

private:
	FText GameFrame_Description;
	FText RenderingFrame_Description;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FTimersViewColumnFactory
{
public:
	static void CreateTimersViewColumns(TArray<TSharedRef<FTableColumn>>& Columns);
	static void CreateCallersViewColumns(TArray<TSharedRef<FTableColumn>>& Columns);
	static void CreateCalleesViewColumns(TArray<TSharedRef<FTableColumn>>& Columns);

	static TSharedRef<FTableColumn> CreateNameColumn();
	static TSharedRef<FTableColumn> CreateIdColumn();
	static TSharedRef<FTableColumn> CreateMetaGroupNameColumn();
	static TSharedRef<FTableColumn> CreateTypeColumn();
	static TSharedRef<FTableColumn> CreateInstanceCountColumn();
	static TSharedRef<FTableColumn> CreateChildInstanceCountColumn();
	static TSharedRef<FTableColumn> CreateAverageInstanceCountColumn();

	static TSharedRef<FTableColumn> CreateTotalInclusiveTimeColumn();
	static TSharedRef<FTableColumn> CreateParentPercentageInclusiveTimeColumn();
	static TSharedRef<FTableColumn> CreateRootPercentageInclusiveTimeColumn();
	static TSharedRef<FTableColumn> CreateMaxInclusiveTimeColumn();
	static TSharedRef<FTableColumn> CreateAverageInclusiveTimeColumn();
	static TSharedRef<FTableColumn> CreateMedianInclusiveTimeColumn();
	static TSharedRef<FTableColumn> CreateMinInclusiveTimeColumn();

	static TSharedRef<FTableColumn> CreateTotalExclusiveTimeColumn();
	static TSharedRef<FTableColumn> CreatePercentageExclusiveInclusiveTimeColumn();
	static TSharedRef<FTableColumn> CreateMaxExclusiveTimeColumn();
	static TSharedRef<FTableColumn> CreateAverageExclusiveTimeColumn();
	static TSharedRef<FTableColumn> CreateMedianExclusiveTimeColumn();
	static TSharedRef<FTableColumn> CreateMinExclusiveTimeColumn();

private:
	static constexpr float TotalTimeColumnInitialWidth = 60.0f;
	static constexpr float TimeMsColumnInitialWidth = 50.0f;
	static constexpr float PercentageColumnInitialWidth = 50.0f;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
