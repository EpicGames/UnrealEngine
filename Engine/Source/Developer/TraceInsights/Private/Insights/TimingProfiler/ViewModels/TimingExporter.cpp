// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimingExporter.h"

#include "HAL/PlatformFileManager.h"
#include "Logging/MessageLog.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/Bookmarks.h"
#include "TraceServices/Model/Counters.h"
#include "TraceServices/Model/Frames.h"
#include "TraceServices/Model/Regions.h"
#include "TraceServices/Model/Threads.h"
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"

// TraceInsights
#include "Insights/Common/Utf8FileWriter.h"
#include "Insights/Log.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Tracks/GpuTimingTrack.h"
#include "Insights/TimingProfiler/Tracks/VerseTimingTrack.h"
#include "Insights/TimingProfiler/ViewModels/FrameTimingTrack.h"
#include "Insights/TimingProfiler/ViewModels/FrameTrackHelper.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::FTimingExporter"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimingExporter
////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingExporter::FTimingExporter(const TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingExporter::~FTimingExporter()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TUniquePtr<IFileHandle> FTimingExporter::OpenExportFile(const TCHAR* InFilename) const
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	FString Path = FPaths::GetPath(InFilename);
	if (!PlatformFile.DirectoryExists(*Path))
	{
		PlatformFile.CreateDirectoryTree(*Path);
	}

	TUniquePtr<IFileHandle> ExportFileHandle{ PlatformFile.OpenWrite(InFilename) };

	return ExportFileHandle;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingExporter::Error(const TCHAR* InTitle, const TCHAR* InMessage) const
{
	FName LogListingName = FTimingProfilerManager::Get()->GetLogListingName();
	FMessageLog ReportMessageLog((LogListingName != NAME_None) ? LogListingName : TEXT("Insights"));
	ReportMessageLog.Error(FText::FromString(InTitle));
	ReportMessageLog.Error(FText::FromString(InMessage));
	ReportMessageLog.Notify();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTimingExporter::ExportThreadsAsText(const FString& Filename, FExportThreadsParams& Params) const
{
	const TCHAR* ErrorTitle = TEXT("Failed to export threads!");

	if (Params.Columns != nullptr)
	{
		Error(ErrorTitle, TEXT("Custom list of columns is not yet supported!"));
		return -1;
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	TUniquePtr<IFileHandle> ExportFileHandle = OpenExportFile(*Filename);
	if (!ExportFileHandle)
	{
		Error(ErrorTitle, *FString::Printf(TEXT("Cannot write the export file (\"%s\")."), *Filename));
		return -1;
	}
	bool bIsCSV = Filename.EndsWith(TEXT(".csv"));
	FUtf8FileWriter Writer(ExportFileHandle.Get(), bIsCSV);

	// Write header.
	{
		Writer.Append(UTF8TEXTVIEW("Id"));
		Writer.AppendSeparator();
		Writer.Append(UTF8TEXTVIEW("Name"));
		Writer.AppendSeparator();
		Writer.Append(UTF8TEXTVIEW("Group"));
		Writer.AppendLineEnd();
	}

	int32 ThreadCount = 0;

	// Write values.
	{
		// Old GPU Profiler tracks
		{
			Writer.AppendValue(FGpuTimingTrack::Gpu1ThreadId);
			Writer.AppendSeparator();
			Writer.Append(UTF8TEXTVIEW("GPU1"));
			Writer.AppendSeparator();
			Writer.Append(UTF8TEXTVIEW("GPU"));
			Writer.AppendLineEnd();
			++ThreadCount;

			Writer.AppendValue(FGpuTimingTrack::Gpu2ThreadId);
			Writer.AppendSeparator();
			Writer.Append(UTF8TEXTVIEW("GPU2"));
			Writer.AppendSeparator();
			Writer.Append(UTF8TEXTVIEW("GPU"));
			Writer.AppendLineEnd();
			++ThreadCount;
		}

		// Verse track
		{
			Writer.AppendValue(FVerseTimingTrack::VerseThreadId);
			Writer.AppendSeparator();
			Writer.Append(UTF8TEXTVIEW("VerseSampling"));
			Writer.AppendSeparator();
			Writer.Append(UTF8TEXTVIEW("Verse"));
			Writer.AppendLineEnd();
			++ThreadCount;
		}

		// Iterate the Frame tracks.
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

			const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(Session);
			for (uint32 FrameType = 0; FrameType < (uint32)ETraceFrameType::TraceFrameType_Count; ++FrameType)
			{
				Writer.AppendValue(GetFrameTypeNonCollidingId(FrameType));
				Writer.AppendSeparator();
				Writer.Append(FFrameTimingTrack::GetFrameTrackName(FrameType));
				Writer.AppendSeparator();
				Writer.Append(UTF8TEXTVIEW("Frame Tracks"));
				Writer.AppendLineEnd();
				++ThreadCount;
			}
		}

		// Iterate the GPU Queues (new GPU Profiler).
		{
			const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(Session);
			if (TimingProfilerProvider)
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

				TimingProfilerProvider->EnumerateGpuQueues(
					[&](const TraceServices::FGpuQueueInfo& QueueInfo)
					{
						Writer.AppendValue(GetNonCollidingId(QueueInfo.Id));
						Writer.AppendSeparator();
						Writer.AppendValue(QueueInfo.GetDisplayName());
						Writer.AppendSeparator();
						Writer.Append(UTF8TEXTVIEW("GPU"));
						Writer.AppendLineEnd();
						++ThreadCount;
					});
			}
		}

		// Iterate the CPU threads.
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

			const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(Session);

			ThreadProvider.EnumerateThreads(
				[&](const TraceServices::FThreadInfo& ThreadInfo)
				{
					Writer.AppendValue(ThreadInfo.Id);
					Writer.AppendSeparator();
					Writer.AppendValue(ThreadInfo.Name);
					Writer.AppendSeparator();
					Writer.AppendValue(ThreadInfo.GroupName);
					Writer.AppendLineEnd();
					++ThreadCount;
				});
		}
	}

	Writer.Flush();
	ExportFileHandle->Flush();
	ExportFileHandle.Reset();

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TraceInsights, Log, "Exported %d threads to file in %.3fs (\"%ls\").", ThreadCount, TotalTime, *Filename);

	return ThreadCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTimingExporter::ExportTimersAsText(const FString& Filename, FExportTimersParams& Params) const
{
	const TCHAR* ErrorTitle = TEXT("Failed to export timers!");

	if (Params.Columns != nullptr)
	{
		Error(ErrorTitle, TEXT("Custom list of columns is not yet supported!"));
		return -1;
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	TUniquePtr<IFileHandle> ExportFileHandle = OpenExportFile(*Filename);
	if (!ExportFileHandle)
	{
		Error(ErrorTitle, *FString::Printf(TEXT("Cannot write the export file (\"%s\")."), *Filename));
		return -1;
	}
	bool bIsCSV = Filename.EndsWith(TEXT(".csv"));
	FUtf8FileWriter Writer(ExportFileHandle.Get(), bIsCSV);

	// Write header.
	{
		Writer.Append(UTF8TEXTVIEW("Id"));
		Writer.AppendSeparator();
		Writer.Append(UTF8TEXTVIEW("Type"));
		Writer.AppendSeparator();
		Writer.Append(UTF8TEXTVIEW("Name"));
		Writer.AppendSeparator();
		Writer.Append(UTF8TEXTVIEW("File"));
		Writer.AppendSeparator();
		Writer.Append(UTF8TEXTVIEW("Line"));
		Writer.AppendLineEnd();
	}

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(Session);
	if (!TimingProfilerProvider)
	{
		Error(ErrorTitle, TEXT("Unable to access TimingProfilerProvider."));
		return -1;
	}

	uint32 TimerCount = 0;

	// Write values.
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

		const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();

		TimerCount = TimerReader.GetTimerCount();
		for (uint32 TimerIndex = 0; TimerIndex < TimerCount; ++TimerIndex)
		{
			const TraceServices::FTimingProfilerTimer& Timer = *(TimerReader.GetTimer(TimerIndex));
			Writer.AppendValue(Timer.Id);
			Writer.AppendSeparator();
			switch (Timer.Type)
			{
			case TraceServices::ETimingProfilerTimerType::GpuScope:
				Writer.Append(UTF8TEXTVIEW("GPU"));
				break;
			case TraceServices::ETimingProfilerTimerType::CpuScope:
				Writer.Append(UTF8TEXTVIEW("CPU"));
				break;
			case TraceServices::ETimingProfilerTimerType::CpuSampling:
				Writer.Append(UTF8TEXTVIEW("CpuSampling"));
				break;
			case TraceServices::ETimingProfilerTimerType::VerseSampling:
				Writer.Append(UTF8TEXTVIEW("VerseSampling"));
				break;
			default:
				Writer.Append(UTF8TEXTVIEW("Unknown"));
				break;
			}
			Writer.AppendSeparator();
			if (Params.bShowFormatInTimerNames)
			{
				TStringBuilder<1024> NameStringBuilder;
				TimingProfilerProvider->MakeTimerDisplayName(Timer, NameStringBuilder);
				Writer.AppendValue(NameStringBuilder.ToView());
			}
			else
			{
				Writer.AppendValue(Timer.Name);
			}
			Writer.AppendSeparator();
			if (Timer.File)
			{
				Writer.AppendValue(Timer.File);
			}
			Writer.AppendSeparator();
			Writer.AppendValue(Timer.Line);
			Writer.AppendLineEnd();
		}
	}

	Writer.Flush();
	ExportFileHandle->Flush();
	ExportFileHandle.Reset();

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TraceInsights, Log, "Exported %u timers to file in %.3fs (\"%ls\").", TimerCount, TotalTime, *Filename);

	return int32(TimerCount);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FName FTimingExporter::ExportTimingEvents_ThreadIdColumn("ThreadId");
const FName FTimingExporter::ExportTimingEvents_ThreadNameColumn("ThreadName");
const FName FTimingExporter::ExportTimingEvents_TimerIdColumn("TimerId");
const FName FTimingExporter::ExportTimingEvents_TimerNameColumn("TimerName");
const FName FTimingExporter::ExportTimingEvents_StartTimeColumn("StartTime");
const FName FTimingExporter::ExportTimingEvents_EndTimeColumn("EndTime");
const FName FTimingExporter::ExportTimingEvents_DurationColumn("Duration");
const FName FTimingExporter::ExportTimingEvents_DepthColumn("Depth");

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingExporter::ExportTimingEvents_InitColumns() const
{
	if (ExportTimingEventsColumns.IsEmpty())
	{
		ExportTimingEventsColumns.Add(ExportTimingEvents_ThreadIdColumn);
		ExportTimingEventsColumns.Add(ExportTimingEvents_ThreadNameColumn);
		ExportTimingEventsColumns.Add(ExportTimingEvents_TimerIdColumn);
		ExportTimingEventsColumns.Add(ExportTimingEvents_TimerNameColumn);
		ExportTimingEventsColumns.Add(ExportTimingEvents_StartTimeColumn);
		ExportTimingEventsColumns.Add(ExportTimingEvents_EndTimeColumn);
		ExportTimingEventsColumns.Add(ExportTimingEvents_DurationColumn);
		ExportTimingEventsColumns.Add(ExportTimingEvents_DepthColumn);

		ExportTimingEventsDefaultColumns.Add(ExportTimingEvents_ThreadIdColumn);
		ExportTimingEventsDefaultColumns.Add(ExportTimingEvents_TimerIdColumn);
		ExportTimingEventsDefaultColumns.Add(ExportTimingEvents_StartTimeColumn);
		ExportTimingEventsDefaultColumns.Add(ExportTimingEvents_EndTimeColumn);
		ExportTimingEventsDefaultColumns.Add(ExportTimingEvents_DepthColumn);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingExporter::MakeExportTimingEventsColumnList(const FString& InColumnsString, TArray<FName>& OutColumnList) const
{
	ExportTimingEvents_InitColumns();

	TArray<FString> Columns;
	InColumnsString.ParseIntoArray(Columns, TEXT(","), true);

	for (const FString& ColumnWildcard : Columns)
	{
		FName ColumnName(*ColumnWildcard);
		if (ExportTimingEventsColumns.Contains(ColumnName))
		{
			OutColumnList.Add(ColumnName);
		}
		else
		{
			for (const FName& Column : ExportTimingEventsColumns)
			{
				if (Column.GetPlainNameString().MatchesWildcard(ColumnWildcard))
				{
					OutColumnList.Add(Column);
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingExporter::ExportTimingEvents_WriteHeader(FExportTimingEventsInternalParams& Params) const
{
	bool bFirst = true;
	for (const FName& Column : Params.Columns)
	{
		if (ExportTimingEventsColumns.Contains(Column))
		{
			if (bFirst)
			{
				bFirst = false;
			}
			else
			{
				Params.Writer.AppendSeparator();
			}
			Params.Writer.AppendValue(Column.GetPlainNameString());
		}
	}
	Params.Writer.AppendLineEnd();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTimingExporter::ExportTimingEvents_WriteEvents(FExportTimingEventsInternalParams& Params) const
{
	int32 TimingEventCount = 0;

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(Session);
	if (TimingProfilerProvider)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

		const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();

		auto TimelineEnumerator =
			[&Params, &TimingEventCount, &TimerReader]
			(const TraceServices::ITimingProfilerProvider::Timeline& Timeline)
			{
				// Iterate timing events.
				Timeline.EnumerateEvents(Params.UserParams.IntervalStartTime, Params.UserParams.IntervalEndTime,
					[&Params, &TimingEventCount, &TimerReader]
					(double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event)
					{
						if (!Params.UserParams.TimingEventFilter || Params.UserParams.TimingEventFilter(EventStartTime, EventEndTime, EventDepth, Event, &TimerReader))
						{
							bool bFirst = true;
							for (const FName& Column : Params.Columns)
							{
								if (Params.Exporter.ExportTimingEventsColumns.Contains(Column))
								{
									if (bFirst)
									{
										bFirst = false;
									}
									else
									{
										Params.Writer.AppendSeparator();
									}

									if (Column == ExportTimingEvents_ThreadIdColumn)
									{
										Params.Writer.AppendValue(Params.ThreadId);
									}
									else if (Column == ExportTimingEvents_ThreadNameColumn)
									{
										Params.Writer.AppendValue(Params.ThreadName);
									}
									else if (Column == ExportTimingEvents_TimerIdColumn)
									{
										uint32 TimerIndex = Event.TimerIndex;
										if (int32(TimerIndex) < 0) //-V547
										{
											TimerIndex = TimerReader.GetOriginalTimerIdFromMetadata(TimerIndex);
										}

										Params.Writer.AppendValue(TimerIndex);
									}
									else if (Column == ExportTimingEvents_TimerNameColumn)
									{
										const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(Event.TimerIndex);
										if (Timer != nullptr)
										{
											Params.Writer.AppendValue(Timer->Name);
										}
										else
										{
											Params.Writer.GetStringBuilder().Appendf(UTF8TEXT("[0x%x]"), Event.TimerIndex);
										}
									}
									else if (Column == ExportTimingEvents_StartTimeColumn)
									{
										Params.Writer.AppendValueAsTimestamp(EventStartTime);
									}
									else if (Column == ExportTimingEvents_EndTimeColumn)
									{
										Params.Writer.AppendValueAsTimestamp(EventEndTime);
									}
									else if (Column == ExportTimingEvents_DurationColumn)
									{
										Params.Writer.AppendValueAsDuration(EventEndTime - EventStartTime);
									}
									else if (Column == ExportTimingEvents_DepthColumn)
									{
										Params.Writer.AppendValue(EventDepth);
									}
								}
							}
							Params.Writer.AppendLineEnd();
							++TimingEventCount;
						}

						return TraceServices::EEventEnumerate::Continue;
					}
				); // EnumerateEvents
			};

		auto FrameEnumerator =
			[&Params, &TimingEventCount]
			(const TraceServices::FFrame& Frame)
			{
				bool bFirst = true;
				for (const FName& Column : Params.Columns)
				{
					if (Params.Exporter.ExportTimingEventsColumns.Contains(Column))
					{
						if (bFirst)
						{
							bFirst = false;
						}
						else
						{
							Params.Writer.AppendSeparator();
						}

						if (Column == ExportTimingEvents_ThreadIdColumn)
						{
							Params.Writer.AppendValue(Params.ThreadId);
						}
						else if (Column == ExportTimingEvents_ThreadNameColumn)
						{
							Params.Writer.AppendValue(Params.ThreadName);
						}
						else if (Column == ExportTimingEvents_TimerIdColumn)
						{
							Params.Writer.AppendValue((uint32)Frame.Index);
						}
						else if (Column == ExportTimingEvents_TimerNameColumn)
						{
							Params.Writer.AppendValue(*FFrameTimingTrack::GetFrameName((uint32)Frame.FrameType, Frame.Index));
						}
						else if (Column == ExportTimingEvents_StartTimeColumn)
						{
							Params.Writer.AppendValueAsTimestamp(Frame.StartTime);
						}
						else if (Column == ExportTimingEvents_EndTimeColumn)
						{
							Params.Writer.AppendValueAsTimestamp(Frame.EndTime);
						}
						else if (Column == ExportTimingEvents_DurationColumn)
						{
							Params.Writer.AppendValueAsDuration(Frame.EndTime - Frame.StartTime);
						}
						else if (Column == ExportTimingEvents_DepthColumn)
						{
							Params.Writer.Append(UTF8TEXTVIEW("1"));
						}
					}
				}
				Params.Writer.AppendLineEnd();
				++TimingEventCount;
			};

		// Iterate the GPU timelines (old GPU Profiler).
		{
			if (!Params.UserParams.ThreadFilter || Params.UserParams.ThreadFilter(FGpuTimingTrack::Gpu1ThreadId))
			{
				Params.ThreadId = FGpuTimingTrack::Gpu1ThreadId;
				Params.ThreadName = TEXT("GPU1");
				uint32 GpuTimelineIndex1 = 0;
				TimingProfilerProvider->GetGpuTimelineIndex(GpuTimelineIndex1);
				TimingProfilerProvider->ReadTimeline(GpuTimelineIndex1, TimelineEnumerator);
			}

			if (!Params.UserParams.ThreadFilter || Params.UserParams.ThreadFilter(FGpuTimingTrack::Gpu2ThreadId))
			{
				Params.ThreadId = FGpuTimingTrack::Gpu2ThreadId;
				Params.ThreadName = TEXT("GPU2");
				uint32 GpuTimelineIndex2 = 0;
				TimingProfilerProvider->GetGpu2TimelineIndex(GpuTimelineIndex2);
				TimingProfilerProvider->ReadTimeline(GpuTimelineIndex2, TimelineEnumerator);
			}
		}

		// Iterate the GPU timelines for the GPU Queues (new GPU Profiler).
		{
			TimingProfilerProvider->EnumerateGpuQueues(
				[TimingProfilerProvider, &TimelineEnumerator, &Params]
				(const TraceServices::FGpuQueueInfo& QueueInfo)
				{
					if (!Params.UserParams.ThreadFilter || Params.UserParams.ThreadFilter(GetNonCollidingId(QueueInfo.Id)))
					{
						FString QueueName = QueueInfo.GetDisplayName();
						Params.ThreadId = GetNonCollidingId(QueueInfo.Id);
						Params.ThreadName = *QueueName;
						TimingProfilerProvider->ReadTimeline(QueueInfo.TimelineIndex, TimelineEnumerator);

						Params.ThreadName = nullptr; // The pointer would no longer be valid outside this scope.
					}
				});
		}

		// Iterate the Verse timelines.
		{
			if (!Params.UserParams.ThreadFilter || Params.UserParams.ThreadFilter(FVerseTimingTrack::VerseThreadId))
			{
				Params.ThreadId = FVerseTimingTrack::VerseThreadId;
				Params.ThreadName = TEXT("VerseSampling");
				uint32 VerseTimelineIndex = 0;
				TimingProfilerProvider->GetVerseTimelineIndex(VerseTimelineIndex);
				TimingProfilerProvider->ReadTimeline(VerseTimelineIndex, TimelineEnumerator);
			}
		}

		// Iterate the Frame timelines.
		{
			const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(Session);
			for (uint32 FrameType = 0; FrameType < (uint32)ETraceFrameType::TraceFrameType_Count; ++FrameType)
			{
				if (!Params.UserParams.ThreadFilter || Params.UserParams.ThreadFilter(GetFrameTypeNonCollidingId(FrameType)))
				{
					FString ThreadName = FFrameTimingTrack::GetFrameTrackName(FrameType);
					Params.ThreadId = GetFrameTypeNonCollidingId(FrameType);
					Params.ThreadName = *ThreadName;
					FramesProvider.EnumerateFrames((ETraceFrameType) FrameType, Params.UserParams.IntervalStartTime, Params.UserParams.IntervalEndTime, FrameEnumerator);
					Params.ThreadName = nullptr;
				}
			}
		}

		// Iterate the CPU threads and their corresponding timelines.
		{
			const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(Session);
			ThreadProvider.EnumerateThreads(
				[&TimelineEnumerator, &Params, TimingProfilerProvider]
				(const TraceServices::FThreadInfo& ThreadInfo)
				{
					if (!Params.UserParams.ThreadFilter || Params.UserParams.ThreadFilter(ThreadInfo.Id))
					{
						Params.ThreadId = ThreadInfo.Id;
						Params.ThreadName = ThreadInfo.Name;
						uint32 CpuTimelineIndex = 0;
						TimingProfilerProvider->GetCpuThreadTimelineIndex(ThreadInfo.Id, CpuTimelineIndex);
						TimingProfilerProvider->ReadTimeline(CpuTimelineIndex, TimelineEnumerator);
					}
				});
		}
	}

	return TimingEventCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTimingExporter::ExportTimingEventsAsTextByRegions(const FString& FilenamePattern, FExportTimingEventsParams& Params) const
{
	const TCHAR* ErrorTitle = TEXT("Failed to export timing events (by regions)!");

	TMap<FString, FTimeRegionGroup> RegionGroups;
	GetRegions(Params.Region, RegionGroups);

	if (RegionGroups.Num() == 0)
	{
		Error(ErrorTitle, *FString::Printf(TEXT("Unable to find any region with name pattern '%s'."), *Params.Region));
		return -1;
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	// Export timing statistics for each region.
	FExportTimingEventsParams RegionParams = Params;
	RegionParams.Region.Reset();
	int32 ExportedRegionCount = EnumerateRegions(RegionGroups, FilenamePattern,
		[this, &RegionParams]
		(const FString& Filename, const FString& RegionName, double IntervalStartTime, double IntervalEndTime)
		{
			RegionParams.IntervalStartTime = IntervalStartTime;
			RegionParams.IntervalEndTime = IntervalEndTime;
			UE_LOGF(TraceInsights, Display, "Exporting timing statistics for region '%ls' [%f .. %f] to '%ls'", *RegionName, RegionParams.IntervalStartTime, RegionParams.IntervalEndTime, *Filename);
			ExportTimingEventsAsText(Filename, RegionParams);
		});

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TraceInsights, Log, "Exported timing statistics for %d regions in %.3fs.", ExportedRegionCount, TotalTime);
	return ExportedRegionCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTimingExporter::ExportTimingEventsAsText(const FString& Filename, FExportTimingEventsParams& Params) const
{
	const TCHAR* ErrorTitle = TEXT("Failed to export timing events!");

	if (!Params.Region.IsEmpty())
	{
		return ExportTimingEventsAsTextByRegions(Filename, Params);
	}

	if (Filename.Contains(TEXT("*")) || Filename.Contains(TEXT("{region}")))
	{
		Error(ErrorTitle, TEXT("Filename contains \"{region}\" pattern, but no valid \"-region\" parameter is specified."));
		return -1;
	}

	ExportTimingEvents_InitColumns();
	const TArray<FName>& Columns = Params.Columns ? *Params.Columns : ExportTimingEventsDefaultColumns;

	FStopwatch Stopwatch;
	Stopwatch.Start();

	TUniquePtr<IFileHandle> ExportFileHandle = OpenExportFile(*Filename);
	if (!ExportFileHandle)
	{
		Error(ErrorTitle, *FString::Printf(TEXT("Cannot write the export file (\"%s\")."), *Filename));
		return -1;
	}
	bool bIsCSV = Filename.EndsWith(TEXT(".csv"));
	FUtf8FileWriter Writer(ExportFileHandle.Get(), bIsCSV);

	FExportTimingEventsInternalParams InternalParams = { *this, Params, Columns, Writer, 0 };

	// Write header.
	ExportTimingEvents_WriteHeader(InternalParams);

	// Write values.
	int32 TimingEventCount = ExportTimingEvents_WriteEvents(InternalParams);

	Writer.Flush();
	ExportFileHandle->Flush();
	ExportFileHandle.Reset();

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TraceInsights, Log, "Exported %d timing events to file in %.3fs (\"%ls\").", TimingEventCount, TotalTime, *Filename);

	return TimingEventCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingExporter::GetRegions(const FString& InRegionNamePattern, TMap<FString, FTimeRegionGroup>& OutRegionGroups) const
{
	class FRegionNameSpec
	{
	public:
		FRegionNameSpec(const FString& InNamePatternList)
		{
			InNamePatternList.ParseIntoArray(NamePatterns, TEXT(","), true);
		}

		bool Match(const FString& InRegionName)
		{
			for (const FString& NamePattern : NamePatterns)
			{
				if (InRegionName.MatchesWildcard(NamePattern))
				{
					return true;
				}
			}
			return false;
		}

	private:
		TArray<FString> NamePatterns;
	};
	FRegionNameSpec RegionNameSpec(InRegionNamePattern);

	// Detect regions
	int32 RegionCount = 0;
	FStopwatch DetectRegionsStopwatch;
	DetectRegionsStopwatch.Start();
	{
		const TraceServices::IRegionProvider& RegionProvider = TraceServices::ReadRegionProvider(Session);
		TraceServices::FProviderReadScopeLock RegionProviderScopedLock(RegionProvider);

		UE_LOGF(TraceInsights, Log, "Looking for regions: '%ls'", *InRegionNamePattern);

		RegionProvider.GetDefaultTimeline().EnumerateRegions(0.0, std::numeric_limits<double>::max(),
			[&RegionCount, &RegionNameSpec, &OutRegionGroups]
			(const TraceServices::FTimeRegion& InRegion) -> bool
			{
				if (InRegion.Timer == nullptr || InRegion.Timer->Name == nullptr)
				{
					return true;
				}
				const FString RegionName = InRegion.Timer->Name;
				if (RegionNameSpec.Match(RegionName))
				{
					// Handle duplicate region names. Regions with same name may appear multiple times.
					// We append numbers to allow for unique export filenames.
					FTimeRegionGroup* ExistingRegionGroup = OutRegionGroups.Find(RegionName);
					if (!ExistingRegionGroup)
					{
						ExistingRegionGroup = &OutRegionGroups.Add(RegionName, FTimeRegionGroup{});
					}
					ExistingRegionGroup->Intervals.Add(FTimeRegionInterval{ InRegion.BeginTime, InRegion.EndTime });
					++RegionCount;
				}
				return true;
			});
	}
	DetectRegionsStopwatch.Stop();
	const double DetectRegionsTime = DetectRegionsStopwatch.GetAccumulatedTime();
	UE_LOGF(TraceInsights, Display, "Detected %d regions in %.3fs.", RegionCount, DetectRegionsTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTimingExporter::EnumerateRegions(const TMap<FString, FTimeRegionGroup>& InRegionGroups, const FString& InFilenamePattern,
	TFunction<void(const FString& /*Filename*/, const FString& /*RegionName*/, double /*IntervalStartTime*/, double /*IntervalEndTime*/)> InCallback) const
{
	constexpr int32 MaxIntervalsPerRegion = 100;
	constexpr int32 MaxExportedRegions = 10000;
	int32 ExportedRegionCount = 0;

	for (auto& KV : InRegionGroups)
	{
		FString RegionName(KV.Key);
		const FString InvalidFileSystemChars = FPaths::GetInvalidFileSystemChars();
		for (int32 CharIndex = 0; CharIndex < InvalidFileSystemChars.Len(); CharIndex++)
		{
			FString Char = FString().AppendChar(InvalidFileSystemChars[CharIndex]);
			RegionName.ReplaceInline(*Char, TEXT("_"));
		}
		RegionName.TrimStartAndEndInline();

		int32 IntervalIndex = 0;
		for (const FTimeRegionInterval& Interval : KV.Value.Intervals)
		{
			FString Filename(InFilenamePattern);
			if (IntervalIndex == 0)
			{
				Filename.ReplaceInline(TEXT("*"), *RegionName); // for backward compatibility
				Filename.ReplaceInline(TEXT("{region}"), *RegionName);
			}
			else
			{
				FString UniqueRegionName = FString::Printf(TEXT("%s_%d"), *RegionName, IntervalIndex);
				Filename.ReplaceInline(TEXT("*"), *UniqueRegionName); // for backward compatibility
				Filename.ReplaceInline(TEXT("{region}"), *UniqueRegionName);
			}
			++IntervalIndex;

			InCallback(Filename, KV.Key, Interval.StartTime, Interval.EndTime);

			++ExportedRegionCount;

			const TCHAR* ErrorTitle = TEXT("Failed to enumerate regions!");

			// Avoid writing too many files...
			if (IntervalIndex >= MaxIntervalsPerRegion)
			{
				Error(ErrorTitle, *FString::Printf(TEXT("Too many intervals for region '%s'! Exporting to separate file per interval for this region is not allowed to continue."), *KV.Key));
				break;
			}
			if (ExportedRegionCount >= MaxExportedRegions)
			{
				Error(ErrorTitle, TEXT("Too many regions! Exporting to separate file per region is not allowed to continue."));
				break;
			}
		}

		if (ExportedRegionCount >= MaxExportedRegions)
		{
			break;
		}
	}

	return ExportedRegionCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTimingExporter::ExportTimerStatisticsAsTextByRegions(const FString& FilenamePattern, FExportTimerStatisticsParams& Params) const
{
	const TCHAR* ErrorTitle = TEXT("Failed to export timing statistics (by regions)!");

	TMap<FString, FTimeRegionGroup> RegionGroups;
	GetRegions(Params.Region, RegionGroups);

	if (RegionGroups.Num() == 0)
	{
		Error(ErrorTitle, *FString::Printf(TEXT("Unable to find any region with name pattern '%s'."), *Params.Region));
		return -1;
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	// Export timing statistics for each region.
	FExportTimerStatisticsParams RegionParams = Params;
	RegionParams.Region.Reset();
	int32 ExportedRegionCount = EnumerateRegions(RegionGroups, FilenamePattern,
		[this, &RegionParams]
		(const FString& Filename, const FString& RegionName, double IntervalStartTime, double IntervalEndTime)
		{
			RegionParams.IntervalStartTime = IntervalStartTime;
			RegionParams.IntervalEndTime = IntervalEndTime;
			UE_LOGF(TraceInsights, Display, "Exporting timing statistics for region '%ls' [%f .. %f] to '%ls'", *RegionName, RegionParams.IntervalStartTime, RegionParams.IntervalEndTime, *Filename);
			ExportTimerStatisticsAsText(Filename, RegionParams);
		});

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TraceInsights, Log, "Exported timing statistics for %d regions in %.3fs.", ExportedRegionCount, TotalTime);
	return ExportedRegionCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace TimingExporter
{

TraceServices::FCreateAggregationParams::ESortBy MapTimerStatisticsParams(FTimingExporter::FExportTimerStatisticsParams::ESortBy Arg)
{
	switch(Arg)
	{
		default:
			ensureMsgf(false, TEXT("Unmapped FExportTimerStatisticsParams::ESortBy value %d"), static_cast<int32>(Arg));
			// intentional fall-through
		case FTimingExporter::FExportTimerStatisticsParams::ESortBy::DontSort:
			return TraceServices::FCreateAggregationParams::ESortBy::DontSort;

		case FTimingExporter::FExportTimerStatisticsParams::ESortBy::TotalInclusiveTime:
			return TraceServices::FCreateAggregationParams::ESortBy::TotalInclusiveTime;
	}
}

TraceServices::FCreateAggregationParams::ESortOrder MapTimerStatisticsParams(FTimingExporter::FExportTimerStatisticsParams::ESortOrder Arg)
{
	switch (Arg)
	{
		default:
			ensureMsgf(false, TEXT("Unmapped FExportTimerStatisticsParams::ESortOrder value %d"), static_cast<int32>(Arg));
			// intentional fall-through
		case FTimingExporter::FExportTimerStatisticsParams::ESortOrder::DontSort:
			return TraceServices::FCreateAggregationParams::ESortOrder::DontSort;

		case FTimingExporter::FExportTimerStatisticsParams::ESortOrder::Descending:
			return TraceServices::FCreateAggregationParams::ESortOrder::Descending;

		case FTimingExporter::FExportTimerStatisticsParams::ESortOrder::Ascending:
			return TraceServices::FCreateAggregationParams::ESortOrder::Ascending;
	}
}

} // namespace TimingExporter

int32 FTimingExporter::ExportTimerStatisticsAsText(const FString& Filename, FExportTimerStatisticsParams& Params) const
{
	const TCHAR* ErrorTitle = TEXT("Failed to export timing statistics!");

	if (!Params.Region.IsEmpty())
	{
		return ExportTimerStatisticsAsTextByRegions(Filename, Params);
	}

	if (Filename.Contains(TEXT("*")) || Filename.Contains(TEXT("{region}")))
	{
		Error(ErrorTitle, TEXT("Filename contains \"{region}\" pattern, but no valid \"-region\" parameter is specified."));
		return -1;
	}

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(Session);
	if (!TimingProfilerProvider)
	{
		Error(ErrorTitle, TEXT("Unable to access TimingProfilerProvider."));
		return -1;
	}

	TraceServices::ITable<TraceServices::FTimingProfilerAggregatedStats>* StatsTable;
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

		TraceServices::FCreateAggregationParams CreateAggregationParams;
		CreateAggregationParams.IntervalStart = Params.IntervalStartTime;
		CreateAggregationParams.IntervalEnd = Params.IntervalEndTime;
		CreateAggregationParams.bIncludeOldGpu1 = true;
		CreateAggregationParams.bIncludeOldGpu2 = true;
		CreateAggregationParams.GpuQueueFilter = [](uint32) { return true; };
		CreateAggregationParams.CpuThreadFilter = Params.ThreadFilter;
		CreateAggregationParams.SortBy = TimingExporter::MapTimerStatisticsParams(Params.SortBy);
		CreateAggregationParams.SortOrder = TimingExporter::MapTimerStatisticsParams(Params.SortOrder);
		CreateAggregationParams.TableEntryLimit = Params.MaxExportedEvents;

		//@Todo: this does not yet handle the -column and -timers parameters.
		StatsTable = TimingProfilerProvider->CreateAggregation(CreateAggregationParams);
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	bool bSuccess = TraceServices::Table2Csv(*StatsTable, *Filename);

	if (!bSuccess)
	{
		Error(ErrorTitle, *FString::Printf(TEXT("Failed to write the CSV file (\"%s\")!"), *Filename));
		return -1;
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();

	UE_LOGF(TraceInsights, Log, "Exported timing statistics to file in %.3fs (\"%ls\").", TotalTime, *Filename);
	return 1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTimingExporter::ExportTimerCalleesByRegions(const FString& FilenamePattern, const FExportTimerCalleesParams& Params) const
{
	const TCHAR* ErrorTitle = TEXT("Failed to export timer callees (by regions)!");

	TMap<FString, FTimeRegionGroup> RegionGroups;
	GetRegions(Params.Region, RegionGroups);

	if (RegionGroups.Num() == 0)
	{
		Error(ErrorTitle, *FString::Printf(TEXT("Unable to find any region with name pattern '%s'."), *Params.Region));
		return -1;
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	// Export timing callees for each region.
	FExportTimerCalleesParams RegionParams = Params;
	RegionParams.Region.Reset();
	int32 ExportedRegionCount = EnumerateRegions(RegionGroups, FilenamePattern,
		[this, &RegionParams]
		(const FString& Filename, const FString& RegionName, double IntervalStartTime, double IntervalEndTime)
		{
			RegionParams.IntervalStartTime = IntervalStartTime;
			RegionParams.IntervalEndTime = IntervalEndTime;
			UE_LOGF(TraceInsights, Display, "Exporting timing callees for region '%ls' [%f .. %f] to '%ls'", *RegionName, RegionParams.IntervalStartTime, RegionParams.IntervalEndTime, *Filename);
			ExportTimerCalleesAsText(Filename, RegionParams);
		});

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TraceInsights, Log, "Exported timing callees for %d regions in %.3fs.", ExportedRegionCount, TotalTime);
	return ExportedRegionCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTimingExporter::ExportTimerCalleesAsText(const FString& Filename, const FExportTimerCalleesParams& Params) const
{
	const TCHAR* ErrorTitle = TEXT("Failed to export timer callees!");

	if (!Params.Region.IsEmpty())
	{
		return ExportTimerCalleesByRegions(Filename, Params);
	}

	if (Filename.Contains(TEXT("*")) || Filename.Contains(TEXT("{region}")))
	{
		Error(ErrorTitle, TEXT("Filename contains \"{region}\" pattern, but no valid \"-region\" parameter is specified."));
		return -1;
	}

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(Session);
	if (!TimingProfilerProvider)
	{
		Error(ErrorTitle, TEXT("Unable to access TimingProfilerProvider."));
		return -1;
	}

	TUniquePtr<TraceServices::ITimingProfilerButterfly> Butterfly = [this, ErrorTitle, TimingProfilerProvider, &Params]() -> TUniquePtr<TraceServices::ITimingProfilerButterfly>
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

		TraceServices::FCreateButterflyParams ButterflyParams;
		ButterflyParams.IntervalStart = Params.IntervalStartTime;
		// The region end interval may be inf if the capture ended before the region was closed.
		ButterflyParams.IntervalEnd = FMath::Min(Session.GetDurationSeconds(), Params.IntervalEndTime);
		ButterflyParams.GpuQueueFilter = [](uint32) { return true; };
		ButterflyParams.bIncludeOldGpu1 = Params.ThreadFilter(FGpuTimingTrack::Gpu1ThreadId);
		ButterflyParams.bIncludeOldGpu2 = Params.ThreadFilter(FGpuTimingTrack::Gpu2ThreadId);
		ButterflyParams.bIncludeVerseSampling = Params.ThreadFilter(FVerseTimingTrack::VerseThreadId);
		ButterflyParams.CpuThreadFilter = Params.ThreadFilter;

		return TUniquePtr<TraceServices::ITimingProfilerButterfly>{ TimingProfilerProvider->CreateButterfly(ButterflyParams) };
	}();
	if (!Butterfly)
	{
		Error(ErrorTitle, *FString::Printf(TEXT("Cannot create Butterfly for region \"%s\" [%.6f-%.6f]"), *Params.Region, Params.IntervalStartTime, Params.IntervalEndTime));
		return -1;
	}

	TUniquePtr<IFileHandle> ExportFileHandle = OpenExportFile(*Filename);
	if (!ExportFileHandle)
	{
		Error(ErrorTitle, *FString::Printf(TEXT("Cannot write the export file (\"%s\")."), *Filename));
		return -1;
	}

	const bool bIsCSV = Filename.EndsWith(TEXT(".csv"));
	FUtf8FileWriter Writer(ExportFileHandle.Get(), bIsCSV);

	// Write header
	{
		TArray<FUtf8StringView> Columns
		{
			UTF8TEXTVIEW("TimerId"),
			UTF8TEXTVIEW("ParentId"),
			UTF8TEXTVIEW("TimerName"),
			UTF8TEXTVIEW("Count"),
			UTF8TEXTVIEW("Inc.Time"),
			UTF8TEXTVIEW("Exc.Time"),
			UTF8TEXTVIEW("NumFrames")
		};

		bool bFirst = true;
		for (FUtf8StringView ColumnName : Columns)
		{
			if (bFirst)
			{
				bFirst = false;
			}
			else
			{
				Writer.AppendSeparator();
			}
			Writer.Append(ColumnName);
		}
		Writer.AppendLineEnd();
	}

	// Count the number of frames in this region so it can be output and used to calculate frame averages.
	uint64 NumFrames = 0;
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);
		const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(Session);
		FrameProvider.EnumerateFrames(ETraceFrameType::TraceFrameType_Game, Params.IntervalStartTime, Params.IntervalEndTime,
			[&NumFrames]
			(const TraceServices::FFrame& Frame)
			{
				++NumFrames;
			});
	}

	// Write rows
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);
		for (uint32 TimerId : Params.TimerIds)
		{
			TArray<const TraceServices::FTimingProfilerButterflyNode*> NodesToVisit{ &Butterfly->GenerateCalleesTree(TimerId) };
			while (!NodesToVisit.IsEmpty())
			{
				// Root node can be null if the timer id is for a thread we've filtered out.
				const TraceServices::FTimingProfilerButterflyNode* CurrentNode = NodesToVisit.Pop(EAllowShrinking::No);
				if (!CurrentNode || !CurrentNode->Timer)
				{
					continue;
				}

				NodesToVisit.Append(CurrentNode->Children);

				Writer.AppendValue(CurrentNode->Timer->Id);
				Writer.AppendSeparator();
				if (CurrentNode->Parent)
				{
					Writer.AppendValue(CurrentNode->Parent->Timer->Id);
				}
				else
				{
					Writer.Append(UTF8TEXTVIEW("-1"));
				}
				Writer.AppendSeparator();
				if (Params.bShowFormatInTimerNames)
				{
					TStringBuilder<1024> NameStringBuilder;
					TimingProfilerProvider->MakeTimerDisplayName(*CurrentNode->Timer, NameStringBuilder);
					Writer.AppendValue(NameStringBuilder.ToView());
				}
				else
				{
					Writer.AppendValue(CurrentNode->Timer->Name);
				}
				Writer.AppendSeparator();
				Writer.AppendValue(CurrentNode->Count);
				Writer.AppendSeparator();
				Writer.AppendValueAsDuration(CurrentNode->InclusiveTime);
				Writer.AppendSeparator();
				Writer.AppendValueAsDuration(CurrentNode->ExclusiveTime);
				Writer.AppendSeparator();
				Writer.AppendValue(NumFrames);
				Writer.AppendLineEnd();
			}
		}
	}

	Writer.Flush();
	ExportFileHandle->Flush();
	return 1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingExporter::FThreadFilterFunc FTimingExporter::MakeThreadFilterInclusive(const FString& InFilterString, TSet<uint32>& OutIncludedThreads) const
{
	if (InFilterString.Len() == 1 && InFilterString[0] == TEXT('*'))
	{
		return nullptr;
	}

	OutIncludedThreads.Reset();

	TMap<FString, uint32> Threads;

	// Add the GPU pseudo-threads (old GPU Profiler).
	{
		Threads.Add(FString("GPU"), FGpuTimingTrack::Gpu1ThreadId);
		Threads.Add(FString("GPU1"), FGpuTimingTrack::Gpu1ThreadId);
		Threads.Add(FString("GPU2"), FGpuTimingTrack::Gpu2ThreadId);
	}

	// Add the GPU Queue pseudo-threads (new GPU Profiler).
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(Session);
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

			TimingProfilerProvider->EnumerateGpuQueues(
				[&Threads](const TraceServices::FGpuQueueInfo& QueueInfo)
				{
					Threads.Add(QueueInfo.GetDisplayName(), GetNonCollidingId(QueueInfo.Id));
				});
		}
	}

	// Add the Verse Sampling pseudo-thread.
	{
		Threads.Add(FString("VerseSampling"), FVerseTimingTrack::VerseThreadId);
	}

	// Add the CPU threads.
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

		const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(Session);

		ThreadProvider.EnumerateThreads(
			[&Threads](const TraceServices::FThreadInfo& ThreadInfo)
			{
				Threads.Add(FString(ThreadInfo.Name), ThreadInfo.Id);
			});
	}

	// Add the Frame pseudo-threads.
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

		const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(Session);

		for (uint32 FrameType = 0; FrameType < (uint32)ETraceFrameType::TraceFrameType_Count; ++FrameType)
		{
			Threads.Add(FFrameTimingTrack::GetFrameTrackName(FrameType), GetFrameTypeNonCollidingId(FrameType));
		}
	}

	TArray<FString> Filter;
	InFilterString.ParseIntoArray(Filter, TEXT(","), true);

	for (const FString& ThreadWildcard : Filter)
	{
		const uint32* Id = Threads.Find(ThreadWildcard);
		if (Id)
		{
			OutIncludedThreads.Add(*Id);
		}
		else
		{
			for (const auto& KeyValuePair : Threads)
			{
				if (KeyValuePair.Key.MatchesWildcard(ThreadWildcard))
				{
					OutIncludedThreads.Add(KeyValuePair.Value);
				}
			}
		}
	}

	return MakeThreadFilterInclusive(OutIncludedThreads);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TFunction<bool(uint32)> FTimingExporter::MakeThreadFilterInclusive(const TSet<uint32>& IncludedThreads)
{
	if (IncludedThreads.Num() == 0)
	{
		return [](uint32 ThreadId) -> bool
		{
			return false;
		};
	}

	if (IncludedThreads.Num() == 1)
	{
		const uint32 IncludedThreadId = IncludedThreads[FSetElementId::FromInteger(0)];
		return [IncludedThreadId](uint32 ThreadId) -> bool
		{
			return ThreadId == IncludedThreadId;
		};
	}

	return [&IncludedThreads](uint32 ThreadId) -> bool
	{
		return IncludedThreads.Contains(ThreadId);
	};
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TFunction<bool(uint32)> FTimingExporter::MakeThreadFilterExclusive(const TSet<uint32>& ExcludedThreads)
{
	if (ExcludedThreads.Num() == 0)
	{
		return nullptr;
	}

	if (ExcludedThreads.Num() == 1)
	{
		const uint32 ExcludedThreadId = ExcludedThreads[FSetElementId::FromInteger(0)];
		return [ExcludedThreadId](uint32 ThreadId) -> bool
		{
			return ThreadId != ExcludedThreadId;
		};
	}

	return [&ExcludedThreads](uint32 ThreadId) -> bool
	{
		return !ExcludedThreads.Contains(ThreadId);
	};
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingExporter::FTimingEventFilterFunc FTimingExporter::MakeTimingEventFilterByTimersInclusive(const FString& InFilterString, TSet<uint32>& OutIncludedTimers) const
{
	if (InFilterString.Len() == 1 && InFilterString[0] == TEXT('*'))
	{
		return nullptr;
	}

	OutIncludedTimers.Reset();

	TArray<FString> NamePatterns;
	InFilterString.ParseIntoArray(NamePatterns, TEXT(","), true);

	// Iterate the available timers.
	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(Session);
	if (TimingProfilerProvider)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

		const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();

		const uint32 TimerCount = TimerReader.GetTimerCount();
		for (uint32 TimerIndex = 0; TimerIndex < TimerCount; ++TimerIndex)
		{
			const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(TimerIndex);
			check(Timer != nullptr);

			FString TimerName(Timer->Name);
			for (const FString& Pattern : NamePatterns)
			{
				if (TimerName.MatchesWildcard(Pattern))
				{
					OutIncludedTimers.Add(Timer->Id);
				}
			}
		}
	}

	return MakeTimingEventFilterByTimersInclusive(OutIncludedTimers);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingExporter::FTimingEventFilterFunc FTimingExporter::MakeTimingEventFilterByTimersInclusive(const TSet<uint32>& IncludedTimers)
{
	if (IncludedTimers.Num() == 0)
	{
		return [](double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event, const TraceServices::ITimingProfilerTimerReader* TimerReader) -> bool
		{
			return false;
		};
	}

	if (IncludedTimers.Num() == 1)
	{
		const uint32 IncludedTimerId = IncludedTimers[FSetElementId::FromInteger(0)];
		return [IncludedTimerId](double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event, const TraceServices::ITimingProfilerTimerReader* TimerReader) -> bool
		{
			uint32 TimerIndex = Event.TimerIndex;
			if ((int32)TimerIndex < 0)
			{
				TimerIndex = TimerReader->GetOriginalTimerIdFromMetadata(TimerIndex);
			}
			return TimerIndex == IncludedTimerId;
		};
	}

	return [&IncludedTimers](double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event, const TraceServices::ITimingProfilerTimerReader* TimerReader) -> bool
	{
		uint32 TimerIndex = Event.TimerIndex;
		if ((int32)TimerIndex < 0)
		{
			TimerIndex = TimerReader->GetOriginalTimerIdFromMetadata(TimerIndex);
		}
		return IncludedTimers.Contains(TimerIndex);
	};
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingExporter::FTimingEventFilterFunc FTimingExporter::MakeTimingEventFilterByTimersExclusive(const TSet<uint32>& ExcludedTimers)
{
	if (ExcludedTimers.Num() == 0)
	{
		return nullptr;
	}

	if (ExcludedTimers.Num() == 1)
	{
		const uint32 ExcludedTimerId = ExcludedTimers[FSetElementId::FromInteger(0)];
		return [ExcludedTimerId](double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event, const TraceServices::ITimingProfilerTimerReader* TimerReader) -> bool
		{
			uint32 TimerIndex = Event.TimerIndex;
			if ((int32)TimerIndex < 0)
			{
				TimerIndex = TimerReader->GetOriginalTimerIdFromMetadata(TimerIndex);
			}
			return TimerIndex != ExcludedTimerId;
		};
	}

	return [&ExcludedTimers](double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event, const TraceServices::ITimingProfilerTimerReader* TimerReader) -> bool
	{
		uint32 TimerIndex = Event.TimerIndex;
		if ((int32)TimerIndex < 0)
		{
			TimerIndex = TimerReader->GetOriginalTimerIdFromMetadata(TimerIndex);
		}
		return !ExcludedTimers.Contains(TimerIndex);
	};
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTimingExporter::ExportCountersAsText(const FString& Filename, FExportCountersParams& Params) const
{
	const TCHAR* ErrorTitle = TEXT("Failed to export counters!");

	if (Params.Columns != nullptr)
	{
		Error(ErrorTitle, TEXT("Custom list of columns is not yet supported!"));
		return -1;
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	TUniquePtr<IFileHandle> ExportFileHandle = OpenExportFile(*Filename);
	if (!ExportFileHandle)
	{
		Error(ErrorTitle, *FString::Printf(TEXT("Cannot write the export file (\"%s\")."), *Filename));
		return -1;
	}
	bool bIsCSV = Filename.EndsWith(TEXT(".csv"));
	FUtf8FileWriter Writer(ExportFileHandle.Get(), bIsCSV);

	// Write header.
	{
		Writer.Append(UTF8TEXTVIEW("Id"));
		Writer.AppendSeparator();
		Writer.Append(UTF8TEXTVIEW("Type"));
		Writer.AppendSeparator();
		Writer.Append(UTF8TEXTVIEW("Name"));
		Writer.AppendLineEnd();
	}

	int32 CounterCount = 0;

	// Write values.
	if (true) // TraceServices::ReadCounterProvider(Session)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

		const TraceServices::ICounterProvider& CounterProvider = TraceServices::ReadCounterProvider(Session);

		CounterProvider.EnumerateCounters(
			[&](uint32 CounterId, const TraceServices::ICounter& Counter)
			{
				Writer.AppendValue(CounterId);
				Writer.AppendSeparator();
				if (Counter.IsFloatingPoint())
				{
					Writer.Append(UTF8TEXTVIEW("Double"));
				}
				else
				{
					Writer.Append(UTF8TEXTVIEW("Int64"));
				}
				if (Counter.IsResetEveryFrame())
				{
					Writer.Append(UTF8TEXTVIEW("|ResetEveryFrame"));
				}
				Writer.AppendSeparator();
				Writer.AppendValue(Counter.GetName());
				Writer.AppendLineEnd();
				++CounterCount;
			});
	}

	Writer.Flush();
	ExportFileHandle->Flush();
	ExportFileHandle.Reset();

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TraceInsights, Log, "Exported %d counters to file in %.3fs (\"%ls\").", CounterCount, TotalTime, *Filename);

	return CounterCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTimingExporter::ExportCounterAsTextByRegions(const FString& FilenamePattern, uint32 CounterId, FExportCounterParams& Params) const
{
	const TCHAR* ErrorTitle = TEXT("Failed to export counter (by regions)!");

	TMap<FString, FTimeRegionGroup> RegionGroups;
	GetRegions(Params.Region, RegionGroups);

	if (RegionGroups.Num() == 0)
	{
		Error(ErrorTitle, *FString::Printf(TEXT("Unable to find any region with name pattern '%s'."), *Params.Region));
		return -1;
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	// Export counter for each region.
	FExportCounterParams RegionParams = Params;
	RegionParams.Region.Reset();
	int32 ExportedRegionCount = EnumerateRegions(RegionGroups, FilenamePattern,
		[this, CounterId, &RegionParams]
		(const FString& Filename, const FString& RegionName, double IntervalStartTime, double IntervalEndTime)
		{
			RegionParams.IntervalStartTime = IntervalStartTime;
			RegionParams.IntervalEndTime = IntervalEndTime;
			UE_LOGF(TraceInsights, Display, "Exporting counter %u for region '%ls' [%f .. %f] to '%ls'", CounterId , *RegionName, RegionParams.IntervalStartTime, RegionParams.IntervalEndTime, *Filename);
			ExportCounterAsText(Filename, CounterId, RegionParams);
		});

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TraceInsights, Log, "Exported counter values for %d regions in %.3fs.", ExportedRegionCount, TotalTime);
	return ExportedRegionCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTimingExporter::ExportCounterAsText(const FString& FilenamePattern, uint32 CounterId, FExportCounterParams& Params) const
{
	const TCHAR* ErrorTitle = TEXT("Failed to export counter!");

	if (Params.Columns != nullptr)
	{
		Error(ErrorTitle, TEXT("Custom list of columns is not yet supported!"));
		return -1;
	}

	if (!Params.Region.IsEmpty())
	{
		return ExportCounterAsTextByRegions(FilenamePattern, CounterId, Params);
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	FString CounterName;

	if (true) // TraceServices::ReadCounterProvider(Session)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);
		const TraceServices::ICounterProvider& CounterProvider = TraceServices::ReadCounterProvider(Session);
		CounterProvider.ReadCounter(CounterId,
			[&](const TraceServices::ICounter& Counter)
			{
				CounterName = Counter.GetName();
			});
	}

	if (CounterName.IsEmpty())
	{
		Error(ErrorTitle, TEXT("Invalid counter!"));
		return -1;
	}

	FString Filename(FilenamePattern);
	if (Filename.Contains(TEXT("{counter}")))
	{
		FString CounterFilename(CounterName);
		const FString InvalidFileSystemChars = FPaths::GetInvalidFileSystemChars();
		for (int32 CharIndex = 0; CharIndex < InvalidFileSystemChars.Len(); CharIndex++)
		{
			FString Char = FString().AppendChar(InvalidFileSystemChars[CharIndex]);
			CounterFilename.ReplaceInline(*Char, TEXT("_"));
		}
		CounterFilename.TrimStartAndEndInline();
		Filename.ReplaceInline(TEXT("{counter}"), *CounterFilename);
	}

	TUniquePtr<IFileHandle> ExportFileHandle = OpenExportFile(*Filename);
	if (!ExportFileHandle)
	{
		Error(ErrorTitle, *FString::Printf(TEXT("Cannot write the export file (\"%s\")."), *Filename));
		return -1;
	}
	bool bIsCSV = Filename.EndsWith(TEXT(".csv"));
	FUtf8FileWriter Writer(ExportFileHandle.Get(), bIsCSV);

	// Write header.
	if (Params.bExportOps)
	{
		Writer.Append(UTF8TEXTVIEW("Time"));
		Writer.AppendSeparator();
		Writer.Append(UTF8TEXTVIEW("Op"));
		Writer.AppendSeparator();
		Writer.Append(UTF8TEXTVIEW("Value"));
		Writer.AppendLineEnd();
	}
	else
	{
		Writer.Append(UTF8TEXTVIEW("Time"));
		Writer.AppendSeparator();
		Writer.Append(UTF8TEXTVIEW("Value"));
		Writer.AppendLineEnd();
	}

	int32 ValueCount = 0;

	// Write values.
	if (true) // TraceServices::ReadCounterProvider(Session)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

		const TraceServices::ICounterProvider& CounterProvider = TraceServices::ReadCounterProvider(Session);

		CounterProvider.ReadCounter(CounterId,
			[&](const TraceServices::ICounter& Counter)
			{
				// Iterate the counter values.
				if (Params.bExportOps)
				{
					if (Counter.IsFloatingPoint())
					{
						Counter.EnumerateFloatOps(Params.IntervalStartTime, Params.IntervalEndTime, false,
							[&](double Time, TraceServices::ECounterOpType Op, double Value)
							{
								Writer.AppendValueAsTimestamp(Time);
								Writer.AppendSeparator();
								switch (Op)
								{
									case TraceServices::ECounterOpType::Set:
										Writer.Append(UTF8TEXTVIEW("Set"));
										break;
									case TraceServices::ECounterOpType::Add:
										Writer.Append(UTF8TEXTVIEW("Add"));
										break;
									default:
										Writer.AppendValue(int32(Op));
								}
								Writer.AppendSeparator();
								Writer.AppendValue(Value);
								Writer.AppendLineEnd();
								++ValueCount;
							});
					}
					else
					{
						Counter.EnumerateOps(Params.IntervalStartTime, Params.IntervalEndTime, false,
							[&](double Time, TraceServices::ECounterOpType Op, int64 IntValue)
							{
								Writer.AppendValueAsTimestamp(Time);
								Writer.AppendSeparator();
								switch (Op)
								{
									case TraceServices::ECounterOpType::Set:
										Writer.Append(UTF8TEXTVIEW("Set"));
										break;
									case TraceServices::ECounterOpType::Add:
										Writer.Append(UTF8TEXTVIEW("Add"));
										break;
									default:
										Writer.AppendValue(int32(Op));
								}
								Writer.AppendSeparator();
								Writer.AppendValue(IntValue);
								Writer.AppendLineEnd();
								++ValueCount;
							});
					}
				}
				else
				{
					if (Counter.IsFloatingPoint())
					{
						Counter.EnumerateFloatValues(Params.IntervalStartTime, Params.IntervalEndTime, false,
							[&](double Time, double Value)
							{
								Writer.AppendValueAsTimestamp(Time);
								Writer.AppendSeparator();
								Writer.AppendValue(Value);
								Writer.AppendLineEnd();
								++ValueCount;
							});
					}
					else
					{
						Counter.EnumerateValues(Params.IntervalStartTime, Params.IntervalEndTime, false,
							[&](double Time, int64 IntValue)
							{
								Writer.AppendValueAsTimestamp(Time);
								Writer.AppendSeparator();
								Writer.AppendValue(IntValue);
								Writer.AppendLineEnd();
								++ValueCount;
							});
					}
				}
			});
	}

	Writer.Flush();
	ExportFileHandle->Flush();
	ExportFileHandle.Reset();

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TraceInsights, Log, "Exported counter %d (\"%ls\", %d values) to file in %.3fs (\"%ls\").", CounterId, *CounterName, ValueCount, TotalTime, *Filename);

	return 1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FTimingExporter::GetNonCollidingId(uint32 QueueId)
{
	return QueueId + (1 << 16);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FTimingExporter::GetFrameTypeNonCollidingId(uint32 FrameType)
{
	return FrameType + (1 << 24);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
