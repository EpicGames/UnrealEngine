// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

#include <limits>

////////////////////////////////////////////////////////////////////////////////////////////////////

class IFileHandle;

namespace TraceServices
{
	class IAnalysisSession;
	class ITimingProfilerTimerReader;
	struct FTimingProfilerEvent;
}

namespace UE::Insights
{
	class FUtf8FileWriter;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingExporter
{
public:
	typedef TFunction<bool(const FName /*ColumnId*/)> FColumnFilterFunc;
	typedef TFunction<bool(uint32 /*ThreadId*/)> FThreadFilterFunc;

	/**
	* Use only under session read lock.
	*/
	typedef TFunction<bool(double /*EventStartTime*/, double /*EventEndTime*/, uint32 /*EventDepth*/, const TraceServices::FTimingProfilerEvent& /*Event*/, const TraceServices::ITimingProfilerTimerReader* /*TimerReader*/)> FTimingEventFilterFunc;

	struct FExportThreadsParams
	{
		/**
		 * The list of columns to be exported.
		 * If nullptr, it uses the default list of columns.
		 */
		const TArray<FName>* Columns = nullptr;
	};

	struct FExportTimersParams
	{
		/**
		 * The list of columns to be exported.
		 * If nullptr, it uses the default list of columns.
		 */
		const TArray<FName>* Columns = nullptr;

		/**
		 *  When enabled, the format will be appended to the timers static name for events with metadata specs.
		 */
		bool bShowFormatInTimerNames = true;
	};

	struct FExportTimingEventsParams
	{
		/**
		 * The list of columns to be exported.
		 * If nullptr, it uses the default list of columns.
		 */
		const TArray<FName>* Columns = nullptr;

		/**
		 * Filters the threads for which timing events are exported.
		 * If nullptr, exports timing events from all threads.
		 */
		FThreadFilterFunc ThreadFilter = nullptr;

		/**
		 * Filters the timing events.
		 * If nullptr, exports all timing events.
		 */
		FTimingEventFilterFunc TimingEventFilter = nullptr;

		/**
		 * Filters the timing events by time.
		 * Only timing events that intersects the [StartTime, EndTime] interval are exported.
		 */
		double IntervalStartTime = -std::numeric_limits<double>::infinity();
		double IntervalEndTime = +std::numeric_limits<double>::infinity();

		/**
		 * The time region to be exported.
		 * If empty, falls back to IntervalStartTime and IntervalEndTime.
		 */
		FString Region;
	};

	struct FExportTimerStatisticsParams : public FExportTimingEventsParams
	{
		/**
		 * Enum governing field to use for sorting of exported events
		 */
		enum class ESortBy
		{
			DontSort,
			TotalInclusiveTime
		};

		/**
		 * Enum governing sorting order of exported events
		 */
		enum class ESortOrder
		{
			DontSort,
			Descending,
			Ascending
		};

		/**
		 * Whether to sort the exported timers by a field, and which one if so.
		 */
		ESortBy SortBy = ESortBy::DontSort;

		/**
		 * Sorting order of the exported timers, descending or ascending
		 */
		ESortOrder SortOrder = ESortOrder::DontSort;

		/**
		 * Whether to limit the exported timing events (e.g. "top 100"). 0 means none.
		 */
		int MaxExportedEvents = 0;
	};

	struct FExportTimerCalleesParams : private FExportTimingEventsParams
	{
		using FExportTimingEventsParams::ThreadFilter;
		using FExportTimingEventsParams::IntervalStartTime;
		using FExportTimingEventsParams::IntervalEndTime;
		using FExportTimingEventsParams::Region;

		TSet<uint32> TimerIds;

		/**
		 *  When enabled, the format will be appended to the timers static name for events with metadata specs.
		 */
		bool bShowFormatInTimerNames = true;
	};

	struct FExportCountersParams
	{
		/**
		 * The list of columns to be exported.
		 * If nullptr, it uses the default list of columns.
		 */
		const TArray<FName>* Columns = nullptr;
	};

	struct FExportCounterParams
	{
		/**
		 * The list of columns to be exported.
		 * If nullptr, it uses the default list of columns.
		 */
		const TArray<FName>* Columns = nullptr;

		/**
		 * Filters the counter events by time.
		 * Only timing events that intersects the [StartTime, EndTime] interval are exported.
		 */
		double IntervalStartTime = -std::numeric_limits<double>::infinity();
		double IntervalEndTime = +std::numeric_limits<double>::infinity();

		/**
		 * The time region to be exported.
		 * If empty, falls back to IntervalStartTime and IntervalEndTime.
		 */
		FString Region;

		/**
		 * If true, will export values with the corresponding operation type, instead of the final values.
		 */
		bool bExportOps = false;
	};

	struct FTimeRegionInterval
	{
		double StartTime;
		double EndTime;
	};

	struct FTimeRegionGroup
	{
		TArray<FTimeRegionInterval> Intervals;
	};

private:
	struct FExportTimingEventsInternalParams
	{
		const FTimingExporter& Exporter;
		const FExportTimingEventsParams& UserParams;
		const TArray<FName>& Columns;
		FUtf8FileWriter& Writer;
		uint32 ThreadId;
		const TCHAR* ThreadName;
	};

public:
	FTimingExporter(const TraceServices::IAnalysisSession& InSession);
	virtual ~FTimingExporter();

	//////////////////////////////////////////////////////////////////////
	// Exporters

	int32 ExportThreadsAsText(const FString& Filename, FExportThreadsParams& Params) const;

	int32 ExportTimersAsText(const FString& Filename, FExportTimersParams& Params) const;

	int32 ExportTimingEventsAsText(const FString& Filename, FExportTimingEventsParams& Params) const;

	/**
	 * Exports Timer Statistics (min,max, inclusive average, exclusive average, etc.).
	 * Supports specifying a range to export via bookmarks, but does not support timer selection via -timers
	 * or column selection via -columns yet
	 */
	int32 ExportTimerStatisticsAsText(const FString& Filename, FExportTimerStatisticsParams& Params) const;

	int32 ExportTimerCalleesAsText(const FString& Filename, const FExportTimerCalleesParams& Params) const;

	int32 ExportCountersAsText(const FString& Filename, FExportCountersParams& Params) const;

	int32 ExportCounterAsText(const FString& Filename, uint32 CounterId, FExportCounterParams& Params) const;

	//////////////////////////////////////////////////////////////////////
	// Utilities

	void MakeExportTimingEventsColumnList(const FString& InColumnsString, TArray<FName>& OutColumnList) const;

	void GetRegions(const FString& InRegionNamePattern, TMap<FString, FTimeRegionGroup>& OutRegionGroups) const;
	int32 EnumerateRegions(const TMap<FString, FTimeRegionGroup>& InRegionGroups, const FString& InFilenamePattern,
		TFunction<void(const FString& /*Filename*/, const FString& /*RegionName*/, double /*IntervalStartTime*/, double /*IntervalEndTime*/)> InCallback) const;

	//////////////////////////////////////////////////////////////////////
	// Utilities to make FThreadFilterFunc filters.

	FThreadFilterFunc MakeThreadFilterInclusive(const FString& InFilterString, TSet<uint32>& OutIncludedThreads) const;

	/**
	 * Makes a FThreadFilterFunc using a set of included list of threads.
	 * Note: The set is referenced in the returned function.
	 * @param IncludedThreads The set of thread ids to be accepted by filter.
	 * @return The FThreadFilterFunc function.
	 */
	static FThreadFilterFunc MakeThreadFilterInclusive(const TSet<uint32>& IncludedThreads);

	/**
	 * Makes a FThreadFilterFunc using a set of excluded list of threads.
	 * Note: The set is referenced in the returned function.
	 * @param ExcludedThreads The set of thread ids to be rejected by filter.
	 * @return The FThreadFilterFunc function.
	 */
	static FThreadFilterFunc MakeThreadFilterExclusive(const TSet<uint32>& ExcludedThreads);

	//////////////////////////////////////////////////////////////////////
	// Utilities to make FTimingEventFilterFunc filters.

	FTimingEventFilterFunc MakeTimingEventFilterByTimersInclusive(const FString& InFilterString, TSet<uint32>& OutIncludedTimers) const;

	/**
	 * Makes a FTimingEventFilterFunc using a set of included list of timers.
	 * Note: The set is referenced in the returned function.
	 * @param IncludedTimers The set of timer ids to be accepted by filter.
	 * @return The TimingEventFilter function.
	 */
	static FTimingEventFilterFunc MakeTimingEventFilterByTimersInclusive(const TSet<uint32>& IncludedTimers);

	/**
	 * Makes a FTimingEventFilterFunc using a set of excluded list of timers.
	 * Note: The set is referenced in the returned function.
	 * @param ExcludedTimers The set of timer ids to be rejected by filter.
	 * @return The TimingEventFilter function. Can be nullptr (i.e. no filter).
	 */
	static FTimingEventFilterFunc MakeTimingEventFilterByTimersExclusive(const TSet<uint32>& ExcludedTimers);

private:
	TUniquePtr<IFileHandle> OpenExportFile(const TCHAR* InFilename) const;
	void Error(const TCHAR* InTitle, const TCHAR* InMessage) const;
	void ExportTimingEvents_InitColumns() const;
	void ExportTimingEvents_WriteHeader(FExportTimingEventsInternalParams& Params) const;
	int32 ExportTimingEvents_WriteEvents(FExportTimingEventsInternalParams& Params) const;
	int32 ExportTimingEventsAsTextByRegions(const FString& FilenamePattern, FExportTimingEventsParams& Params) const;
	int32 ExportTimerStatisticsAsTextByRegions(const FString& Filename, FExportTimerStatisticsParams& Params) const;
	int32 ExportTimerCalleesByRegions(const FString& FilenamePattern, const FExportTimerCalleesParams& Params) const;
	int32 ExportCounterAsTextByRegions(const FString& Filename, uint32 CounterId, FExportCounterParams& Params) const;
	static uint32 GetNonCollidingId(uint32 QueueId);
	static uint32 GetFrameTypeNonCollidingId(uint32 FrameType);

private:
	const TraceServices::IAnalysisSession& Session;
	mutable TSet<FName> ExportTimingEventsColumns;
	mutable TArray<FName> ExportTimingEventsDefaultColumns;
	mutable TArray<FName> ExportTimerStatisticsDefaultColumns;
	mutable TArray<FName> ExportTimerStatisticsColumns;

	static const FName ExportTimingEvents_ThreadIdColumn;
	static const FName ExportTimingEvents_ThreadNameColumn;
	static const FName ExportTimingEvents_TimerIdColumn;
	static const FName ExportTimingEvents_TimerNameColumn;
	static const FName ExportTimingEvents_StartTimeColumn;
	static const FName ExportTimingEvents_EndTimeColumn;
	static const FName ExportTimingEvents_DurationColumn;
	static const FName ExportTimingEvents_DepthColumn;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
