// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimingRegionsSharedState.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

// TraceServices
#include "TraceServices/Model/AnalysisSession.h"

// TraceInsights
#include "Common/ProviderLock.h"
#include "Insights/InsightsManager.h"
#include "Insights/Common/InsightsMenuBuilder.h"
#include "Insights/InsightsStyle.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/TimingProfiler/Tracks/RegionsTimingTrack.h"
#include "Insights/Widgets/STimingView.h"
#include "TraceServices/Model/Regions.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::TimingRegions"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimingRegionsViewCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingRegionsViewCommands : public TCommands<FTimingRegionsViewCommands>
{
public:
	FTimingRegionsViewCommands();
	virtual ~FTimingRegionsViewCommands() {}
	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> ShowHideTimingRegionsTrack;
	TSharedPtr<FUICommandInfo> ColorTimingRegionsTrackByCategory;
	TSharedPtr<FUICommandInfo> CreateRegionTracksByCategory;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingRegionsViewCommands::FTimingRegionsViewCommands()
	: TCommands<FTimingRegionsViewCommands>(
		TEXT("FTimingRegionsViewCommands"),
		NSLOCTEXT("Contexts", "FTimingRegionsViewCommands", "Insights - Timing View - Timing Regions"),
		NAME_None,
		FInsightsStyle::GetStyleSetName())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

// UI_COMMAND takes long for the compiler to optimize
UE_DISABLE_OPTIMIZATION_SHIP
void FTimingRegionsViewCommands::RegisterCommands()
{
	UI_COMMAND(ShowHideTimingRegionsTrack,
		"Timing Regions Track",
		"Shows/hides the Timing Regions track(s).",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EModifierKey::Control, EKeys::R));
	UI_COMMAND(ColorTimingRegionsTrackByCategory,
		"Color Regions by Category",
		"Color Timing Regions by Category instead of by Name.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());
	UI_COMMAND(CreateRegionTracksByCategory,
		"Split Timing Regions into individual Tracks per Category",
		"Creates a Timing Regions track for each category instead of a single combined one.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());
}
UE_ENABLE_OPTIMIZATION_SHIP

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimingRegionsSharedState
////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingRegionsSharedState::FTimingRegionsSharedState(STimingView* InTimingView)
	: TimingView(InTimingView)
{
	check(TimingView != nullptr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingRegionsSharedState::OnBeginSession(Timing::ITimingViewSession& InSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	AllRegionsTrack.Reset();
	TimingRegionTracksPerCategory.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingRegionsSharedState::OnEndSession(Timing::ITimingViewSession& InSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	AllRegionsTrack.Reset();
	TimingRegionTracksPerCategory.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingRegionsSharedState::CreateRequiredTracks()
{
	// delete all tracks
	if (!bShowHideRegionsTrack)
	{
		if (AllRegionsTrack)
		{
			TimingView->RemoveScrollableTrack(AllRegionsTrack);
			AllRegionsTrack.Reset();
		}
		
		for (auto& KV : TimingRegionTracksPerCategory)
		{
			TimingView->RemoveScrollableTrack(KV.Value);
		}
		TimingRegionTracksPerCategory.Reset();
		return;
	}
	
	if (bCreateRegionTracksByCategory)
	{   // delete combined track
		if (AllRegionsTrack)
		{
			TimingView->RemoveScrollableTrack(AllRegionsTrack);
			AllRegionsTrack.Reset();
		}
	}
	else
	{   // create combined track
		if (!AllRegionsTrack)
		{
			AllRegionsTrack = MakeShared<FTimingRegionsTrack>(*this);
			AllRegionsTrack->SetOrder(FTimingTrackOrder::First + 100);
			AllRegionsTrack->SetVisibilityFlag(bShowHideRegionsTrack);
			TimingView->AddScrollableTrack(AllRegionsTrack);
		}
	}

	if (!bCreateRegionTracksByCategory)
	{   // delete per category tracks
		for (auto& KV : TimingRegionTracksPerCategory)
		{
			TimingView->RemoveScrollableTrack(KV.Value);
		}
		TimingRegionTracksPerCategory.Reset();
	}
	else
	{   // create per category tracks
		// known categories might change during analysis so always create on demand
		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		const TraceServices::IRegionProvider& RegionProvider = TraceServices::ReadRegionProvider(*Session);
		TraceServices::FProviderReadScopeLock RegionProviderScopedLock(RegionProvider);

		RegionProvider.EnumerateTimelinesByCategory([this, &RegionProvider](const TraceServices::IRegionTimeline& Timeline, const TCHAR* Category)
		{			
			if (!TimingRegionTracksPerCategory.Contains(Category))
			{
				auto NewTrack = MakeShared<FTimingRegionsTrack>(*this);
				NewTrack->SetRegionsCategory(Category);
				NewTrack->SetOrder( Category == RegionProvider.GetUncategorizedRegionCategoryName() ? FTimingTrackOrder::First + 100 : FTimingTrackOrder::First + 101);
				NewTrack->SetVisibilityFlag(true);
				TimingView->AddScrollableTrack(NewTrack);
				TimingRegionTracksPerCategory.Add(Category, NewTrack);
			}
		});
	}
}

void FTimingRegionsSharedState::Tick(Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	if (&InSession != TimingView)
	{
		return;
	}

	CreateRequiredTracks();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingRegionsSharedState::ShowHideRegionsTrack()
{
	bShowHideRegionsTrack = !bShowHideRegionsTrack;

	CreateRequiredTracks();
	
	if (TimingView)
	{
		TimingView->HandleTrackVisibilityChanged();
	}
}

void FTimingRegionsSharedState::ToggleColorRegionsByCategory()
{
	bColorRegionsByCategory = !bColorRegionsByCategory;
	// redraw whatever tracks exist right now
	if (AllRegionsTrack.IsValid())
	{
		AllRegionsTrack->SetDirtyFlag();
	}
	for (auto& KV : TimingRegionTracksPerCategory)
	{
		KV.Value->SetDirtyFlag();
	}
}

void FTimingRegionsSharedState::ToggleShouldCreateRegionTracksByCategory()
{
	bCreateRegionTracksByCategory = !bCreateRegionTracksByCategory;

	CreateRequiredTracks();
	
	if (TimingView)
	{
		TimingView->HandleTrackVisibilityChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingRegionsSharedState::ExtendOtherTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InOutMenuBuilder)
{
	if (&InSession != TimingView)
	{
		return;
	}

	InOutMenuBuilder.BeginSection("Timing Regions", LOCTEXT("ContextMenu_Section_Regions", "Timing Regions"));
	{
		InOutMenuBuilder.AddMenuEntry(FTimingRegionsViewCommands::Get().ShowHideTimingRegionsTrack);
		InOutMenuBuilder.AddMenuEntry(FTimingRegionsViewCommands::Get().ColorTimingRegionsTrackByCategory);
		InOutMenuBuilder.AddMenuEntry(FTimingRegionsViewCommands::Get().CreateRegionTracksByCategory);
	}
	InOutMenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingRegionsSharedState::BindCommands()
{
	FTimingRegionsViewCommands::Register();

	TSharedPtr<FUICommandList> CommandList = TimingView->GetCommandList();
	ensure(CommandList.IsValid());

	CommandList->MapAction(
		FTimingRegionsViewCommands::Get().ShowHideTimingRegionsTrack,
		FExecuteAction::CreateSP(this, &FTimingRegionsSharedState::ShowHideRegionsTrack),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTimingRegionsSharedState::IsRegionsTrackVisible));

	CommandList->MapAction(
		FTimingRegionsViewCommands::Get().ColorTimingRegionsTrackByCategory,
		FExecuteAction::CreateSP(this, &FTimingRegionsSharedState::ToggleColorRegionsByCategory),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTimingRegionsSharedState::ShouldColorRegionsByCategory));
	
	CommandList->MapAction(
		FTimingRegionsViewCommands::Get().CreateRegionTracksByCategory,
		FExecuteAction::CreateSP(this, &FTimingRegionsSharedState::ToggleShouldCreateRegionTracksByCategory),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTimingRegionsSharedState::ShouldCreateRegionTracksByCategory));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
