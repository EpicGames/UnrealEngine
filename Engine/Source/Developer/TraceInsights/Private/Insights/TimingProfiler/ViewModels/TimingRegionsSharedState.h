// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Templates/SharedPointer.h"

// TraceInsights
#include "Insights/ITimingViewExtender.h"

namespace TraceServices
{
	class IAnalysisSession;
}

namespace UE::Insights::TimingProfiler
{

class FTimingRegionsTrack;
class STimingView;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingRegionsSharedState : public Timing::ITimingViewExtender, public TSharedFromThis<FTimingRegionsSharedState>
{
	friend class FTimingRegionsTrack;

public:
	explicit FTimingRegionsSharedState(STimingView* InTimingView);
	virtual ~FTimingRegionsSharedState() override = default;

	//////////////////////////////////////////////////
	// ITimingViewExtender interface

	virtual void OnBeginSession(Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(Timing::ITimingViewSession& InSession) override;
	virtual void Tick(Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;
	virtual void ExtendOtherTracksFilterMenu(Timing::ITimingViewSession& InSession, FMenuBuilder& InOutMenuBuilder) override;

	//////////////////////////////////////////////////

	void BindCommands();

	bool IsRegionsTrackVisible() const { return bShowHideRegionsTrack; }
	void ShowHideRegionsTrack();
	bool ShouldColorRegionsByCategory() const { return bColorRegionsByCategory; }
	void ToggleColorRegionsByCategory();
	bool ShouldCreateRegionTracksByCategory() const { return bCreateRegionTracksByCategory; }
	void ToggleShouldCreateRegionTracksByCategory();

private:
	/// creates/destroys AllRegions/per-category tracks to make sure they match with bCreateRegionTracksByCategory
	void CreateRequiredTracks();

	STimingView* TimingView = nullptr;

	// unfiltered view
	TSharedPtr<FTimingRegionsTrack> AllRegionsTrack;
	// filtered views/tracks per category
	TMap<const TCHAR*, TSharedPtr<FTimingRegionsTrack>, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<const TCHAR*, TSharedPtr<FTimingRegionsTrack>>> TimingRegionTracksPerCategory;

	bool bShowHideRegionsTrack = true;
	bool bColorRegionsByCategory = false;
	bool bCreateRegionTracksByCategory = true;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
