// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFrameTrack.h"

#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformTime.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "Rendering/DrawElements.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SScrollBar.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/Frames.h"
#include "TraceServices/Model/StackSamples.h"
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/Log.h"
#include "Insights/TimingProfiler/Models/UserAnnotation.h"
#include "Insights/TimingProfiler/Models/UserAnnotationStore.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/UserAnnotationsTimingViewExtender.h"
#include "Insights/TraceInsightsModule.h"
#include "Insights/TimingProfiler/Tracks/ThreadTimingTrack.h"
#include "Insights/TimingProfiler/ViewModels/FrameStatsHelper.h"
#include "Insights/TimingProfiler/ViewModels/FrameTrackHelper.h"
#include "Insights/TimingProfiler/ViewModels/ThreadTimingSharedState.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/TimingProfilerCommon.h"
#include "Insights/Widgets/SLogView.h"
#include "Insights/Widgets/STimingView.h"

#include <limits>

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::SFrameTrack"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

SFrameTrack::SFrameTrack()
{
	Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SFrameTrack::~SFrameTrack()
{
	UnregisterTimingViewCallbacks();

	if (OnAnnotationStoreChangedHandle.IsValid())
	{
		if (FTraceInsightsModule* Module = static_cast<FTraceInsightsModule*>(
				FModuleManager::GetModulePtr<IUnrealInsightsModule>("TraceInsights")))
		{
			Module->GetUserAnnotationsExtender().GetOnAnnotationStoreChanged().Remove(OnAnnotationStoreChangedHandle);
		}
	}

	TSharedPtr<FTimingProfilerManager> ProfilerManager = FTimingProfilerManager::Get();
	TSharedPtr<STimingProfilerWindow> ProfilerWindow = ProfilerManager.IsValid() ? ProfilerManager->GetProfilerWindow() : nullptr;
	if (ProfilerWindow.IsValid())
	{
		TArray<uint32> TimerIds;
		for (TSharedPtr<FFrameTrackSeries>& Series : AllSeries)
		{
			if (Series->Is<FTimerFrameStatsTrackSeries>())
			{
				TimerIds.Add(Series->As<FTimerFrameStatsTrackSeries>().GetTimerId());
			}
		}
		AllSeries.Reset();
		for (uint32 TimerId : TimerIds)
		{
			ProfilerWindow->OnTimerAddedToGraphsChanged(TimerId);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::Reset()
{
	const FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();

	Viewport.Reset();
	FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.SetScaleLimits(0.0001f, 16.0f); // 10000 [sample/px] to 16 [px/sample]
	ViewportX.SetScale(16.0f);
	FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();
	ViewportY.SetScaleLimits(0.01, 1000000.0);
	ViewportY.SetScale(1500.0);
	bIsViewportDirty = true;

	bIsStateDirty = true;

	bIsAutoZoomEnabled = true;
	AutoZoomViewportPos = ViewportX.GetPos();
	AutoZoomViewportScale = ViewportX.GetScale();
	AutoZoomViewportSize = 0.0f;

	bZoomTimingViewOnFrameSelection = Settings.IsAutoZoomOnFrameSelectionEnabled();

	AnalysisSyncNextTimestamp = 0;

	bShowUpperThresholdLine = Settings.IsShowUpperThresholdLineEnabled();
	bShowLowerThresholdLine = Settings.IsShowLowerThresholdLineEnabled();

	UpperThresholdTime = FMath::Clamp(Settings.GetUpperThresholdTime(), MinThresholdTime, MaxThresholdTime);
	LowerThresholdTime = FMath::Clamp(Settings.GetLowerThresholdTime(), MinThresholdTime, MaxThresholdTime);

	bShowUpperThresholdAsFps = Settings.IsShowUpperThresholdAsFpsEnabled();
	bShowLowerThresholdAsFps = Settings.IsShowLowerThresholdAsFpsEnabled();

	MousePosition = FVector2D::ZeroVector;

	MousePositionOnButtonDown = FVector2D::ZeroVector;
	ViewportPosXOnButtonDown = 0.0f;

	MousePositionOnButtonUp = FVector2D::ZeroVector;

	bIsLMB_Pressed = false;
	bIsRMB_Pressed = false;

	bIsScrolling = false;

	bDrawVerticalAxisLabelsOnLeftSide = false;

	HoveredSample.Reset();
	SelectedSample.Reset();
	TooltipOpacity = 0.0f;
	TooltipSizeX = 70.0f;

	//ThisGeometry

	CursorType = ECursorType::Default;

	NumUpdatedFrames = 0;
	UpdateDurationHistory.Reset();
	DrawDurationHistory.Reset();
	OnPaintDurationHistory.Reset();
	LastOnPaintTime = FPlatformTime::Cycles64();

	AllSeries.Empty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SOverlay)
		.Visibility(EVisibility::SelfHitTestInvisible)

		+ SOverlay::Slot()
		.VAlign(VAlign_Top)
		.Padding(FMargin(0, 0, 0, 0))
		[
			SAssignNew(HorizontalScrollBar, SScrollBar)
			.Orientation(Orient_Horizontal)
			.AlwaysShowScrollbar(false)
			.Visibility(EVisibility::Visible)
			.Thickness(FVector2D(5.0f, 5.0f))
			.RenderOpacity(0.75)
			.OnUserScrolled(this, &SFrameTrack::HorizontalScrollBar_OnUserScrolled)
		]
	];

	UpdateHorizontalScrollBar();

	BindCommands();

	// Subscribe to the annotations extender's store-changed broadcast; one-shot acquire below.
	if (FTraceInsightsModule* Module = static_cast<FTraceInsightsModule*>(
			FModuleManager::GetModulePtr<IUnrealInsightsModule>("TraceInsights")))
	{
		FUserAnnotationsTimingViewExtender& Extender = Module->GetUserAnnotationsExtender();
		OnAnnotationStoreChangedHandle = Extender.GetOnAnnotationStoreChanged().AddSP(
			this, &SFrameTrack::RefreshAnnotationStore);
		RefreshAnnotationStore();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::RefreshAnnotationStore()
{
	FTraceInsightsModule* Module = static_cast<FTraceInsightsModule*>(
		FModuleManager::GetModulePtr<IUnrealInsightsModule>("TraceInsights"));
	if (!Module)
	{
		AnnotationStoreWeak.Reset();
		return;
	}
	// Assigns null when no current session — prevents stale sibling-window store during trace switch.
	AnnotationStoreWeak = Module->GetUserAnnotationsExtender().GetAnnotationStoreForCurrentSession();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::RegisterTimingViewCallbacks()
{
	TSharedPtr<STimingView> TimingView = TimingViewWeak.Pin();
	if (TimingView.IsValid())
	{
		auto OnTrackAddedRemovedLambda =
			[this](const TSharedPtr<const FBaseTimingTrack> Track)
			{
				if (Track->Is<FThreadTimingTrack>())
				{
					// If there are more series than the default frame series.
					if (AllSeries.Num() > ETraceFrameType::TraceFrameType_Count)
					{
						bIsStateDirty = true;
					}
				}
			};

		OnTrackAddedHandle = TimingView->OnTrackAdded().AddLambda(OnTrackAddedRemovedLambda);
		OnTrackRemovedHandle = TimingView->OnTrackRemoved().AddLambda(OnTrackAddedRemovedLambda);

		OnTrackVisibilityChangedHandle = TimingView->OnTrackVisibilityChanged().AddLambda(
			[this]()
			{
				// If there are more series than the default frame series.
				if (AllSeries.Num() > ETraceFrameType::TraceFrameType_Count)
				{
					bIsStateDirty = true;
				}
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::UnregisterTimingViewCallbacks()
{
	TSharedPtr<STimingView> TimingView = TimingViewWeak.Pin();
	if (TimingView.IsValid())
	{
		TimingView->OnTrackAdded().Remove(OnTrackAddedHandle);
		TimingView->OnTrackRemoved().Remove(OnTrackRemovedHandle);
		TimingView->OnTrackVisibilityChanged().Remove(OnTrackVisibilityChangedHandle);
	}

	OnTrackAddedHandle.Reset();
	OnTrackRemovedHandle.Reset();
	OnTrackVisibilityChangedHandle.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	TSharedPtr<STimingView> OldTimingView = TimingViewWeak.Pin();
	TSharedPtr<STimingView> NewTimingView = GetTimingView();
	if (OldTimingView.Get() != NewTimingView.Get())
	{
		UnregisterTimingViewCallbacks();
		TimingViewWeak = NewTimingView;
		RegisterTimingViewCallbacks();
		bIsStateDirty = true;
	}

	if (ThisGeometry != AllottedGeometry || bIsViewportDirty)
	{
		bIsViewportDirty = false;
		const float ViewWidth = static_cast<float>(AllottedGeometry.GetLocalSize().X);
		const float ViewHeight = static_cast<float>(AllottedGeometry.GetLocalSize().Y);
		Viewport.SetSize(ViewWidth, ViewHeight);
		bIsStateDirty = true;
	}

	ThisGeometry = AllottedGeometry;

	FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

	if (!bIsScrolling)
	{
		// Elastic snap to horizontal limits.
		if (ViewportX.UpdatePosWithinLimits())
		{
			bIsStateDirty = true;
		}
	}

	// Disable auto-zoom if viewport's position or scale has changed.
	if (AutoZoomViewportPos != ViewportX.GetPos() ||
		AutoZoomViewportScale != ViewportX.GetScale())
	{
		bIsAutoZoomEnabled = false;
	}

	// Update auto-zoom if viewport size has changed.
	bool bAutoZoom = bIsAutoZoomEnabled && AutoZoomViewportSize != ViewportX.GetSize();

	const uint64 Time = FPlatformTime::Cycles64();
	if (Time > AnalysisSyncNextTimestamp)
	{
		const uint64 WaitTime = static_cast<uint64>(0.1 / FPlatformTime::GetSecondsPerCycle64()); // 100ms
		AnalysisSyncNextTimestamp = Time + WaitTime;

		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(*Session.Get());

			for (int32 FrameType = 0; FrameType < TraceFrameType_Count; ++FrameType)
			{
				TSharedPtr<FFrameTrackSeries> SeriesPtr = FindOrAddSeries(static_cast<ETraceFrameType>(FrameType));

				int32 NumFrames = static_cast<int32>(FramesProvider.GetFrameCount(static_cast<ETraceFrameType>(FrameType)));
				if (NumFrames > ViewportX.GetMaxValue())
				{
					ViewportX.SetMinMaxInterval(0, NumFrames);
					UpdateHorizontalScrollBar();
					bIsStateDirty = true;

					if (bIsAutoZoomEnabled)
					{
						bAutoZoom = true;
					}
				}
			}
		}
	}

	if (bAutoZoom)
	{
		AutoZoom();
	}

	if (bIsStateDirty)
	{
		bIsStateDirty = false;
		UpdateState();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FFrameTrackSeries> SFrameTrack::FindOrAddSeries(ETraceFrameType FrameType)
{
	TSharedPtr<FFrameTrackSeries>* ExistingSeries = AllSeries.FindByPredicate(
		[FrameType]
		(TSharedPtr<FFrameTrackSeries> Series)
		{
			return !Series->Is<FTimerFrameStatsTrackSeries>() &&
				   Series->GetFrameType() == FrameType;
		});

	if (ExistingSeries != nullptr)
	{
		return ExistingSeries->ToSharedRef();
	}

	LLM_SCOPE_BYTAG(Insights);

	TSharedRef<FFrameTrackSeries> SeriesRef = MakeShared<FFrameTrackSeries>(FrameType);
	SeriesRef->SetColor(FFrameTrackDrawHelper::GetColorByFrameType(FrameType));
	SeriesRef->SetName(FFrameTrackDrawHelper::FrameTypeToText(FrameType));
	AllSeries.Add(SeriesRef);
	return SeriesRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FFrameTrackSeries> SFrameTrack::FindSeries(ETraceFrameType FrameType) const
{
	const TSharedPtr<FFrameTrackSeries>* ExistingSeries = AllSeries.FindByPredicate(
		[FrameType]
		(TSharedPtr<FFrameTrackSeries> Series)
		{
			return !Series->Is<FTimerFrameStatsTrackSeries>() &&
				   Series->GetFrameType() == FrameType;
		});

	if (ExistingSeries != nullptr)
	{
		return *ExistingSeries;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FFrameTrackSeries> SFrameTrack::FindFrameStatsSeries(ETraceFrameType FrameType, uint32 TimerId) const
{
	const TSharedPtr<FFrameTrackSeries>* ExistingSeries = AllSeries.FindByPredicate(
		[FrameType, TimerId]
		(TSharedPtr<FFrameTrackSeries> Series)
		{
			return Series->Is<FTimerFrameStatsTrackSeries>() &&
				   Series->As<FTimerFrameStatsTrackSeries>().GetFrameType() == FrameType &&
				   Series->As<FTimerFrameStatsTrackSeries>().GetTimerId() == TimerId;
		});

	if (ExistingSeries)
	{
		return *ExistingSeries;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::UpdateState()
{
	LLM_SCOPE_BYTAG(Insights);

	FStopwatch Stopwatch;
	Stopwatch.Start();

	// Reset stats.
	for (TSharedPtr<FFrameTrackSeries> Series : AllSeries)
	{
		Series->SetNumAggregatedFrames(0);
	}
	NumUpdatedFrames = 0;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(*Session.Get());

		const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

		const uint64 StartIndex = static_cast<uint64>(FMath::Max(0, ViewportX.GetValueAtOffset(0.0f)));
		const uint64 EndIndex = static_cast<uint64>(ViewportX.GetValueAtOffset(ViewportX.GetSize()));

		// Frame Series
		for (int32 FrameType = 0; FrameType < TraceFrameType_Count; ++FrameType)
		{
			TSharedPtr<FFrameTrackSeries> SeriesPtr = FindOrAddSeries(static_cast<ETraceFrameType>(FrameType));

			FFrameTrackSeriesBuilder Builder(*SeriesPtr, Viewport);

			FramesProvider.EnumerateFrames(static_cast<ETraceFrameType>(FrameType), StartIndex, EndIndex,
				[&Builder]
				(const TraceServices::FFrame& Frame)
				{
					Builder.AddFrame(Frame);
				});

			NumUpdatedFrames += Builder.GetNumAddedFrames();
		}

		// Timer Series
		for (int32 Index = 0; Index < AllSeries.Num(); ++Index)
		{
			TSharedPtr<FFrameTrackSeries> Series = AllSeries[Index];
			if (!Series->Is<FTimerFrameStatsTrackSeries>())
			{
				continue;
			}
			const FTimerFrameStatsTrackSeries& TimerSeries = Series->As<FTimerFrameStatsTrackSeries>();
			const uint32 TimerId = TimerSeries.GetTimerId();

			FFrameTrackSeriesBuilder Builder(*Series, Viewport);

			TArray<FFrameStatsCachedEvent> Frames;
			FramesProvider.EnumerateFrames(static_cast<ETraceFrameType>(Series->GetFrameType()), StartIndex, EndIndex,
				[&Frames]
				(const TraceServices::FFrame& Frame)
				{
					FFrameStatsCachedEvent Event;
					Event.FrameStartTime = Frame.StartTime;
					Event.FrameEndTime = Frame.EndTime;
					Event.Duration.store(0.0f);
					Frames.Add(Event);
				});

			if (Frames.Num() > 0)
			{
				TSharedPtr<STimingView> TimingView = GetTimingView();
				if (TimingView.IsValid())
				{
					// Attempt to compute only from visible timelines.
					TSet<uint32> VisibleTimelines;
					TSet<uint32> VisibleCpuSamplingThreads;
					TSharedPtr<FThreadTimingSharedState> ThreadSharedState = TimingView->GetThreadTimingSharedState();
					if (ThreadSharedState.IsValid())
					{
						ThreadSharedState->GetVisibleTimelineIndexes(VisibleTimelines);
						ThreadSharedState->GetVisibleCpuSamplingThreads(VisibleCpuSamplingThreads);
					}

					// CPU & GPU timelines
					if (VisibleTimelines.Num() > 0)
					{
						FFrameStatsHelper::ComputeFrameStatsForTimer(Frames, TimerId, VisibleTimelines);
					}

					// CPU Sampling timelines
					if (VisibleCpuSamplingThreads.Num() > 0)
					{
						const TraceServices::IStackSamplesProvider* StackSamplesProvider = TraceServices::ReadStackSamplesProvider(*Session.Get());
						if (StackSamplesProvider)
						{
							for (uint32 ThreadId : VisibleCpuSamplingThreads)
							{
								TraceServices::FProviderReadScopeLock _(*StackSamplesProvider);
								const TraceServices::ITimingProfilerProvider::Timeline& Timeline = *StackSamplesProvider->GetTimeline(ThreadId);
								FFrameStatsHelper::ComputeFrameStatsForTimer(Frames, TimerId, Timeline);
							}
						}
					}
				}
				else
				{
					// Compute the stats for all timelines.
					FFrameStatsHelper::ComputeFrameStatsForTimer(Frames, TimerId);
				}

				uint64 CurrentIndex = StartIndex;
				for (FFrameStatsCachedEvent& Event : Frames)
				{
					TraceServices::FFrame NewFrame;
					NewFrame.StartTime = Event.FrameStartTime;
					NewFrame.EndTime = Event.FrameStartTime + Event.Duration.load();
					NewFrame.Index = CurrentIndex++;
					Builder.AddFrame(NewFrame);
				}
			}

			NumUpdatedFrames += Builder.GetNumAddedFrames();
		}
	}

	Stopwatch.Stop();
	UpdateDurationHistory.AddValue(Stopwatch.AccumulatedTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FFrameTrackSampleRef SFrameTrack::GetSampleAtMousePosition(double X, double Y)
{
	if (!bIsStateDirty)
	{
		float SampleW = Viewport.GetSampleWidth();
		int32 SampleIndex = FMath::FloorToInt(static_cast<float>(X) / SampleW);
		if (SampleIndex >= 0)
		{
			const float MY = static_cast<float>(Y);

			// Search in reverse paint order.
			for (int32 SeriesIndex = AllSeries.Num() - 1; SeriesIndex >= 0; --SeriesIndex)
			{
				TSharedPtr<FFrameTrackSeries> SeriesPtr = AllSeries[SeriesIndex];

				if (!SeriesPtr->IsVisible())
				{
					continue;
				}

				if (SeriesPtr.IsValid())
				{
					if (SeriesPtr->GetNumAggregatedFrames() > 0 &&
						SampleIndex < SeriesPtr->GetNumSamples())
					{
						const FFrameTrackSample& Sample = SeriesPtr->GetSample(SampleIndex);
						if (Sample.NumFrames > 0)
						{
							const FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();

							const float ViewHeight = FMath::RoundToFloat(Viewport.GetHeight());
							const float BaselineY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(0.0));

							float ValueY;
							if (Sample.LargestFrameDuration == std::numeric_limits<double>::infinity())
							{
								ValueY = ViewHeight;
							}
							else
							{
								ValueY = FMath::RoundToFloat(ViewportY.GetOffsetForValue(Sample.LargestFrameDuration));
							}

							constexpr float ToleranceY = 3.0f; // [pixels]

							const float BottomY = FMath::Min(ViewHeight, ViewHeight - BaselineY + ToleranceY);
							const float TopY = FMath::Max(0.0f, ViewHeight - ValueY - ToleranceY);

							if (MY >= TopY && MY < BottomY)
							{
								LLM_SCOPE_BYTAG(Insights);
								return FFrameTrackSampleRef(SeriesPtr, MakeShared<FFrameTrackSample>(Sample));
							}
						}
					}
				}
			}
		}
	}
	return FFrameTrackSampleRef(nullptr, nullptr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::SelectFrameAtMousePosition(double X, double Y, bool JoinCurrentSelection)
{
	FFrameTrackSampleRef SampleRef = GetSampleAtMousePosition(X, Y);
	if (!SampleRef.IsValid())
	{
		SampleRef = GetSampleAtMousePosition(X - 1.0, Y);
	}
	if (!SampleRef.IsValid())
	{
		SampleRef = GetSampleAtMousePosition(X + 1.0, Y);
	}

	if (SampleRef.IsValid())
	{
		TSharedPtr<STimingView> TimingView = GetTimingView();
		if (TimingView.IsValid())
		{
			double StartTime = SampleRef.Sample->LargestFrameStartTime;
			double Duration = SampleRef.Sample->LargestFrameDuration;

			if (JoinCurrentSelection)
			{
				double EndTime = StartTime + Duration;
				StartTime = FMath::Min(StartTime, TimingView->GetSelectionStartTime());
				EndTime = FMath::Max(EndTime, TimingView->GetSelectionEndTime());
				Duration = EndTime - StartTime;
			}

			TimingView->SetAutoScroll(false);

			if (bZoomTimingViewOnFrameSelection)
			{
				const double EndTime = FMath::Min(StartTime + Duration, TimingView->GetViewport().GetMaxValidTime());
				const double AdjustedDuration = EndTime - StartTime;
				TimingView->ZoomOnTimeInterval(StartTime - AdjustedDuration * 0.1, AdjustedDuration * 1.2);
			}
			else
			{
				TimingView->CenterOnTimeInterval(StartTime, Duration);
			}

			TimingView->SelectTimeInterval(StartTime, Duration);
			FSlateApplication::Get().SetKeyboardFocus(TimingView);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 SFrameTrack::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const bool bEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	FDrawContext DrawContext(AllottedGeometry, MyCullingRect, InWidgetStyle, DrawEffects, OutDrawElements, LayerId);

	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	FSlateFontInfo SummaryFont = FAppStyle::Get().GetFontStyle("SmallFont");

	const FSlateBrush* WhiteBrush = FAppStyle::Get().GetBrush("WhiteBrush");

	const float ViewWidth = static_cast<float>(AllottedGeometry.Size.X);
	const float ViewHeight = static_cast<float>(AllottedGeometry.Size.Y);

	int32 NumDrawSamples = 0;

	//////////////////////////////////////////////////
	{
		FStopwatch Stopwatch;
		Stopwatch.Start();

		FFrameTrackDrawHelper Helper(DrawContext, Viewport);

		Helper.SetThresholds(UpperThresholdTime, LowerThresholdTime);

		Helper.DrawBackground();

		// Draw the horizontal axis grid (background layer).
		DrawHorizontalAxisGrid(DrawContext, WhiteBrush, SummaryFont, true);

		// Draw frames, for each visible Series.
		for (TSharedPtr<FFrameTrackSeries> Series : AllSeries)
		{
			if (!Series->IsVisible())
			{
				continue;
			}

			if (Series.IsValid())
			{
				Helper.DrawCached(*Series);
			}
		}

		NumDrawSamples = Helper.GetNumDrawSamples();

		TSharedPtr<FFrameTrackSeries> GameFrameSeries = FindSeries(TraceFrameType_Game);
		if (GameFrameSeries.IsValid())
		{
			TSharedPtr<STimingView> TimingView = GetTimingView();
			if (TimingView.IsValid())
			{
				// Highlight the area corresponding to viewport of Timing View.
				const double StartTime = TimingView->GetViewport().GetStartTime();
				const double EndTime = TimingView->GetViewport().GetEndTime();
				Helper.DrawHighlightedInterval(*GameFrameSeries, StartTime, EndTime);
			}
		}

		// Draw annotation indicators. Suppress when annotations are globally toggled off.
		{
			TSharedPtr<FUserAnnotationStore> AnnotationStore = AnnotationStoreWeak.Pin();
			if (AnnotationStore.IsValid() && AnnotationStore->AreAnnotationsVisible())
			{
				const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
				constexpr float HeadW = 6.0f;  // Width of the indicator head
				constexpr float HeadH = 10.0f; // Height of the indicator head
				constexpr float StemW = 1.0f;   // Width of the vertical stem line

				for (const FUserAnnotation& Annotation : AnnotationStore->GetAllAnnotations())
				{
					if (!Annotation.bVisible || Annotation.GameFrameNumber == 0)
					{
						continue;
					}

					const float X = ViewportX.GetOffsetForValue(static_cast<int32>(Annotation.GameFrameNumber));
					if (X >= -HeadW && X <= ViewWidth + HeadW)
					{
						const FLinearColor HeadColor(Annotation.Color.R, Annotation.Color.G, Annotation.Color.B, 0.9f);
						const FLinearColor StemColor(Annotation.Color.R, Annotation.Color.G, Annotation.Color.B, 0.5f);
						// Draw head (solid rectangle at the top)
						DrawContext.DrawBox(X - HeadW * 0.5f, 0.0f, HeadW, HeadH, WhiteBrush, HeadColor);
						// Draw stem (vertical line spanning the full frame graph height)
						DrawContext.DrawBox(X - StemW * 0.5f, HeadH, StemW, ViewHeight - HeadH, WhiteBrush, StemColor);
					}
				}
				DrawContext.LayerId++;
			}
		}

		// Draw the horizontal axis grid (foreground layer).
		DrawHorizontalAxisGrid(DrawContext, WhiteBrush, SummaryFont, false);

		if (bShowLowerThresholdLine)
		{
			const FLinearColor LineColor(0.1f, 0.7f, 0.1f, 1.0f);
			const FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();
			const float RoundedViewHeight = FMath::RoundToFloat(ViewportY.GetSize());
			const float LineY = RoundedViewHeight - FMath::RoundToFloat(ViewportY.GetOffsetForValue(LowerThresholdTime));
			DrawContext.DrawBox(0.0, LineY, ViewWidth, 1.0, WhiteBrush, LineColor);
			DrawContext.LayerId++;
		}
		if (bShowUpperThresholdLine)
		{
			const FLinearColor LineColor(1.0f, 0.1f, 0.1f, 1.0f);
			const FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();
			const float RoundedViewHeight = FMath::RoundToFloat(ViewportY.GetSize());
			const float LineY = RoundedViewHeight - FMath::RoundToFloat(ViewportY.GetOffsetForValue(UpperThresholdTime));
			DrawContext.DrawBox(0.0, LineY, ViewWidth, 1.0, WhiteBrush, LineColor);
			DrawContext.LayerId++;
		}

		// Draw the vertical axis grid.
		DrawVerticalAxisGrid(DrawContext, WhiteBrush, SummaryFont);

		// Highlight the mouse hovered sample (frame).
		if (HoveredSample.IsValid())
		{
			Helper.DrawHoveredSample(*HoveredSample.Sample);
		}

		// Draw tooltip for hovered sample (frame).
		if (HoveredSample.IsValid())
		{
			constexpr float TooltipDesiredOpacity = 1.0f;
			if (TooltipOpacity < TooltipDesiredOpacity)
			{
				// slow fade in
				TooltipOpacity = TooltipOpacity * 0.9f + TooltipDesiredOpacity * 0.1f;
			}
			else
			{
				// fast fade out
				TooltipOpacity = TooltipOpacity * 0.75f + TooltipDesiredOpacity * 0.25f;
			}

			// First line: "Rendering Frame 1,234"
			TStringBuilder<512> StringBuilder;

			StringBuilder.Append(HoveredSample.Series->GetName().ToString());
			StringBuilder.Append(TEXT(" "));
			StringBuilder.Append(FText::AsNumber(HoveredSample.Sample->LargestFrameIndex).ToString());
			const FString Text1(StringBuilder);

			// Second line: "1m 2.34s + 16.67ms (60 fps)"
			StringBuilder.Reset();
			StringBuilder.Append(FormatTimeAuto(HoveredSample.Sample->LargestFrameStartTime, HoveredSample.Sample->LargestFrameStartTime > 60.0 ? 3 : 2));
			StringBuilder.Append(TEXT(" + "));
			StringBuilder.Append(FormatTimeAuto(HoveredSample.Sample->LargestFrameDuration, 2));
			StringBuilder.Appendf(TEXT(" (%.1f fps)"), 1.0 / HoveredSample.Sample->LargestFrameDuration);
			const FString Text2(StringBuilder);

			const float FontScale = DrawContext.Geometry.Scale;
			const FVector2f TextSize1(FontMeasureService->Measure(Text1, SummaryFont, FontScale) / FontScale);
			const FVector2f TextSize2(FontMeasureService->Measure(Text2, SummaryFont, FontScale) / FontScale);

			const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

			const float FrameX = ViewportX.GetOffsetForValue(HoveredSample.Sample->LargestFrameIndex);
			const float CX0 = FMath::RoundToFloat(FrameX + Viewport.GetSampleWidth() / 2.0f);

			constexpr float DX = 3.0f;
			const float DX1 = FMath::RoundToFloat(TextSize1.X / 2.0f);
			const float DX2 = FMath::RoundToFloat(TextSize2.X / 2.0f);
			const float TooltipDesiredSizeX = FMath::Max(DX1, DX2) + DX;

			if (TooltipSizeX != TooltipDesiredSizeX)
			{
				TooltipSizeX = TooltipSizeX * 0.75f + TooltipDesiredSizeX * 0.25f;

				if (FMath::IsNearlyEqual(TooltipSizeX, TooltipDesiredSizeX))
				{
					TooltipSizeX = TooltipDesiredSizeX;
				}
			}

			float CX = CX0;
			if (CX > ViewportX.GetSize() - TooltipSizeX)
			{
				CX = FMath::RoundToFloat(ViewportX.GetSize() - TooltipSizeX);
			}
			if (CX - TooltipSizeX < 0)
			{
				CX = TooltipSizeX;
			}

			constexpr float BoxY = 11.0f;
			constexpr float BoxH = 26.0f;
			constexpr float LineDY = 12.0f;

			const FLinearColor BackgroundColor(0.9f, 0.9f, 0.9f, TooltipOpacity);
			DrawContext.DrawBox(CX - TooltipSizeX, BoxY, 2 * TooltipSizeX, BoxH, WhiteBrush, BackgroundColor);
			const int32 ArrowSize = 4;
			for (int32 ArrowY = 0; ArrowY < ArrowSize; ++ArrowY)
			{
				const int32 LineWidth = ArrowSize - ArrowY;
				DrawContext.DrawBox(CX0 - float(LineWidth), BoxY + BoxH + float(ArrowY), float(2 * LineWidth - 1), 1.0f, WhiteBrush, BackgroundColor);
			}
			DrawContext.LayerId++;

			const FLinearColor TextColor1 =
				HoveredSample.Series->GetFrameType() == TraceFrameType_Rendering ?
					FLinearColor(0.5f, 0.1f, 0.1f, TooltipOpacity) :
				HoveredSample.Series->GetFrameType() == TraceFrameType_Game ?
					FLinearColor(0.1f, 0.1f, 0.5f, TooltipOpacity) :
					FLinearColor(0.1f, 0.1f, 0.1f, TooltipOpacity);
			const FLinearColor TextColor2(0.05f, 0.05f, 0.05f, TooltipOpacity);
			DrawContext.DrawText(CX - DX1, BoxY          + 1.0f, Text1, SummaryFont, TextColor1);
			DrawContext.DrawText(CX - DX2, BoxY + LineDY + 1.0f, Text2, SummaryFont, TextColor2);
			DrawContext.LayerId++;
		}
		else
		{
			TooltipOpacity = 0.0f;
		}

		Stopwatch.Stop();
		DrawDurationHistory.AddValue(Stopwatch.AccumulatedTime);
	}
	//////////////////////////////////////////////////

	const bool bShouldDisplayDebugInfo = FInsightsManager::Get()->IsDebugInfoEnabled();
	if (bShouldDisplayDebugInfo)
	{
		const float FontScale = DrawContext.Geometry.Scale;
		const float MaxFontCharHeight = static_cast<float>(FontMeasureService->Measure(TEXT("!"), SummaryFont, FontScale).Y / FontScale);
		const float DbgDY = MaxFontCharHeight;

		const float DbgW = 280.0f;
		const float DbgH = DbgDY * 4 + 3.0f;
		const float DbgX = ViewWidth - DbgW - 20.0f;
		float DbgY = 7.0f;

		const FLinearColor DbgBackgroundColor(1.0f, 1.0f, 1.0f, 0.9f);
		const FLinearColor DbgTextColor(0.0f, 0.0f, 0.0f, 0.9f);

		DrawContext.LayerId++;
		DrawContext.DrawBox(DbgX - 2.0f, DbgY - 2.0f, DbgW, DbgH, WhiteBrush, DbgBackgroundColor);
		DrawContext.LayerId++;

		// Time interval since last OnPaint call.
		const uint64 CurrentTime = FPlatformTime::Cycles64();
		const uint64 OnPaintDuration = CurrentTime - LastOnPaintTime;
		LastOnPaintTime = CurrentTime;
		OnPaintDurationHistory.AddValue(OnPaintDuration); // saved for last 32 OnPaint calls
		const uint64 AvgOnPaintDuration = OnPaintDurationHistory.ComputeAverage();
		const uint64 AvgOnPaintDurationMs = FStopwatch::Cycles64ToMilliseconds(AvgOnPaintDuration);
		const double AvgOnPaintFps = AvgOnPaintDurationMs != 0 ? 1.0 / FStopwatch::Cycles64ToSeconds(AvgOnPaintDuration) : 0.0;

		const uint64 AvgUpdateDurationMs = FStopwatch::Cycles64ToMilliseconds(UpdateDurationHistory.ComputeAverage());
		const uint64 AvgDrawDurationMs = FStopwatch::Cycles64ToMilliseconds(DrawDurationHistory.ComputeAverage());

		// Draw performance info.
		DrawContext.DrawText
		(
			DbgX, DbgY,
			FString::Printf(TEXT("U: %" UINT64_FMT " ms, D: %" UINT64_FMT " ms + %" UINT64_FMT " ms = %" UINT64_FMT " ms (%" INT64_FMT " fps)"),
				AvgUpdateDurationMs, // caching time
				AvgDrawDurationMs, // drawing time
				AvgOnPaintDurationMs - AvgDrawDurationMs, // other overhead to OnPaint calls
				AvgOnPaintDurationMs, // average time between two OnPaint calls
				FMath::RoundToInt(AvgOnPaintFps)), // framerate of OnPaint calls
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		// Draw number of draw calls.
		DrawContext.DrawText
		(
			DbgX, DbgY,
			FString::Printf(TEXT("U: %s frames, D: %s samples"),
				*FText::AsNumber(NumUpdatedFrames).ToString(),
				*FText::AsNumber(NumDrawSamples).ToString()),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		// Draw viewport's horizontal info.
		DrawContext.DrawText
		(
			DbgX, DbgY,
			Viewport.GetHorizontalAxisViewport().ToDebugString(TEXT("X"), TEXT("frame")),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		// Draw viewport's vertical info.
		DrawContext.DrawText
		(
			DbgX, DbgY,
			Viewport.GetVerticalAxisViewport().ToDebugString(TEXT("Y")),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;
	}

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled && IsEnabled());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::DrawVerticalAxisGrid(FDrawContext& DrawContext, const FSlateBrush* Brush, const FSlateFontInfo& Font) const
{
	const float ViewWidth = Viewport.GetWidth();

	const FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();
	const float RoundedViewHeight = FMath::RoundToFloat(ViewportY.GetSize());

	const FLinearColor GridColor(0.0f, 0.0f, 0.0f, 0.1f);
	const FLinearColor TextBgColor(0.05f, 0.05f, 0.05f, 1.0f);

	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const float FontScale = DrawContext.Geometry.Scale;

	// Available axis, pre-ordered by value.
	struct FAxis
	{
		int32 Priority; // lower value means higher priority
		double Value; // time value
	};
	const FAxis AvailableAxis[] =
	{
		{ 0, 0.0 },
		{ 3, 0.001 },       //    1 ms (1000 fps)
		{ 3, 0.002 },       //    2 ms (500 fps)
		{ 3, 0.003 },       //    3 ms (333 fps)
		{ 3, 0.004 },       //    4 ms (250 fps)
		{ 2, 0.005 },       //    5 ms (200 fps)
		{ 2, 1.0 / 150.0 }, //  6.6 ms (150 fps)
		{ 1, 1.0 / 120.0 }, //  8.3 ms (120 fps)
		{ 2, 1.0 / 100.0 }, //   10 ms (100 fps)
		{ 3, 1.0 / 90.0 },  // 11.1 ms (90 fps)
		{ 4, 1.0 / 80.0 },  // 12.5 ms (80 fps)
		{ 4, 1.0 / 70.0 },  // 14.3 ms (70 fps)
		{ 1, 1.0 / 60.0 },  // 16.7 ms (60 fps)
		{ 2, 1.0 / 50.0 },  //   20 ms (50 fps)
		{ 3, 1.0 / 40.0 },  //   25 ms (40 fps)
		{ 1, 1.0 / 30.0 },  // 33.3 ms (30 fps)
		{ 2, 1.0 / 20.0 },  //   50 ms (20 fps)
		{ 3, 1.0 / 15.0 },  // 66.7 ms (15 fps)
		{ 2, 1.0 / 10.0 },  //  100 ms (10 fps)
		{ 3, 1.0 / 5.0 },   //  200 ms (5 fps)
		{ 3, 1.0 },         // 1s
		{ 3, 10.0 },        // 10s
		{ 3, 60.0 },        // 1m
		{ 3, 600.0 },       // 10m
		{ 3, 3600.0 },      // 1h
	};
	constexpr int32 NumAvailableAxis = UE_ARRAY_COUNT(AvailableAxis);

	struct FVisibleAxis
	{
		double Value;
		float Y;
		float LabelY;
	};
	FVisibleAxis VisibleAxis[NumAvailableAxis];
	int32 NumVisibleAxis = 0;

	constexpr float TextH = 14.0f;
	constexpr float MinDY = 13.0f; // min vertical distance between horizontal grid lines

	int32 PreviousPriority = 0;
	float PreviousLabelY = -MinDY;
	for (int32 Index = 0; Index < NumAvailableAxis; ++Index)
	{
		const FAxis& Axis = AvailableAxis[Index];

		const float Y = RoundedViewHeight - FMath::RoundToFloat(ViewportY.GetOffsetForValue(Axis.Value));
		const float LabelY = FMath::Clamp(Y - TextH / 2, 0.0f, RoundedViewHeight - TextH);

		if (Y < 0)
		{
			break; // we are done; the rest of axis are offscreen
		}
		if (Y > RoundedViewHeight + TextH)
		{
			continue; // skip the current axis
		}

		// Does the label overlaps with the label of the previous axis?
		if (FMath::Abs(PreviousLabelY - LabelY) < MinDY)
		{
			if (Axis.Priority < PreviousPriority)
			{
				--NumVisibleAxis; // the current axis replaces the previous axis
			}
			else
			{
				continue; // skip the current axis
			}
		}

		PreviousPriority = Axis.Priority;
		PreviousLabelY = LabelY;

		FVisibleAxis& CurrentVisibleAxis = VisibleAxis[NumVisibleAxis++];
		CurrentVisibleAxis.Value = Axis.Value;
		CurrentVisibleAxis.Y = Y;
		CurrentVisibleAxis.LabelY = LabelY;
	}

	for (int32 Index = 0; Index < NumVisibleAxis; ++Index)
	{
		const FVisibleAxis& Axis = VisibleAxis[Index];

		FLinearColor TextColor;
		if (Axis.Value <= LowerThresholdTime)
		{
			TextColor = FLinearColor(0.5f, 1.0f, 0.5f, 1.0f);
		}
		else if (Axis.Value <= UpperThresholdTime)
		{
			TextColor = FLinearColor(1.0f, 1.0f, 0.5f, 1.0f);
		}
		else
		{
			TextColor = FLinearColor(1.0f, 0.5f, 0.5f, 1.0f);
		}

		// Draw horizontal grid line.
		DrawContext.DrawBox(0, Axis.Y, ViewWidth, 1.0f, Brush, GridColor);

		const FString LabelText = (Axis.Value == 0.0) ? TEXT("0") :
								  (Axis.Value <= 1.0) ? FString::Printf(TEXT("%s (%.0f fps)"), *FormatTimeAuto(Axis.Value), 1.0 / Axis.Value) :
														FormatTimeAuto(Axis.Value);

		const float LabelTextWidth = static_cast<float>(FontMeasureService->Measure(LabelText, Font, FontScale).X / FontScale);
		const float LabelX = bDrawVerticalAxisLabelsOnLeftSide ? 0.0f : ViewWidth - LabelTextWidth - 4.0f;

		// Draw background for value text.
		DrawContext.DrawBox(LabelX, Axis.LabelY, LabelTextWidth + 4.0f, TextH, Brush, TextBgColor);

		// Draw value text.
		DrawContext.DrawText(LabelX + 2.0f, Axis.LabelY + 1.0f, LabelText, Font, TextColor);
	}
	DrawContext.LayerId++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::DrawHorizontalAxisGrid(FDrawContext& DrawContext, const FSlateBrush* Brush, const FSlateFontInfo& Font, bool bDrawBackgroundLayer) const
{
	const FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

	const float RoundedViewWidth = FMath::RoundToFloat(ViewportX.GetSize());

	constexpr float MinDX = 125.0f; // min horizontal distance between vertical grid lines

	const int32 LeftIndex = ViewportX.GetValueAtOffset(0.0f);
	const int32 GridIndex = ViewportX.GetValueAtOffset(MinDX);
	const int32 RightIndex = ViewportX.GetValueAtOffset(RoundedViewWidth);
	const int32 Delta = GridIndex - LeftIndex;

	if (Delta > 0)
	{
		// Compute rounding based on magnitude of visible range of samples (Delta).
		int32 Power10 = 1;
		int32 Delta10 = Delta;
		while (Delta10 > 0)
		{
			Delta10 /= 10;
			Power10 *= 10;
		}
		if (Power10 >= 100)
		{
			Power10 /= 100;
		}
		else
		{
			Power10 = 1;
		}

		const int32 Grid = ((Delta + Power10 - 1) / Power10) * Power10; // next value divisible with a multiple of 10

		// Skip grid lines for negative indices.
		int32 StartIndex = ((LeftIndex + Grid - 1) / Grid) * Grid;
		while (StartIndex < 0)
		{
			StartIndex += Grid;
		}

		if (bDrawBackgroundLayer)
		{
			const float ViewHeight = Viewport.GetHeight();

			// Draw vertical grid lines.
			const FLinearColor GridColor(0.0f, 0.0f, 0.0f, 0.1f);
			for (int32 Index = StartIndex; Index < RightIndex; Index += Grid)
			{
				const float X = FMath::RoundToFloat(ViewportX.GetOffsetForValue(Index));
				DrawContext.DrawBox(X, 0.0f, 1.0f, ViewHeight, Brush, GridColor);
			}
			DrawContext.LayerId++;
		}
		else
		{
			const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			const float FontScale = DrawContext.Geometry.Scale;

			// Draw labels.
			const FLinearColor LabelBoxColor(0.05f, 0.05f, 0.05f, 1.0f);
			const FLinearColor LabelTextColor(1.0f, 1.0f, 1.0f, 0.7f);
			for (int32 Index = StartIndex; Index < RightIndex; Index += Grid)
			{
				const float X = FMath::RoundToFloat(ViewportX.GetOffsetForValue(Index));
				const FString LabelText = FText::AsNumber(Index).ToString();
				const float LabelTextWidth = static_cast<float>(FontMeasureService->Measure(LabelText, Font, FontScale).X / FontScale);
				DrawContext.DrawBox(X, 10.0f, LabelTextWidth + 4.0f, 12.0f, Brush, LabelBoxColor);
				DrawContext.DrawText(X + 2.0f, 10.0f, LabelText, Font, LabelTextColor);
			}
			DrawContext.LayerId++;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SFrameTrack::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	MousePositionOnButtonDown = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	ViewportPosXOnButtonDown = Viewport.GetHorizontalAxisViewport().GetPos();

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bIsLMB_Pressed = true;

		// Capture mouse.
		Reply = FReply::Handled().CaptureMouse(SharedThis(this));
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		bIsRMB_Pressed = true;

		// Capture mouse, so we can scroll outside this widget.
		Reply = FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SFrameTrack::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	MousePositionOnButtonUp = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	const bool bIsValidForMouseClick = MousePositionOnButtonUp.Equals(MousePositionOnButtonDown, MOUSE_SNAP_DISTANCE);

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (bIsLMB_Pressed)
		{
			if (bIsScrolling)
			{
				bIsScrolling = false;
				CursorType = ECursorType::Default;
			}
			else if (bIsValidForMouseClick)
			{
				const bool JoinCurrentSelection = MouseEvent.IsShiftDown();

				if (!JoinCurrentSelection)
				{
					SelectedSample = HoveredSample;
				}

				SelectFrameAtMousePosition(
					static_cast<float>(MousePositionOnButtonUp.X),
					static_cast<float>(MousePositionOnButtonUp.Y),
					JoinCurrentSelection);
			}

			bIsLMB_Pressed = false;

			// Release the mouse.
			Reply = FReply::Handled().ReleaseMouseCapture();
		}
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (bIsRMB_Pressed)
		{
			if (bIsScrolling)
			{
				bIsScrolling = false;
				CursorType = ECursorType::Default;
			}
			else if (bIsValidForMouseClick)
			{
				SelectedSample = HoveredSample;
				ShowContextMenu(MouseEvent);
			}

			bIsRMB_Pressed = false;

			// Release mouse as we no longer scroll.
			Reply = FReply::Handled().ReleaseMouseCapture();
		}
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SFrameTrack::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	if (!MouseEvent.GetCursorDelta().IsZero())
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) ||
			MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
		{
			if (HasMouseCapture())
			{
				if (!bIsScrolling)
				{
					bIsScrolling = true;
					CursorType = ECursorType::Hand;

					HoveredSample.Reset();
				}

				FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
				const float PosX = ViewportPosXOnButtonDown + static_cast<float>(MousePositionOnButtonDown.X - MousePosition.X);
				ViewportX.ScrollAtValue(ViewportX.GetValueAtPos(PosX)); // align viewport position with sample (frame index)
				UpdateHorizontalScrollBar();
				bIsStateDirty = true;
			}
		}
		else
		{
			if (!HoveredSample.IsValid())
			{
				TooltipOpacity = 0.0f;
			}
			HoveredSample = GetSampleAtMousePosition(MousePosition.X, MousePosition.Y);
			if (!HoveredSample.IsValid())
			{
				HoveredSample = GetSampleAtMousePosition(MousePosition.X - 1.0, MousePosition.Y);
			}
			if (!HoveredSample.IsValid())
			{
				HoveredSample = GetSampleAtMousePosition(MousePosition.X + 1.0, MousePosition.Y);
			}
			if (HoveredSample.IsValid())
			{
				constexpr float VerticalAxisLabelAreaWidth = 100.0f;
				FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
				if (MousePosition.X > ViewportX.GetSize() - VerticalAxisLabelAreaWidth)
				{
					bDrawVerticalAxisLabelsOnLeftSide = true;
				}
				else if (MousePosition.X < VerticalAxisLabelAreaWidth)
				{
					bDrawVerticalAxisLabelsOnLeftSide = false;
				}
			}
		}

		Reply = FReply::Handled();
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	if (!HasMouseCapture())
	{
		bIsLMB_Pressed = false;
		bIsRMB_Pressed = false;

		HoveredSample.Reset();

		CursorType = ECursorType::Default;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SFrameTrack::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	if (MouseEvent.GetModifierKeys().IsShiftDown())
	{
		FAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();

		// Zoom in/out vertically.
		const double Delta = MouseEvent.GetWheelDelta();
		constexpr double ZoomStep = 0.25; // as percent
		double ScaleY;

		if (Delta > 0)
		{
			ScaleY = ViewportY.GetScale() * FMath::Pow(1.0 + ZoomStep, Delta);
		}
		else
		{
			ScaleY = ViewportY.GetScale() * FMath::Pow(1.0 / (1.0 + ZoomStep), -Delta);
		}

		ViewportY.SetScale(ScaleY);
		//UpdateVerticalScrollBar();
	}
	else //if (MouseEvent.GetModifierKeys().IsControlDown())
	{
		// Zoom in/out horizontally.
		const float Delta = MouseEvent.GetWheelDelta();
		ZoomHorizontally(Delta, static_cast<float>(MousePosition.X));
	}

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::ZoomHorizontally(const float Delta, const float X)
{
	FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.RelativeZoomWithFixedOffset(Delta, X);
	ViewportX.ScrollAtValue(ViewportX.GetValueAtPos(ViewportX.GetPos())); // align viewport position with sample (frame index)
	UpdateHorizontalScrollBar();
	bIsStateDirty = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SFrameTrack::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FCursorReply SFrameTrack::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	FCursorReply CursorReply = FCursorReply::Unhandled();

	if (CursorType == ECursorType::Arrow)
	{
		CursorReply = FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}
	else if (CursorType == ECursorType::Hand)
	{
		CursorReply = FCursorReply::Cursor(EMouseCursor::GrabHand);
	}

	return CursorReply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::ShowContextMenu(const FPointerEvent& MouseEvent)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, NULL);

	auto CreateSeriesMenuWidget = [](FLinearColor InIconColor, FText InText) -> TSharedRef<SWidget>
	{
		return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(InText)
		]

		+ SHorizontalBox::Slot()
		.Padding(FMargin(8.0f, 0.0f, 8.0f, 0.0f))
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.FilledCircle"))
			.DesiredSizeOverride(FVector2D(12.0, 12.0))
			.ColorAndOpacity(InIconColor)
		];
	};

	MenuBuilder.BeginSection("Frames", LOCTEXT("ContextMenu_Section_Frames", "Frames"));
	{
		FUIAction Action_ShowGameFrames
		(
			FExecuteAction::CreateSP(this, &SFrameTrack::ContextMenu_ShowFrameSeries_Execute, TraceFrameType_Game),
			FCanExecuteAction::CreateSP(this, &SFrameTrack::ContextMenu_ShowFrameSeries_CanExecute, TraceFrameType_Game),
			FIsActionChecked::CreateSP(this, &SFrameTrack::ContextMenu_ShowFrameSeries_IsChecked, TraceFrameType_Game)
		);
		MenuBuilder.AddMenuEntry
		(
			Action_ShowGameFrames,
			CreateSeriesMenuWidget(
				FLinearColor(0.3f, 0.3f, 0.7f, 1.0f),
				LOCTEXT("ContextMenu_ShowGameFrames", "Game Frames")),
			NAME_None,
			LOCTEXT("ContextMenu_ShowGameFrames_Desc", "Shows/hides the Game frames."),
			EUserInterfaceActionType::ToggleButton
		);

		FUIAction Action_ShowRenderingFrames
		(
			FExecuteAction::CreateSP(this, &SFrameTrack::ContextMenu_ShowFrameSeries_Execute, TraceFrameType_Rendering),
			FCanExecuteAction::CreateSP(this, &SFrameTrack::ContextMenu_ShowFrameSeries_CanExecute, TraceFrameType_Rendering),
			FIsActionChecked::CreateSP(this, &SFrameTrack::ContextMenu_ShowFrameSeries_IsChecked, TraceFrameType_Rendering)
		);
		MenuBuilder.AddMenuEntry
		(
			Action_ShowRenderingFrames,
			CreateSeriesMenuWidget(
				FLinearColor(0.7f, 0.3f, 0.3f, 1.0f),
				LOCTEXT("ContextMenu_ShowRenderingFrames", "Rendering Frames")),
			NAME_None,
			LOCTEXT("ContextMenu_ShowRenderingFrames_Desc", "Shows/hides the Rendering frames."),
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Timers", LOCTEXT("ContextMenu_Section_Timers", "Timers"));

	for (TSharedPtr<FFrameTrackSeries> Series : AllSeries)
	{
		if (!Series->Is<FTimerFrameStatsTrackSeries>())
		{
			continue;
		}


		FTimerFrameStatsTrackSeries& TimerSeries = Series->As<FTimerFrameStatsTrackSeries>();
		ETraceFrameType FrameType = static_cast<ETraceFrameType>(TimerSeries.GetFrameType());
		FUIAction Action_ShowFrameStatsSeries
		(
			FExecuteAction::CreateSP(this, &SFrameTrack::ContextMenu_ShowFrameStats_Execute, FrameType, TimerSeries.GetTimerId()),
			FCanExecuteAction::CreateSP(this, &SFrameTrack::ContextMenu_ShowFrameStats_CanExecute, FrameType, TimerSeries.GetTimerId()),
			FIsActionChecked::CreateSP(this, &SFrameTrack::ContextMenu_ShowFrameStats_IsChecked, FrameType, TimerSeries.GetTimerId())
		);
		MenuBuilder.AddMenuEntry
		(
			Action_ShowFrameStatsSeries,
			CreateSeriesMenuWidget(TimerSeries.GetColor(), TimerSeries.GetName()),
			NAME_None,
			FText::Format(LOCTEXT("ContextMenu_ShowFrameStatsSeries_Desc", "Shows/hides the {0} timer series."), TimerSeries.GetName()),
			EUserInterfaceActionType::ToggleButton
		);
	}

	MenuBuilder.EndSection();

	MenuBuilder.AddSeparator();

	MenuBuilder.AddSubMenu
	(
		LOCTEXT("ContextMenu_ThresholdsSubMenu", "Setup Thresholds"),
		LOCTEXT("ContextMenu_ThresholdsSubMenu_Desc", "Setup thresholds."),
		FNewMenuDelegate::CreateSP(this, &SFrameTrack::CreateThresholdsMenu),
		false,
		FSlateIcon()
	);

	MenuBuilder.BeginSection("Zoom", LOCTEXT("ContextMenu_Section_Zoom", "Zoom"));
	{
		FUIAction Action_AutoZoom
		(
			FExecuteAction::CreateSP(this, &SFrameTrack::ContextMenu_AutoZoom_Execute),
			FCanExecuteAction::CreateSP(this, &SFrameTrack::ContextMenu_AutoZoom_CanExecute),
			FIsActionChecked::CreateSP(this, &SFrameTrack::ContextMenu_AutoZoom_IsChecked)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_AutoZoom", "Auto Zoom"),
			LOCTEXT("ContextMenu_AutoZoom_Desc", "Enables auto zoom. Makes the entire session time range to fit into the Frames track's view."),
			FSlateIcon(),
			Action_AutoZoom,
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		FUIAction Action_ZoomTimingViewOnFrameSelection
		(
			FExecuteAction::CreateSP(this, &SFrameTrack::ContextMenu_ZoomTimingViewOnFrameSelection_Execute),
			FCanExecuteAction::CreateSP(this, &SFrameTrack::ContextMenu_ZoomTimingViewOnFrameSelection_CanExecute),
			FIsActionChecked::CreateSP(this, &SFrameTrack::ContextMenu_ZoomTimingViewOnFrameSelection_IsChecked)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_ZoomTimingViewOnFrameSelection", "Zoom Timing View on Frame Selection"),
			LOCTEXT("ContextMenu_ZoomTimingViewOnFrameSelection_Desc", "If enabled, the Timing view will also be zoomed when a frame is selected.\n(This option is persistent to the UnrealInsightsSettings.ini file.)"),
			FSlateIcon(),
			Action_ZoomTimingViewOnFrameSelection,
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	CreateSelectedFrameMenu(MenuBuilder);

	TSharedRef<SWidget> MenuWidget = MenuBuilder.MakeWidget();

	FWidgetPath EventPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
	const FVector2D ScreenSpacePosition = MouseEvent.GetScreenSpacePosition();
	FSlateApplication::Get().PushMenu(SharedThis(this), EventPath, MenuWidget, ScreenSpacePosition, FPopupTransitionEffect::ContextMenu);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::CreateThresholdsMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Thresholds");// , LOCTEXT("ContextMenu_Section_Thresholds", "Thresholds"));

	MenuBuilder.AddMenuEntry
	(
		LOCTEXT("ContextMenu_ShowUpperThresholdLine", "Show Upper Threshold Line"),
		LOCTEXT("ContextMenu_ShowUpperThresholdLine_Desc", "Shows/hides the red horizontal line for the upper threshold."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					bShowUpperThresholdLine = !bShowUpperThresholdLine;

					// Persistent option. Save it to the config file.
					FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
					Settings.SetAndSaveShowUpperThresholdLineEnabled(bShowUpperThresholdLine);
				}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda(
				[this]() -> bool
				{
					return bShowUpperThresholdLine;
				})),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.AddMenuEntry
	(
		LOCTEXT("ContextMenu_ShowLowerThresholdLine", "Show Lower Threshold Line"),
		LOCTEXT("ContextMenu_ShowLowerThresholdLine_Desc", "Shows/hides the green horizontal line for the lower threshold."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					bShowLowerThresholdLine = !bShowLowerThresholdLine;

					// Persistent option. Save it to the config file.
					FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
					Settings.SetAndSaveShowLowerThresholdLineEnabled(bShowLowerThresholdLine);
				}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda(
				[this]() -> bool
				{
					return bShowLowerThresholdLine;
				})),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry
	(
		FUIAction(FExecuteAction(), FCanExecuteAction()),
		CreateUpperThresholdWidget(),
		NAME_None,
		LOCTEXT("UpperThresholdCustomTooltip", "Upper Threshold\nFrames with duration longer than this threshold will have a red color tint.\nCan be specified as a frame duration, in seconds [0.001 .. 1.0] or as a framerate [1 fps ... 1000 fps]."),
		EUserInterfaceActionType::None
	);

	MenuBuilder.AddMenuEntry
	(
		FUIAction(FExecuteAction(), FCanExecuteAction()),
		CreateLowerThresholdWidget(),
		NAME_None,
		LOCTEXT("LowerThresholdCustomTooltip", "Lower Threshold\nFrames with duration shorter than this threshold will have a green color tint.\nCan be specified as a frame duration, in seconds [0.001 .. 1.0] or as a framerate [1 fps ... 1000 fps]."),
		EUserInterfaceActionType::None
	);

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry
	(
		FUIAction(FExecuteAction(), FCanExecuteAction()),
		CreateThresholdPresetsWidget(),
		NAME_None,
		LOCTEXT("ThresholdPresetsTooltip", "Threshold Presets"),
		EUserInterfaceActionType::None
	);

	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SFrameTrack::CreateUpperThresholdWidget()
{
	return SNew(SHorizontalBox)

	+ SHorizontalBox::Slot()
	.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.MinDesiredWidth(110.0f)
		.Text(LOCTEXT("UpperThresholdText", "Upper Threshold:"))
	]

	+ SHorizontalBox::Slot()
	.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(SEditableTextBox)
		.MinDesiredWidth(50.0f)
		.HintText(LOCTEXT("UpperThresholdCustomHint", "30 fps"))
		.Text_Lambda([this]
		{
			if (bShowUpperThresholdAsFps)
			{
				return FText::FromString(FString::Printf(TEXT("%g fps"), 1.0 / UpperThresholdTime));
			}
			else
			{
				return FText::FromString(FString::Printf(TEXT("%g"), UpperThresholdTime));
			}
		})
		.OnTextChanged_Lambda([this](const FText& InText)
		{
			FString ValueStr = InText.ToString().TrimStartAndEnd();
			if (ValueStr.IsEmpty())
			{
				ValueStr = TEXT("30 fps");
			}
			if (ValueStr.EndsWith(TEXT("fps")))
			{
				double FPS = FCString::Atof(*ValueStr);
				UpperThresholdTime = 1.0 / FMath::Clamp(FPS, 1.0 / MaxThresholdTime, 1.0 / MinThresholdTime);
				bShowUpperThresholdAsFps = true;
			}
			else
			{
				double Time = FCString::Atof(*ValueStr);
				UpperThresholdTime = FMath::Clamp(Time, MinThresholdTime, MaxThresholdTime);
				bShowUpperThresholdAsFps = false;
			}
			if (LowerThresholdTime > UpperThresholdTime)
			{
				LowerThresholdTime = UpperThresholdTime;
			}
			SaveThresholds();
		})
	]

	+ SHorizontalBox::Slot()
	.Padding(FMargin(8.0f, 0.0f, 12.0f, 0.0f))
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.Text_Lambda([this]
		{
			FString ThresholdTimeStr = FormatTimeAuto(UpperThresholdTime, 2);
			if (bShowUpperThresholdAsFps)
			{
				return FText::FromString(FString::Printf(TEXT("%s"), *ThresholdTimeStr));
			}
			else
			{
				return FText::FromString(FString::Printf(TEXT("%s (%.2f fps)"), *ThresholdTimeStr, 1.0 / UpperThresholdTime));
			}
		})
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SFrameTrack::CreateLowerThresholdWidget()
{
	return SNew(SHorizontalBox)

	+ SHorizontalBox::Slot()
	.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.MinDesiredWidth(110.0f)
		.Text(LOCTEXT("LowerThresholdText", "Lower Threshold:"))
	]

	+ SHorizontalBox::Slot()
	.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(SEditableTextBox)
		.MinDesiredWidth(50.0f)
		.HintText(LOCTEXT("LowerThresholdCustomHint", "60 fps"))
		.Text_Lambda([this]
		{
			if (bShowLowerThresholdAsFps)
			{
				return FText::FromString(FString::Printf(TEXT("%g fps"), 1.0 / LowerThresholdTime));
			}
			else
			{
				return FText::FromString(FString::Printf(TEXT("%g"), LowerThresholdTime));
			}
		})
		.OnTextChanged_Lambda([this](const FText& InText)
		{
			FString ValueStr = InText.ToString().TrimStartAndEnd();
			if (ValueStr.IsEmpty())
			{
				ValueStr = TEXT("60 fps");
			}
			if (ValueStr.EndsWith(TEXT("fps")))
			{
				double FPS = FCString::Atof(*ValueStr);
				LowerThresholdTime = 1.0 / FMath::Clamp(FPS, 1.0 / MaxThresholdTime, 1.0 / MinThresholdTime);
				bShowLowerThresholdAsFps = true;
			}
			else
			{
				double Time = FCString::Atof(*ValueStr);
				LowerThresholdTime = FMath::Clamp(Time, MinThresholdTime, MaxThresholdTime);
				bShowLowerThresholdAsFps = false;
			}
			if (UpperThresholdTime < LowerThresholdTime)
			{
				UpperThresholdTime = LowerThresholdTime;
			}
			SaveThresholds();
		})
	]

	+ SHorizontalBox::Slot()
	.Padding(FMargin(8.0f, 0.0f, 12.0f, 0.0f))
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.Text_Lambda([this]
		{
			FString ThresholdTimeStr = FormatTimeAuto(LowerThresholdTime, 2);
			if (bShowLowerThresholdAsFps)
			{
				return FText::FromString(FString::Printf(TEXT("%s"), *ThresholdTimeStr));
			}
			else
			{
				return FText::FromString(FString::Printf(TEXT("%s (%.2f fps)"), *ThresholdTimeStr, 1.0 / LowerThresholdTime));
			}
		})
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::SaveThresholds()
{
	// Persistent option. Save it to the config file.
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
	Settings.SetAndSaveThresholds(UpperThresholdTime, LowerThresholdTime, bShowUpperThresholdAsFps, bShowLowerThresholdAsFps);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::SetThresholdsFPS(double InUpperThresholdFPS, double InLowerThresholdFPS)
{
	UpperThresholdTime = 1.0 / InUpperThresholdFPS;
	LowerThresholdTime = 1.0 / InLowerThresholdFPS;
	bShowUpperThresholdAsFps = true;
	bShowLowerThresholdAsFps = true;
	SaveThresholds();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SFrameTrack::CreateThresholdPresetsWidget()
{
	return SNew(SHorizontalBox)

	+ SHorizontalBox::Slot()
	.Padding(FMargin(-30.0f, 0.0f, 0.0f, 0.0f))
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(SButton)
		.ContentPadding(FMargin(-6.0f, 0.0f, -6.0f, 0.0f))
		.Text(LOCTEXT("ThresholdPreset20fps", "15/20 fps"))
		.OnClicked_Lambda([this]() -> FReply
		{
			SetThresholdsFPS(15.0, 20.0);
			return FReply::Handled();
		})
	]

	+ SHorizontalBox::Slot()
	.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(SButton)
		.ContentPadding(FMargin(-6.0f, 0.0f, -6.0f, 0.0f))
		.Text(LOCTEXT("ThresholdPreset30fps", "20/30 fps"))
		.OnClicked_Lambda([this]() -> FReply
		{
			SetThresholdsFPS(20.0, 30.0);
			return FReply::Handled();
		})
	]

	+ SHorizontalBox::Slot()
	.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(SButton)
		.ContentPadding(FMargin(-6.0f, 0.0f, -6.0f, 0.0f))
		.Text(LOCTEXT("ThresholdPreset60fps", "30/60 fps"))
		.OnClicked_Lambda([this]() -> FReply
		{
			SetThresholdsFPS(30.0, 60.0);
			return FReply::Handled();
		})
	]

	+ SHorizontalBox::Slot()
	.Padding(FMargin(4.0f, 0.0f, 9.0f, 0.0f))
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(SButton)
		.ContentPadding(FMargin(-6.0f, 0.0f, -6.0f, 0.0f))
		.Text(LOCTEXT("ThresholdPreset120fps", "60/120 fps"))
		.OnClicked_Lambda([this]() -> FReply
		{
			SetThresholdsFPS(60.0, 120.0);
			return FReply::Handled();
		})
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::CreateSelectedFrameMenu(FMenuBuilder& MenuBuilder)
{
	FText SelectedFrameSectionName;
	if (SelectedSample.IsValid())
	{
		SelectedFrameSectionName = FText::Format(LOCTEXT("ContextMenu_Section_SelectedFrame_Fmt", "{0} {1}"),
			FFrameTrackDrawHelper::FrameTypeToText(SelectedSample.Series->GetFrameType()),
			FText::AsNumber(SelectedSample.Sample->LargestFrameIndex));
	}
	else
	{
		SelectedFrameSectionName = LOCTEXT("ContextMenu_Section_NoFrameSelected", "No Frame Selected");
	}
	MenuBuilder.BeginSection("SelectedFrame", SelectedFrameSectionName);

	FUIAction Action_ScrollLogView
	(
		FExecuteAction::CreateSP(this, &SFrameTrack::ContextMenu_ScrollLogView_Execute),
		FCanExecuteAction::CreateSP(this, &SFrameTrack::ContextMenu_ScrollLogView_CanExecute)
	);
	FText Label;
	if (SelectedSample.IsValid())
	{
		FText StartTimeText = FText::FromString(FormatTimeAuto(SelectedSample.Sample->LargestFrameStartTime, 2));
		Label = FText::Format(LOCTEXT("ContextMenu_ScrollLogView_Fmt", "Scroll Log View (\u2192 {0})"), StartTimeText);
	}
	else
	{
		Label = LOCTEXT("ContextMenu_ScrollLogView", "Scroll Log View");
	}
	MenuBuilder.AddMenuEntry
	(
		Label,
		LOCTEXT("ContextMenu_ScrollLogView_Desc", "Scrolls the Log View at the message with the closest timestamp to the start time of the selected frame."),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.LogView"),
		Action_ScrollLogView,
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::ContextMenu_ShowFrameSeries_Execute(ETraceFrameType FrameType)
{
	TSharedPtr<FFrameTrackSeries> Series = FindSeries(FrameType);
	if (Series.IsValid())
	{
		Series->SetVisibility(!Series->IsVisible());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SFrameTrack::ContextMenu_ShowFrameSeries_CanExecute(ETraceFrameType FrameType) const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SFrameTrack::ContextMenu_ShowFrameSeries_IsChecked(ETraceFrameType FrameType) const
{
	TSharedPtr<FFrameTrackSeries> Series = FindSeries(FrameType);
	return Series.IsValid() ? Series->IsVisible() : false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::ContextMenu_ShowFrameStats_Execute(ETraceFrameType FrameType, uint32 TimerId)
{
	TSharedPtr<FFrameTrackSeries> Series = FindFrameStatsSeries(FrameType, TimerId);
	if (Series.IsValid())
	{
		Series->SetVisibility(!Series->IsVisible());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SFrameTrack::ContextMenu_ShowFrameStats_CanExecute(ETraceFrameType FrameType, uint32 TimerId) const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SFrameTrack::ContextMenu_ShowFrameStats_IsChecked(ETraceFrameType FrameType, uint32 TimerId) const
{
	TSharedPtr<FFrameTrackSeries> Series = FindFrameStatsSeries(FrameType, TimerId);
	return Series.IsValid() ? Series->IsVisible() : false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::ContextMenu_AutoZoom_Execute()
{
	bIsAutoZoomEnabled = !bIsAutoZoomEnabled;

	if (bIsAutoZoomEnabled)
	{
		AutoZoom();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SFrameTrack::ContextMenu_AutoZoom_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SFrameTrack::ContextMenu_AutoZoom_IsChecked() const
{
	return bIsAutoZoomEnabled;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::AutoZoom()
{
	FAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

	AutoZoomViewportPos = ViewportX.GetMinPos();
	ViewportX.ScrollAtPos(AutoZoomViewportPos);

	AutoZoomViewportSize = ViewportX.GetSize();

	if (AutoZoomViewportSize > 0.0f &&
		ViewportX.GetMaxValue() - ViewportX.GetMinValue() > 0)
	{
		float DX = ViewportX.GetMaxPos() - ViewportX.GetMinPos();

		// Auto zoom in.
		while (DX < AutoZoomViewportSize)
		{
			const float OldScale = ViewportX.GetScale();
			ViewportX.RelativeZoomWithFixedOffset(+0.1f, 0.0f);
			ViewportX.ScrollAtPos(AutoZoomViewportPos);
			DX = ViewportX.GetMaxPos() - ViewportX.GetMinPos();
			if (OldScale == ViewportX.GetScale())
			{
				break;
			}
		}

		// Auto zoom out (until entire session frame range fits into view).
		while (DX > AutoZoomViewportSize)
		{
			const float OldScale = ViewportX.GetScale();
			ViewportX.RelativeZoomWithFixedOffset(-0.1f, 0.0f);
			ViewportX.ScrollAtPos(AutoZoomViewportPos);
			DX = ViewportX.GetMaxPos() - ViewportX.GetMinPos();
			if (OldScale == ViewportX.GetScale())
			{
				break;
			}
		}
	}

	AutoZoomViewportScale = ViewportX.GetScale();

	UpdateHorizontalScrollBar();
	bIsStateDirty = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::ContextMenu_ZoomTimingViewOnFrameSelection_Execute()
{
	bZoomTimingViewOnFrameSelection = !bZoomTimingViewOnFrameSelection;

	// Persistent option. Save it to the config file.
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
	Settings.SetAndSaveAutoZoomOnFrameSelection(bZoomTimingViewOnFrameSelection);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SFrameTrack::ContextMenu_ZoomTimingViewOnFrameSelection_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SFrameTrack::ContextMenu_ZoomTimingViewOnFrameSelection_IsChecked() const
{
	return bZoomTimingViewOnFrameSelection;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::ContextMenu_ScrollLogView_Execute()
{
	if (!SelectedSample.IsValid())
	{
		return;
	}

	TSharedPtr<STimingProfilerWindow> TimingWindow = FTimingProfilerManager::Get()->GetProfilerWindow();
	if (!TimingWindow.IsValid())
	{
		return;
	}

	TSharedPtr<SLogView> LogView = TimingWindow->GetLogView();
	if (!LogView.IsValid())
	{
		return;
	}

	LogView->SelectLogMessageByClosestTime(SelectedSample.Sample->LargestFrameStartTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SFrameTrack::ContextMenu_ScrollLogView_CanExecute() const
{
	if (!SelectedSample.IsValid())
	{
		return false;
	}

	TSharedPtr<STimingProfilerWindow> TimingWindow = FTimingProfilerManager::Get()->GetProfilerWindow();
	if (!TimingWindow.IsValid())
	{
		return false;
	}

	TSharedPtr<SLogView> LogView = TimingWindow->GetLogView();
	if (!LogView.IsValid())
	{
		return false;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::BindCommands()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::HorizontalScrollBar_OnUserScrolled(float ScrollOffset)
{
	Viewport.GetHorizontalAxisViewport().OnUserScrolled(HorizontalScrollBar, ScrollOffset);
	bIsStateDirty = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SFrameTrack::UpdateHorizontalScrollBar()
{
	Viewport.GetHorizontalAxisViewport().UpdateScrollBar(HorizontalScrollBar);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SFrameTrack::HasFrameStatSeries(ETraceFrameType FrameType, uint32 TimerId) const
{
	const TSharedPtr<FFrameTrackSeries>* ExistingSeries = AllSeries.FindByPredicate([FrameType, TimerId](TSharedPtr<FFrameTrackSeries> Series)
		{
			return Series->Is<FTimerFrameStatsTrackSeries>() &&
					Series->As<FTimerFrameStatsTrackSeries>().GetFrameType() == FrameType &&
					Series->As<FTimerFrameStatsTrackSeries>().GetTimerId() == TimerId;
		});

	return ExistingSeries != nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerFrameStatsTrackSeries& SFrameTrack::AddTimerFrameStatSeries(ETraceFrameType FrameType, uint32 TimerId, FLinearColor Color, FText Name)
{
	TSharedPtr<FFrameTrackSeries> ExistingSeries = FindFrameStatsSeries(FrameType, TimerId);

	if (ExistingSeries.IsValid())
	{
		return ExistingSeries->As<FTimerFrameStatsTrackSeries>();
	}
	TSharedRef<FTimerFrameStatsTrackSeries> SeriesRef = MakeShared<FTimerFrameStatsTrackSeries>(FrameType, TimerId);
	SeriesRef->SetColor(Color);
	SeriesRef->SetName(Name);
	AllSeries.Add(SeriesRef);

	bIsStateDirty = true;

	return *SeriesRef;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SFrameTrack::RemoveTimerFrameStatSeries(ETraceFrameType FrameType, uint32 TimerId)
{
	int32 NumRemoved = AllSeries.RemoveAll([FrameType, TimerId](TSharedPtr<FFrameTrackSeries> Series)
		{
			return Series->Is<FTimerFrameStatsTrackSeries>() &&
				Series->As<FTimerFrameStatsTrackSeries>().GetFrameType() == FrameType &&
				Series->As<FTimerFrameStatsTrackSeries>().GetTimerId() == TimerId;
		});

	ensure(NumRemoved == 1);

	bIsStateDirty = true;

	return NumRemoved >= 1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SFrameTrack::HasAnySeriesForTimer(uint32 TimerId) const
{
	for (const TSharedPtr<FFrameTrackSeries>& Series : AllSeries)
	{
		if (Series->Is<FTimerFrameStatsTrackSeries>())
		{
			if (Series->As<FTimerFrameStatsTrackSeries>().GetTimerId() == TimerId)
			{
				return true;
			}
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 SFrameTrack::GetNumSeriesForTimer(uint32 TimerId) const
{
	uint32 NumSeries = 0;

	for (const TSharedPtr<FFrameTrackSeries>& Series : AllSeries)
	{
		if (Series->Is<FTimerFrameStatsTrackSeries>())
		{
			if (Series->As<FTimerFrameStatsTrackSeries>().GetTimerId() == TimerId)
			{
				++NumSeries;
			}
		}
	}

	return NumSeries;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<STimingView> SFrameTrack::GetTimingView() const
{
	TSharedPtr<STimingProfilerWindow> Window = FTimingProfilerManager::Get()->GetProfilerWindow();
	return Window.IsValid() ? Window->GetTimingView() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
