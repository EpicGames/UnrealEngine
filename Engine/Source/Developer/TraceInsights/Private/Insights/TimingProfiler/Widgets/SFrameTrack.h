// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Geometry.h"
#include "Rendering/RenderingCommon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

// TraceInsightsCore
#include "InsightsCore/Common/FixedCircularBuffer.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/FrameTrackHelper.h"
#include "Insights/TimingProfiler/ViewModels/FrameTrackViewport.h"

class FMenuBuilder;
class SScrollBar;

namespace UE::Insights::TimingProfiler
{

class FUserAnnotationStore;
class STimingView;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FFrameTrackSampleRef
{
	TSharedPtr<FFrameTrackSeries> Series;
	TSharedPtr<FFrameTrackSample> Sample;

	FFrameTrackSampleRef()
		: Series(), Sample()
	{
	}

	FFrameTrackSampleRef(TSharedPtr<FFrameTrackSeries> InSeries, TSharedPtr<FFrameTrackSample> InSample)
		: Series(InSeries), Sample(InSample)
	{
	}

	FFrameTrackSampleRef(const FFrameTrackSampleRef& Other)
		: Series(Other.Series), Sample(Other.Sample)
	{
	}

	FFrameTrackSampleRef& operator=(const FFrameTrackSampleRef& Other)
	{
		Series = Other.Series;
		Sample = Other.Sample;
		return *this;
	}

	void Reset()
	{
		Series.Reset();
		Sample.Reset();
	}

	bool IsValid() const
	{
		return Series.IsValid() && Sample.IsValid();
	}

	bool Equals(const FFrameTrackSampleRef& Other) const
	{
		return Series == Other.Series
			&& ((Sample == Other.Sample) || (Sample.IsValid() && Other.Sample.IsValid() && Sample->Equals(*Other.Sample)));
	}

	static bool AreEquals(const FFrameTrackSampleRef& A, const FFrameTrackSampleRef& B)
	{
		return A.Equals(B);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Widget used to present frames data in a bar track.
 */
class SFrameTrack : public SCompoundWidget
{
public:
	/** Number of pixels. */
	static constexpr float MOUSE_SNAP_DISTANCE = 2.0f;

	enum class ECursorType
	{
		Default,
		Arrow,
		Hand,
	};

public:
	/** Default constructor. */
	SFrameTrack();

	/** Virtual destructor. */
	virtual ~SFrameTrack();

	/** Resets internal widget's data to the default one. */
	void Reset();

	SLATE_BEGIN_ARGS(SFrameTrack)
	{
		_Clipping = EWidgetClipping::ClipToBounds;
	}
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 * @param InArgs The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs);

	/**
	 * Ticks this widget. Override in derived classes, but always call the parent implementation.
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

	bool HasFrameStatSeries(ETraceFrameType FrameType, uint32 TimerId) const;
	FTimerFrameStatsTrackSeries& AddTimerFrameStatSeries(ETraceFrameType FrameType, uint32 TimerId, FLinearColor Color, FText Name);
	bool RemoveTimerFrameStatSeries(ETraceFrameType FrameType, uint32 TimerId);

	bool HasAnySeriesForTimer(uint32 TimerId) const;
	uint32 GetNumSeriesForTimer(uint32 TimerId) const;
	TArray<TSharedPtr<FFrameTrackSeries>>& GetSeries() { return AllSeries; };

protected:
	TSharedRef<FFrameTrackSeries> FindOrAddSeries(ETraceFrameType FrameType);
	TSharedPtr<FFrameTrackSeries> FindSeries(ETraceFrameType FrameType) const;
	TSharedPtr<FFrameTrackSeries> FindFrameStatsSeries(ETraceFrameType FrameType, uint32 TimerId) const;
	void UpdateState();

	void DrawHorizontalAxisGrid(FDrawContext& DrawContext, const FSlateBrush* Brush, const FSlateFontInfo& Font, bool bDrawBackgroundLayer) const;
	void DrawVerticalAxisGrid(FDrawContext& DrawContext, const FSlateBrush* Brush, const FSlateFontInfo& Font) const;

	FFrameTrackSampleRef GetSampleAtMousePosition(double X, double Y);
	void SelectFrameAtMousePosition(double X, double Y, bool JoinCurrentSelection);

	void ShowContextMenu(const FPointerEvent& MouseEvent);
	void CreateThresholdsMenu(FMenuBuilder& MenuBuilder);
	TSharedRef<SWidget> CreateUpperThresholdWidget();
	TSharedRef<SWidget> CreateLowerThresholdWidget();
	void SaveThresholds();
	void SetThresholdsFPS(double InUpperThresholdFPS, double InLowerThresholdFPS);
	TSharedRef<SWidget> CreateThresholdPresetsWidget();
	void CreateSelectedFrameMenu(FMenuBuilder& MenuBuilder);

	void ContextMenu_ShowFrameSeries_Execute(ETraceFrameType FrameType);
	bool ContextMenu_ShowFrameSeries_CanExecute(ETraceFrameType FrameType) const;
	bool ContextMenu_ShowFrameSeries_IsChecked(ETraceFrameType FrameType) const;

	void ContextMenu_ShowFrameStats_Execute(ETraceFrameType FrameType, uint32 TimerId);
	bool ContextMenu_ShowFrameStats_CanExecute(ETraceFrameType FrameType, uint32 TimerId) const;
	bool ContextMenu_ShowFrameStats_IsChecked(ETraceFrameType FrameType, uint32 TimerId) const;

	void ContextMenu_AutoZoom_Execute();
	bool ContextMenu_AutoZoom_CanExecute() const;
	bool ContextMenu_AutoZoom_IsChecked() const;
	void AutoZoom();

	void ContextMenu_ZoomTimingViewOnFrameSelection_Execute();
	bool ContextMenu_ZoomTimingViewOnFrameSelection_CanExecute() const;
	bool ContextMenu_ZoomTimingViewOnFrameSelection_IsChecked() const;

	void ContextMenu_ScrollLogView_Execute();
	bool ContextMenu_ScrollLogView_CanExecute() const;

	/** Binds our UI commands to delegates. */
	void BindCommands();

	/**
	 * Called when the user scrolls the horizontal scrollbar.
	 * @param ScrollOffset Scroll offset as a fraction between 0 and 1.
	 */
	void HorizontalScrollBar_OnUserScrolled(float ScrollOffset);
	void UpdateHorizontalScrollBar();

	void ZoomHorizontally(const float Delta, const float X);

	TSharedPtr<STimingView> GetTimingView() const;
	void RegisterTimingViewCallbacks();
	void UnregisterTimingViewCallbacks();

protected:
	/** The track's viewport. Encapsulates info about position and scale. */
	FFrameTrackViewport Viewport;
	bool bIsViewportDirty = false;

	/** Cached info for all frame series. */
	TArray<TSharedPtr<FFrameTrackSeries>> AllSeries;

	bool bIsStateDirty = false;

	bool bIsAutoZoomEnabled = false;

	float AutoZoomViewportPos = false;
	float AutoZoomViewportScale = false;
	float AutoZoomViewportSize = false;

	bool bZoomTimingViewOnFrameSelection = false;

	uint64 AnalysisSyncNextTimestamp = 0;

	//////////////////////////////////////////////////
	// Thresholds

	bool bShowUpperThresholdLine = false;
	bool bShowLowerThresholdLine = false;

	static constexpr double MinThresholdTime = 0.001; // == 1ms == 1000 fps
	static constexpr double MaxThresholdTime = 1.0; // == 1s == 1 fps

	double UpperThresholdTime = 1.0 / 30.0;
	double LowerThresholdTime = 1.0 / 60.0;

	bool bShowUpperThresholdAsFps = true;
	bool bShowLowerThresholdAsFps = true;

	//////////////////////////////////////////////////

	TSharedPtr<SScrollBar> HorizontalScrollBar;

	//////////////////////////////////////////////////
	// Panning and Zooming behaviors

	/** The current mouse position. */
	FVector2D MousePosition;

	/** Mouse position during the call on mouse button down. */
	FVector2D MousePositionOnButtonDown;
	float ViewportPosXOnButtonDown = 0.0f;

	/** Mouse position during the call on mouse button up. */
	FVector2D MousePositionOnButtonUp;

	bool bIsLMB_Pressed = false;
	bool bIsRMB_Pressed = false;

	/** True, if the user is currently interactively scrolling the view (ex.: by holding the left mouse button and dragging). */
	bool bIsScrolling = false;

	mutable bool bDrawVerticalAxisLabelsOnLeftSide = false;

	//////////////////////////////////////////////////
	// Selection

	FFrameTrackSampleRef HoveredSample;
	FFrameTrackSampleRef SelectedSample;

	mutable float TooltipOpacity = 0.0f;
	mutable float TooltipSizeX = 0.0f;

	//////////////////////////////////////////////////
	// Misc

	FGeometry ThisGeometry;

	/** Cursor type. */
	ECursorType CursorType = ECursorType::Default;

	TWeakPtr<STimingView> TimingViewWeak;
	FDelegateHandle OnTrackVisibilityChangedHandle;
	FDelegateHandle OnTrackAddedHandle;
	FDelegateHandle OnTrackRemovedHandle;

	//////////////////////////////////////////////////
	// User Annotations

	/** Weak reference to the user annotation store, for drawing indicators. */
	TWeakPtr<FUserAnnotationStore> AnnotationStoreWeak;

	/** Handle for the annotations-extender "store available" subscription. Unsubscribed in the
	 *  destructor so we don't fire into a dead widget if the module outlives the tab. */
	FDelegateHandle OnAnnotationStoreChangedHandle;

	/** Refreshes AnnotationStoreWeak from the current session's store. Invoked on subscribe
	 *  (catches the already-active session) and on the extender's broadcast (catches later
	 *  session begin or store-sharing swaps). */
	void RefreshAnnotationStore();

	// Debug stats
	int32 NumUpdatedFrames = 0;
	TFixedCircularBuffer<uint64, 32> UpdateDurationHistory;
	mutable TFixedCircularBuffer<uint64, 32> DrawDurationHistory;
	mutable TFixedCircularBuffer<uint64, 32> OnPaintDurationHistory;
	mutable uint64 LastOnPaintTime = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
