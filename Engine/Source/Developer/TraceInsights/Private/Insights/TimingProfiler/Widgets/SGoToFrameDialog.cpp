// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/TimingProfiler/Widgets/SGoToFrameDialog.h"

#include "Framework/Application/SlateApplication.h"
#include "Input/Events.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

// TraceInsights
#include "Common/ProviderLock.h"
#include "TraceServices/Model/Frames.h"
#include "Insights/InsightsManager.h"
#include "Insights/TimingProfilerCommon.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::SGoToFrameDialog"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

void SGoToFrameDialog::Construct(const FArguments& InArgs)
{
	ParentWindowPtr = InArgs._ParentWindow;
	bSelectTimeRange = InArgs._InitialSelectTimeRange;
	bZoomTimingView = InArgs._InitialZoomTimingView;
	NavigateDelegate = InArgs._OnNavigate;

	// Construct frame type options. Their index maps to ETraceFrameType currently. 
	// static_assert is here to make sure that if the enum ever changes order, this drop-down must be updated.
	FrameTypeOptions.Add(MakeShared<FString>(TEXT("Game Frame")));
	FrameTypeOptions.Add(MakeShared<FString>(TEXT("Render Frame")));
	static_assert((int32)TraceFrameType_Game == 0 && (int32)TraceFrameType_Rendering == 1, "ETraceFrameType values must match FrameTypeOptions indices");

	// Ensure initially selected frametype is always valid. This is passed in from config, so clamp it here.
	const int32 ClampedFrameType = FMath::Clamp((int32)InArgs._InitialFrameType, 0, FrameTypeOptions.Num() - 1);

	const FString InitialFrameNumberStr = (InArgs._InitialFrameNumber >= 0) ? FString::Printf(TEXT("%lld"), InArgs._InitialFrameNumber) : FString();

	// Let Escape close the modal immediately, instead of first unfocusing text fields
	TFunction<FReply(const FGeometry&, const FKeyEvent&)> CloseOnEscapeLambda = [this](const FGeometry&, const FKeyEvent& KeyEvent) -> FReply
	{
		if (KeyEvent.GetKey() == EKeys::Escape)
		{
			if (TSharedPtr<SWindow> ParentWindow = ParentWindowPtr.Pin())
			{
				ParentWindow->RequestDestroyWindow();
			}
			return FReply::Handled();
		}
		return FReply::Unhandled();
	};

	constexpr float RowPadding = 4.0f;
	constexpr float OuterPadding = 12.0f;

	ChildSlot
	[
		SNew(SBorder)
		.Padding(OuterPadding)
		[
			SNew(SGridPanel)
			.FillColumn(1, 1.0f)

			// Row 0: Frame Type dropdown
			+ SGridPanel::Slot(0, 0)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 8.0f, RowPadding)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FrameType", "Frame Type:"))
			]
			+ SGridPanel::Slot(1, 0)
			.Padding(0.0f, 0.0f, 0.0f, RowPadding)
			[
				SNew(SBox)
				.MinDesiredWidth(160.0f)
				[
					SAssignNew(FrameTypeCombo, STextComboBox)
					.OptionsSource(&FrameTypeOptions)
					.InitiallySelectedItem(FrameTypeOptions[ClampedFrameType])
				]
			]

			// Row 1: Frame Number editable text
			+ SGridPanel::Slot(0, 1)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 8.0f, RowPadding)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FrameNumber", "Frame Number:"))
			]
			+ SGridPanel::Slot(1, 1)
			.Padding(0.0f, 0.0f, 0.0f, RowPadding)
			[
				SAssignNew(FrameNumberBox, SEditableTextBox)
				.ClearKeyboardFocusOnCommit(false)
				.SelectAllTextWhenFocused(true)
				.Text(FText::FromString(InitialFrameNumberStr))
				.OnTextCommitted(this, &SGoToFrameDialog::OnFrameNumberTextCommitted)
				.OnKeyDownHandler_Lambda(CloseOnEscapeLambda)
			]

			// Row 2: Select Time Range
			+ SGridPanel::Slot(0, 2)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 8.0f, RowPadding)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SelectTimeRange", "Select Time Range:"))
			]
			+ SGridPanel::Slot(1, 2)
			.Padding(0.0f, 0.0f, 0.0f, RowPadding)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return bSelectTimeRange ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState) { bSelectTimeRange = (NewState == ECheckBoxState::Checked); })
			]

			// Row 3: Zoom timing view option
			+ SGridPanel::Slot(0, 3)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 8.0f, RowPadding)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Zoom", "Zoom Timing View:"))
			]
			+ SGridPanel::Slot(1, 3)
			.Padding(0.0f, 0.0f, 0.0f, RowPadding)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return bZoomTimingView ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState) { bZoomTimingView = (NewState == ECheckBoxState::Checked); })
			]

			// Row 4: Navigate button
			+ SGridPanel::Slot(0, 4)
			.ColumnSpan(2)
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 4.0f, 0.0f, RowPadding)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("Navigate", "Navigate"))
				.OnClicked(this, &SGoToFrameDialog::OnNavigateClicked)
			]

			// Row 5: Warning text
			+ SGridPanel::Slot(0, 5)
			.ColumnSpan(2)
			.Padding(0.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(WarningText, STextBlock)
				.ColorAndOpacity(FLinearColor::Yellow)
				.AutoWrapText(true)
				.Visibility(EVisibility::Collapsed)
			]
		]
	];

}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SGoToFrameDialog::OnFrameNumberTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	// Quality of life: When user presses enter while focused on frame number input field, execute navigation
	if (InCommitType == ETextCommit::OnEnter)
	{
		ExecuteNavigation();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SGoToFrameDialog::OnNavigateClicked()
{
	ExecuteNavigation();
	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SGoToFrameDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		if (TSharedPtr<SWindow> ParentWindow = ParentWindowPtr.Pin())
		{
			ParentWindow->RequestDestroyWindow();
		}
		return FReply::Handled();
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SGoToFrameDialog::ExecuteNavigation()
{
	const FString FrameNumberStr = FrameNumberBox->GetText().ToString();
	if (FrameNumberStr.IsEmpty())
	{
		WarningText->SetText(LOCTEXT("EnterFrameNumber", "Enter a frame number"));
		WarningText->SetVisibility(EVisibility::Visible);
		FSlateApplication::Get().SetKeyboardFocus(FrameNumberBox, EFocusCause::SetDirectly);
		return;
	}

	// If the string isn't a number, show warning. Allow the user to correct.
	if (!FrameNumberStr.IsNumeric())
	{
		WarningText->SetText(LOCTEXT("InvalidFrameNumber", "Enter a valid frame number"));
		WarningText->SetVisibility(EVisibility::Visible);
		FSlateApplication::Get().SetKeyboardFocus(FrameNumberBox, EFocusCause::SetDirectly);
		return;
	}

	// Parse integer and clamp to non-negative
	const uint64 FrameNumber = (uint64)FMath::Max(0LL, FCString::Strtoi64(*FrameNumberStr, nullptr, 10));

	// Process drop-down for Game Frame/Render Frame. Selected index corresponds to ETraceFrameType (checked earlier with static_assert).
	ETraceFrameType FrameType = TraceFrameType_Game;
	if (FrameTypeCombo->GetSelectedItem().IsValid())
	{
		const int32 FrameTypeIndex = FrameTypeOptions.IndexOfByPredicate([this](const TSharedPtr<FString>& Item)
		{
			return Item == FrameTypeCombo->GetSelectedItem();
		});
		if (FrameTypeIndex != INDEX_NONE)
		{
			FrameType = (ETraceFrameType)FrameTypeIndex;
		}
	}

	// Look up the frame, may fail: sparse frames, or frame number too high.
	double FrameStartTime = 0.0;
	double FrameEndTime = 0.0;
	bool bFrameFound = false;
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*Session.Get());
		const TraceServices::FFrame* Frame = FrameProvider.GetFrame(FrameType, FrameNumber);
		if (Frame != nullptr)
		{
			FrameStartTime = Frame->StartTime;
			FrameEndTime = Frame->EndTime;
			bFrameFound = true;
		}
	}

	// If the frame wasn't found, show warning. Don't close window to allow the user to correct.
	if (!bFrameFound)
	{
		const FText TypeName = (FrameType == TraceFrameType_Game) ? LOCTEXT("FrameTypeGame", "Game") : LOCTEXT("FrameTypeRender", "Render");
		const FText FrameNumberText = FText::AsNumber(FrameNumber, &FNumberFormattingOptions().SetUseGrouping(false));
		WarningText->SetText(FText::Format(LOCTEXT("FrameNotFound", "{0} Frame {1} was not found"), TypeName, FrameNumberText));
		WarningText->SetVisibility(EVisibility::Visible);
		FSlateApplication::Get().SetKeyboardFocus(FrameNumberBox, EFocusCause::SetDirectly);
		return;
	}

	// All parameters are valid and resolvable. Execute the navigate delegate bound to by STimingView.
	// STimingView will handle the actual browsing since it has necessary context.
	NavigateDelegate.ExecuteIfBound(FrameStartTime, FrameEndTime, FrameNumber, bSelectTimeRange, bZoomTimingView, FrameType);

	// Close modal on navigate with valid params
	if (TSharedPtr<SWindow> ParentWindow = ParentWindowPtr.Pin())
	{
		ParentWindow->RequestDestroyWindow();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
