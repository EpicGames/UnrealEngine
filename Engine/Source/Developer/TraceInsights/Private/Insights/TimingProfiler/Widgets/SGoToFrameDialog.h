// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

// TraceInsights
#include "Insights/InsightsSettings.h"

class SEditableTextBox;
class STextBlock;
class STextComboBox;
class SWindow;

namespace UE::Insights::TimingProfiler
{

DECLARE_DELEGATE_SixParams(FOnGoToFrameNavigate, double /*StartTime*/, double /*EndTime*/, uint64 /*FrameNumber*/, bool /*bSelectTimeRange*/, bool /*bZoomTimingView*/, ETraceFrameType /*FrameType*/);

class SGoToFrameDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGoToFrameDialog)
		: _ParentWindow()
		, _InitialFrameType(TraceFrameType_Game)
		, _InitialFrameNumber(-1)
		, _InitialSelectTimeRange(true)
		, _InitialZoomTimingView(true)
	{}
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ARGUMENT(ETraceFrameType, InitialFrameType)
		SLATE_ARGUMENT(int64, InitialFrameNumber)
		SLATE_ARGUMENT(bool, InitialSelectTimeRange)
		SLATE_ARGUMENT(bool, InitialZoomTimingView)
		SLATE_EVENT(FOnGoToFrameNavigate, OnNavigate)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	TSharedPtr<SEditableTextBox> GetFrameNumberBox() const { return FrameNumberBox; }

private:
	FReply OnNavigateClicked();
	void OnFrameNumberTextCommitted(const FText& InText, ETextCommit::Type InCommitType);
	void ExecuteNavigation();

	TWeakPtr<SWindow> ParentWindowPtr;

	TSharedPtr<SEditableTextBox> FrameNumberBox;
	TSharedPtr<STextComboBox> FrameTypeCombo;
	TSharedPtr<STextBlock> WarningText;

	TArray<TSharedPtr<FString>> FrameTypeOptions;

	FOnGoToFrameNavigate NavigateDelegate;
	bool bSelectTimeRange = true;
	bool bZoomTimingView = false;
};

} // namespace UE::Insights::TimingProfiler
