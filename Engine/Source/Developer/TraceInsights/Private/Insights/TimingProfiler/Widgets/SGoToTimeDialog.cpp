// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/TimingProfiler/Widgets/SGoToTimeDialog.h"

#include "Framework/Application/SlateApplication.h"
#include "Input/Events.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

// TraceInsightsCore
#include "InsightsCore/Filter/ViewModels/TimeFilterValueConverter.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::SGoToTimeDialog"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

void SGoToTimeDialog::Construct(const FArguments& InArgs)
{
	ParentWindowPtr = InArgs._ParentWindow;
	NavigateDelegate = InArgs._OnNavigate;
	bSelectTimeRange = InArgs._InitialSelectTimeRange;
	bZoomTimingView = InArgs._InitialZoomTimingView;

	// Use utility class that can parse time inputs with various units. Construct it here to access the hint and tooltip text.
	const FTimeFilterValueConverter TimeConverter;

	const FString InitialTimeStr = (InArgs._InitialTime >= 0.0) ? FString::Printf(TEXT("%g"), InArgs._InitialTime) : FString();

	InitialTimeRangeDuration = InArgs._InitialTimeRangeDuration;
	const FString InitialTimeRangeDurationStr = FString::Printf(TEXT("%g"), InitialTimeRangeDuration);

	TFunction<FReply(const FGeometry&, const FKeyEvent&)> CloseOnEscapeLambda = [this](const FGeometry&, const FKeyEvent& KeyEvent) -> FReply
	{
		if (KeyEvent.GetKey() == EKeys::Escape)
		{
			if (TSharedPtr<SWindow> ParentWindow = ParentWindowPtr.Pin())
			{
				ParentWindow->RequestDestroyWindow();
			}
			return FReply::Handled();
		}
		return FReply::Unhandled();
	};

	constexpr float RowPadding = 4.0f;
	constexpr float OuterPadding = 12.0f;
	ChildSlot
	[
		SNew(SBorder)
		.Padding(OuterPadding)
		[
			SNew(SGridPanel)
			.FillColumn(1, 1.0f)

			// Row 0: Time input
			+ SGridPanel::Slot(0, 0)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 8.0f, RowPadding)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TimeLabel", "Time:"))
			]
			+ SGridPanel::Slot(1, 0)
			.Padding(0.0f, 0.0f, 0.0f, RowPadding)
			[
				SNew(SBox)
				.MinDesiredWidth(160.0f)
				[
					SAssignNew(TimeBox, SEditableTextBox)
					.ClearKeyboardFocusOnCommit(false)
					.SelectAllTextWhenFocused(true)
					.Text(FText::FromString(InitialTimeStr))
					.HintText(TimeConverter.GetHintText())
					.ToolTipText(TimeConverter.GetTooltipText())
					.OnTextCommitted(this, &SGoToTimeDialog::OnTimeTextCommitted)
					.OnKeyDownHandler_Lambda(CloseOnEscapeLambda)
				]
			]

			// Row 1: Time Range Duration
			+ SGridPanel::Slot(0, 1)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 8.0f, RowPadding)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TimeRangeDuration", "Time Range Duration:"))
			]
			+ SGridPanel::Slot(1, 1)
			.Padding(0.0f, 0.0f, 0.0f, RowPadding)
			[
				SNew(SBox)
				.MinDesiredWidth(160.0f)
				[
					SAssignNew(TimeRangeDurationBox, SEditableTextBox)
					.ClearKeyboardFocusOnCommit(false)
					.Text(FText::FromString(InitialTimeRangeDurationStr))
					.HintText(TimeConverter.GetHintText())
					.ToolTipText(TimeConverter.GetTooltipText())
					.OnTextCommitted(this, &SGoToTimeDialog::OnTimeTextCommitted)
					.OnKeyDownHandler_Lambda(CloseOnEscapeLambda)
				]
			]

			// Row 2: Select Time Range
			+ SGridPanel::Slot(0, 2)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 8.0f, RowPadding)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SelectTimeRange", "Select Time Range:"))
			]
			+ SGridPanel::Slot(1, 2)
			.Padding(0.0f, 0.0f, 0.0f, RowPadding)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return bSelectTimeRange ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState) { bSelectTimeRange = (NewState == ECheckBoxState::Checked); })
			]

			// Row 3: Zoom Timing View
			+ SGridPanel::Slot(0, 3)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 8.0f, RowPadding)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Zoom", "Zoom Timing View:"))
			]
			+ SGridPanel::Slot(1, 3)
			.Padding(0.0f, 0.0f, 0.0f, RowPadding)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return bZoomTimingView ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState) { bZoomTimingView = (NewState == ECheckBoxState::Checked); })
			]

			// Row 4: Navigate button
			+ SGridPanel::Slot(0, 4)
			.ColumnSpan(2)
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 4.0f, 0.0f, RowPadding)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("Navigate", "Navigate"))
				.OnClicked(this, &SGoToTimeDialog::OnNavigateClicked)
			]

			// Row 5: Warning text (shown when input is invalid)
			+ SGridPanel::Slot(0, 5)
			.ColumnSpan(2)
			.Padding(0.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(WarningText, STextBlock)
				.ColorAndOpacity(FLinearColor::Yellow)
				.AutoWrapText(true)
				.Visibility(EVisibility::Collapsed)
			]
		]
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SGoToTimeDialog::OnTimeTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnEnter)
	{
		ExecuteNavigation();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SGoToTimeDialog::OnNavigateClicked()
{
	ExecuteNavigation();
	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SGoToTimeDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		if (TSharedPtr<SWindow> ParentWindow = ParentWindowPtr.Pin())
		{
			ParentWindow->RequestDestroyWindow();
		}
		return FReply::Handled();
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SGoToTimeDialog::ExecuteNavigation()
{
	FString TimeStr = TimeBox->GetText().ToString();
	TimeStr.ReplaceInline(TEXT(","), TEXT("."));
	TimeStr.TrimStartAndEndInline();

	if (TimeStr.IsEmpty())
	{
		WarningText->SetText(LOCTEXT("EnterTime", "Enter a time"));
		WarningText->SetVisibility(EVisibility::Visible);
		FSlateApplication::Get().SetKeyboardFocus(TimeBox, EFocusCause::SetDirectly);
		return;
	}

	FString TimeRangeDurationStr = TimeRangeDurationBox->GetText().ToString();
	TimeRangeDurationStr.ReplaceInline(TEXT(","), TEXT("."));
	TimeRangeDurationStr.TrimStartAndEndInline();

	if ((bSelectTimeRange || bZoomTimingView) && TimeRangeDurationStr.IsEmpty())
	{
		WarningText->SetText(LOCTEXT("EnterTimeRangeDuration", "Enter a time range duration"));
		WarningText->SetVisibility(EVisibility::Visible);
		FSlateApplication::Get().SetKeyboardFocus(TimeRangeDurationBox, EFocusCause::SetDirectly);
		return;
	}

	const FTimeFilterValueConverter TimeConverter;

	// Show warning if timestamp failed to parse
	double TargetTime = 0.0;
	FText TimeError;
	if (!TimeConverter.Convert(TimeStr, TargetTime, TimeError))
	{
		WarningText->SetText(TimeError);
		WarningText->SetVisibility(EVisibility::Visible);
		FSlateApplication::Get().SetKeyboardFocus(TimeBox, EFocusCause::SetDirectly);
		return;
	}

	// Show warning if select or zoom is enabled, but time range duration failed to parse
	double TimeRangeDuration = InitialTimeRangeDuration;
	double ParsedTimeRangeDuration = 0.0;
	FText TimeRangeDurationError;
	if (!TimeConverter.Convert(TimeRangeDurationStr, ParsedTimeRangeDuration, TimeRangeDurationError))
	{
		if (bSelectTimeRange || bZoomTimingView)
		{
			WarningText->SetText(TimeRangeDurationError);
			WarningText->SetVisibility(EVisibility::Visible);
			FSlateApplication::Get().SetKeyboardFocus(TimeRangeDurationBox, EFocusCause::SetDirectly);
			return;
		}

		// If neither is enabled, ignore the parsing error and continue to navigate. We still tried to
		// parse the duration so that preferences are saved regardless of whether currently in use or not.
	}
	else
	{
		// Parsed duration successfully. Clamp to a minimum value to avoid extreme zoom in.
		TimeRangeDuration = FMath::Max(0.00001, ParsedTimeRangeDuration);
	}

	// Parameters are valid. Broadcast delegate so that STimingView can execute the command.
	NavigateDelegate.ExecuteIfBound(TargetTime, bSelectTimeRange, bZoomTimingView, TimeRangeDuration);

	// Close modal on navigate with valid params
	if (TSharedPtr<SWindow> ParentWindow = ParentWindowPtr.Pin())
	{
		ParentWindow->RequestDestroyWindow();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
