// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SEditableTextBox;
class STextBlock;
class SWindow;

namespace UE::Insights::TimingProfiler
{

DECLARE_DELEGATE_FourParams(FOnGoToTimeNavigate, double /*TargetTime*/, bool /*bSelectTimeRange*/, bool /*bZoomTimingView*/, double /*TimeRangeDuration*/);

class SGoToTimeDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGoToTimeDialog)
		: _ParentWindow()
		, _InitialTime(-1.0)
		, _InitialSelectTimeRange(true)
		, _InitialZoomTimingView(true)
		, _InitialTimeRangeDuration(1.0)
	{}
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ARGUMENT(double, InitialTime)
		SLATE_ARGUMENT(bool, InitialSelectTimeRange)
		SLATE_ARGUMENT(bool, InitialZoomTimingView)
		SLATE_ARGUMENT(double, InitialTimeRangeDuration)
		SLATE_EVENT(FOnGoToTimeNavigate, OnNavigate)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	TSharedPtr<SEditableTextBox> GetTimeBox() const { return TimeBox; }

private:
	FReply OnNavigateClicked();
	void OnTimeTextCommitted(const FText& InText, ETextCommit::Type InCommitType);
	void ExecuteNavigation();

	TWeakPtr<SWindow> ParentWindowPtr;

	double InitialTimeRangeDuration;
	TSharedPtr<SEditableTextBox> TimeBox;
	TSharedPtr<SEditableTextBox> TimeRangeDurationBox;
	TSharedPtr<STextBlock> WarningText;

	FOnGoToTimeNavigate NavigateDelegate;
	bool bSelectTimeRange = true;
	bool bZoomTimingView = true;
};

} // namespace UE::Insights::TimingProfiler
