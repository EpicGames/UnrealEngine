// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStatsTableCell.h"

#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/TimingProfiler/Widgets/SStatsTableRow.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::SStatsView"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SStatsTableCell::Construct(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	TablePtr = InArgs._TablePtr;
	ColumnPtr = InArgs._ColumnPtr;
	StatsNodePtr = InArgs._StatsNodePtr;

	ensure(TablePtr.IsValid());
	ensure(ColumnPtr.IsValid());
	ensure(StatsNodePtr.IsValid());

	SetHoveredCellDelegate = InArgs._OnSetHoveredCell;

	ChildSlot
	[
		GenerateWidgetForColumn(InArgs, TableRow)
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SStatsTableCell::GenerateWidgetForColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	if (InArgs._IsNameColumn)
	{
		return GenerateWidgetForNameColumn(InArgs, TableRow);
	}
	else
	{
		return GenerateWidgetForStatsColumn(InArgs, TableRow);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SStatsTableCell::GenerateWidgetForNameColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	return
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SExpanderArrow, TableRow)
		]

		// Info icon + tooltip
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Visibility(this, &SStatsTableCell::GetHintIconVisibility)
			.Image(FInsightsStyle::GetBrush("Icons.Hint.TreeItem"))
			.ToolTip(GetRowToolTip(TableRow))
		]

		// Color box
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.Visibility(this, &SStatsTableCell::GetBoxVisibility)
			.WidthOverride(14.0f)
			.HeightOverride(14.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
				.BorderBackgroundColor(this, &SStatsTableCell::GetBoxColorAndOpacity)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
			]
		]

		// Name
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(this, &SStatsTableCell::GetDisplayName)
			.HighlightText(InArgs._HighlightText)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			.ColorAndOpacity(this, &SStatsTableCell::GetColorAndOpacity)
			.ShadowColorAndOpacity(this, &SStatsTableCell::GetShadowColorAndOpacity)
		]

		// Name Suffix
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Visibility(this, &SStatsTableCell::HasExtraDisplayName)
			.Text(this, &SStatsTableCell::GetExtraDisplayName)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			.ColorAndOpacity(this, &SStatsTableCell::GetExtraColorAndOpacity)
			.ShadowColorAndOpacity(this, &SStatsTableCell::GetShadowColorAndOpacity)
		]
	;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<IToolTip> SStatsTableCell::GetRowToolTip(const TSharedRef<ITableRow>& TableRow) const
{
	TSharedRef<SStatsTableRow> Row = StaticCastSharedRef<SStatsTableRow, ITableRow>(TableRow);
	return Row->GetRowToolTip();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SStatsTableCell::GetValueAsText() const
{
	return ColumnPtr->GetValueAsText(*StatsNodePtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SStatsTableCell::GenerateWidgetForStatsColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	return
		SNew(SHorizontalBox)

		// Value
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		.HAlign(ColumnPtr->GetHorizontalAlignment())
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(this, &SStatsTableCell::GetValueAsText)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			.ColorAndOpacity(this, &SStatsTableCell::GetStatsColorAndOpacity)
			.ShadowColorAndOpacity(this, &SStatsTableCell::GetShadowColorAndOpacity)
		]
	;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
