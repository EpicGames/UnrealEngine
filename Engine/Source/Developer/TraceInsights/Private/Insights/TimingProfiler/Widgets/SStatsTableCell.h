// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/StatsNode.h"

class ITableRow;

namespace UE::Insights
{
	class FTable;
	class FTableColumn;
}

namespace UE::Insights::TimingProfiler
{

DECLARE_DELEGATE_ThreeParams(FSetHoveredStatsTableCell, TSharedPtr<FTable> /*TablePtr*/, TSharedPtr<FTableColumn> /*ColumnPtr*/, FStatsNodePtr /*StatsNodePtr*/);

class SStatsTableCell : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStatsTableCell) {}
		SLATE_EVENT(FSetHoveredStatsTableCell, OnSetHoveredCell)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(TSharedPtr<FTable>, TablePtr)
		SLATE_ARGUMENT(TSharedPtr<FTableColumn>, ColumnPtr)
		SLATE_ARGUMENT(FStatsNodePtr, StatsNodePtr)
		SLATE_ARGUMENT(bool, IsNameColumn)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow);

protected:
	TSharedRef<SWidget> GenerateWidgetForColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow);
	TSharedRef<SWidget> GenerateWidgetForNameColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow);
	TSharedRef<SWidget> GenerateWidgetForStatsColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow);

	/**
	 * The system will use this event to notify a widget that the cursor has entered it. This event is NOT bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 */
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
		SetHoveredCellDelegate.ExecuteIfBound(TablePtr, ColumnPtr, StatsNodePtr);
	}

	/**
	 * The system will use this event to notify a widget that the cursor has left it. This event is NOT bubbled.
	 *
	 * @param MouseEvent Information about the input event
	 */
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		SCompoundWidget::OnMouseLeave(MouseEvent);
		SetHoveredCellDelegate.ExecuteIfBound(nullptr, nullptr, nullptr);
	}

	/**
	 * Called during drag and drop when the drag enters a widget.
	 *
	 * Enter/Leave events in slate are meant as lightweight notifications.
	 * So we do not want to capture mouse or set focus in response to these.
	 * However, OnDragEnter must also support external APIs (e.g. OLE Drag/Drop)
	 * Those require that we let them know whether we can handle the content
	 * being dragged OnDragEnter.
	 *
	 * The concession is to return a can_handled/cannot_handle
	 * boolean rather than a full FReply.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether the contents of the DragDropEvent can potentially be processed by this widget.
	 */
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		SCompoundWidget::OnDragEnter(MyGeometry, DragDropEvent);
		SetHoveredCellDelegate.ExecuteIfBound(TablePtr, ColumnPtr, StatsNodePtr);
	}

	/**
	 * Called during drag and drop when the drag leaves a widget.
	 *
	 * @param DragDropEvent   The drag and drop event.
	 */
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent)  override
	{
		SCompoundWidget::OnDragLeave(DragDropEvent);
		SetHoveredCellDelegate.ExecuteIfBound(nullptr, nullptr, nullptr);
	}

	EVisibility GetBoxVisibility() const
	{
		return StatsNodePtr->IsAddedToGraph() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility GetHintIconVisibility() const
	{
		return IsHovered() ? EVisibility::Visible : EVisibility::Hidden;
	}

	TSharedPtr<class IToolTip> GetRowToolTip(const TSharedRef<ITableRow>& TableRow) const;

	FText GetDisplayName() const
	{
		return StatsNodePtr->GetDisplayName();
	}

	FText GetExtraDisplayName() const
	{
		return StatsNodePtr->GetExtraDisplayName();
	}

	EVisibility HasExtraDisplayName() const
	{
		return StatsNodePtr->HasExtraDisplayName() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	FText GetValueAsText() const;

	FSlateColor GetColorAndOpacity() const
	{
		const FLinearColor TextColor =
			StatsNodePtr->IsFiltered() ?
				FLinearColor(1.0f, 1.0f, 1.0f, 0.5f) :
				FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
		return TextColor;
	}

	FSlateColor GetExtraColorAndOpacity() const
	{
		const FLinearColor TextColor =
			StatsNodePtr->IsFiltered() ?
				FLinearColor(0.5f, 0.5f, 0.5f, 0.5f) :
				FLinearColor(0.5f, 0.5f, 0.5f, 1.0f);
		return TextColor;
	}

	FSlateColor GetStatsColorAndOpacity() const
	{
		const FLinearColor TextColor =
			StatsNodePtr->IsFiltered() ?
				FLinearColor(1.0f, 1.0f, 1.0f, 0.5f) :
			StatsNodePtr->GetAggregatedStats().Count == 0 ?
				FLinearColor(1.0f, 1.0f, 1.0f, 0.6f) :
				FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
		return TextColor;
	}

	FSlateColor GetBoxColorAndOpacity() const
	{
		return StatsNodePtr->GetColor();
	}

	FLinearColor GetShadowColorAndOpacity() const
	{
		const FLinearColor ShadowColor =
			StatsNodePtr->IsFiltered() ?
				FLinearColor(0.f, 0.f, 0.f, 0.25f) :
				FLinearColor(0.0f, 0.0f, 0.0f, 0.5f);
		return ShadowColor;
	}

protected:
	/** A shared pointer to the table view model. */
	TSharedPtr<FTable> TablePtr;

	/** A shared pointer to the table column view model. */
	TSharedPtr<FTableColumn> ColumnPtr;

	/** A shared pointer to the stats node. */
	FStatsNodePtr StatsNodePtr;

	FSetHoveredStatsTableCell SetHoveredCellDelegate;
};

} // namespace UE::Insights::TimingProfier
