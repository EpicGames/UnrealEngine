// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStatsTableRow.h"

#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/TimingProfiler/Widgets/SStatsTableCell.h"
#include "Insights/TimingProfiler/Widgets/SStatsViewTooltip.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::SStatsView"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SStatsTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	OnShouldBeEnabled = InArgs._OnShouldBeEnabled;
	IsColumnVisibleDelegate = InArgs._OnIsColumnVisible;
	GetColumnOutlineHAlignmentDelegate = InArgs._OnGetColumnOutlineHAlignmentDelegate;
	SetHoveredCellDelegate = InArgs._OnSetHoveredCell;

	HighlightText = InArgs._HighlightText;

	TablePtr = InArgs._TablePtr;
	StatsNodePtr = InArgs._StatsNodePtr;

	RowToolTip = MakeShared<SStatsCounterTableRowToolTip>(StatsNodePtr);

	SetEnabled(TAttribute<bool>(this, &SStatsTableRow::HandleShouldBeEnabled));

	SMultiColumnTableRow<FStatsNodePtr>::Construct(SMultiColumnTableRow<FStatsNodePtr>::FArguments(), InOwnerTableView);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SStatsTableRow::GenerateWidgetForColumn(const FName& ColumnId)
{
	TSharedPtr<FTableColumn> ColumnPtr = TablePtr->FindColumnChecked(ColumnId);

	return
		SNew(SOverlay)
		.Visibility(EVisibility::SelfHitTestInvisible)

		+SOverlay::Slot()
		.Padding(0.0f)
		[
			SNew(SImage)
			.Image(FInsightsStyle::GetBrush("TreeTable.RowBackground"))
			.ColorAndOpacity(this, &SStatsTableRow::GetBackgroundColorAndOpacity)
		]

		+SOverlay::Slot()
		.Padding(0.0f)
		[
			SNew(SImage)
			.Image(this, &SStatsTableRow::GetOutlineBrush, ColumnId)
			.ColorAndOpacity(this, &SStatsTableRow::GetOutlineColorAndOpacity)
		]

		+SOverlay::Slot()
		[
			SNew(SStatsTableCell, SharedThis(this))
			.Visibility(this, &SStatsTableRow::IsColumnVisible, ColumnId)
			.TablePtr(TablePtr)
			.ColumnPtr(ColumnPtr)
			.StatsNodePtr(StatsNodePtr)
			.HighlightText(HighlightText)
			.IsNameColumn(ColumnPtr->IsHierarchy())
			.OnSetHoveredCell(this, &SStatsTableRow::OnSetHoveredCell)
		];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SStatsTableRow::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return SMultiColumnTableRow<FStatsNodePtr>::OnDragDetected(MyGeometry, MouseEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<IToolTip> SStatsTableRow::GetRowToolTip() const
{
	return RowToolTip.ToSharedRef();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsTableRow::InvalidateContent()
{
	RowToolTip->InvalidateWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SStatsTableRow::GetBackgroundColorAndOpacity() const
{
	if (StatsNodePtr->GetType() == EStatsNodeType::Group)
	{
		return FLinearColor(0.0f, 0.0f, 0.5f, 1.0f);
	}
	else
	{
		switch (StatsNodePtr->GetDataType())
		{
		case EStatsNodeDataType::Double:	return FLinearColor(0.05f, 0.05f, 0.0f, 1.0f);
		case EStatsNodeDataType::Int64:		return FLinearColor(0.01f, 0.01f, 0.0f, 1.0f);
		default:							return FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SStatsTableRow::GetBackgroundColorAndOpacity(double Time) const
{
	const FLinearColor Color =	Time > FTimeValue::Second      ? FLinearColor(0.3f, 0.0f, 0.0f, 1.0f) :
								Time > FTimeValue::Millisecond ? FLinearColor(0.3f, 0.1f, 0.0f, 1.0f) :
								Time > FTimeValue::Microsecond ? FLinearColor(0.0f, 0.1f, 0.0f, 1.0f) :
								                                 FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
	return Color;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SStatsTableRow::GetOutlineColorAndOpacity() const
{
	return FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* SStatsTableRow::GetOutlineBrush(const FName ColumnId) const
{
	EHorizontalAlignment HAlign = HAlign_Center;
	if (GetColumnOutlineHAlignmentDelegate.IsBound())
	{
		HAlign = GetColumnOutlineHAlignmentDelegate.Execute(ColumnId);
	}
	return FInsightsStyle::GetOutlineBrush(HAlign);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsTableRow::HandleShouldBeEnabled() const
{
	if (StatsNodePtr->IsGroup())
	{
		return true;
	}
	if (OnShouldBeEnabled.IsBound())
	{
		return OnShouldBeEnabled.Execute(StatsNodePtr);
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EVisibility SStatsTableRow::IsColumnVisible(const FName ColumnId) const
{
	EVisibility Result = EVisibility::Collapsed;
	if (IsColumnVisibleDelegate.IsBound())
	{
		Result = IsColumnVisibleDelegate.Execute(ColumnId) ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return Result;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsTableRow::OnSetHoveredCell(TSharedPtr<FTable> InTablePtr, TSharedPtr<FTableColumn> InColumnPtr, FStatsNodePtr InStatsNodePtr)
{
	SetHoveredCellDelegate.ExecuteIfBound(InTablePtr, InColumnPtr, InStatsNodePtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
