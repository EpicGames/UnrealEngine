// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStatsView.h"

#include "DesktopPlatformModule.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GenericPlatform/GenericPlatformTime.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformFileManager.h"
#include "Logging/MessageLog.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Views/STableViewBase.h"

// TraceServices
#include "TraceServices/Model/Counters.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "InsightsCore/Table/Widgets/SAsyncOperationStatus.h"

// TraceInsights
#include "Insights/Common/Utf8FileWriter.h"
#include "Insights/InsightsStyle.h"
#include "Insights/Log.h"
#include "Insights/Table/ViewModels/TableCommands.h"
#include "Insights/TimingProfiler/GraphTracks/TimingGraphSeries.h"
#include "Insights/TimingProfiler/GraphTracks/TimingGraphTrack.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/ViewModels/CounterAggregator.h"
#include "Insights/TimingProfiler/ViewModels/StatsNodeHelper.h"
#include "Insights/TimingProfiler/ViewModels/StatsViewColumnFactory.h"
#include "Insights/TimingProfiler/ViewModels/TimingExporter.h"
#include "Insights/TimingProfiler/Widgets/SStatsTableRow.h"
#include "Insights/TimingProfiler/Widgets/SStatsViewTooltip.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/TimingProfilerCommon.h"
#include "Insights/Widgets/STimingView.h"

#include <limits>

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::SStatsView"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FStatsViewCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

class FStatsViewCommands : public TCommands<FStatsViewCommands>
{
public:
	FStatsViewCommands()
	: TCommands<FStatsViewCommands>(
		TEXT("CountersViewCommands"),
		NSLOCTEXT("Contexts", "CountersViewCommands", "Insights - Counters View"),
		NAME_None,
		FInsightsStyle::GetStyleSetName())
	{
	}

	virtual ~FStatsViewCommands()
	{
	}

	// UI_COMMAND takes long for the compiler to optimize
	UE_DISABLE_OPTIMIZATION_SHIP
	virtual void RegisterCommands() override
	{
		UI_COMMAND(Command_GenerateNewColor,
			"Generate New Color",
			"Generates a new random color for the selected counters.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_EditColor,
			"Edit Color...",
			"Changes color for the selected counters.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_CopyToClipboard,
			"Copy To Clipboard",
			"Copies the selection (counters and their aggregated statistics) to clipboard.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Control, EKeys::C));

		UI_COMMAND(Command_CopyNameToClipboard,
			"Copy Name To Clipboard",
			"Copies the name of the selected counter to the clipboard.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::C));

		UI_COMMAND(Command_Export,
			"Export...",
			"Exports the selection (counters and their aggregated statistics) to a text file (tab-separated values or comma-separated values).",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Control, EKeys::S));

		UI_COMMAND(Command_ExportValues,
			"Export Values...",
			"Exports the values of the selected counter to a text file (tab-separated values or comma-separated values).\nExports the values only in the selected time region (if any) or the entire session if no time region is selected.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_ExportOps,
			"Export Operations...",
			"Exports the incremental operations/values of the selected counter to a text file (tab-separated values or comma-separated values).\nExports the ops/values only in the selected time region (if any) or the entire session if no time region is selected.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_ExportCounters,
			"Export Counters...",
			"Exports the list of counters to a text file (tab-separated values or comma-separated values).",
			EUserInterfaceActionType::Button,
			FInputChord());
	}
	UE_ENABLE_OPTIMIZATION_SHIP

	TSharedPtr<FUICommandInfo> Command_GenerateNewColor;
	TSharedPtr<FUICommandInfo> Command_EditColor;
	TSharedPtr<FUICommandInfo> Command_CopyToClipboard;
	TSharedPtr<FUICommandInfo> Command_CopyNameToClipboard;
	TSharedPtr<FUICommandInfo> Command_Export;
	TSharedPtr<FUICommandInfo> Command_ExportValues;
	TSharedPtr<FUICommandInfo> Command_ExportOps;
	TSharedPtr<FUICommandInfo> Command_ExportCounters;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// SStatsView
////////////////////////////////////////////////////////////////////////////////////////////////////

SStatsView::SStatsView()
	: Table(MakeShared<FTable>())
	, ColumnBeingSorted(GetDefaultColumnBeingSorted())
	, ColumnSortMode(GetDefaultColumnSortMode())
	, Aggregator(MakeShared<FCounterAggregator>())
{
	FMemory::Memset(FilterByNodeType, 1);
	FMemory::Memset(FilterByDataType, 1);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SStatsView::~SStatsView()
{
	// Remove ourselves from the Insights manager.
	if (FInsightsManager::Get().IsValid())
	{
		FInsightsManager::Get()->GetSessionChangedEvent().RemoveAll(this);
		FInsightsManager::Get()->GetSessionAnalysisCompletedEvent().RemoveAll(this);
	}

	FStatsViewCommands::Unregister();

	Session.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::InitCommandList()
{
	FStatsViewCommands::Register();
	CommandList = MakeShared<FUICommandList>();
	CommandList->MapAction(FStatsViewCommands::Get().Command_GenerateNewColor, FExecuteAction::CreateSP(this, &SStatsView::ContextMenu_GenerateNewColor_Execute), FCanExecuteAction::CreateSP(this, &SStatsView::ContextMenu_GenerateNewColor_CanExecute));
	CommandList->MapAction(FStatsViewCommands::Get().Command_EditColor, FExecuteAction::CreateSP(this, &SStatsView::ContextMenu_EditColor_Execute), FCanExecuteAction::CreateSP(this, &SStatsView::ContextMenu_EditColor_CanExecute));
	CommandList->MapAction(FStatsViewCommands::Get().Command_CopyToClipboard, FExecuteAction::CreateSP(this, &SStatsView::ContextMenu_CopySelectedToClipboard_Execute), FCanExecuteAction::CreateSP(this, &SStatsView::ContextMenu_CopySelectedToClipboard_CanExecute));
	CommandList->MapAction(FStatsViewCommands::Get().Command_CopyNameToClipboard, FExecuteAction::CreateSP(this, &SStatsView::ContextMenu_CopySelectedNameToClipboard_Execute), FCanExecuteAction::CreateSP(this, &SStatsView::ContextMenu_CopySelectedNameToClipboard_CanExecute));
	CommandList->MapAction(FStatsViewCommands::Get().Command_Export, FExecuteAction::CreateSP(this, &SStatsView::ContextMenu_Export_Execute), FCanExecuteAction::CreateSP(this, &SStatsView::ContextMenu_Export_CanExecute));
	CommandList->MapAction(FStatsViewCommands::Get().Command_ExportValues, FExecuteAction::CreateSP(this, &SStatsView::ContextMenu_ExportValues_Execute), FCanExecuteAction::CreateSP(this, &SStatsView::ContextMenu_ExportValues_CanExecute));
	CommandList->MapAction(FStatsViewCommands::Get().Command_ExportOps, FExecuteAction::CreateSP(this, &SStatsView::ContextMenu_ExportOps_Execute), FCanExecuteAction::CreateSP(this, &SStatsView::ContextMenu_ExportOps_CanExecute));
	CommandList->MapAction(FStatsViewCommands::Get().Command_ExportCounters, FExecuteAction::CreateSP(this, &SStatsView::ContextMenu_ExportCounters_Execute), FCanExecuteAction::CreateSP(this, &SStatsView::ContextMenu_ExportCounters_CanExecute));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SStatsView::Construct(const FArguments& InArgs)
{
	SAssignNew(ExternalScrollbar, SScrollBar)
	.AlwaysShowScrollbar(true);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoHeight()
		.Padding(2.0f, 2.0f, 2.0f, 2.0f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				// Search box
				+ SHorizontalBox::Slot()
				.Padding(2.0f)
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SAssignNew(SearchBox, SSearchBox)
					.HintText(LOCTEXT("SearchBoxHint", "Search counters or groups"))
					.OnTextChanged(this, &SStatsView::SearchBox_OnTextChanged)
					.IsEnabled(this, &SStatsView::SearchBox_IsEnabled)
					.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search counter or group."))
				]

				// Filter by type (Int64)
				+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 4.0f, 0.0f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					GetToggleButtonForDataType(EStatsNodeDataType::Int64)
				]

				// Filter by type (Double)
				+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 4.0f, 0.0f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					GetToggleButtonForDataType(EStatsNodeDataType::Double)
				]

				// Filter out counters with zero instance count
				+ SHorizontalBox::Slot()
				.Padding(2.0f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
					.HAlign(HAlign_Center)
					.Padding(3.0f)
					.OnCheckStateChanged(this, &SStatsView::FilterOutZeroCountStats_OnCheckStateChanged)
					.IsChecked(this, &SStatsView::FilterOutZeroCountStats_IsChecked)
					.ToolTipText(LOCTEXT("FilterOutZeroCountStats_Tooltip", "Filter out the counters having zero total instance count (aggregated stats)."))
					[
						SNew(SImage)
						.Image(FInsightsStyle::Get().GetBrush("Icons.ZeroCountFilter"))
					]
				]
			]

			// Group by
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("GroupByText", "Group by"))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.MinDesiredWidth(128.0f)
					[
						SAssignNew(GroupByComboBox, SComboBox<TSharedPtr<EStatsGroupingMode>>)
						.ToolTipText(this, &SStatsView::GroupBy_GetSelectedTooltipText)
						.OptionsSource(&GroupByOptionsSource)
						.OnSelectionChanged(this, &SStatsView::GroupBy_OnSelectionChanged)
						.OnGenerateWidget(this, &SStatsView::GroupBy_OnGenerateWidget)
						[
							SNew(STextBlock)
							.Text(this, &SStatsView::GroupBy_GetSelectedText)
						]
					]
				]
			]
		]

		// Tree view
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.0f, 2.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(0.0f)
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(TreeView, STreeView<FStatsNodePtr>)
					.ExternalScrollbar(ExternalScrollbar)
					.SelectionMode(ESelectionMode::Multi)
					.TreeItemsSource(&FilteredGroupNodes)
					.OnGetChildren(this, &SStatsView::TreeView_OnGetChildren)
					.OnGenerateRow(this, &SStatsView::TreeView_OnGenerateRow)
					.OnSelectionChanged(this, &SStatsView::TreeView_OnSelectionChanged)
					.OnMouseButtonDoubleClick(this, &SStatsView::TreeView_OnMouseButtonDoubleClick)
					.OnContextMenuOpening(FOnContextMenuOpening::CreateSP(this, &SStatsView::TreeView_GetMenuContent))
					.HeaderRow
					(
						SAssignNew(TreeViewHeaderRow, SHeaderRow)
						.Visibility(EVisibility::Visible)
					)
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				.Padding(16.0f)
				[
					SAssignNew(AsyncOperationStatus, SAsyncOperationStatus, Aggregator)
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f)
			[
				SNew(SBox)
				.WidthOverride(FOptionalSize(13.0f))
				[
					ExternalScrollbar.ToSharedRef()
				]
			]
		]

		// Status bar
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(0.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
			.BorderBackgroundColor(FLinearColor(0.05f, 0.1f, 0.2f, 1.0f))
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Margin(FMargin(4.0f, 1.0f, 4.0f, 1.0f))
				.Text(LOCTEXT("EmptyAggregationNote", "Please select a time range to update the aggregated statistics!"))
				.ColorAndOpacity(FLinearColor(1.0f, 0.75f, 0.5f, 1.0f))
				.Visibility_Lambda([this]()
				{
					return Aggregator->IsEmptyTimeInterval() && !Aggregator->IsRunning() ?
						EVisibility::Visible : EVisibility::Collapsed;
				})
			]
		]
	];

	InitializeAndShowHeaderColumns();

	// Create the search filters: text based, type based etc.
	TextFilter = MakeShared<FStatsNodeTextFilter>(FStatsNodeTextFilter::FItemToStringArray::CreateSP(this, &SStatsView::HandleItemToStringArray));
	Filters = MakeShared<FStatsNodeFilterCollection>();
	Filters->Add(TextFilter);

	CreateGroupByOptionsSources();
	CreateSortings();

	InitCommandList();

	// Register ourselves with the Insights manager.
	FInsightsManager::Get()->GetSessionChangedEvent().AddSP(this, &SStatsView::InsightsManager_OnSessionChanged);
	FInsightsManager::Get()->GetSessionAnalysisCompletedEvent().AddSP(this, &SStatsView::InsightsManager_OnSessionAnalysisCompleted);

	// Update the Session (i.e. when analysis session was already started).
	InsightsManager_OnSessionChanged();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> SStatsView::TreeView_GetMenuContent()
{
	int32 NumSelectedNodes = TreeView->GetNumItemsSelected();
	FStatsNodePtr SingleSelectedNode;
	bool bIsValidNode = false;

	if (NumSelectedNodes == 1)
	{
		TArray<FStatsNodePtr> SelectedNodes = TreeView->GetSelectedItems();
		check(SelectedNodes.Num() == 1);
		SingleSelectedNode = SelectedNodes[0];
		if (SingleSelectedNode.IsValid())
		{
			bIsValidNode = (SingleSelectedNode->GetType() != EStatsNodeType::Group);
		}
		else
		{
			NumSelectedNodes = 0;
		}
	}

	FText SelectionStr;
	if (NumSelectedNodes == 0)
	{
		SelectionStr = LOCTEXT("NothingSelected", "Nothing selected");
	}
	else if (NumSelectedNodes == 1)
	{
		check(SingleSelectedNode.IsValid());
		FString ItemName = SingleSelectedNode->GetDisplayName().ToString();
		const int32 MaxStringLen = 64;
		if (ItemName.Len() > MaxStringLen)
		{
			ItemName = ItemName.Left(MaxStringLen) + TEXT("...");
		}
		SelectionStr = FText::FromString(ItemName);
	}
	else
	{
		SelectionStr = FText::Format(LOCTEXT("MultipleSelection_Fmt", "{0} selected items"), FText::AsNumber(NumSelectedNodes));
	}

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList.ToSharedRef());

	// Selection menu
	MenuBuilder.BeginSection("Selection", LOCTEXT("ContextMenu_Section_Selection", "Selection"));
	{
		struct FLocal
		{
			static bool ReturnFalse()
			{
				return false;
			}
		};

		FUIAction DummyUIAction;
		DummyUIAction.CanExecuteAction = FCanExecuteAction::CreateStatic(&FLocal::ReturnFalse);
		MenuBuilder.AddMenuEntry
		(
			SelectionStr,
			LOCTEXT("ContextMenu_Selection", "Currently selected items"),
			FSlateIcon(),
			DummyUIAction,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	// Counter options section
	MenuBuilder.BeginSection("CounterOptions", LOCTEXT("ContextMenu_Section_CounterOptions", "Counter Options"));
	{
		auto CanExecute = [this, bIsValidNode]()
		{
			return bIsValidNode && GetTimingView().IsValid();
		};

		// Add/remove series to/from graph track
		{
			FUIAction Action_ToggleCounterInGraphTrack;
			Action_ToggleCounterInGraphTrack.CanExecuteAction = FCanExecuteAction::CreateLambda(CanExecute);
			Action_ToggleCounterInGraphTrack.ExecuteAction = FExecuteAction::CreateSP(this, &SStatsView::ToggleTimingViewMainGraphEventSeries, SingleSelectedNode);

			if (bIsValidNode &&
				IsSeriesInTimingViewMainGraph(SingleSelectedNode))
			{
				MenuBuilder.AddMenuEntry
				(
					LOCTEXT("ContextMenu_RemoveFromGraphTrack", "Remove series from graph track"),
					LOCTEXT("ContextMenu_RemoveFromGraphTrack_Desc", "Removes the series containing event instances of the selected counter from the Main Graph track."),
					FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.RemoveGraphSeries"),
					Action_ToggleCounterInGraphTrack,
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}
			else
			{
				MenuBuilder.AddMenuEntry
				(
					LOCTEXT("ContextMenu_AddToGraphTrack", "Add series to graph track"),
					LOCTEXT("ContextMenu_AddToGraphTrack_Desc", "Adds a series containing event instances of the selected counter to the Main Graph track."),
					FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.AddGraphSeries"),
					Action_ToggleCounterInGraphTrack,
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}
		}

		MenuBuilder.AddMenuEntry
		(
			FStatsViewCommands::Get().Command_GenerateNewColor,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Refresh")
		);

		MenuBuilder.AddMenuEntry
		(
			FStatsViewCommands::Get().Command_EditColor,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.EyeDropper")
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Misc", LOCTEXT("ContextMenu_Section_Misc", "Miscellaneous"));
	{
		MenuBuilder.AddMenuEntry
		(
			FStatsViewCommands::Get().Command_CopyToClipboard,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);

		MenuBuilder.AddMenuEntry
		(
			FStatsViewCommands::Get().Command_CopyNameToClipboard,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);

		MenuBuilder.AddMenuEntry
		(
			FStatsViewCommands::Get().Command_Export,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);

		MenuBuilder.AddMenuEntry
		(
			FStatsViewCommands::Get().Command_ExportValues,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);

		MenuBuilder.AddMenuEntry
		(
			FStatsViewCommands::Get().Command_ExportOps,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);

		MenuBuilder.AddMenuEntry
		(
			FStatsViewCommands::Get().Command_ExportCounters,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::TreeView_BuildSortByMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("SortColumn", LOCTEXT("ContextMenu_Section_SortColumn", "Sort Column"));

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (Column.IsVisible() && Column.CanBeSorted())
		{
			FUIAction Action_SortByColumn
			(
				FExecuteAction::CreateSP(this, &SStatsView::ContextMenu_SortByColumn_Execute, Column.GetId()),
				FCanExecuteAction::CreateSP(this, &SStatsView::ContextMenu_SortByColumn_CanExecute, Column.GetId()),
				FIsActionChecked::CreateSP(this, &SStatsView::ContextMenu_SortByColumn_IsChecked, Column.GetId())
			);
			MenuBuilder.AddMenuEntry
			(
				Column.GetTitleName(),
				Column.GetDescription(),
				FSlateIcon(),
				Action_SortByColumn,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}
	}

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("SortMode", LOCTEXT("ContextMenu_Section_SortMode", "Sort Mode"));
	{
		FUIAction Action_SortAscending
		(
			FExecuteAction::CreateSP(this, &SStatsView::ContextMenu_SortMode_Execute, EColumnSortMode::Ascending),
			FCanExecuteAction::CreateSP(this, &SStatsView::ContextMenu_SortMode_CanExecute, EColumnSortMode::Ascending),
			FIsActionChecked::CreateSP(this, &SStatsView::ContextMenu_SortMode_IsChecked, EColumnSortMode::Ascending)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_SortAscending", "Sort Ascending"),
			LOCTEXT("ContextMenu_SortAscending_Desc", "Sorts ascending."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortUp"),
			Action_SortAscending,
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		FUIAction Action_SortDescending
		(
			FExecuteAction::CreateSP(this, &SStatsView::ContextMenu_SortMode_Execute, EColumnSortMode::Descending),
			FCanExecuteAction::CreateSP(this, &SStatsView::ContextMenu_SortMode_CanExecute, EColumnSortMode::Descending),
			FIsActionChecked::CreateSP(this, &SStatsView::ContextMenu_SortMode_IsChecked, EColumnSortMode::Descending)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_SortDescending", "Sort Descending"),
			LOCTEXT("ContextMenu_SortDescending_Desc", "Sorts descending."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortDown"),
			Action_SortDescending,
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::TreeView_BuildViewColumnMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Columns", LOCTEXT("ContextMenu_Section_Columns", "Columns"));

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		FUIAction Action_ToggleColumn
		(
			FExecuteAction::CreateSP(this, &SStatsView::ToggleColumnVisibility, Column.GetId()),
			FCanExecuteAction::CreateSP(this, &SStatsView::CanToggleColumnVisibility, Column.GetId()),
			FIsActionChecked::CreateSP(this, &SStatsView::IsColumnVisible, Column.GetId())
		);
		MenuBuilder.AddMenuEntry
		(
			Column.GetTitleName(),
			Column.GetDescription(),
			FSlateIcon(),
			Action_ToggleColumn,
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::InitializeAndShowHeaderColumns()
{
	// Create columns.
	TArray<TSharedRef<FTableColumn>> Columns;
	FStatsViewColumnFactory::CreateStatsViewColumns(Columns);
	Table->SetColumns(Columns);

	// Show columns.
	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		if (ColumnRef->ShouldBeVisible())
		{
			ShowColumn(ColumnRef->GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SStatsView::GetColumnHeaderText(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.GetShortName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SStatsView::TreeViewHeaderRow_GenerateColumnMenu(const FTableColumn& Column)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, NULL);

	MenuBuilder.BeginSection("Sorting", LOCTEXT("ContextMenu_Section_Sorting", "Sorting"));
	{
		if (Column.CanBeSorted())
		{
			FUIAction Action_SortAscending
			(
				FExecuteAction::CreateSP(this, &SStatsView::HeaderMenu_SortMode_Execute, Column.GetId(), EColumnSortMode::Ascending),
				FCanExecuteAction::CreateSP(this, &SStatsView::HeaderMenu_SortMode_CanExecute, Column.GetId(), EColumnSortMode::Ascending),
				FIsActionChecked::CreateSP(this, &SStatsView::HeaderMenu_SortMode_IsChecked, Column.GetId(), EColumnSortMode::Ascending)
			);
			MenuBuilder.AddMenuEntry
			(
				FText::Format(LOCTEXT("ContextMenu_SortAscending_Fmt", "Sort Ascending (by {0})"), Column.GetTitleName()),
				FText::Format(LOCTEXT("ContextMenu_SortAscending_Desc_Fmt", "Sorts ascending by {0}."), Column.GetTitleName()),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortUp"),
				Action_SortAscending,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);

			FUIAction Action_SortDescending
			(
				FExecuteAction::CreateSP(this, &SStatsView::HeaderMenu_SortMode_Execute, Column.GetId(), EColumnSortMode::Descending),
				FCanExecuteAction::CreateSP(this, &SStatsView::HeaderMenu_SortMode_CanExecute, Column.GetId(), EColumnSortMode::Descending),
				FIsActionChecked::CreateSP(this, &SStatsView::HeaderMenu_SortMode_IsChecked, Column.GetId(), EColumnSortMode::Descending)
			);
			MenuBuilder.AddMenuEntry
			(
				FText::Format(LOCTEXT("ContextMenu_SortDescending_Fmt", "Sort Descending (by {0})"), Column.GetTitleName()),
				FText::Format(LOCTEXT("ContextMenu_SortDescending_Desc_Fmt", "Sorts descending by {0}."), Column.GetTitleName()),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortDown"),
				Action_SortDescending,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_SortBy_SubMenu", "Sort By"),
			LOCTEXT("ContextMenu_SortBy_SubMenu_Desc", "Sorts by a column."),
			FNewMenuDelegate::CreateSP(this, &SStatsView::TreeView_BuildSortByMenu),
			false,
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.SortBy")
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("ColumnVisibility", LOCTEXT("ContextMenu_Section_ColumnVisibility", "Column Visibility"));
	{
		if (Column.CanBeHidden())
		{
			FUIAction Action_HideColumn
			(
				FExecuteAction::CreateSP(this, &SStatsView::HideColumn, Column.GetId()),
				FCanExecuteAction::CreateSP(this, &SStatsView::CanHideColumn, Column.GetId())
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_HideColumn", "Hide"),
				LOCTEXT("ContextMenu_HideColumn_Desc", "Hides the selected column."),
				FSlateIcon(),
				Action_HideColumn,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_ViewColumn_SubMenu", "View Column"),
			LOCTEXT("ContextMenu_ViewColumn_SubMenu_Desc", "Hides or shows columns."),
			FNewMenuDelegate::CreateSP(this, &SStatsView::TreeView_BuildViewColumnMenu),
			false,
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ViewColumn")
		);

		FUIAction Action_ShowAllColumns
		(
			FExecuteAction::CreateSP(this, &SStatsView::ContextMenu_ShowAllColumns_Execute),
			FCanExecuteAction::CreateSP(this, &SStatsView::ContextMenu_ShowAllColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_ShowAllColumns", "Show All Columns"),
			LOCTEXT("ContextMenu_ShowAllColumns_Desc", "Resets tree view to show all columns."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_ShowAllColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);

		FUIAction Action_ShowMinMaxMedColumns
		(
			FExecuteAction::CreateSP(this, &SStatsView::ContextMenu_ShowMinMaxMedColumns_Execute),
			FCanExecuteAction::CreateSP(this, &SStatsView::ContextMenu_ShowMinMaxMedColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_ShowMinMaxMedColumns", "Reset Columns to Min/Max/Median Preset"),
			LOCTEXT("ContextMenu_ShowMinMaxMedColumns_Desc", "Resets columns to Min/Max/Median preset."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_ShowMinMaxMedColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);

		FUIAction Action_ResetColumns
		(
			FExecuteAction::CreateSP(this, &SStatsView::ContextMenu_ResetColumns_Execute),
			FCanExecuteAction::CreateSP(this, &SStatsView::ContextMenu_ResetColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_ResetColumns", "Reset Columns to Default"),
			LOCTEXT("ContextMenu_ResetColumns_Desc", "Resets columns to default."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_ResetColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::InsightsManager_OnSessionChanged()
{
	TSharedPtr<const TraceServices::IAnalysisSession> NewSession = FInsightsManager::Get()->GetSession();
	if (NewSession != Session)
	{
		Session = NewSession;
		Reset();
	}
	else
	{
		UpdateTree();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::InsightsManager_OnSessionAnalysisCompleted()
{
	// Re-sync the list of counters to update the "<unknown>" counter names.
	RebuildTree(true);

	// Aggregate stats automatically for the entire session (but only if user didn't made a time selection yet).
	if (Aggregator->IsEmptyTimeInterval() && !Aggregator->IsRunning())
	{
		TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();
		InsightsManager->UpdateSessionDuration();
		const double SessionDuration = InsightsManager->GetSessionDuration();

		constexpr double Delta = 0.0; // session padding

		Aggregator->Cancel();
		Aggregator->SetTimeInterval(0.0 - Delta, SessionDuration + Delta);
		Aggregator->Start();

		if (ColumnBeingSorted == NAME_None)
		{
			// Restore sorting...
			SetSortModeForColumn(GetDefaultColumnBeingSorted(), GetDefaultColumnSortMode());
			TreeView_Refresh();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::UpdateTree()
{
	CreateGroups();
	SortTreeNodes();
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ApplyFiltering()
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	FilteredGroupNodes.Reset();

	// Apply filter to all groups and its children.
	const int32 NumGroups = GroupNodes.Num();
	for (int32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
	{
		FStatsNodePtr& GroupPtr = GroupNodes[GroupIndex];
		GroupPtr->ClearFilteredChildren();
		const bool bIsGroupVisible = Filters->PassesAllFilters(GroupPtr);

		const TArray<FBaseTreeNodePtr>& GroupChildren = GroupPtr->GetChildren();
		int32 NumVisibleChildren = 0;
		for (const FBaseTreeNodePtr& ChildPtr : GroupChildren)
		{
			check(ChildPtr->Is<FStatsNode>());
			const FStatsNodePtr& NodePtr = StaticCastSharedPtr<FStatsNode>(ChildPtr);

			const bool bIsChildVisible =
				(!bFilterOutZeroCountStats || NodePtr->GetAggregatedStats().Count > 0) &&
				FilterByNodeType[static_cast<int>(NodePtr->GetType())] &&
				FilterByDataType[static_cast<int>(NodePtr->GetDataType())] &&
				Filters->PassesAllFilters(NodePtr);

			if (bIsChildVisible)
			{
				// Add a child.
				GroupPtr->AddFilteredChild(NodePtr);
				NumVisibleChildren++;
			}
		}

		if (bIsGroupVisible || NumVisibleChildren > 0)
		{
			// Add a group.
			FilteredGroupNodes.Add(GroupPtr);
			GroupPtr->SetExpansion(true);
		}
		else
		{
			GroupPtr->SetExpansion(false);
		}
	}

	// Only expand tree nodes if we have a text filter.
	const bool bNonEmptyTextFilter = !TextFilter->GetRawFilterText().IsEmpty();
	if (bNonEmptyTextFilter)
	{
		if (!bExpansionSaved)
		{
			ExpandedNodes.Empty();
			TreeView->GetExpandedItems(ExpandedNodes);
			bExpansionSaved = true;
		}

		for (const FStatsNodePtr& GroupPtr : FilteredGroupNodes)
		{
			TreeView->SetItemExpansion(GroupPtr, GroupPtr->IsExpanded());
		}
	}
	else
	{
		if (bExpansionSaved)
		{
			// Restore previously expanded nodes when the text filter is disabled.
			TreeView->ClearExpandedItems();
			for (auto It = ExpandedNodes.CreateConstIterator(); It; ++It)
			{
				TreeView->SetItemExpansion(*It, true);
			}
			bExpansionSaved = false;
		}
	}

	// Update aggregations for groups.
	for (FStatsNodePtr& GroupPtr : FilteredGroupNodes)
	{
		FAggregatedStats& AggregatedStats = GroupPtr->GetAggregatedStats();

		GroupPtr->ResetAggregatedStats();

		AggregatedStats.DoubleStats.Min = std::numeric_limits<double>::max();
		AggregatedStats.DoubleStats.Max = std::numeric_limits<double>::lowest();
		constexpr double NotAvailableDoubleValue = std::numeric_limits<double>::quiet_NaN();
		AggregatedStats.DoubleStats.Average = NotAvailableDoubleValue;
		AggregatedStats.DoubleStats.Median = NotAvailableDoubleValue;
		AggregatedStats.DoubleStats.LowerQuartile = NotAvailableDoubleValue;
		AggregatedStats.DoubleStats.UpperQuartile = NotAvailableDoubleValue;

		AggregatedStats.Int64Stats.Min = std::numeric_limits<int64>::max();
		AggregatedStats.Int64Stats.Max = std::numeric_limits<int64>::lowest();
		constexpr int64 NotAvailableIntegerValue = 0;// std::numeric_limits<int64>::max();
		AggregatedStats.Int64Stats.Average = NotAvailableIntegerValue;
		AggregatedStats.Int64Stats.Median = NotAvailableIntegerValue;
		AggregatedStats.Int64Stats.LowerQuartile = NotAvailableIntegerValue;
		AggregatedStats.Int64Stats.UpperQuartile = NotAvailableIntegerValue;

		EStatsNodeDataType GroupDataType = EStatsNodeDataType::InvalidOrMax;

		const TArray<FBaseTreeNodePtr>& GroupChildren = GroupPtr->GetFilteredChildren();
		for (const FBaseTreeNodePtr& ChildPtr : GroupChildren)
		{
			check(ChildPtr->Is<FStatsNode>());
			const FStatsNodePtr& NodePtr = StaticCastSharedPtr<FStatsNode>(ChildPtr);
			const FAggregatedStats& NodeAggregatedStats = NodePtr->GetAggregatedStats();

			if (NodeAggregatedStats.Count > 0)
			{
				AggregatedStats.Count += NodeAggregatedStats.Count;

				AggregatedStats.DoubleStats.Sum += NodeAggregatedStats.DoubleStats.Sum;
				AggregatedStats.DoubleStats.Min = FMath::Min(AggregatedStats.DoubleStats.Min, NodeAggregatedStats.DoubleStats.Min);
				AggregatedStats.DoubleStats.Max = FMath::Max(AggregatedStats.DoubleStats.Max, NodeAggregatedStats.DoubleStats.Max);

				AggregatedStats.Int64Stats.Sum += NodeAggregatedStats.Int64Stats.Sum;
				AggregatedStats.Int64Stats.Min = FMath::Min(AggregatedStats.Int64Stats.Min, NodeAggregatedStats.Int64Stats.Min);
				AggregatedStats.Int64Stats.Max = FMath::Max(AggregatedStats.Int64Stats.Max, NodeAggregatedStats.Int64Stats.Max);
			}

			if (GroupDataType == EStatsNodeDataType::InvalidOrMax)
			{
				GroupDataType = NodePtr->GetDataType();
			}
			else
			{
				if (GroupDataType != NodePtr->GetDataType())
				{
					GroupDataType = EStatsNodeDataType::Undefined;
				}
			}
		}

		// If not all children have same type, reset aggregated stats for group.
		if (GroupDataType >= EStatsNodeDataType::Undefined)
		{
			AggregatedStats.DoubleStats.Sum = NotAvailableDoubleValue;
			AggregatedStats.DoubleStats.Min = NotAvailableDoubleValue;
			AggregatedStats.DoubleStats.Max = NotAvailableDoubleValue;

			AggregatedStats.Int64Stats.Sum = NotAvailableIntegerValue;
			AggregatedStats.Int64Stats.Min = NotAvailableIntegerValue;
			AggregatedStats.Int64Stats.Max = NotAvailableIntegerValue;
		}
		GroupPtr->SetDataType(GroupDataType);
	}

	// Request tree refresh
	TreeView->RequestTreeRefresh();

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.1)
	{
		UE_LOGF(LogTimingProfiler, Log, "[Counters] Tree view filtered in %.3fs (%d counters)",
			TotalTime, StatsNodes.Num());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::HandleItemToStringArray(const FStatsNodePtr& FStatsNodePtr, TArray<FString>& OutSearchStrings) const
{
	OutSearchStrings.Add(FStatsNodePtr->GetDisplayName().ToString());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SStatsView::GetToggleButtonForNodeType(const EStatsNodeType NodeType)
{
	return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.Padding(FMargin(4.0f, 2.0f, 4.0f, 2.0f))
		.HAlign(HAlign_Center)
		.OnCheckStateChanged(this, &SStatsView::FilterByStatsType_OnCheckStateChanged, NodeType)
		.IsChecked(this, &SStatsView::FilterByStatsType_IsChecked, NodeType)
		.ToolTipText(StatsNodeTypeHelper::ToDescription(NodeType))
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SImage)
				.Image(StatsNodeTypeHelper::GetIcon(NodeType))
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(StatsNodeTypeHelper::ToText(NodeType))
			]
		];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SStatsView::GetToggleButtonForDataType(const EStatsNodeDataType DataType)
{
	return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.Padding(FMargin(4.0f, 2.0f, 4.0f, 2.0f))
		.HAlign(HAlign_Center)
		.OnCheckStateChanged(this, &SStatsView::FilterByStatsDataType_OnCheckStateChanged, DataType)
		.IsChecked(this, &SStatsView::FilterByStatsDataType_IsChecked, DataType)
		.ToolTipText(StatsNodeDataTypeHelper::ToDescription(DataType))
		[
			SNew(SHorizontalBox)

			//+ SHorizontalBox::Slot()
			//.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			//.AutoWidth()
			//.VAlign(VAlign_Center)
			//[
			//	SNew(SImage)
			//	.Image(StatsNodeDataTypeHelper::GetIcon(DataType))
			//]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(StatsNodeDataTypeHelper::ToText(DataType))
			]
		];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::FilterOutZeroCountStats_OnCheckStateChanged(ECheckBoxState NewRadioState)
{
	bFilterOutZeroCountStats = (NewRadioState == ECheckBoxState::Checked);
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState SStatsView::FilterOutZeroCountStats_IsChecked() const
{
	return bFilterOutZeroCountStats ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::FilterByStatsType_OnCheckStateChanged(ECheckBoxState NewRadioState, const EStatsNodeType InNodeType)
{
	FilterByNodeType[static_cast<int>(InNodeType)] = (NewRadioState == ECheckBoxState::Checked);
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState SStatsView::FilterByStatsType_IsChecked(const EStatsNodeType InNodeType) const
{
	return FilterByNodeType[static_cast<int>(InNodeType)] ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::FilterByStatsDataType_OnCheckStateChanged(ECheckBoxState NewRadioState, const EStatsNodeDataType InDataType)
{
	FilterByDataType[static_cast<int>(InDataType)] = (NewRadioState == ECheckBoxState::Checked);
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState SStatsView::FilterByStatsDataType_IsChecked(const EStatsNodeDataType InDataType) const
{
	return FilterByDataType[static_cast<int>(InDataType)] ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// TreeView
////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::TreeView_Refresh()
{
	if (TreeView.IsValid())
	{
		TreeView->RequestTreeRefresh();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::TreeView_OnSelectionChanged(FStatsNodePtr SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		//FStatsNodePtr SelectedNode = GetSingleSelectedStatsNode();
		//if (SelectedNode.IsValid() && SelectedNode->GetType() != EStatsNodeType::Group)
		//{
		//	FTimingProfilerManager::Get()->SetSelectedCounter(SelectedNode->GetCounterId());
		//}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FStatsNodePtr SStatsView::GetSingleSelectedStatsNode() const
{
	if (TreeView->GetNumItemsSelected() != 1)
	{
		return nullptr;
	}
	TArray<FStatsNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	check(SelectedNodes.Num() == 1);
	return SelectedNodes[0];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::TreeView_OnGetChildren(FStatsNodePtr InParent, TArray<FStatsNodePtr>& OutChildren)
{
	const TArray<FBaseTreeNodePtr>& Children = InParent->GetFilteredChildren();
	OutChildren.Reset(Children.Num());
	for (const FBaseTreeNodePtr& Child : Children)
	{
		check(Child->Is<FStatsNode>());
		OutChildren.Add(StaticCastSharedPtr<FStatsNode>(Child));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::TreeView_OnMouseButtonDoubleClick(FStatsNodePtr NodePtr)
{
	if (NodePtr->GetType() == EStatsNodeType::Group)
	{
		const bool bIsGroupExpanded = TreeView->IsItemExpanded(NodePtr);
		TreeView->SetItemExpansion(NodePtr, !bIsGroupExpanded);
	}
	else
	{
		ToggleTimingViewMainGraphEventSeries(NodePtr);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Tree View's Table Row
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableRow> SStatsView::TreeView_OnGenerateRow(FStatsNodePtr NodePtr, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<ITableRow> TableRow =
		SNew(SStatsTableRow, OwnerTable)
		.OnShouldBeEnabled(this, &SStatsView::TableRow_ShouldBeEnabled)
		.OnIsColumnVisible(this, &SStatsView::IsColumnVisible)
		.OnSetHoveredCell(this, &SStatsView::TableRow_SetHoveredCell)
		.OnGetColumnOutlineHAlignmentDelegate(this, &SStatsView::TableRow_GetColumnOutlineHAlignment)
		.HighlightText(this, &SStatsView::TableRow_GetHighlightText)
		.TablePtr(Table)
		.StatsNodePtr(NodePtr);

	return TableRow;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::TableRow_ShouldBeEnabled(FStatsNodePtr NodePtr) const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::TableRow_SetHoveredCell(TSharedPtr<FTable> InTablePtr, TSharedPtr<FTableColumn> InColumnPtr, FStatsNodePtr InNodePtr)
{
	HoveredColumnId = InColumnPtr ? InColumnPtr->GetId() : FName();

	const bool bIsAnyMenusVisible = FSlateApplication::Get().AnyMenusVisible();
	if (!HasMouseCapture() && !bIsAnyMenusVisible)
	{
		HoveredNodePtr = InNodePtr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EHorizontalAlignment SStatsView::TableRow_GetColumnOutlineHAlignment(const FName ColumnId) const
{
	const TIndirectArray<SHeaderRow::FColumn>& Columns = TreeViewHeaderRow->GetColumns();

	// First column
	if (Columns.Num() == 0 ||
		Columns[0].ColumnId == ColumnId)
	{
		return HAlign_Left;
	}
	// Last column
	else if (Columns[Columns.Num() - 1].ColumnId == ColumnId)
	{
		return HAlign_Right;
	}
	// Middle columns
	{
		return HAlign_Center;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SStatsView::TableRow_GetHighlightText() const
{
	return SearchBox->GetText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SearchBox
////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::SearchBox_OnTextChanged(const FText& InFilterText)
{
	TextFilter->SetRawFilterText(InFilterText);
	SearchBox->SetError(TextFilter->GetFilterErrorText());
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::SearchBox_IsEnabled() const
{
	return StatsNodes.Num() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Grouping
////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::CreateGroups()
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	if (GroupingMode == EStatsGroupingMode::Flat)
	{
		GroupNodes.Reset();

		FStatsNodePtr GroupPtr = MakeShared<FStatsNode>(LOCTEXT("GroupName_All", "All"));
		GroupNodes.Add(GroupPtr);

		for (const FStatsNodePtr& NodePtr : StatsNodes)
		{
			GroupPtr->AddChildAndSetParent(NodePtr);
		}
		TreeView->SetItemExpansion(GroupPtr, true);
	}
	// Creates groups based on stat metadata groups.
	else if (GroupingMode == EStatsGroupingMode::ByMetaGroupName)
	{
		TMap<FName, FStatsNodePtr> GroupNodeSet;
		for (const FStatsNodePtr& NodePtr : StatsNodes)
		{
			const FName GroupName = NodePtr->GetMetaGroupName();
			FStatsNodePtr GroupPtr = GroupNodeSet.FindRef(GroupName);
			if (!GroupPtr)
			{
				const FText DisplayName = FText::FromName(GroupName);
				GroupPtr = GroupNodeSet.Add(GroupName, MakeShared<FStatsNode>(DisplayName));
			}
			GroupPtr->AddChildAndSetParent(NodePtr);
			TreeView->SetItemExpansion(GroupPtr, true);
		}
		GroupNodeSet.KeySort([](const FName& A, const FName& B) { return A.Compare(B) < 0; }); // sort groups by name
		GroupNodeSet.GenerateValueArray(GroupNodes);
	}
	// Creates one group for each node type.
	else if (GroupingMode == EStatsGroupingMode::ByType)
	{
		TMap<EStatsNodeType, FStatsNodePtr> GroupNodeSet;
		for (const FStatsNodePtr& NodePtr : StatsNodes)
		{
			const EStatsNodeType NodeType = NodePtr->GetType();
			FStatsNodePtr GroupPtr = GroupNodeSet.FindRef(NodeType);
			if (!GroupPtr)
			{
				const FText GroupName = StatsNodeTypeHelper::ToText(NodeType);
				GroupPtr = GroupNodeSet.Add(NodeType, MakeShared<FStatsNode>(GroupName));
			}
			GroupPtr->AddChildAndSetParent(NodePtr);
			TreeView->SetItemExpansion(GroupPtr, true);
		}
		GroupNodeSet.KeySort([](const EStatsNodeType& A, const EStatsNodeType& B) { return A < B; }); // sort groups by type
		GroupNodeSet.GenerateValueArray(GroupNodes);
	}
	// Creates one group for each data type.
	else if (GroupingMode == EStatsGroupingMode::ByDataType)
	{
		TMap<EStatsNodeDataType, FStatsNodePtr> GroupNodeSet;
		for (const FStatsNodePtr& NodePtr : StatsNodes)
		{
			const EStatsNodeDataType DataType = NodePtr->GetDataType();
			FStatsNodePtr GroupPtr = GroupNodeSet.FindRef(DataType);
			if (!GroupPtr)
			{
				const FText GroupName = StatsNodeDataTypeHelper::ToText(DataType);
				GroupPtr = GroupNodeSet.Add(DataType, MakeShared<FStatsNode>(GroupName));
			}
			GroupPtr->AddChildAndSetParent(NodePtr);
			TreeView->SetItemExpansion(GroupPtr, true);
		}
		GroupNodeSet.KeySort([](const EStatsNodeDataType& A, const EStatsNodeDataType& B) { return A < B; }); // sort groups by data type
		GroupNodeSet.GenerateValueArray(GroupNodes);
	}
	// Creates one group for one letter.
	else if (GroupingMode == EStatsGroupingMode::ByName)
	{
		TMap<TCHAR, FStatsNodePtr> GroupNodeSet;
		for (const FStatsNodePtr& NodePtr : StatsNodes)
		{
			FString FirstLetterStr(NodePtr->GetDisplayName().ToString().Left(1).ToUpper());
			const TCHAR FirstLetter = FirstLetterStr.Len() > 0 ? FirstLetterStr[0] : TEXT('\0');
			FStatsNodePtr GroupPtr = GroupNodeSet.FindRef(FirstLetter);
			if (!GroupPtr)
			{
				const FText GroupName = FText::FromString(FirstLetterStr);
				GroupPtr = GroupNodeSet.Add(FirstLetter, MakeShared<FStatsNode>(GroupName));
			}
			GroupPtr->AddChildAndSetParent(NodePtr);
		}
		GroupNodeSet.KeySort([](const TCHAR& A, const TCHAR& B) { return A < B; }); // sort groups alphabetically
		GroupNodeSet.GenerateValueArray(GroupNodes);
	}
	// Creates one group for each logarithmic range ie. 0, [1 .. 10), [10 .. 100), [100 .. 1K), etc.
	else if (GroupingMode == EStatsGroupingMode::ByCount)
	{
		const TCHAR* Orders[] =
		{
			TEXT("1"), TEXT("10"), TEXT("100"),
			TEXT("1K"), TEXT("10K"), TEXT("100K"),
			TEXT("1M"), TEXT("10M"), TEXT("100M"),
			TEXT("1G"), TEXT("10G"), TEXT("100G"),
			TEXT("1T")
		};
		const uint32 MaxOrder = UE_ARRAY_COUNT(Orders);
		TMap<uint32, FStatsNodePtr> GroupNodeSet;
		for (const FStatsNodePtr& NodePtr : StatsNodes)
		{
			uint64 InstanceCount = NodePtr->GetAggregatedStats().Count;
			uint32 Order = 0;
			while (InstanceCount)
			{
				InstanceCount /= 10;
				Order++;
			}
			FStatsNodePtr GroupPtr = GroupNodeSet.FindRef(Order);
			if (!GroupPtr)
			{
				const FText GroupName =
					(Order == 0) ?
						LOCTEXT("GroupName_ZeroCount", "Count == 0") :
					(Order < MaxOrder) ?
						FText::Format(LOCTEXT("GroupName_CountFmt1", "Count: [{0} .. {1})"), FText::FromString(Orders[Order - 1]), FText::FromString(Orders[Order])) :
						FText::Format(LOCTEXT("GroupName_CountFmt2", "Count >= {0}"), FText::FromString(Orders[MaxOrder - 1]));
				GroupPtr = GroupNodeSet.Add(Order, MakeShared<FStatsNode>(GroupName));
			}
			GroupPtr->AddChildAndSetParent(NodePtr);
		}
		GroupNodeSet.KeySort([](const uint32& A, const uint32& B) { return A > B; }); // sort groups by order
		GroupNodeSet.GenerateValueArray(GroupNodes);
	}
	else
	{
		unimplemented();
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.1)
	{
		UE_LOGF(LogTimingProfiler, Log, "[Counters] Tree view grouping updated in %.3fs (%d counters)",
			TotalTime, StatsNodes.Num());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::CreateGroupByOptionsSources()
{
	GroupByOptionsSource.Reset(3);

	// Must be added in order of elements in the EStatsGroupingMode.
	GroupByOptionsSource.Add(MakeShared<EStatsGroupingMode>(EStatsGroupingMode::Flat));
	GroupByOptionsSource.Add(MakeShared<EStatsGroupingMode>(EStatsGroupingMode::ByName));
	GroupByOptionsSource.Add(MakeShared<EStatsGroupingMode>(EStatsGroupingMode::ByMetaGroupName));
	GroupByOptionsSource.Add(MakeShared<EStatsGroupingMode>(EStatsGroupingMode::ByType));
	GroupByOptionsSource.Add(MakeShared<EStatsGroupingMode>(EStatsGroupingMode::ByDataType));
	GroupByOptionsSource.Add(MakeShared<EStatsGroupingMode>(EStatsGroupingMode::ByCount));

	EStatsGroupingModePtr* GroupingModePtrPtr = GroupByOptionsSource.FindByPredicate([&](const EStatsGroupingModePtr InGroupingModePtr) { return *InGroupingModePtr == GroupingMode; });
	if (GroupingModePtrPtr != nullptr)
	{
		GroupByComboBox->SetSelectedItem(*GroupingModePtrPtr);
	}

	GroupByComboBox->RefreshOptions();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::GroupBy_OnSelectionChanged(TSharedPtr<EStatsGroupingMode> NewGroupingMode, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct &&
		NewGroupingMode.IsValid() &&
		*NewGroupingMode != GroupingMode)
	{
		GroupingMode = *NewGroupingMode;

		CreateGroups();
		SortTreeNodes();
		ApplyFiltering();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SStatsView::GroupBy_OnGenerateWidget(TSharedPtr<EStatsGroupingMode> InGroupingMode) const
{
	check(InGroupingMode.IsValid());
	return SNew(STextBlock)
		.Text(StatsNodeGroupingHelper::ToText(*InGroupingMode))
		.ToolTipText(StatsNodeGroupingHelper::ToDescription(*InGroupingMode));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SStatsView::GroupBy_GetSelectedText() const
{
	return StatsNodeGroupingHelper::ToText(GroupingMode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SStatsView::GroupBy_GetSelectedTooltipText() const
{
	return StatsNodeGroupingHelper::ToDescription(GroupingMode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorting
////////////////////////////////////////////////////////////////////////////////////////////////////

const FName SStatsView::GetDefaultColumnBeingSorted()
{
	return FStatsViewColumns::NameColumnID;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const EColumnSortMode::Type SStatsView::GetDefaultColumnSortMode()
{
	return EColumnSortMode::Ascending;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::CreateSortings()
{
	AvailableSorters.Reset();
	CurrentSorter = nullptr;

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		if (ColumnRef->CanBeSorted())
		{
			TSharedPtr<ITableCellValueSorter> SorterPtr = ColumnRef->GetValueSorter();
			if (ensure(SorterPtr.IsValid()))
			{
				AvailableSorters.Add(SorterPtr);
			}
		}
	}

	UpdateCurrentSortingByColumn();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::UpdateCurrentSortingByColumn()
{
	TSharedPtr<FTableColumn> ColumnPtr = Table->FindColumn(ColumnBeingSorted);
	CurrentSorter = ColumnPtr.IsValid() ? ColumnPtr->GetValueSorter() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::SortTreeNodes()
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	if (CurrentSorter.IsValid())
	{
		for (FStatsNodePtr& Root : GroupNodes)
		{
			SortTreeNodesRec(*Root, *CurrentSorter);
		}
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.1)
	{
		UE_LOGF(LogTimingProfiler, Log, "[Counters] Tree view sorted (%ls, %c) in %.3fs (%d counters)",
			CurrentSorter.IsValid() ? *CurrentSorter->GetShortName().ToString() : TEXT("N/A"),
			(ColumnSortMode == EColumnSortMode::Type::Descending) ? TEXT('D') : TEXT('A'),
			TotalTime, StatsNodes.Num());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::SortTreeNodesRec(FStatsNode& Node, const ITableCellValueSorter& Sorter)
{
	ESortMode SortMode = (ColumnSortMode == EColumnSortMode::Type::Descending) ? ESortMode::Descending : ESortMode::Ascending;
	Node.SortChildren(Sorter, SortMode);

#if 0 // Current groupings creates only one level.
	for (FBaseTreeNodePtr ChildPtr : Node.GetChildren())
	{
		if (ChildPtr->GetChildrenCount() > 0)
		{
			check(ChildPtr->Is<FStatsNode>());
			SortTreeNodesRec(*StaticCastSharedPtr<FStatsNode>(ChildPtr), Sorter);
		}
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EColumnSortMode::Type SStatsView::GetSortModeForColumn(const FName ColumnId) const
{
	if (ColumnBeingSorted != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return ColumnSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::SetSortModeForColumn(const FName& ColumnId, const EColumnSortMode::Type SortMode)
{
	ColumnBeingSorted = ColumnId;
	ColumnSortMode = SortMode;
	UpdateCurrentSortingByColumn();

	SortTreeNodes();
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode)
{
	SetSortModeForColumn(ColumnId, SortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortMode action (HeaderMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::HeaderMenu_SortMode_IsChecked(const FName ColumnId, const EColumnSortMode::Type InSortMode)
{
	return ColumnBeingSorted == ColumnId && ColumnSortMode == InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::HeaderMenu_SortMode_CanExecute(const FName ColumnId, const EColumnSortMode::Type InSortMode) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.CanBeSorted();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::HeaderMenu_SortMode_Execute(const FName ColumnId, const EColumnSortMode::Type InSortMode)
{
	SetSortModeForColumn(ColumnId, InSortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortMode action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_SortMode_IsChecked(const EColumnSortMode::Type InSortMode)
{
	return ColumnSortMode == InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_SortMode_CanExecute(const EColumnSortMode::Type InSortMode) const
{
	return true; //ColumnSortMode != InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ContextMenu_SortMode_Execute(const EColumnSortMode::Type InSortMode)
{
	SetSortModeForColumn(ColumnBeingSorted, InSortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortByColumn action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_SortByColumn_IsChecked(const FName ColumnId)
{
	return ColumnId == ColumnBeingSorted;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_SortByColumn_CanExecute(const FName ColumnId) const
{
	return true; //ColumnId != ColumnBeingSorted;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ContextMenu_SortByColumn_Execute(const FName ColumnId)
{
	SetSortModeForColumn(ColumnId, EColumnSortMode::Descending);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ShowColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::CanShowColumn(const FName ColumnId) const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ShowColumn(const FName ColumnId)
{
	FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	Column.Show();

	SHeaderRow::FColumn::FArguments ColumnArgs;
	ColumnArgs
		.ColumnId(Column.GetId())
		.DefaultLabel(Column.GetShortName())
		.ToolTip(SStatsViewTooltip::GetColumnTooltip(Column))
		.HAlignHeader(Column.GetHorizontalAlignment())
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Fill)
		.InitialSortMode(Column.GetInitialSortMode())
		.SortMode(this, &SStatsView::GetSortModeForColumn, Column.GetId())
		.OnSort(this, &SStatsView::OnSortModeChanged)
		.FillWidth(Column.GetInitialWidth())
		//.FixedWidth(Column.IsFixedWidth() ? Column.GetInitialWidth() : TOptional<float>())
		.HeaderContent()
		[
			SNew(SBox)
			.HeightOverride(24.0f)
			.Padding(FMargin(0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SStatsView::GetColumnHeaderText, Column.GetId())
			]
		]
		.MenuContent()
		[
			TreeViewHeaderRow_GenerateColumnMenu(Column)
		];

	int32 ColumnIndex = 0;
	const int32 NewColumnPosition = Table->GetColumnPositionIndex(ColumnId);
	const int32 NumColumns = TreeViewHeaderRow->GetColumns().Num();
	for (; ColumnIndex < NumColumns; ColumnIndex++)
	{
		const SHeaderRow::FColumn& CurrentColumn = TreeViewHeaderRow->GetColumns()[ColumnIndex];
		const int32 CurrentColumnPosition = Table->GetColumnPositionIndex(CurrentColumn.ColumnId);
		if (NewColumnPosition < CurrentColumnPosition)
		{
			break;
		}
	}

	TreeViewHeaderRow->InsertColumn(ColumnArgs, ColumnIndex);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// HideColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::CanHideColumn(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.CanBeHidden();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::HideColumn(const FName ColumnId)
{
	FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	Column.Hide();

	TreeViewHeaderRow->RemoveColumn(ColumnId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ToggleColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::IsColumnVisible(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::CanToggleColumnVisibility(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return !Column.IsVisible() || Column.CanBeHidden();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ToggleColumnVisibility(const FName ColumnId)
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	if (Column.IsVisible())
	{
		HideColumn(ColumnId);
	}
	else
	{
		ShowColumn(ColumnId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// "Show All Columns" action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_ShowAllColumns_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ContextMenu_ShowAllColumns_Execute()
{
	ColumnBeingSorted = GetDefaultColumnBeingSorted();
	ColumnSortMode = GetDefaultColumnSortMode();
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (!Column.IsVisible())
		{
			ShowColumn(Column.GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// "Show Min/Max/Median Columns" action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_ShowMinMaxMedColumns_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ContextMenu_ShowMinMaxMedColumns_Execute()
{
	TSet<FName> Preset =
	{
		FStatsViewColumns::NameColumnID,
		//FStatsViewColumns::MetaGroupNameColumnID,
		//FStatsViewColumns::TypeColumnID,
		FStatsViewColumns::CountColumnID,
		FStatsViewColumns::SumColumnID,
		FStatsViewColumns::MaxColumnID,
		FStatsViewColumns::UpperQuartileColumnID,
		//FStatsViewColumns::AverageColumnID,
		FStatsViewColumns::MedianColumnID,
		FStatsViewColumns::LowerQuartileColumnID,
		FStatsViewColumns::MinColumnID,
		FStatsViewColumns::DiffColumnID,
	};

	ColumnBeingSorted = FStatsViewColumns::CountColumnID;
	ColumnSortMode = EColumnSortMode::Descending;
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		const bool bShouldBeVisible = Preset.Contains(Column.GetId());

		if (bShouldBeVisible && !Column.IsVisible())
		{
			ShowColumn(Column.GetId());
		}
		else if (!bShouldBeVisible && Column.IsVisible())
		{
			HideColumn(Column.GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ResetColumns action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_ResetColumns_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ContextMenu_ResetColumns_Execute()
{
	ColumnBeingSorted = GetDefaultColumnBeingSorted();
	ColumnSortMode = GetDefaultColumnSortMode();
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (Column.ShouldBeVisible() && !Column.IsVisible())
		{
			ShowColumn(Column.GetId());
		}
		else if (!Column.ShouldBeVisible() && Column.IsVisible())
		{
			HideColumn(Column.GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::Reset()
{
	Aggregator->Cancel();
	Aggregator->SetTimeInterval(0.0, 0.0);

	RebuildTree(true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Check if we need to update the list of counters, but not too often.
	static uint64 NextTimestamp = 0;
	const uint64 Time = FPlatformTime::Cycles64();
	if (Time > NextTimestamp)
	{
		RebuildTree(false);

		// 1000 counters --> check each 150ms
		// 10000 counters --> check each 600ms
		// 100000 counters --> check each 5.1s
		const double WaitTimeSec = 0.1 + StatsNodes.Num() / 20000.0;
		const uint64 WaitTime = static_cast<uint64>(WaitTimeSec / FPlatformTime::GetSecondsPerCycle64());
		NextTimestamp = Time + WaitTime;
	}

	Aggregator->Tick(Session, InCurrentTime, InDeltaTime, [this]() { FinishAggregation(); });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::RebuildTree(bool bResync)
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	FStopwatch SyncStopwatch;
	SyncStopwatch.Start();

	if (bResync)
	{
		StatsNodes.Empty();
		StatsNodesIdMap.Empty();
	}

	uint32 AddedNodeCount = 0;

	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::ICounterProvider& CountersProvider = TraceServices::ReadCounterProvider(*Session.Get());

		const uint32 PreviousNodeCount = StatsNodes.Num();
		const uint32 CounterCount = static_cast<uint32>(CountersProvider.GetCounterCount());
		if (CounterCount != PreviousNodeCount)
		{
			check(CounterCount > PreviousNodeCount);

			StatsNodes.Reserve(CounterCount);
			StatsNodesIdMap.Reserve(CounterCount);


			// Add nodes only for new counters.
			uint32 CounterIndex = PreviousNodeCount;
			CountersProvider.EnumerateCounters(
				[this, &CounterIndex, &AddedNodeCount]
				(uint32 CounterId, const TraceServices::ICounter& Counter)
				{
					auto DeduceGroup = [](const TraceServices::ICounter& Counter)
					{
						static const FName MemoryGroup(TEXT("Memory"));
						static const FName BandwidthGroup(TEXT("Bandwidth"));
						static const FName PercentGroup(TEXT("Percent"));
						static const FName MiscFloatGroup(TEXT("Misc_double"));
						static const FName MiscInt64Group(TEXT("Misc_int64"));

						switch (Counter.GetDisplayHint())
						{
						case TraceServices::CounterDisplayHint_Memory:
							return MemoryGroup;
						case TraceServices::CounterDisplayHint_Bandwidth:
							return BandwidthGroup;
						case TraceServices::CounterDisplayHint_Percent:
							return PercentGroup;
						}
						return Counter.IsFloatingPoint() ? MiscFloatGroup : MiscInt64Group;
					};

					FStatsNodePtr NodePtr = StatsNodesIdMap.FindRef(CounterId);
					if (!NodePtr)
					{
						++CounterIndex;
						const FText DisplayName = FText::FromName(Counter.GetName());
						const FName Group = Counter.GetGroup() ? Counter.GetGroup() : DeduceGroup(Counter);
						const EStatsNodeType Type = EStatsNodeType::Counter;
						const EStatsNodeDataType DataType = Counter.IsFloatingPoint() ? EStatsNodeDataType::Double : EStatsNodeDataType::Int64;
						NodePtr = MakeShared<FStatsNode>(CounterId, DisplayName, Group, Type, DataType);
						UpdateNode(NodePtr);
						StatsNodes.Add(NodePtr);
						StatsNodesIdMap.Add(CounterId, NodePtr);
						++AddedNodeCount;
					}
				});
			check(AddedNodeCount == CounterIndex - PreviousNodeCount);
			ensure(StatsNodes.Num() == CounterCount);
		}
	}

	SyncStopwatch.Stop();

	if (bResync || AddedNodeCount > 0)
	{
		// Disable sorting if too many items.
		if (StatsNodes.Num() > 10000)
		{
			ColumnBeingSorted = NAME_None;
			ColumnSortMode = GetDefaultColumnSortMode();
			UpdateCurrentSortingByColumn();
		}

		UpdateTree();
		Aggregator->Cancel();
		Aggregator->Start();

		// Save selection.
		TArray<FStatsNodePtr> SelectedItems;
		TreeView->GetSelectedItems(SelectedItems);

		TreeView->RebuildList();

		// Restore selection.
		if (SelectedItems.Num() > 0)
		{
			TreeView->ClearSelection();
			for (FStatsNodePtr& NodePtr : SelectedItems)
			{
				NodePtr = GetCounterNode(NodePtr->GetCounterId());
			}
			SelectedItems.RemoveAll([](const FStatsNodePtr& NodePtr) { return !NodePtr.IsValid(); });
			if (SelectedItems.Num() > 0)
			{
				TreeView->SetItemSelection(SelectedItems, true);
				TreeView->RequestScrollIntoView(SelectedItems.Last());
			}
		}
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.01)
	{
		const double SyncTime = SyncStopwatch.GetAccumulatedTime();
		UE_LOGF(LogTimingProfiler, Log, "[Counters] Tree view rebuilt in %.4fs (sync: %.4fs + update: %.4fs) --> %d counters (%d added)",
			TotalTime, SyncTime, TotalTime - SyncTime, StatsNodes.Num(), AddedNodeCount);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::UpdateNode(FStatsNodePtr NodePtr)
{
	bool bAddedToGraphFlag = false;

	if (!NodePtr->IsGroup())
	{
		TSharedPtr<FTimingGraphTrack> GraphTrack = GetTimingViewMainGraphTrack();
		if (GraphTrack.IsValid())
		{
			TSharedPtr<FTimingGraphSeries> Series = GraphTrack->GetStatsCounterSeries(NodePtr->GetCounterId());
			bAddedToGraphFlag = Series.IsValid();
		}
	}

	NodePtr->SetAddedToGraphFlag(bAddedToGraphFlag);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ResetStats()
{
	Aggregator->Cancel();
	Aggregator->SetTimeInterval(0.0, 0.0);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::UpdateStats(double StartTime, double EndTime)
{
	Aggregator->Cancel();
	Aggregator->SetTimeInterval(StartTime, EndTime);
	Aggregator->Start();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::FinishAggregation()
{
	for (const FStatsNodePtr& NodePtr : StatsNodes)
	{
		NodePtr->ResetAggregatedStats();
	}

	Aggregator->ApplyResultsTo(StatsNodesIdMap);
	Aggregator->ResetResults();

	// Invalidate all tree table rows.
	for (const FStatsNodePtr& NodePtr : StatsNodes)
	{
		TSharedPtr<ITableRow> TableRowPtr = TreeView->WidgetFromItem(NodePtr);
		if (TableRowPtr.IsValid())
		{
			TSharedPtr<SStatsTableRow> RowPtr = StaticCastSharedPtr<SStatsTableRow, ITableRow>(TableRowPtr);
			RowPtr->InvalidateContent();
		}
	}

	UpdateTree(); // grouping + sorting + filtering

	// Ensure the last selected item is visible.
	TArray<FStatsNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	if (SelectedNodes.Num() > 0)
	{
		TreeView->RequestScrollIntoView(SelectedNodes.Last());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FStatsNodePtr SStatsView::GetCounterNode(uint32 CounterId) const
{
	return StatsNodesIdMap.FindRef(CounterId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::SelectCounterNode(uint32 CounterId)
{
	FStatsNodePtr NodePtr = GetCounterNode(CounterId);
	if (NodePtr)
	{
		TreeView->SetSelection(NodePtr);
		TreeView->RequestScrollIntoView(NodePtr);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTimingGraphTrack> SStatsView::GetTimingViewMainGraphTrack() const
{
	TSharedPtr<STimingView> TimingView = GetTimingView();
	return TimingView.IsValid() ? TimingView->GetMainTimingGraphTrack() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ToggleGraphSeries(TSharedRef<FTimingGraphTrack> GraphTrack, FStatsNodeRef NodePtr) const
{
	const uint32 CounterId = NodePtr->GetCounterId();
	TSharedPtr<FTimingGraphSeries> Series = GraphTrack->GetStatsCounterSeries(CounterId);
	if (Series.IsValid())
	{
		GraphTrack->RemoveStatsCounterSeries(CounterId);
		GraphTrack->SetDirtyFlag();
		NodePtr->SetAddedToGraphFlag(false);
	}
	else
	{
		GraphTrack->Show();
		Series = GraphTrack->AddStatsCounterSeries(CounterId, NodePtr->GetColor());
		GraphTrack->SetDirtyFlag();
		NodePtr->SetAddedToGraphFlag(true);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::IsSeriesInTimingViewMainGraph(FStatsNodePtr CounterNode) const
{
	TSharedPtr<FTimingGraphTrack> GraphTrack = GetTimingViewMainGraphTrack();
	if (GraphTrack.IsValid())
	{
		const uint32 CounterId = CounterNode->GetCounterId();
		TSharedPtr<FTimingGraphSeries> Series = GraphTrack->GetStatsCounterSeries(CounterId);

		return Series.IsValid();
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ToggleTimingViewMainGraphEventSeries(FStatsNodePtr CounterNode) const
{
	TSharedPtr<FTimingGraphTrack> GraphTrack = GetTimingViewMainGraphTrack();
	if (GraphTrack.IsValid())
	{
		ToggleGraphSeries(GraphTrack.ToSharedRef(), CounterNode.ToSharedRef());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Copy to Clipboard
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_CopySelectedToClipboard_CanExecute() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ContextMenu_CopySelectedToClipboard_Execute()
{
	if (Table->IsValid())
	{
		const ESortMode SortMode = ColumnSortMode == EColumnSortMode::Ascending ? Insights::ESortMode::Ascending : Insights::ESortMode::Descending;
		UE::Insights::CopyToClipboard(Table.ToSharedRef(), TreeView->GetSelectedItems(), CurrentSorter, SortMode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_CopySelectedNameToClipboard_CanExecute() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ContextMenu_CopySelectedNameToClipboard_Execute()
{
	if (Table->IsValid())
	{
		const ESortMode SortMode = ColumnSortMode == EColumnSortMode::Ascending ? Insights::ESortMode::Ascending : Insights::ESortMode::Descending;
		UE::Insights::CopyNameToClipboard(Table.ToSharedRef(), TreeView->GetSelectedItems(), CurrentSorter, SortMode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Export
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_Export_CanExecute() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ContextMenu_Export_Execute()
{
	if (!Table->IsValid())
	{
		return;
	}

	TArray<FStatsNodePtr> SelectedStatsNodes = TreeView->GetSelectedItems();
	if (SelectedStatsNodes.Num() == 0)
	{
		return;
	}

	TArray<FBaseTreeNodePtr> SelectedNodes;
	for (FStatsNodePtr Node : SelectedStatsNodes)
	{
		SelectedNodes.Add(Node);
	}

	const FString DialogTitle = LOCTEXT("Export_Title", "Export Aggregated Counter Stats").ToString();
	const FString DefaultFile = TEXT("CounterStats.tsv");
	FString Filename;
	if (!OpenSaveTextFileDialog(DialogTitle, DefaultFile, Filename))
	{
		return;
	}

	TUniquePtr<IFileHandle> ExportFileHandle{ OpenExportFile(*Filename) };
	if (!ExportFileHandle)
	{
		return;
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	bool bIsCSV = Filename.EndsWith(TEXT(".csv"));
	FUtf8FileWriter Writer(ExportFileHandle.Get(), bIsCSV);

	TArray<TSharedRef<FTableColumn>> VisibleColumns;
	Table->GetVisibleColumns(VisibleColumns);

	// Write header.
	{
		bool bIsFirstColumn = true;
		for (const TSharedRef<FTableColumn>& ColumnRef : VisibleColumns)
		{
			if (bIsFirstColumn)
			{
				bIsFirstColumn = false;
			}
			else
			{
				Writer.AppendSeparator();
			}

			FString ColumnName = ColumnRef->GetShortName().ToString();
			Writer.AppendValue(ColumnName);
		}
		Writer.AppendLineEnd();
	}

	if (CurrentSorter.IsValid())
	{
		CurrentSorter->Sort(SelectedNodes, ColumnSortMode == EColumnSortMode::Ascending ? ESortMode::Ascending : ESortMode::Descending);
	}

	const int32 NodeCount = SelectedNodes.Num();
	for (int32 Index = 0; Index < NodeCount; Index++)
	{
		const FBaseTreeNodePtr& Node = SelectedNodes[Index];

		bool bIsFirstColumn = true;
		for (const TSharedRef<FTableColumn>& ColumnRef : VisibleColumns)
		{
			if (bIsFirstColumn)
			{
				bIsFirstColumn = false;
			}
			else
			{
				Writer.AppendSeparator();
			}

			FString Value = ColumnRef->GetValueAsSerializableString(*Node);
			Writer.AppendValue(Value);
		}
		Writer.AppendLineEnd();
	}

	Writer.Flush();
	ExportFileHandle->Flush();
	ExportFileHandle.Reset();

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TraceInsights, Log, "Exported aggregated counter stats to file in %.3fs (\"%ls\").", TotalTime, *Filename);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_ExportValues_CanExecute() const
{
	FStatsNodePtr SelectedNode = GetSingleSelectedStatsNode();
	return SelectedNode.IsValid() && !SelectedNode->IsGroup();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ContextMenu_ExportValues_Execute() const
{
	if (!Session.IsValid())
	{
		return;
	}

	FStatsNodePtr SelectedNode = GetSingleSelectedStatsNode();
	if (!SelectedNode.IsValid() || SelectedNode->IsGroup())
	{
		return;
	}

	const uint32 CounterId = SelectedNode->GetCounterId();
	FString CounterName = SelectedNode->GetDisplayName().ToString();

	const FString DialogTitle = LOCTEXT("ExportValues_Title", "Export Counter Values").ToString();

	FString DefaultFile = CounterName.Replace(TEXT("(1/frame)"), TEXT("(1 per frame)"));
	DefaultFile.ReplaceCharInline(TEXT('.'), TEXT('_'));
	DefaultFile = FPaths::MakeValidFileName(DefaultFile, TCHAR('_'));
	if (DefaultFile.IsEmpty() || DefaultFile[DefaultFile.Len() - 1] == TEXT(' '))
	{
		DefaultFile += TEXT('_');
	}
	DefaultFile = TEXT("CounterValues ") + DefaultFile + TEXT(".tsv");

	FString Filename;
	if (!OpenSaveTextFileDialog(DialogTitle, DefaultFile, Filename))
	{
		return;
	}

	FTimingExporter Exporter(*Session.Get());
	FTimingExporter::FExportCounterParams Params; // default

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Limit the time interval for enumeration (if a time range selection is made in Timing view).

	Params.IntervalStartTime = -std::numeric_limits<double>::infinity();
	Params.IntervalEndTime = +std::numeric_limits<double>::infinity();

	TSharedPtr<STimingView> TimingView = GetTimingView();
	if (TimingView.IsValid())
	{
		const double SelectionStartTime = TimingView->GetSelectionStartTime();
		const double SelectionEndTime = TimingView->GetSelectionEndTime();
		if (SelectionStartTime < SelectionEndTime)
		{
			Params.IntervalStartTime = SelectionStartTime;
			Params.IntervalEndTime = SelectionEndTime;
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	Exporter.ExportCounterAsText(*Filename, CounterId, Params);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_ExportOps_CanExecute() const
{
	FStatsNodePtr SelectedNode = GetSingleSelectedStatsNode();
	return SelectedNode.IsValid() && !SelectedNode->IsGroup();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ContextMenu_ExportOps_Execute() const
{
	if (!Session.IsValid())
	{
		return;
	}

	FStatsNodePtr SelectedNode = GetSingleSelectedStatsNode();
	if (!SelectedNode.IsValid() || SelectedNode->IsGroup())
	{
		return;
	}

	const uint32 CounterId = SelectedNode->GetCounterId();
	FString CounterName = SelectedNode->GetDisplayName().ToString();

	const FString DialogTitle = LOCTEXT("ExportOps_Title", "Export Counter Ops").ToString();

	FString DefaultFile = CounterName.Replace(TEXT("(1/frame)"), TEXT("(1 per frame)"));
	DefaultFile.ReplaceCharInline(TEXT('.'), TEXT('_'));
	DefaultFile = FPaths::MakeValidFileName(DefaultFile, TCHAR('_'));
	if (DefaultFile.IsEmpty() || DefaultFile[DefaultFile.Len() - 1] == TEXT(' '))
	{
		DefaultFile += TEXT('_');
	}
	DefaultFile = TEXT("CounterOps ") + DefaultFile + TEXT(".tsv");

	FString Filename;
	if (!OpenSaveTextFileDialog(DialogTitle, DefaultFile, Filename))
	{
		return;
	}

	FTimingExporter Exporter(*Session.Get());
	FTimingExporter::FExportCounterParams Params; // default

	Params.bExportOps = true; // export "operations" instead of "values"

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Limit the time interval for enumeration (if a time range selection is made in Timing view).

	Params.IntervalStartTime = -std::numeric_limits<double>::infinity();
	Params.IntervalEndTime = +std::numeric_limits<double>::infinity();

	TSharedPtr<STimingView> TimingView = GetTimingView();
	if (TimingView.IsValid())
	{
		const double SelectionStartTime = TimingView->GetSelectionStartTime();
		const double SelectionEndTime = TimingView->GetSelectionEndTime();
		if (SelectionStartTime < SelectionEndTime)
		{
			Params.IntervalStartTime = SelectionStartTime;
			Params.IntervalEndTime = SelectionEndTime;
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	Exporter.ExportCounterAsText(*Filename, CounterId, Params);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_ExportCounters_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ContextMenu_ExportCounters_Execute() const
{
	if (!Session.IsValid())
	{
		return;
	}

	const FString DialogTitle = LOCTEXT("ExportCounters_Title", "Export Counters").ToString();
	const FString DefaultFile = TEXT("Counters.tsv");
	FString Filename;
	if (!OpenSaveTextFileDialog(DialogTitle, DefaultFile, Filename))
	{
		return;
	}

	FTimingExporter Exporter(*Session.Get());
	FTimingExporter::FExportCountersParams Params; // default
	Exporter.ExportCountersAsText(*Filename, Params);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::OpenSaveTextFileDialog(const FString& InDialogTitle, const FString& InDefaultFile, FString& OutFilename) const
{
	TArray<FString> SaveFilenames;
	bool bDialogResult = false;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		const FString DefaultPath = FPaths::ProjectSavedDir();
		bDialogResult = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			InDialogTitle,
			DefaultPath,
			InDefaultFile,
			TEXT("Comma-Separated Values (*.csv)|*.csv|Tab-Separated Values (*.tsv)|*.tsv|Text Files (*.txt)|*.txt|All Files (*.*)|*.*"),
			EFileDialogFlags::None,
			SaveFilenames
		);
	}

	if (!bDialogResult || SaveFilenames.Num() == 0)
	{
		return false;
	}

	OutFilename = SaveFilenames[0];
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

IFileHandle* SStatsView::OpenExportFile(const TCHAR* InFilename) const
{
	IFileHandle* ExportFileHandle = FPlatformFileManager::Get().GetPlatformFile().OpenWrite(InFilename);

	if (ExportFileHandle == nullptr)
	{
		FName LogListingName = FTimingProfilerManager::Get()->GetLogListingName();
		FMessageLog ReportMessageLog((LogListingName != NAME_None) ? LogListingName : TEXT("Other"));
		ReportMessageLog.Error(LOCTEXT("FailedToOpenFile", "Export failed. Failed to open file for write."));
		ReportMessageLog.Notify();
		return nullptr;
	}

	return ExportFileHandle;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Generate and Edit Colors
////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_GenerateNewColor_CanExecute() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ContextMenu_GenerateNewColor_Execute() const
{
	TArray<FStatsNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	for (const FStatsNodePtr& SelectedNode : SelectedNodes)
	{
		if (SelectedNode->Is<FStatsNode>())
		{
			FStatsNode& StatsNode = SelectedNode->As<FStatsNode>();
			SetRandomColorToNode(StatsNode);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::SetColorToNode(FStatsNode& StatsNode, FLinearColor CustomColor) const
{
	StatsNode.SetColor(CustomColor);

	const uint32 CounterId = StatsNode.GetCounterId();
	const FLinearColor BorderColor(CustomColor.R + 0.4f, CustomColor.G + 0.4f, CustomColor.B + 0.4f, 1.0f);

	TSharedPtr<FTimingGraphTrack> GraphTrack = GetTimingViewMainGraphTrack();
	if (GraphTrack.IsValid())
	{
		TSharedPtr<FTimingGraphSeries> CounterSeries = GraphTrack->GetStatsCounterSeries(CounterId);
		if (CounterSeries.IsValid())
		{
			CounterSeries->SetColor(CustomColor, BorderColor);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::SetRandomColorToNode(FStatsNode& StatsNode) const
{
	uint64 Time = FPlatformTime::Cycles64();
	uint32 Hash = (Time & 0xFFFFFFFF) ^ (Time >> 32);
	Hash = ((Hash << 5) + Hash) * 0xfb23618f;

	const uint8 H = Hash & 0xFF;
	const uint8 S = 128 + ((Hash >> 8) & 0x7F);
	const uint8 V = 128 + ((Hash >> 16) & 0x7F);
	FLinearColor CustomColor = FLinearColor::MakeFromHSV8(H, S, V);
	CustomColor.A = 1.0f;

	SetColorToNode(StatsNode, CustomColor);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SStatsView::ContextMenu_EditColor_CanExecute() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ContextMenu_EditColor_Execute()
{
	EditableColorValue = FLinearColor(0.5f, 0.5f, 0.5f, 1.0f);

	TArray<FStatsNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	for (const FStatsNodePtr& SelectedNode : SelectedNodes)
	{
		if (SelectedNode->Is<FStatsNode>())
		{
			FStatsNode& StatsNode = SelectedNode->As<FStatsNode>();
			EditableColorValue = StatsNode.GetColor();
			break;
		}
	}

	FColorPickerArgs PickerArgs;
	{
		PickerArgs.bUseAlpha = true;
		PickerArgs.bOnlyRefreshOnMouseUp = false;
		PickerArgs.bOnlyRefreshOnOk = false;
		PickerArgs.bExpandAdvancedSection = false;
		PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &SStatsView::SetEditableColor);
		PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &SStatsView::ColorPickerCancelled);
		PickerArgs.InitialColor = EditableColorValue;
		PickerArgs.ParentWidget = SharedThis(this);
	}

	OpenColorPicker(PickerArgs);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor SStatsView::GetEditableColor() const
{
	return EditableColorValue;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::SetEditableColor(FLinearColor NewColor)
{
	EditableColorValue = NewColor;

	TArray<FStatsNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	for (const FStatsNodePtr& SelectedNode : SelectedNodes)
	{
		if (SelectedNode->Is<FStatsNode>())
		{
			FStatsNode& StatsNode = SelectedNode->As<FStatsNode>();
			SetColorToNode(StatsNode, EditableColorValue);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsView::ColorPickerCancelled(FLinearColor OriginalColor)
{
	SetEditableColor(OriginalColor);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Misc
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<STimingProfilerWindow> SStatsView::GetProfilerWindow() const
{
	return FTimingProfilerManager::Get()->GetProfilerWindow();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<STimingView> SStatsView::GetTimingView() const
{
	TSharedPtr<STimingProfilerWindow> ProfilerWindow = GetProfilerWindow();
	return ProfilerWindow.IsValid() ? ProfilerWindow->GetTimingView() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SStatsView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return CommandList->ProcessCommandBindings(InKeyEvent) == true ? FReply::Handled() : FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
