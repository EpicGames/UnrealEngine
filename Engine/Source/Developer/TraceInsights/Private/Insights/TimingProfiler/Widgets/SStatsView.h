// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Misc/FilterCollection.h"
#include "Misc/TextFilter.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/StatsGroupingAndSorting.h"
#include "Insights/TimingProfiler/ViewModels/StatsNode.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMenuBuilder;
class FUICommandList;

namespace TraceServices
{
	class IAnalysisSession;
}

namespace UE::Insights
{
	class FTable;
	class FTableColumn;
	class ITableCellValueSorter;
	class SAsyncOperationStatus;
}

namespace UE::Insights::TimingProfiler
{

class FCounterAggregator;
class FTimingGraphTrack;
class STimingProfilerWindow;
class STimingView;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** The filter collection - used for updating the list of counter nodes. */
typedef TFilterCollection<const FStatsNodePtr&> FStatsNodeFilterCollection;

/** The text based filter - used for updating the list of counter nodes. */
typedef TTextFilter<const FStatsNodePtr&> FStatsNodeTextFilter;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * A custom widget used to display the list of counters.
 */
class SStatsView : public SCompoundWidget
{
public:
	/** Default constructor. */
	SStatsView();

	/** Virtual destructor. */
	virtual ~SStatsView();

	SLATE_BEGIN_ARGS(SStatsView) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs);

	TSharedPtr<FTable> GetTable() const { return Table; }

	void Reset();

	/**
	 * Rebuilds the tree (if necessary).
	 * @param bResync - If true, it forces a resync with list of counters from Analysis, even if the list did not changed since last sync.
	 */
	void RebuildTree(bool bResync);

	void ResetStats();
	void UpdateStats(double StartTime, double EndTime);

	FStatsNodePtr GetCounterNode(uint32 CounterId) const;
	void SelectCounterNode(uint32 CounterId);

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	void InitCommandList();

	void UpdateTree();

	void UpdateNode(FStatsNodePtr NodePtr);

	void FinishAggregation();

	/** Called when the session has changed. */
	void InsightsManager_OnSessionChanged();

	/** Called when the analysis was completed. */
	void InsightsManager_OnSessionAnalysisCompleted();

	/**
	 * Populates OutSearchStrings with the strings that should be used in searching.
	 *
	 * @param GroupOrStatNodePtr - the group and stat node to get a text description from.
	 * @param OutSearchStrings   - an array of strings to use in searching.
	 *
	 */
	void HandleItemToStringArray(const FStatsNodePtr& GroupOrStatNodePtr, TArray<FString>& OutSearchStrings) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Context Menu

	TSharedPtr<SWidget> TreeView_GetMenuContent();
	void TreeView_BuildSortByMenu(FMenuBuilder& MenuBuilder);
	void TreeView_BuildViewColumnMenu(FMenuBuilder& MenuBuilder);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Copy to Clipboard

	bool ContextMenu_CopySelectedToClipboard_CanExecute() const;
	void ContextMenu_CopySelectedToClipboard_Execute();

	bool ContextMenu_CopySelectedNameToClipboard_CanExecute() const;
	void ContextMenu_CopySelectedNameToClipboard_Execute();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Export

	bool ContextMenu_Export_CanExecute() const;
	void ContextMenu_Export_Execute();

	bool ContextMenu_ExportValues_CanExecute() const;
	void ContextMenu_ExportValues_Execute() const;

	bool ContextMenu_ExportOps_CanExecute() const;
	void ContextMenu_ExportOps_Execute() const;

	bool ContextMenu_ExportCounters_CanExecute() const;
	void ContextMenu_ExportCounters_Execute() const;

	bool OpenSaveTextFileDialog(const FString& InDialogTitle, const FString& InDefaultFile, FString& OutFilename) const;
	class IFileHandle* OpenExportFile(const TCHAR* InFilename) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Generate and Edit Counters Colors

	bool ContextMenu_GenerateNewColor_CanExecute() const;
	void ContextMenu_GenerateNewColor_Execute() const;

	bool ContextMenu_EditColor_CanExecute() const;
	void ContextMenu_EditColor_Execute();

	void SetColorToNode(FStatsNode& StatsNode, FLinearColor CustomColor) const;
	void SetRandomColorToNode(FStatsNode& StatsNode) const;

	FLinearColor GetEditableColor() const;
	void SetEditableColor(FLinearColor NewColor);
	void ColorPickerCancelled(FLinearColor OriginalColor);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Columns' Header

	void InitializeAndShowHeaderColumns();

	FText GetColumnHeaderText(const FName ColumnId) const;

	TSharedRef<SWidget> TreeViewHeaderRow_GenerateColumnMenu(const FTableColumn& Column);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Misc

	void TreeView_Refresh();

	/**
	 * Called by STreeView to retrieves the children for the specified parent item.
	 * @param InParent    - The parent node to retrieve the children from.
	 * @param OutChildren - List of children for the parent node.
	 */
	void TreeView_OnGetChildren(FStatsNodePtr InParent, TArray<FStatsNodePtr>& OutChildren);

	/** Called by STreeView when selection has changed. */
	void TreeView_OnSelectionChanged(FStatsNodePtr SelectedItem, ESelectInfo::Type SelectInfo);

	FStatsNodePtr GetSingleSelectedStatsNode() const;

	/** Called by STreeView when a tree item is double clicked. */
	void TreeView_OnMouseButtonDoubleClick(FStatsNodePtr TreeNode);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Table Row

	/** Called by STreeView to generate a table row for the specified item. */
	TSharedRef<ITableRow> TreeView_OnGenerateRow(FStatsNodePtr TreeNode, const TSharedRef<STableViewBase>& OwnerTable);

	bool TableRow_ShouldBeEnabled(FStatsNodePtr NodePtr) const;

	void TableRow_SetHoveredCell(TSharedPtr<FTable> TablePtr, TSharedPtr<FTableColumn> ColumnPtr, FStatsNodePtr NodePtr);
	EHorizontalAlignment TableRow_GetColumnOutlineHAlignment(const FName ColumnId) const;

	FText TableRow_GetHighlightText() const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Filtering

	/** Populates the group and stat tree with items based on the current data. */
	void ApplyFiltering();

	void FilterOutZeroCountStats_OnCheckStateChanged(ECheckBoxState NewRadioState);
	ECheckBoxState FilterOutZeroCountStats_IsChecked() const;

	TSharedRef<SWidget> GetToggleButtonForNodeType(const EStatsNodeType InNodeType);
	void FilterByStatsType_OnCheckStateChanged(ECheckBoxState NewRadioState, const EStatsNodeType InNodeType);
	ECheckBoxState FilterByStatsType_IsChecked(const EStatsNodeType InNodeType) const;

	TSharedRef<SWidget> GetToggleButtonForDataType(const EStatsNodeDataType InDataType);
	void FilterByStatsDataType_OnCheckStateChanged(ECheckBoxState NewRadioState, const EStatsNodeDataType InDataType);
	ECheckBoxState FilterByStatsDataType_IsChecked(const EStatsNodeDataType InDataType) const;

	bool SearchBox_IsEnabled() const;
	void SearchBox_OnTextChanged(const FText& InFilterText);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Grouping

	void CreateGroups();
	void CreateGroupByOptionsSources();

	void GroupBy_OnSelectionChanged(TSharedPtr<EStatsGroupingMode> NewGroupingMode, ESelectInfo::Type SelectInfo);

	TSharedRef<SWidget> GroupBy_OnGenerateWidget(TSharedPtr<EStatsGroupingMode> InGroupingMode) const;

	FText GroupBy_GetSelectedText() const;

	FText GroupBy_GetSelectedTooltipText() const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Sorting

	static const FName GetDefaultColumnBeingSorted();
	static const EColumnSortMode::Type GetDefaultColumnSortMode();

	void CreateSortings();

	void UpdateCurrentSortingByColumn();
	void SortTreeNodes();
	void SortTreeNodesRec(FStatsNode& Node, const ITableCellValueSorter& Sorter);

	EColumnSortMode::Type GetSortModeForColumn(const FName ColumnId) const;
	void SetSortModeForColumn(const FName& ColumnId, EColumnSortMode::Type SortMode);
	void OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Sorting Actions

	// SortMode (HeaderMenu)
	bool HeaderMenu_SortMode_IsChecked(const FName ColumnId, const EColumnSortMode::Type InSortMode);
	bool HeaderMenu_SortMode_CanExecute(const FName ColumnId, const EColumnSortMode::Type InSortMode) const;
	void HeaderMenu_SortMode_Execute(const FName ColumnId, const EColumnSortMode::Type InSortMode);

	// SortMode (ContextMenu)
	bool ContextMenu_SortMode_IsChecked(const EColumnSortMode::Type InSortMode);
	bool ContextMenu_SortMode_CanExecute(const EColumnSortMode::Type InSortMode) const;
	void ContextMenu_SortMode_Execute(const EColumnSortMode::Type InSortMode);

	// SortByColumn (ContextMenu)
	bool ContextMenu_SortByColumn_IsChecked(const FName ColumnId);
	bool ContextMenu_SortByColumn_CanExecute(const FName ColumnId) const;
	void ContextMenu_SortByColumn_Execute(const FName ColumnId);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Column Visibility Actions

	// ShowColumn
	bool CanShowColumn(const FName ColumnId) const;
	void ShowColumn(const FName ColumnId);

	// HideColumn
	bool CanHideColumn(const FName ColumnId) const;
	void HideColumn(const FName ColumnId);

	// ToggleColumnVisibility
	bool IsColumnVisible(const FName ColumnId) const;
	bool CanToggleColumnVisibility(const FName ColumnId) const;
	void ToggleColumnVisibility(const FName ColumnId);

	// ShowAllColumns (ContextMenu)
	bool ContextMenu_ShowAllColumns_CanExecute() const;
	void ContextMenu_ShowAllColumns_Execute();

	// MinMaxMedColumns (ContextMenu)
	bool ContextMenu_ShowMinMaxMedColumns_CanExecute() const;
	void ContextMenu_ShowMinMaxMedColumns_Execute();

	// ResetColumns (ContextMenu)
	bool ContextMenu_ResetColumns_CanExecute() const;
	void ContextMenu_ResetColumns_Execute();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Plot (Graph Series)

	TSharedPtr<FTimingGraphTrack> GetTimingViewMainGraphTrack() const;

	void ToggleGraphSeries(TSharedRef<FTimingGraphTrack> GraphTrack, FStatsNodeRef NodePtr) const;
	bool IsSeriesInTimingViewMainGraph(FStatsNodePtr CounterNode) const;
	void ToggleTimingViewMainGraphEventSeries(FStatsNodePtr CounterNode) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	TSharedPtr<STimingProfilerWindow> GetProfilerWindow() const;
	TSharedPtr<STimingView> GetTimingView() const;

private:
	/** Table view model. */
	TSharedPtr<FTable> Table;

	/** The analysis session used to populate this widget. */
	TSharedPtr<const TraceServices::IAnalysisSession> Session;

	TSharedPtr<FUICommandList> CommandList;

	/** Eye-Dropper selected color for graphing.*/
	FLinearColor EditableColorValue;

	//////////////////////////////////////////////////
	// Tree View, Columns

	/** The tree widget which holds the list of groups and counter corresponding with each group. */
	TSharedPtr<STreeView<FStatsNodePtr>> TreeView;

	/** Holds the tree view header row widget which display all columns in the tree view. */
	TSharedPtr<SHeaderRow> TreeViewHeaderRow;

	/** External scrollbar used to synchronize tree view position. */
	TSharedPtr<SScrollBar> ExternalScrollbar;

	//////////////////////////////////////////////////
	// Hovered Column, Hovered Counter Node

	/** Name of the column currently being hovered by the mouse. */
	FName HoveredColumnId;

	/** A shared pointer to the counter node currently being hovered by the mouse. */
	FStatsNodePtr HoveredNodePtr;

	//////////////////////////////////////////////////
	// Stats Nodes

	/** An array of group and counter nodes generated from the metadata. */
	TArray<FStatsNodePtr> GroupNodes;

	/** A filtered array of group and counter nodes to be displayed in the tree widget. */
	TArray<FStatsNodePtr> FilteredGroupNodes;

	/** All counter nodes. */
	TArray<FStatsNodePtr> StatsNodes;

	/** All counter nodes, stored as CounterId -> FStatsNodePtr. */
	TMap<uint32, FStatsNodePtr> StatsNodesIdMap;

	/** Currently expanded group nodes. */
	TSet<FStatsNodePtr> ExpandedNodes;

	/** If true, the expanded nodes have been saved before applying a text filter. */
	bool bExpansionSaved = false;

	//////////////////////////////////////////////////
	// Search box and filters

	/** The search box widget used to filter items displayed in the tree. */
	TSharedPtr<SSearchBox> SearchBox;

	/** The text based filter. */
	TSharedPtr<FStatsNodeTextFilter> TextFilter;

	/** The filter collection. */
	TSharedPtr<FStatsNodeFilterCollection> Filters;

	/** The on/off filter flag for each node type. */
	bool FilterByNodeType[static_cast<int>(EStatsNodeType::InvalidOrMax)];

	/** The on/off filter flag for each data type. */
	bool FilterByDataType[static_cast<int>(EStatsNodeDataType::InvalidOrMax)];

	/** Filter out the counters having zero total instance count (aggregated stats). */
	bool bFilterOutZeroCountStats = false;

	//////////////////////////////////////////////////
	// Grouping

	TArray<TSharedPtr<EStatsGroupingMode>> GroupByOptionsSource;

	TSharedPtr<SComboBox<TSharedPtr<EStatsGroupingMode>>> GroupByComboBox;

	/** How we group the counters? */
	EStatsGroupingMode GroupingMode = EStatsGroupingMode::Flat;

	//////////////////////////////////////////////////
	// Sorting

	/** All available sorters. */
	TArray<TSharedPtr<ITableCellValueSorter>> AvailableSorters;

	/** Current sorter. It is nullptr if sorting is disabled. */
	TSharedPtr<ITableCellValueSorter> CurrentSorter;

	/** Name of the column currently being sorted. Can be NAME_None if sorting is disabled (CurrentSorting == nullptr) or if a complex sorting is used (CurrentSorting != nullptr). */
	FName ColumnBeingSorted;

	/** How we sort the nodes? Ascending or Descending. */
	EColumnSortMode::Type ColumnSortMode = EColumnSortMode::Type::Ascending;

	//////////////////////////////////////////////////

	TSharedRef<FCounterAggregator> Aggregator;
	TSharedPtr<SAsyncOperationStatus> AsyncOperationStatus;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
