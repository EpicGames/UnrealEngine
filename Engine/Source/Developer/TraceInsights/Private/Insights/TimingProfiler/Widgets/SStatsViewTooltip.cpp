// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStatsViewTooltip.h"

#include "SlateOptMacros.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

// TraceServices
#include "TraceServices/Model/AnalysisSession.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/TimingProfiler/ViewModels/StatsNode.h"
#include "Insights/TimingProfiler/ViewModels/StatsNodeHelper.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::SStatsView"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedPtr<SToolTip> SStatsViewTooltip::GetTableTooltip(const FTable& Table)
{
	TSharedPtr<SToolTip> ColumnTooltip =
		SNew(SToolTip)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Table.GetDisplayName())
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Table.GetDescription())
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			]
		];

	return ColumnTooltip;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SToolTip> SStatsViewTooltip::GetColumnTooltip(const FTableColumn& Column)
{
	TSharedPtr<SToolTip> ColumnTooltip =
		SNew(SToolTip)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Column.GetTitleName())
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Column.GetDescription())
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			]
		];

	return ColumnTooltip;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SToolTip> SStatsViewTooltip::GetRowTooltip(const TSharedPtr<FStatsNode> StatsNodePtr)
{
	const FText InstanceCountText = FText::AsNumber(StatsNodePtr->GetAggregatedStats().Count);

	FText SumText = StatsNodePtr->GetTextForAggregatedStatsSum(true);
	FText MinText = StatsNodePtr->GetTextForAggregatedStatsMin(true);
	FText MaxText = StatsNodePtr->GetTextForAggregatedStatsMax(true);
	FText AvgText = StatsNodePtr->GetTextForAggregatedStatsAverage(true);
	FText MedText = StatsNodePtr->GetTextForAggregatedStatsMedian(true);
	//FText LowText = StatsNodePtr->GetTextForAggregatedStatsLowerQuartile(true);
	//FText UppText = StatsNodePtr->GetTextForAggregatedStatsUpperQuartile(true);

	TSharedPtr<SGridPanel> GridPanel;
	TSharedPtr<SHorizontalBox> HBox;

	TSharedPtr<SToolTip> TableCellTooltip =
		SNew(SToolTip)
		[
			SAssignNew(HBox, SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SGridPanel)

					// Id: [Id]
					+ SGridPanel::Slot(0, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_Id", "Id:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(1, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(FText::AsNumber(StatsNodePtr->GetCounterId()))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					]

					// Name: [Name]
					+ SGridPanel::Slot(0, 1)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_Name", "Name:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(1, 1)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.WrapTextAt(512.0f)
						.WrappingPolicy(ETextWrappingPolicy::AllowPerCharacterWrapping)
						.Text(StatsNodePtr->GetDisplayName())
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					]

					// Item Type: [Type]
					+ SGridPanel::Slot(0, 3)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_Type", "Item Type:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(1, 3)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(StatsNodeTypeHelper::ToText(StatsNodePtr->GetType()))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					]

					// Data Type: [DataType]
					+ SGridPanel::Slot(0, 4)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_DataType", "Data Type:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(1, 4)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(StatsNodeDataTypeHelper::ToText(StatsNodePtr->GetDataType()))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SGridPanel)

					+ SGridPanel::Slot(0, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_NumInstances", "Num Instances:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(1, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(InstanceCountText)
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SAssignNew(GridPanel, SGridPanel)

					// Aggregated stats are added here.
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]
			]
		];

	int32 Row = 1;
	AddAggregatedStatsRow(GridPanel, Row, LOCTEXT("TT_Sum",     "Sum:"),            SumText);
	AddAggregatedStatsRow(GridPanel, Row, LOCTEXT("TT_Max",     "Max:"),            MaxText);
	//AddAggregatedStatsRow(GridPanel, Row, LOCTEXT("TT_UpperQ",  "Upper Quartile:"), UppText);
	AddAggregatedStatsRow(GridPanel, Row, LOCTEXT("TT_Average", "Average:"),        AvgText);
	AddAggregatedStatsRow(GridPanel, Row, LOCTEXT("TT_Median",  "Median:"),         MedText);
	//AddAggregatedStatsRow(GridPanel, Row, LOCTEXT("TT_LowerQ",  "Lower Quartile:"), LowText);
	AddAggregatedStatsRow(GridPanel, Row, LOCTEXT("TT_Min",     "Min:"),            MinText);

	return TableCellTooltip;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SStatsViewTooltip::AddAggregatedStatsRow(TSharedPtr<SGridPanel> Grid, int32& Row, const FText& Name, const FText& Value)
{
	Grid->AddSlot(0, Row)
		.Padding(2.0f)
		[
			SNew(STextBlock)
			.Text(Name)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
		];

	Grid->AddSlot(1, Row)
		.Padding(2.0f)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(Value)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			.ColorAndOpacity(Row == 1 ? FLinearColor::Gray : FLinearColor::White)
		];

	Row++;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
