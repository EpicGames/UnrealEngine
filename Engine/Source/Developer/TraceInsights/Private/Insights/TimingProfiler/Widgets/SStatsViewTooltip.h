// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Widgets/IToolTip.h"
#include "Widgets/SToolTip.h"

class SGridPanel;

namespace UE::Insights
{
	class FTable;
	class FTableColumn;
}

namespace UE::Insights::TimingProfiler
{

class FStatsNode;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Stats Counters View Tooltip */
class SStatsViewTooltip
{
public:
	SStatsViewTooltip() = delete;

	static TSharedPtr<SToolTip> GetTableTooltip(const FTable& Table);
	static TSharedPtr<SToolTip> GetColumnTooltip(const FTableColumn& Column);
	static TSharedPtr<SToolTip> GetRowTooltip(const TSharedPtr<FStatsNode> TreeNodePtr);

private:
	static void AddAggregatedStatsRow(TSharedPtr<SGridPanel> Grid, int32& Row, const FText& Name, const FText& Value);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class SStatsCounterTableRowToolTip : public IToolTip
{
public:
	SStatsCounterTableRowToolTip(const TSharedPtr<FStatsNode> InTreeNodePtr) : TreeNodePtr(InTreeNodePtr) {}
	virtual ~SStatsCounterTableRowToolTip() = default;

	virtual TSharedRef<class SWidget> AsWidget()
	{
		CreateToolTipWidget();
		return ToolTipWidget.ToSharedRef();
	}

	virtual TSharedRef<SWidget> GetContentWidget()
	{
		CreateToolTipWidget();
		return ToolTipWidget->GetContentWidget();
	}

	virtual void SetContentWidget(const TSharedRef<SWidget>& InContentWidget)
	{
		CreateToolTipWidget();
		ToolTipWidget->SetContentWidget(InContentWidget);
	}

	void InvalidateWidget()
	{
		ToolTipWidget.Reset();
	}

	virtual bool IsEmpty() const { return false; }
	virtual bool IsInteractive() const { return false; }
	virtual void OnOpening() {}
	virtual void OnClosed() {}

private:
	void CreateToolTipWidget()
	{
		if (!ToolTipWidget.IsValid())
		{
			ToolTipWidget = SStatsViewTooltip::GetRowTooltip(TreeNodePtr);
		}
	}

private:
	TSharedPtr<SToolTip> ToolTipWidget;
	const TSharedPtr<FStatsNode> TreeNodePtr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
