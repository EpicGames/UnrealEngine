// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimeMarkerEditBlock.h"

#include "SlateOptMacros.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/TimeMarker.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::STimeMarkerEditBlock"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STimeMarkerEditBlock::Construct(const FArguments& InArgs, TSharedRef<FTimeMarker> InTimeMarker)
{
	TimeMarker = InTimeMarker;

	Padding = InArgs._Padding;
	PreviousTimeMarker = InArgs._PreviousTimeMarker;
	OnGetTimingViewCallback = InArgs._OnGetTimingView;
	OnTimeMarkerChangedCallback = InArgs._OnTimeMarkerChanged;

	const FText TimeMarkerName = FText::FromString(TimeMarker->GetName());

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(Padding.Left, Padding.Top + 1.0f, 4.0f, Padding.Bottom + 1.0f)
		[
			SNew(SCheckBox)
			.ToolTipText(LOCTEXT("VisibilityTooltip", "Time Marker Visibility\nShows the time marker in the Timing View."))
			.IsChecked_Lambda([this]()
				{
					return TimeMarker->IsVisible() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
			.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
				{
					TimeMarker->SetVisibility(NewState == ECheckBoxState::Checked);
				})
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0.0f, 1.0f, 0.0f, 1.0f)
		[
			SNew(STextBlock)
			.Text(TimeMarkerName)
			.ToolTipText(FText::Format(
				LOCTEXT("NameTooltipFmt", "Time Marker '{0}'\nDouble click the name to move the time marker to the center of the Timing View."),
				TimeMarkerName))
			.ColorAndOpacity(FSlateColor(TimeMarker->GetColor()))
			.OnDoubleClicked(this, &STimeMarkerEditBlock::OnLabelDoubleClicked)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.0f, 1.0f, 0.0f, 1.0f)
		[
			SNew(SEditableTextBox)
			.MinDesiredWidth(110.0f)
			.Text_Lambda([this]()
				{
					return FText::FromString(FString::Printf(TEXT("%.9f"), TimeMarker->GetTime()));
				})
			.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type InCommitType)
				{
					const double Time = FCString::Atod(*InText.ToString());
					TimeMarker->SetTime(Time);
					OnTimeMarkerChanged();
				})
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.0f, 1.0f, Padding.Right, 1.0f)
		[
			SNew(STextBlock)
			.Text_Lambda([this]()
				{
					double Time = TimeMarker->GetTime();
					if (PreviousTimeMarker.IsSet() && PreviousTimeMarker.Get().IsValid())
					{
						double PrevTime = PreviousTimeMarker.Get()->GetTime();
						return FText::FromString(FString::Printf(TEXT("%s (+%s)"),
							*FormatTime(Time, 0.1),
							*FormatTime(Time - PrevTime, 0.1)));
					}
					else
					{
						return FText::FromString(FormatTime(Time, 0.1));
					}
				})
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<STimingView> STimeMarkerEditBlock::GetTimingView() const
{
	if (OnGetTimingViewCallback.IsBound())
	{
		return OnGetTimingViewCallback.Execute(TimeMarker.ToSharedRef());
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimeMarkerEditBlock::OnTimeMarkerChanged()
{
	OnTimeMarkerChangedCallback.ExecuteIfBound(TimeMarker.ToSharedRef());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimeMarkerEditBlock::OnLabelDoubleClicked(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent)
{
	TSharedPtr<STimingView> TimingView = GetTimingView();
	if (TimingView.IsValid())
	{
		// Move timer to the center of the timing view.
		const double Time = (TimingView->GetViewport().GetStartTime() + TimingView->GetViewport().GetEndTime()) / 2.0;
		TimeMarker->SetTime(Time);
	}
	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
