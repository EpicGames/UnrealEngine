// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Input/Events.h"
#include "Input/Reply.h"
#include "Layout/Geometry.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Insights::TimingProfiler
{

class FTimeMarker;
class STimingView;

using FOnGetTimingView = TDelegate<TSharedPtr<STimingView>(TSharedRef<FTimeMarker>)>;
using FOnTimeMarkerChanged = TDelegate<void(TSharedRef<FTimeMarker>)>;

class STimeMarkerEditBlock : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STimeMarkerEditBlock) {}
		SLATE_ARGUMENT(FMargin, Padding)
		SLATE_ATTRIBUTE(TSharedPtr<FTimeMarker>, PreviousTimeMarker)
		SLATE_EVENT(FOnGetTimingView, OnGetTimingView)
		SLATE_EVENT(FOnTimeMarkerChanged, OnTimeMarkerChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FTimeMarker> InTimeMarker);

private:
	TSharedPtr<STimingView> GetTimingView() const;
	void OnTimeMarkerChanged();
	FReply OnLabelDoubleClicked(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent);

private:
	TSharedPtr<FTimeMarker> TimeMarker;

	FMargin Padding;

	/** The previous TimeMarker; used to display delta time. */
	TAttribute<TSharedPtr<FTimeMarker>> PreviousTimeMarker;

	/** Called whenever this widget needs to know the TimingView widget where the TimeMarker is located. */
	FOnGetTimingView OnGetTimingViewCallback;

	/** Called whenever the time marker is changed by this widget. */
	FOnTimeMarkerChanged OnTimeMarkerChangedCallback;
};

} // namespace UE::Insights::TimingProfiler
