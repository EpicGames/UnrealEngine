// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimerTableCell.h"

#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/TimingProfiler/Widgets/STimerTableRow.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::STimersView"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void STimerTableCell::Construct(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	TablePtr = InArgs._TablePtr;
	ColumnPtr = InArgs._ColumnPtr;
	TimerNodePtr = InArgs._TimerNodePtr;

	ensure(TablePtr.IsValid());
	ensure(ColumnPtr.IsValid());
	ensure(TimerNodePtr.IsValid());

	SetHoveredCellDelegate = InArgs._OnSetHoveredCell;

	ChildSlot
	[
		GenerateWidgetForColumn(InArgs, TableRow)
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimerTableCell::GenerateWidgetForColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	if (InArgs._IsNameColumn)
	{
		return GenerateWidgetForNameColumn(InArgs, TableRow);
	}
	else
	{
		return GenerateWidgetForStatsColumn(InArgs, TableRow);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimerTableCell::GenerateWidgetForNameColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	return
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SExpanderArrow, TableRow)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SOverlay)

			// Hot path icon
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Visibility(this, &STimerTableCell::GetHotPathIconVisibility)
				.ColorAndOpacity(FLinearColor(1.0f, 0.3f, 0.3f, 1.0f))
				.Image(FInsightsStyle::GetBrush("Icons.HotPath.TreeItem"))
			]

			// Info icon + tooltip
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Visibility(this, &STimerTableCell::GetHintIconVisibility)
				.Image(FInsightsStyle::GetBrush("Icons.Hint.TreeItem"))
				.ToolTip(GetRowToolTip(TableRow))
			]
		]

		// Color box
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.Visibility(this, &STimerTableCell::GetBoxVisibility)
			.WidthOverride(14.0f)
			.HeightOverride(14.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
				.BorderBackgroundColor(this, &STimerTableCell::GetBoxColorAndOpacity)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
			]
		]

		// Name
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(this, &STimerTableCell::GetDisplayName)
			.HighlightText(InArgs._HighlightText)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			.ColorAndOpacity(this, &STimerTableCell::GetColorAndOpacity)
			.ShadowColorAndOpacity(this, &STimerTableCell::GetShadowColorAndOpacity)
		]

		// Name Suffix
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Visibility(this, &STimerTableCell::HasExtraDisplayName)
			.Text(this, &STimerTableCell::GetExtraDisplayName)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			.ColorAndOpacity(this, &STimerTableCell::GetExtraColorAndOpacity)
			.ShadowColorAndOpacity(this, &STimerTableCell::GetShadowColorAndOpacity)
		]
	;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<IToolTip> STimerTableCell::GetRowToolTip(const TSharedRef<ITableRow>& TableRow) const
{
	TSharedRef<STimerTableRow> Row = StaticCastSharedRef<STimerTableRow, ITableRow>(TableRow);
	return Row->GetRowToolTip();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STimerTableCell::GetValueAsText() const
{
	return ColumnPtr->GetValueAsText(*TimerNodePtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimerTableCell::GenerateWidgetForStatsColumn(const FArguments& InArgs, const TSharedRef<ITableRow>& TableRow)
{
	return
		SNew(SHorizontalBox)

		// Value
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		.HAlign(ColumnPtr->GetHorizontalAlignment())
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(this, &STimerTableCell::GetValueAsText)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			.ColorAndOpacity(this, &STimerTableCell::GetStatsColorAndOpacity)
			.ShadowColorAndOpacity(this, &STimerTableCell::GetShadowColorAndOpacity)
		]
	;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
