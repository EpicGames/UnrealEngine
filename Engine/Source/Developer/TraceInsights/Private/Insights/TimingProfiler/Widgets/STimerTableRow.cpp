// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimerTableRow.h"

#include "SlateOptMacros.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Images/SImage.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/TimingProfiler/Widgets/STimersViewTooltip.h"
#include "Insights/TimingProfiler/Widgets/STimerTableCell.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::STimersView"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void STimerTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	OnShouldBeEnabled = InArgs._OnShouldBeEnabled;
	IsColumnVisibleDelegate = InArgs._OnIsColumnVisible;
	GetColumnOutlineHAlignmentDelegate = InArgs._OnGetColumnOutlineHAlignmentDelegate;
	SetHoveredCellDelegate = InArgs._OnSetHoveredCell;

	HighlightText = InArgs._HighlightText;

	TablePtr = InArgs._TablePtr;
	TimerNodePtr = InArgs._TimerNodePtr;

	RowToolTip = MakeShared<STimerTableRowToolTip>(TimerNodePtr);

	SetEnabled(TAttribute<bool>(this, &STimerTableRow::HandleShouldBeEnabled));

	SMultiColumnTableRow<FTimerNodePtr>::Construct(SMultiColumnTableRow<FTimerNodePtr>::FArguments(), InOwnerTableView);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimerTableRow::GenerateWidgetForColumn(const FName& ColumnId)
{
	TSharedPtr<FTableColumn> ColumnPtr = TablePtr->FindColumnChecked(ColumnId);

	return
		SNew(SOverlay)
		.Visibility(EVisibility::SelfHitTestInvisible)

		+SOverlay::Slot()
		.Padding(0.0f)
		[
			SNew(SImage)
			.Image(FInsightsStyle::GetBrush("TreeTable.RowBackground"))
			.ColorAndOpacity(this, &STimerTableRow::GetBackgroundColorAndOpacity)
		]

		+SOverlay::Slot()
		.Padding(0.0f)
		[
			SNew(SImage)
			.Image(this, &STimerTableRow::GetOutlineBrush, ColumnId)
			.ColorAndOpacity(this, &STimerTableRow::GetOutlineColorAndOpacity)
		]

		+SOverlay::Slot()
		[
			SNew(STimerTableCell, SharedThis(this))
			.Visibility(this, &STimerTableRow::IsColumnVisible, ColumnId)
			.TablePtr(TablePtr)
			.ColumnPtr(ColumnPtr)
			.TimerNodePtr(TimerNodePtr)
			.HighlightText(HighlightText)
			.IsNameColumn(ColumnPtr->IsHierarchy())
			.OnSetHoveredCell(this, &STimerTableRow::OnSetHoveredCell)
		];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimerTableRow::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	//im:TODO
	//if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	//{
	//	if (TimerNode->IsGroup())
	//	{
	//		// Add all timer Ids for the group.
	//		TArray<int32> TimerIds;
	//		const TArray<FTimerNodePtr>& FilteredChildren = TimerNode->GetFilteredChildren();
	//		const int32 NumFilteredChildren = FilteredChildren.Num();
	//
	//		TimerIds.Reserve(NumFilteredChildren);
	//		for (int32 Nx = 0; Nx < NumFilteredChildren; ++Nx)
	//		{
	//			TimerIds.Add(FilteredChildren[Nx]->GetId());
	//		}
	//
	//		return FReply::Handled().BeginDragDrop(FStatIDDragDropOp::NewGroup(TimerIds, TimerNode->GetName().GetPlainNameString()));
	//	}
	//	else
	//	{
	//		return FReply::Handled().BeginDragDrop(FStatIDDragDropOp::NewSingle(TimerNode->GetId(), TimerNode->GetName().GetPlainNameString()));
	//	}
	//}

	return SMultiColumnTableRow<FTimerNodePtr>::OnDragDetected(MyGeometry, MouseEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<IToolTip> STimerTableRow::GetRowToolTip() const
{
	return RowToolTip.ToSharedRef();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTableRow::InvalidateContent()
{
	RowToolTip->InvalidateWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STimerTableRow::GetBackgroundColorAndOpacity() const
{
	if (TimerNodePtr->GetType() == ETimerNodeType::Group)
	{
		return FLinearColor(0.0f, 0.0f, 0.5f, 1.0f);
	}
	else
	{
		return GetBackgroundColorAndOpacity(TimerNodePtr->GetAggregatedStats().TotalInclusiveTime);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STimerTableRow::GetBackgroundColorAndOpacity(double Time) const
{
	const FLinearColor Color =	Time > FTimeValue::Second      ? FLinearColor(0.3f, 0.0f, 0.0f, 1.0f) :
								Time > FTimeValue::Millisecond ? FLinearColor(0.3f, 0.1f, 0.0f, 1.0f) :
								Time > FTimeValue::Microsecond ? FLinearColor(0.0f, 0.1f, 0.0f, 1.0f) :
								                                 FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
	return Color;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STimerTableRow::GetOutlineColorAndOpacity() const
{
	return FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* STimerTableRow::GetOutlineBrush(const FName ColumnId) const
{
	EHorizontalAlignment HAlign = HAlign_Center;
	if (GetColumnOutlineHAlignmentDelegate.IsBound())
	{
		HAlign = GetColumnOutlineHAlignmentDelegate.Execute(ColumnId);
	}
	return FInsightsStyle::GetOutlineBrush(HAlign);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTableRow::HandleShouldBeEnabled() const
{
	if (TimerNodePtr->IsGroup())
	{
		return true;
	}
	if (OnShouldBeEnabled.IsBound())
	{
		return OnShouldBeEnabled.Execute(TimerNodePtr);
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EVisibility STimerTableRow::IsColumnVisible(const FName ColumnId) const
{
	EVisibility Result = EVisibility::Collapsed;
	if (IsColumnVisibleDelegate.IsBound())
	{
		Result = IsColumnVisibleDelegate.Execute(ColumnId) ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return Result;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTableRow::OnSetHoveredCell(TSharedPtr<FTable> InTablePtr, TSharedPtr<FTableColumn> InColumnPtr, FTimerNodePtr InTimerNodePtr)
{
	SetHoveredCellDelegate.ExecuteIfBound(InTablePtr, InColumnPtr, InTimerNodePtr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
