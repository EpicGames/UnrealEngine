// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/TimerNodeHelper.h"

class IToolTip;

namespace UE::Insights
{
	class FTable;
	class FTableColumn;
}

namespace UE::Insights::TimingProfiler
{

class STimerTableRowToolTip;

DECLARE_DELEGATE_RetVal_OneParam(bool, FTimerNodeShouldBeEnabledDelegate, FTimerNodePtr /*NodePtr*/);
DECLARE_DELEGATE_RetVal_OneParam(bool, FIsColumnVisibleDelegate, const FName /*ColumnId*/);
DECLARE_DELEGATE_RetVal_OneParam(EHorizontalAlignment, FGetColumnOutlineHAlignmentDelegate, const FName /*ColumnId*/);
DECLARE_DELEGATE_ThreeParams(FSetHoveredTimerTableCell, TSharedPtr<FTable> /*TablePtr*/, TSharedPtr<FTableColumn> /*ColumnPtr*/, FTimerNodePtr /*TimerNodePtr*/);

/** Widget that represents a table row in the tree control. Generates widgets for each column on demand. */
class STimerTableRow : public SMultiColumnTableRow<FTimerNodePtr>
{
public:
	SLATE_BEGIN_ARGS(STimerTableRow) {}
		SLATE_EVENT(FTimerNodeShouldBeEnabledDelegate, OnShouldBeEnabled)
		SLATE_EVENT(FIsColumnVisibleDelegate, OnIsColumnVisible)
		SLATE_EVENT(FGetColumnOutlineHAlignmentDelegate, OnGetColumnOutlineHAlignmentDelegate)
		SLATE_EVENT(FSetHoveredTimerTableCell, OnSetHoveredCell)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(TSharedPtr<FTable>, TablePtr)
		SLATE_ARGUMENT(FTimerNodePtr, TimerNodePtr)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override;

	/**
	 * Called when Slate detects that a widget started to be dragged.
	 * Usage:
	 * A widget can ask Slate to detect a drag.
	 * OnMouseDown() reply with FReply::Handled().DetectDrag(SharedThis(this)).
	 * Slate will either send an OnDragDetected() event or do nothing.
	 * If the user releases a mouse button or leaves the widget before
	 * a drag is triggered (maybe user started at the very edge) then no event will be
	 * sent.
	 *
	 * @param  InMyGeometry  Widget geometry
	 * @param  InMouseEvent  MouseMove that triggered the drag
	 *
	 */
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	TSharedRef<IToolTip> GetRowToolTip() const;
	void InvalidateContent();

protected:
	FSlateColor GetBackgroundColorAndOpacity() const;
	FSlateColor GetBackgroundColorAndOpacity(double Time) const;
	FSlateColor GetOutlineColorAndOpacity() const;
	const FSlateBrush* GetOutlineBrush(const FName ColumnId) const;
	bool HandleShouldBeEnabled() const;
	EVisibility IsColumnVisible(const FName ColumnId) const;
	void OnSetHoveredCell(TSharedPtr<FTable> InTablePtr, TSharedPtr<FTableColumn> InColumnPtr, FTimerNodePtr InTimerNodePtr);

protected:
	/** A shared pointer to the table view model. */
	TSharedPtr<FTable> TablePtr;

	/** Data context for this table row. */
	FTimerNodePtr TimerNodePtr;

	FTimerNodeShouldBeEnabledDelegate OnShouldBeEnabled;
	FIsColumnVisibleDelegate IsColumnVisibleDelegate;
	FSetHoveredTimerTableCell SetHoveredCellDelegate;
	FGetColumnOutlineHAlignmentDelegate GetColumnOutlineHAlignmentDelegate;

	/** Text to be highlighted on timer name. */
	TAttribute<FText> HighlightText;

	TSharedPtr<STimerTableRowToolTip> RowToolTip;
};

} // namespace UE::Insights::TimingProfiler
