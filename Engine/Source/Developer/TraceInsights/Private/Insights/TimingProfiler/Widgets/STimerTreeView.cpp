// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimerTreeView.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SToolTip.h"

// TraceInsightsCore
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "InsightsCore/Table/Widgets/SAsyncOperationStatus.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/Table/ViewModels/TableCommands.h"
#include "Insights/TimingProfiler/GraphTracks/TimingGraphSeries.h"
#include "Insights/TimingProfiler/GraphTracks/TimingGraphTrack.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/ViewModels/TimerButterflyAggregator.h"
#include "Insights/TimingProfiler/ViewModels/TimersViewColumnFactory.h"
#include "Insights/TimingProfiler/Widgets/SFrameTrack.h"
#include "Insights/TimingProfiler/Widgets/STimersViewTooltip.h"
#include "Insights/TimingProfiler/Widgets/STimerTableRow.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::STimerTreeView"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimerTreeViewCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimerTreeViewCommands : public TCommands<FTimerTreeViewCommands>
{
public:
	FTimerTreeViewCommands()
	: TCommands<FTimerTreeViewCommands>(
		TEXT("TimerTreeViewCommands"),
		NSLOCTEXT("Contexts", "TimerTreeViewCommands", "Insights - Timer Tree View"),
		NAME_None,
		FInsightsStyle::GetStyleSetName())
	{
	}

	virtual ~FTimerTreeViewCommands()
	{
	}

	// UI_COMMAND takes long for the compiler to optimize
	UE_DISABLE_OPTIMIZATION_SHIP
	virtual void RegisterCommands() override
	{
		UI_COMMAND(Command_CopyToClipboard,
			"Copy To Clipboard",
			"Copies the selection to clipboard.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Control, EKeys::C));

		UI_COMMAND(Command_CopyNameToClipboard,
			"Copy Name To Clipboard",
			"Copies the name of the selected timer to the clipboard.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::C));

		UI_COMMAND(Command_OpenSource,
			"Open Source",
			"Opens the source file of the selected timer in the registered IDE.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_FindMaxInstance,
			"Maximum Duration Instance",
			"Navigates to and selects the timing event instance with the maximum duration, for the selected timer.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_FindMinInstance,
			"Minimum Duration Instance",
			"Navigates to and selects the timing event instance with the minimum duration, for the selected timer.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_FindMaxInstanceInSelection,
			"Maximum Duration Instance in Selection",
			"Navigates to and selects the timing event instance with the maximum duration, for the selected timer, in the selected time range.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_FindMinInstanceInSelection,
			"Minimum Duration Instance in Selection",
			"Navigates to and selects the timing event instance with the minimum duration, for the selected timer, in the selected time range.",
			EUserInterfaceActionType::Button,
			FInputChord());
	}
	UE_ENABLE_OPTIMIZATION_SHIP

	TSharedPtr<FUICommandInfo> Command_CopyToClipboard;
	TSharedPtr<FUICommandInfo> Command_CopyNameToClipboard;
	TSharedPtr<FUICommandInfo> Command_OpenSource;
	TSharedPtr<FUICommandInfo> Command_FindMaxInstance;
	TSharedPtr<FUICommandInfo> Command_FindMinInstance;
	TSharedPtr<FUICommandInfo> Command_FindMaxInstanceInSelection;
	TSharedPtr<FUICommandInfo> Command_FindMinInstanceInSelection;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// STimerTreeView
////////////////////////////////////////////////////////////////////////////////////////////////////

STimerTreeView::STimerTreeView()
	: Table(MakeShared<FTable>())
	, ColumnBeingSorted(GetDefaultColumnBeingSorted())
	, ColumnSortMode(GetDefaultColumnSortMode())
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

STimerTreeView::~STimerTreeView()
{
	FTimerTreeViewCommands::Unregister();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::InitCommandList()
{
	FTimerTreeViewCommands::Register();
	CommandList = MakeShared<FUICommandList>();
	CommandList->MapAction(FTimerTreeViewCommands::Get().Command_CopyToClipboard, FExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_CopyToClipboard_Execute), FCanExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_CopyToClipboard_CanExecute));
	CommandList->MapAction(FTimerTreeViewCommands::Get().Command_CopyNameToClipboard, FExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_CopyTimerNameToClipboard_Execute), FCanExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_CopyTimerNameToClipboard_CanExecute));
	CommandList->MapAction(FTimerTreeViewCommands::Get().Command_OpenSource, FExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_OpenSource_Execute), FCanExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_OpenSource_CanExecute));
	CommandList->MapAction(FTimerTreeViewCommands::Get().Command_FindMaxInstance, FExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_FindInstance_Execute, ESelectEventType::Max), FCanExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_FindInstance_CanExecute));
	CommandList->MapAction(FTimerTreeViewCommands::Get().Command_FindMinInstance, FExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_FindInstance_Execute, ESelectEventType::Min), FCanExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_FindInstance_CanExecute));
	CommandList->MapAction(FTimerTreeViewCommands::Get().Command_FindMaxInstanceInSelection, FExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_FindInstanceInSelection_Execute, ESelectEventType::Max), FCanExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_FindInstanceInSelection_CanExecute));
	CommandList->MapAction(FTimerTreeViewCommands::Get().Command_FindMinInstanceInSelection, FExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_FindInstanceInSelection_Execute, ESelectEventType::Min), FCanExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_FindInstanceInSelection_CanExecute));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STimerTreeView::Construct(const FArguments& InArgs, const FText& InViewName, ETreeType InTreeType)
{
	ViewName = InViewName;
	TreeType = InTreeType;

	TSharedRef<FTimerButterflyAggregator> TimerButterflyAggregator = FTimingProfilerManager::Get()->GetTimerButterflyAggregator();

	SAssignNew(ExternalScrollbar, SScrollBar)
	.AlwaysShowScrollbar(true);

	ChildSlot
	[
		SNew(SVerticalBox)

		// Tree view
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(0.0f)
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(TreeView, STreeView<FTimerNodePtr>)
					.ExternalScrollbar(ExternalScrollbar)
					.SelectionMode(ESelectionMode::Multi)
					.TreeItemsSource(&TreeNodes)
					.OnGetChildren(this, &STimerTreeView::TreeView_OnGetChildren)
					.OnGenerateRow(this, &STimerTreeView::TreeView_OnGenerateRow)
					//.OnSelectionChanged(this, &STimerTreeView::TreeView_OnSelectionChanged)
					//.OnMouseButtonDoubleClick(this, &STimerTreeView::TreeView_OnMouseButtonDoubleClick)
					.OnContextMenuOpening(FOnContextMenuOpening::CreateSP(this, &STimerTreeView::TreeView_GetMenuContent))
					.HeaderRow
					(
						SAssignNew(TreeViewHeaderRow, SHeaderRow)
						.Visibility(EVisibility::Visible)
					)
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				.Padding(16.0f)
				[
					SAssignNew(AsyncOperationStatus, SAsyncOperationStatus, TimerButterflyAggregator)
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f)
			[
				SNew(SBox)
				.WidthOverride(FOptionalSize(13.0f))
				[
					ExternalScrollbar.ToSharedRef()
				]
			]
		]

		// Status bar
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
			.BorderBackgroundColor(FLinearColor(0.05f, 0.1f, 0.2f, 1.0f))
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Margin(FMargin(4.0f, 1.0f, 4.0f, 1.0f))
				.Text(LOCTEXT("SelectionWarning", "Please select a time range and a timer!"))
				.ColorAndOpacity(FLinearColor(1.0f, 0.75f, 0.5f, 1.0f))
				.Visibility_Lambda([]()
				{
					TSharedPtr<FTimingProfilerManager> TimingProfilerManager = FTimingProfilerManager::Get();
					if (!TimingProfilerManager.IsValid())
					{
						return EVisibility::Collapsed;
					}
					TSharedRef<FTimerButterflyAggregator> Aggregator = TimingProfilerManager->GetTimerButterflyAggregator();
					if (Aggregator->IsRunning())
					{
						return EVisibility::Collapsed;
					}
					return !TimingProfilerManager->IsValidTimeSelection() || !TimingProfilerManager->IsValidSelectedTimer() ?
						EVisibility::Visible : EVisibility::Collapsed;
				})
			]
		]
	];

	InitializeAndShowHeaderColumns();

	CreateSortings();

	InitCommandList();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> STimerTreeView::TreeView_GetMenuContent()
{
	int32 NumSelectedNodes = TreeView->GetNumItemsSelected();
	FTimerNodePtr SingleSelectedNode = nullptr;
	bool bIsValidNode = false;

	if (NumSelectedNodes == 1)
	{
		TArray<FTimerNodePtr> SelectedNodes = TreeView->GetSelectedItems();
		check(SelectedNodes.Num() == 1);
		SingleSelectedNode = SelectedNodes[0];
		if (SingleSelectedNode.IsValid())
		{
			bIsValidNode = (SingleSelectedNode->GetType() != ETimerNodeType::Group);
		}
		else
		{
			NumSelectedNodes = 0;
		}
	}

	FText SelectionStr;
	if (NumSelectedNodes == 0)
	{
		SelectionStr = LOCTEXT("NothingSelected", "Nothing selected");
	}
	else if (NumSelectedNodes == 1)
	{
		check(SingleSelectedNode.IsValid());
		FString ItemName = SingleSelectedNode->GetDisplayName().ToString();
		const int32 MaxStringLen = 64;
		if (ItemName.Len() > MaxStringLen)
		{
			ItemName = ItemName.Left(MaxStringLen) + TEXT("...");
		}
		SelectionStr = FText::FromString(ItemName);
	}
	else
	{
		SelectionStr = FText::Format(LOCTEXT("MultipleSelection_Fmt", "{0} selected items"), FText::AsNumber(NumSelectedNodes));
	}

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList.ToSharedRef());

	// Selection menu
	MenuBuilder.BeginSection("Selection", LOCTEXT("ContextMenu_Section_Selection", "Selection"));
	{
		struct FLocal
		{
			static bool ReturnFalse()
			{
				return false;
			}
		};

		FUIAction DummyUIAction;
		DummyUIAction.CanExecuteAction = FCanExecuteAction::CreateStatic(&FLocal::ReturnFalse);
		MenuBuilder.AddMenuEntry
		(
			SelectionStr,
			LOCTEXT("ContextMenu_Selection", "Currently selected items"),
			FSlateIcon(),
			DummyUIAction,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	// Timer options section
	MenuBuilder.BeginSection("TimerOptions", LOCTEXT("ContextMenu_Section_TimerOptions", "Timer Options"));
	{
		auto CanExecute = [this, bIsValidNode]()
		{
			return bIsValidNode && GetTimingView().IsValid();
		};

		// Highlight event
		{
			FUIAction Action_ToggleHighlight;
			Action_ToggleHighlight.CanExecuteAction = FCanExecuteAction::CreateLambda(CanExecute);
			Action_ToggleHighlight.ExecuteAction = FExecuteAction::CreateSP(this, &STimerTreeView::ToggleTimingViewEventFilter, SingleSelectedNode);

			TSharedPtr<STimingView> TimingView = GetTimingView();

			if (bIsValidNode &&
				TimingView.IsValid() &&
				TimingView->IsFilterByEventType(SingleSelectedNode->GetTimerId()))
			{
				MenuBuilder.AddMenuEntry
				(
					LOCTEXT("ContextMenu_StopHighlightEvent", "Stop Highlighting Event"),
					LOCTEXT("ContextMenu_StopHighlightEvent_Desc", "Stops highlighting timing event instances for the selected timer."),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Visible"),
					Action_ToggleHighlight,
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}
			else
			{
				MenuBuilder.AddMenuEntry
				(
					LOCTEXT("ContextMenu_HighlightEvent", "Highlight Event"),
					LOCTEXT("ContextMenu_HighlightEvent_Desc", "Highlights all timing event instances for the selected timer."),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Visible"),
					Action_ToggleHighlight,
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}
		}

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_PlotTimer_SubMenu", "Plot Timer"),
			LOCTEXT("ContextMenu_PlotTimer_SubMenu_Desc", "Options to add the timer series to graph or frame tracks."),
			FNewMenuDelegate::CreateSP(this, &STimerTreeView::TreeView_BuildPlotTimerMenu),
			false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.AddGraphSeries")
		);

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_FindInstance_SubMenu", "Find Instance"),
			LOCTEXT("ContextMenu_PlotInstance_SubMenu_Desc", "Find the instance of this timer with the minimum or maximum duration."),
			FNewMenuDelegate::CreateSP(this, &STimerTreeView::TreeView_FindMenu),
			false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FindInstance")
		);

		// Open Source in IDE
		{
			ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
			ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();

			FString File;
			uint32 Line = 0;
			bool bIsValidSource = bIsValidNode && SingleSelectedNode->GetSourceFileAndLine(File, Line);
			FString UsablePath;
			const bool bPathExists = FInsightsManager::Get()->GetSourceFilePathHelper().GetUsableFilePath(File, UsablePath);

			FText ItemLabel;
			FText ItemToolTip;

			if (SourceCodeAccessor.CanAccessSourceCode())
			{
				ItemLabel = FText::Format(LOCTEXT("ContextMenu_OpenSource", "Open Source in {0}"), SourceCodeAccessor.GetNameText());
				if (bIsValidSource)
				{
					ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSource_Desc1", "Opens the source file of the selected timer in {0}.\n{1} ({2})"),
						SourceCodeAccessor.GetNameText(),
						FText::FromString(UsablePath),
						FText::AsNumber(Line, &FNumberFormattingOptions::DefaultNoGrouping()));
				}
				else
				{
					ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSource_Desc2", "Opens the source file of the selected timer in {0}."),
						SourceCodeAccessor.GetNameText());
				}
			}
			else
			{
				ItemLabel = LOCTEXT("ContextMenu_OpenSourceNA", "Open Source");
				if (bIsValidSource)
				{
					ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSourceNA_Desc1", "{0} ({1})\nSource Code Accessor is not available."),
						FText::FromString(UsablePath),
						FText::AsNumber(Line, &FNumberFormattingOptions::DefaultNoGrouping()));
				}
				else
				{
					ItemToolTip = LOCTEXT("ContextMenu_OpenSourceNA_Desc2", "Source Code Accessor is not available.");
				}
			}

			MenuBuilder.AddMenuEntry(
				FTimerTreeViewCommands::Get().Command_OpenSource,
				NAME_None,
				ItemLabel,
				ItemToolTip,
				FSlateIcon(SourceCodeAccessor.GetStyleSet(), SourceCodeAccessor.GetOpenIconName()));
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Misc", LOCTEXT("ContextMenu_Section_Misc", "Miscellaneous"));
	{
		MenuBuilder.AddMenuEntry
		(
			FTimerTreeViewCommands::Get().Command_CopyToClipboard,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);

		MenuBuilder.AddMenuEntry
		(
			FTimerTreeViewCommands::Get().Command_CopyNameToClipboard,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::TreeView_BuildSortByMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("SortColumn", LOCTEXT("ContextMenu_Section_SortColumn", "Sort Column"));

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (Column.IsVisible() && Column.CanBeSorted())
		{
			FUIAction Action_SortByColumn
			(
				FExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_SortByColumn_Execute, Column.GetId()),
				FCanExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_SortByColumn_CanExecute, Column.GetId()),
				FIsActionChecked::CreateSP(this, &STimerTreeView::ContextMenu_SortByColumn_IsChecked, Column.GetId())
			);
			MenuBuilder.AddMenuEntry
			(
				Column.GetTitleName(),
				Column.GetDescription(),
				FSlateIcon(),
				Action_SortByColumn,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}
	}

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("SortMode", LOCTEXT("ContextMenu_Section_SortMode", "Sort Mode"));
	{
		FUIAction Action_SortAscending
		(
			FExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_SortMode_Execute, EColumnSortMode::Ascending),
			FCanExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_SortMode_CanExecute, EColumnSortMode::Ascending),
			FIsActionChecked::CreateSP(this, &STimerTreeView::ContextMenu_SortMode_IsChecked, EColumnSortMode::Ascending)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_SortAscending", "Sort Ascending"),
			LOCTEXT("ContextMenu_SortAscending_Desc", "Sorts ascending."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortUp"),
			Action_SortAscending,
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		FUIAction Action_SortDescending
		(
			FExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_SortMode_Execute, EColumnSortMode::Descending),
			FCanExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_SortMode_CanExecute, EColumnSortMode::Descending),
			FIsActionChecked::CreateSP(this, &STimerTreeView::ContextMenu_SortMode_IsChecked, EColumnSortMode::Descending)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_SortDescending", "Sort Descending"),
			LOCTEXT("ContextMenu_SortDescending_Desc", "Sorts descending."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortDown"),
			Action_SortDescending,
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::TreeView_BuildViewColumnMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Columns", LOCTEXT("ContextMenu_Section_Columns", "Columns"));

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		FUIAction Action_ToggleColumn
		(
			FExecuteAction::CreateSP(this, &STimerTreeView::ToggleColumnVisibility, Column.GetId()),
			FCanExecuteAction::CreateSP(this, &STimerTreeView::CanToggleColumnVisibility, Column.GetId()),
			FIsActionChecked::CreateSP(this, &STimerTreeView::IsColumnVisible, Column.GetId())
		);
		MenuBuilder.AddMenuEntry
		(
			Column.GetTitleName(),
			Column.GetDescription(),
			FSlateIcon(),
			Action_ToggleColumn,
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::TreeView_BuildPlotTimerMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.SetSearchable(false);

	FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
	const bool bIsValidNode = SelectedNode.IsValid() && SelectedNode->GetType() != ETimerNodeType::Group;

	auto CanExecuteAddToGraphTrack = [this, bIsValidNode]()
	{
		return bIsValidNode && GetTimingView().IsValid();
	};

	auto CanExecuteAddToFramesTrack = [this, bIsValidNode]()
	{
		return bIsValidNode && GetFrameTrack().IsValid();
	};

	MenuBuilder.BeginSection("Instance", LOCTEXT("Plot_Series_Instance_Section", "Instance"));

	// Add/remove series to/from graph track
	{
		FUIAction Action_ToggleTimerInGraphTrack;
		Action_ToggleTimerInGraphTrack.CanExecuteAction = FCanExecuteAction::CreateLambda(CanExecuteAddToGraphTrack);
		Action_ToggleTimerInGraphTrack.ExecuteAction = FExecuteAction::CreateSP(this, &STimerTreeView::ToggleTimingViewMainGraphEventInstanceSeries, SelectedNode);

		if (bIsValidNode &&
			IsInstanceSeriesInTimingViewMainGraph(SelectedNode))
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_RemoveFromGraphTrack", "Remove instance series from graph track"),
				LOCTEXT("ContextMenu_RemoveFromGraphTrack_Desc", "Removes the series containing event instances of the selected timer from the Main Graph track."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.RemoveGraphSeries"),
				Action_ToggleTimerInGraphTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		else
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_AddToGraphTrack", "Add instance series to graph track"),
				LOCTEXT("ContextMenu_AddToGraphTrack_Desc", "Adds a series containing event instances of the selected timer to the Main Graph track."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.AddGraphSeries"),
				Action_ToggleTimerInGraphTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
	}

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Game Frame", LOCTEXT("Plot_Series_GameFrame_Section", "Game Frame"));

	// Add/remove game frame stats series to/from graph track
	{
		FUIAction Action_ToggleFrameStatsTimerInGraphTrack;
		Action_ToggleFrameStatsTimerInGraphTrack.CanExecuteAction = FCanExecuteAction::CreateLambda(CanExecuteAddToGraphTrack);
		Action_ToggleFrameStatsTimerInGraphTrack.ExecuteAction = FExecuteAction::CreateSP(this, &STimerTreeView::ToggleTimingViewMainGraphEventFrameStatsSeries, SelectedNode, ETraceFrameType::TraceFrameType_Game);

		if (bIsValidNode &&
			IsFrameStatsSeriesInTimingViewMainGraph(SelectedNode, ETraceFrameType::TraceFrameType_Game))
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_RemoveGameFrameStatsFromGraphTrack", "Remove game frame stats series from graph track"),
				LOCTEXT("ContextMenu_RemoveGameFrameStatsFromGraphTrack_Desc", "Remove the game frame stats series for this timer."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.RemoveGraphSeries"),
				Action_ToggleFrameStatsTimerInGraphTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		else
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_AddGameFrameStatsSeriesToGraphTrack", "Add game frame stats series to graph track"),
				LOCTEXT("ContextMenu_AddGameFrameStatsSeriesToGraphTrack_Desc", "Adds a game frame stats series for this timer. Each data entry is computed as the sum of all instances of this timer in a game frame."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.AddGraphSeries"),
				Action_ToggleFrameStatsTimerInGraphTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
	}

	// Add/remove game frame stats series to/from frame track
	{
		FUIAction Action_ToggleFrameStatsTimerInFrameTrack;
		Action_ToggleFrameStatsTimerInFrameTrack.CanExecuteAction = FCanExecuteAction::CreateLambda(CanExecuteAddToFramesTrack);
		Action_ToggleFrameStatsTimerInFrameTrack.ExecuteAction = FExecuteAction::CreateSP(this, &STimerTreeView::ToggleFrameTrackSeries, SelectedNode, ETraceFrameType::TraceFrameType_Game);

		if (bIsValidNode &&
			IsSeriesInFrameTrack(SelectedNode, ETraceFrameType::TraceFrameType_Game))
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_RemoveGameFrameStatsSeriesFromFrameTrack", "Remove game frame stats series from frame track"),
				LOCTEXT("ContextMenu_RemoveGameFrameStatsSeriesFromFrameTrack_Desc", "Remove the game frame stats series for this timer from the frame track."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.RemoveGraphSeries"),
				Action_ToggleFrameStatsTimerInFrameTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		else
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_AddGameFrameStatsSeriesToFrameTrack", "Add game frame stats series to the frame track"),
				LOCTEXT("ContextMenu_AddGameFrameStatsSeriesToFrameTrack_Desc", "Adds a game frame stats series for this timer to the frame track. Each data entry is computed as the sum of all instances of this timer in a game frame."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.AddGraphSeries"),
				Action_ToggleFrameStatsTimerInFrameTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
	}

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Rendering Frame", LOCTEXT("Plot_Series_RenderingFrame_Section", "Rendering frame"));

	// Add/remove rendering frame stats series to/from graph track
	{
		FUIAction Action_ToggleFrameStatsTimerInGraphTrack;
		Action_ToggleFrameStatsTimerInGraphTrack.CanExecuteAction = FCanExecuteAction::CreateLambda(CanExecuteAddToGraphTrack);
		Action_ToggleFrameStatsTimerInGraphTrack.ExecuteAction = FExecuteAction::CreateSP(this, &STimerTreeView::ToggleTimingViewMainGraphEventFrameStatsSeries, SelectedNode, ETraceFrameType::TraceFrameType_Rendering);

		if (bIsValidNode &&
			IsFrameStatsSeriesInTimingViewMainGraph(SelectedNode, ETraceFrameType::TraceFrameType_Rendering))
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_RemoveRenderingFrameStatsFromGraphTrack", "Remove rendering frame stats series from graph track"),
				LOCTEXT("ContextMenu_RemoveRenderingFrameStatsFromGraphTrack_Desc", "Remove the rendering frame stats series for this timer."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.RemoveGraphSeries"),
				Action_ToggleFrameStatsTimerInGraphTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		else
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_AddRenderingFrameStatsSeriesToGraphTrack", "Add rendering frame stats series to graph track"),
				LOCTEXT("ContextMenu_AddRenderingFrameStatsSeriesToGraphTrack_Desc", "Adds a rendering frame stats series for this timer. Each data entry is computed as the sum of all instances of this timer in a rendering frame."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.AddGraphSeries"),
				Action_ToggleFrameStatsTimerInGraphTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
	}

	// Add/remove rendering frame stats series to/from frame track
	{
		FUIAction Action_ToggleFrameStatsTimerInFrameTrack;
		Action_ToggleFrameStatsTimerInFrameTrack.CanExecuteAction = FCanExecuteAction::CreateLambda(CanExecuteAddToFramesTrack);
		Action_ToggleFrameStatsTimerInFrameTrack.ExecuteAction = FExecuteAction::CreateSP(this, &STimerTreeView::ToggleFrameTrackSeries, SelectedNode, ETraceFrameType::TraceFrameType_Rendering);

		if (bIsValidNode &&
			IsSeriesInFrameTrack(SelectedNode, ETraceFrameType::TraceFrameType_Rendering))
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_RemoveRenderingFrameStatsFromFrameTrac", "Remove rendering frame stats series from frame track"),
				LOCTEXT("ContextMenu_RemoveRenderingFrameStatsFromFrameTrack_Desc", "Remove the rendering frame stats series for this timer from the frame track."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.RemoveGraphSeries"),
				Action_ToggleFrameStatsTimerInFrameTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		else
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_AddRenderingFrameStatsSeriesToFrameTrack", "Add rendering frame stats series to the frame track"),
				LOCTEXT("ContextMenu_AddRenderingFrameStatsSeriesToFrameTrack_Desc", "Adds a rendering frame stats series for this timer to the frame track. Each data entry is computed as the sum of all instances of this timer in a rendering frame."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.AddGraphSeries"),
				Action_ToggleFrameStatsTimerInFrameTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
	}

	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::TreeView_FindMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.SetSearchable(false);

	MenuBuilder.AddMenuEntry
	(
		FTimerTreeViewCommands::Get().Command_FindMaxInstance,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FindMaxInstance")
	);

	MenuBuilder.AddMenuEntry
	(
		FTimerTreeViewCommands::Get().Command_FindMinInstance,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FindMinInstance")
	);

	MenuBuilder.AddMenuSeparator();

	MenuBuilder.AddMenuEntry
	(
		FTimerTreeViewCommands::Get().Command_FindMaxInstanceInSelection,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FindMaxInstance")
	);

	MenuBuilder.AddMenuEntry
	(
		FTimerTreeViewCommands::Get().Command_FindMinInstanceInSelection,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FindMinInstance")
	);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::InitializeAndShowHeaderColumns()
{
	// Create columns.
	TArray<TSharedRef<FTableColumn>> Columns;
	if (TreeType == ETreeType::Callers)
	{
		FTimersViewColumnFactory::CreateCallersViewColumns(Columns);
	}
	else if (TreeType == ETreeType::Callees)
	{
		FTimersViewColumnFactory::CreateCalleesViewColumns(Columns);
	}
	else
	{
		FTimersViewColumnFactory::CreateTimersViewColumns(Columns);
	}
	if (ensure(Columns.Num() > 0 && Columns[0]->IsHierarchy()))
	{
		Columns[0]->SetShortName(ViewName);
		Columns[0]->SetTitleName(ViewName);
	}
	Table->SetColumns(Columns);

	// Show columns.
	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		if (ColumnRef->ShouldBeVisible())
		{
			ShowColumn(ColumnRef->GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STimerTreeView::GetColumnHeaderText(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.GetShortName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimerTreeView::TreeViewHeaderRow_GenerateColumnMenu(const FTableColumn& Column)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, NULL);

	MenuBuilder.BeginSection("Sorting", LOCTEXT("ContextMenu_Section_Sorting", "Sorting"));
	{
		if (Column.CanBeSorted())
		{
			FUIAction Action_SortAscending
			(
				FExecuteAction::CreateSP(this, &STimerTreeView::HeaderMenu_SortMode_Execute, Column.GetId(), EColumnSortMode::Ascending),
				FCanExecuteAction::CreateSP(this, &STimerTreeView::HeaderMenu_SortMode_CanExecute, Column.GetId(), EColumnSortMode::Ascending),
				FIsActionChecked::CreateSP(this, &STimerTreeView::HeaderMenu_SortMode_IsChecked, Column.GetId(), EColumnSortMode::Ascending)
			);
			MenuBuilder.AddMenuEntry
			(
				FText::Format(LOCTEXT("ContextMenu_SortAscending_Fmt", "Sort Ascending (by {0})"), Column.GetTitleName()),
				FText::Format(LOCTEXT("ContextMenu_SortAscending_Desc_Fmt", "Sorts ascending by {0}."), Column.GetTitleName()),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortUp"),
				Action_SortAscending,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);

			FUIAction Action_SortDescending
			(
				FExecuteAction::CreateSP(this, &STimerTreeView::HeaderMenu_SortMode_Execute, Column.GetId(), EColumnSortMode::Descending),
				FCanExecuteAction::CreateSP(this, &STimerTreeView::HeaderMenu_SortMode_CanExecute, Column.GetId(), EColumnSortMode::Descending),
				FIsActionChecked::CreateSP(this, &STimerTreeView::HeaderMenu_SortMode_IsChecked, Column.GetId(), EColumnSortMode::Descending)
			);
			MenuBuilder.AddMenuEntry
			(
				FText::Format(LOCTEXT("ContextMenu_SortDescending_Fmt", "Sort Descending (by {0})"), Column.GetTitleName()),
				FText::Format(LOCTEXT("ContextMenu_SortDescending_Desc_Fmt", "Sorts descending by {0}."), Column.GetTitleName()),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortDown"),
				Action_SortDescending,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_SortBy_SubMenu", "Sort By"),
			LOCTEXT("ContextMenu_SortBy_SubMenu_Desc", "Sorts by a column."),
			FNewMenuDelegate::CreateSP(this, &STimerTreeView::TreeView_BuildSortByMenu),
			false,
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.SortBy")
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("ColumnVisibility", LOCTEXT("ContextMenu_Section_ColumnVisibility", "Column Visibility"));
	{
		if (Column.CanBeHidden())
		{
			FUIAction Action_HideColumn
			(
				FExecuteAction::CreateSP(this, &STimerTreeView::HideColumn, Column.GetId()),
				FCanExecuteAction::CreateSP(this, &STimerTreeView::CanHideColumn, Column.GetId())
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_HideColumn", "Hide"),
				LOCTEXT("ContextMenu_HideColumn_Desc", "Hides the selected column."),
				FSlateIcon(),
				Action_HideColumn,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_ViewColumn_SubMenu", "View Column"),
			LOCTEXT("ContextMenu_ViewColumn_SubMenu_Desc", "Hides or shows columns."),
			FNewMenuDelegate::CreateSP(this, &STimerTreeView::TreeView_BuildViewColumnMenu),
			false,
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ViewColumn")
		);

		FUIAction Action_ShowAllColumns
		(
			FExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_ShowAllColumns_Execute),
			FCanExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_ShowAllColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_ShowAllColumns", "Show All Columns"),
			LOCTEXT("ContextMenu_ShowAllColumns_Desc", "Resets tree view to show all columns."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_ShowAllColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);

		FUIAction Action_ResetColumns
		(
			FExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_ResetColumns_Execute),
			FCanExecuteAction::CreateSP(this, &STimerTreeView::ContextMenu_ResetColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_ResetColumns", "Reset Columns to Default"),
			LOCTEXT("ContextMenu_ResetColumns_Desc", "Resets columns to default."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_ResetColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// TreeView
////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::TreeView_Refresh()
{
	if (TreeView.IsValid())
	{
		TreeView->RequestTreeRefresh();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::TreeView_OnSelectionChanged(FTimerNodePtr SelectedItem, ESelectInfo::Type SelectInfo)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNodePtr STimerTreeView::GetSingleSelectedTimerNode() const
{
	if (TreeView->GetNumItemsSelected() != 1)
	{
		return nullptr;
	}
	const TArray<FTimerNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	check(SelectedNodes.Num() == 1);
	return SelectedNodes[0];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::TreeView_OnGetChildren(FTimerNodePtr InParent, TArray<FTimerNodePtr>& OutChildren)
{
	constexpr bool bUseFiltering = false;
	if (bUseFiltering)
	{
		const TArray<FBaseTreeNodePtr>& Children = InParent->GetFilteredChildren();
		OutChildren.Reset(Children.Num());
		for (const FBaseTreeNodePtr& Child : Children)
		{
			check(Child->Is<FTimerNode>());
			OutChildren.Add(StaticCastSharedPtr<FTimerNode>(Child));
		}
	}
	else
	{
		const TArray<FBaseTreeNodePtr>& Children = InParent->GetChildren();
		OutChildren.Reset(Children.Num());
		for (const FBaseTreeNodePtr& Child : Children)
		{
			check(Child->Is<FTimerNode>());
			OutChildren.Add(StaticCastSharedPtr<FTimerNode>(Child));
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::TreeView_OnMouseButtonDoubleClick(FTimerNodePtr NodePtr)
{
	if (NodePtr->GetChildrenCount() > 0)
	{
		const bool bIsGroupExpanded = TreeView->IsItemExpanded(NodePtr);
		TreeView->SetItemExpansion(NodePtr, !bIsGroupExpanded);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Tree View's Table Row
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableRow> STimerTreeView::TreeView_OnGenerateRow(FTimerNodePtr NodePtr, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<ITableRow> TableRow =
		SNew(STimerTableRow, OwnerTable)
		.OnShouldBeEnabled(this, &STimerTreeView::TableRow_ShouldBeEnabled)
		.OnIsColumnVisible(this, &STimerTreeView::IsColumnVisible)
		.OnSetHoveredCell(this, &STimerTreeView::TableRow_SetHoveredCell)
		.OnGetColumnOutlineHAlignmentDelegate(this, &STimerTreeView::TableRow_GetColumnOutlineHAlignment)
		.HighlightText(this, &STimerTreeView::TableRow_GetHighlightText)
		.TablePtr(Table)
		.TimerNodePtr(NodePtr);

	return TableRow;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::TableRow_ShouldBeEnabled(FTimerNodePtr NodePtr) const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::TableRow_SetHoveredCell(TSharedPtr<FTable> InTablePtr, TSharedPtr<FTableColumn> InColumnPtr, FTimerNodePtr InNodePtr)
{
	HoveredColumnId = InColumnPtr ? InColumnPtr->GetId() : FName();

	const bool bIsAnyMenusVisible = FSlateApplication::Get().AnyMenusVisible();
	if (!HasMouseCapture() && !bIsAnyMenusVisible)
	{
		HoveredNodePtr = InNodePtr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EHorizontalAlignment STimerTreeView::TableRow_GetColumnOutlineHAlignment(const FName ColumnId) const
{
	const TIndirectArray<SHeaderRow::FColumn>& Columns = TreeViewHeaderRow->GetColumns();

	// First column
	if (Columns.Num() == 0 ||
		Columns[0].ColumnId == ColumnId)
	{
		return HAlign_Left;
	}
	// Last column
	else if (Columns[Columns.Num() - 1].ColumnId == ColumnId)
	{
		return HAlign_Right;
	}
	// Middle columns
	{
		return HAlign_Center;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STimerTreeView::TableRow_GetHighlightText() const
{
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorting
////////////////////////////////////////////////////////////////////////////////////////////////////

const FName STimerTreeView::GetDefaultColumnBeingSorted()
{
	return FTimersViewColumns::TotalInclusiveTimeColumnID;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const EColumnSortMode::Type STimerTreeView::GetDefaultColumnSortMode()
{
	return EColumnSortMode::Descending;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::CreateSortings()
{
	AvailableSorters.Reset();
	CurrentSorter = nullptr;

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		if (ColumnRef->CanBeSorted())
		{
			TSharedPtr<ITableCellValueSorter> SorterPtr = ColumnRef->GetValueSorter();
			if (ensure(SorterPtr.IsValid()))
			{
				AvailableSorters.Add(SorterPtr);
			}
		}
	}

	UpdateCurrentSortingByColumn();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::UpdateCurrentSortingByColumn()
{
	TSharedPtr<FTableColumn> ColumnPtr = Table->FindColumn(ColumnBeingSorted);
	CurrentSorter = ColumnPtr.IsValid() ? ColumnPtr->GetValueSorter() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::SortTreeNodes()
{
	if (CurrentSorter.IsValid())
	{
		for (FTimerNodePtr& Root : TreeNodes)
		{
			SortTreeNodesRec(*Root, *CurrentSorter);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::SortTreeNodesRec(FTimerNode& Node, const ITableCellValueSorter& Sorter)
{
	ESortMode SortMode = (ColumnSortMode == EColumnSortMode::Type::Descending) ? ESortMode::Descending : ESortMode::Ascending;
	Node.SortChildren(Sorter, SortMode);

	for (FBaseTreeNodePtr ChildPtr : Node.GetChildren())
	{
		if (ChildPtr->GetChildrenCount() > 0)
		{
			check(ChildPtr->Is<FTimerNode>());
			SortTreeNodesRec(*StaticCastSharedPtr<FTimerNode>(ChildPtr), Sorter);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EColumnSortMode::Type STimerTreeView::GetSortModeForColumn(const FName ColumnId) const
{
	if (ColumnBeingSorted != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return ColumnSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::SetSortModeForColumn(const FName& ColumnId, const EColumnSortMode::Type SortMode)
{
	ColumnBeingSorted = ColumnId;
	ColumnSortMode = SortMode;
	UpdateCurrentSortingByColumn();

	SortTreeNodes();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode)
{
	SetSortModeForColumn(ColumnId, SortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortMode action (HeaderMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::HeaderMenu_SortMode_IsChecked(const FName ColumnId, const EColumnSortMode::Type InSortMode)
{
	return ColumnBeingSorted == ColumnId && ColumnSortMode == InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::HeaderMenu_SortMode_CanExecute(const FName ColumnId, const EColumnSortMode::Type InSortMode) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.CanBeSorted();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::HeaderMenu_SortMode_Execute(const FName ColumnId, const EColumnSortMode::Type InSortMode)
{
	SetSortModeForColumn(ColumnId, InSortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortMode action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::ContextMenu_SortMode_IsChecked(const EColumnSortMode::Type InSortMode)
{
	return ColumnSortMode == InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::ContextMenu_SortMode_CanExecute(const EColumnSortMode::Type InSortMode) const
{
	return true; //ColumnSortMode != InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ContextMenu_SortMode_Execute(const EColumnSortMode::Type InSortMode)
{
	SetSortModeForColumn(ColumnBeingSorted, InSortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortByColumn action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::ContextMenu_SortByColumn_IsChecked(const FName ColumnId)
{
	return ColumnId == ColumnBeingSorted;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::ContextMenu_SortByColumn_CanExecute(const FName ColumnId) const
{
	return true; //ColumnId != ColumnBeingSorted;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ContextMenu_SortByColumn_Execute(const FName ColumnId)
{
	SetSortModeForColumn(ColumnId, EColumnSortMode::Descending);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ShowColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::CanShowColumn(const FName ColumnId) const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ShowColumn(const FName ColumnId)
{
	FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	Column.Show();

	SHeaderRow::FColumn::FArguments ColumnArgs;
	ColumnArgs
		.ColumnId(Column.GetId())
		.DefaultLabel(Column.GetShortName())
		.ToolTip(STimersViewTooltip::GetColumnTooltipForMode(Column, ETimerAggregationMode::Instance))
		.HAlignHeader(Column.GetHorizontalAlignment())
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Fill)
		.InitialSortMode(Column.GetInitialSortMode())
		.SortMode(this, &STimerTreeView::GetSortModeForColumn, Column.GetId())
		.OnSort(this, &STimerTreeView::OnSortModeChanged)
		.FillWidth(Column.GetInitialWidth())
		//.FixedWidth(Column.IsFixedWidth() ? Column.GetInitialWidth() : TOptional<float>())
		.HeaderContent()
		[
			SNew(SBox)
			.HeightOverride(24.0f)
			.Padding(FMargin(0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &STimerTreeView::GetColumnHeaderText, Column.GetId())
			]
		]
		.MenuContent()
		[
			TreeViewHeaderRow_GenerateColumnMenu(Column)
		];

	int32 ColumnIndex = 0;
	const int32 NewColumnPosition = Table->GetColumnPositionIndex(ColumnId);
	const int32 NumColumns = TreeViewHeaderRow->GetColumns().Num();
	for (; ColumnIndex < NumColumns; ColumnIndex++)
	{
		const SHeaderRow::FColumn& CurrentColumn = TreeViewHeaderRow->GetColumns()[ColumnIndex];
		const int32 CurrentColumnPosition = Table->GetColumnPositionIndex(CurrentColumn.ColumnId);
		if (NewColumnPosition < CurrentColumnPosition)
		{
			break;
		}
	}

	TreeViewHeaderRow->InsertColumn(ColumnArgs, ColumnIndex);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// HideColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::CanHideColumn(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.CanBeHidden();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::HideColumn(const FName ColumnId)
{
	FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	Column.Hide();

	TreeViewHeaderRow->RemoveColumn(ColumnId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ToggleColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::IsColumnVisible(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::CanToggleColumnVisibility(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return !Column.IsVisible() || Column.CanBeHidden();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ToggleColumnVisibility(const FName ColumnId)
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	if (Column.IsVisible())
	{
		HideColumn(ColumnId);
	}
	else
	{
		ShowColumn(ColumnId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// "Show All Columns" action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::ContextMenu_ShowAllColumns_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ContextMenu_ShowAllColumns_Execute()
{
	ColumnBeingSorted = GetDefaultColumnBeingSorted();
	ColumnSortMode = GetDefaultColumnSortMode();
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (!Column.IsVisible())
		{
			ShowColumn(Column.GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ResetColumns action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::ContextMenu_ResetColumns_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ContextMenu_ResetColumns_Execute()
{
	ColumnBeingSorted = GetDefaultColumnBeingSorted();
	ColumnSortMode = GetDefaultColumnSortMode();
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (Column.ShouldBeVisible() && !Column.IsVisible())
		{
			ShowColumn(Column.GetId());
		}
		else if (!Column.ShouldBeVisible() && Column.IsVisible())
		{
			HideColumn(Column.GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::Reset()
{
	TreeNodes.Reset();
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::SetTree(const TraceServices::FTimingProfilerButterflyNode& Root)
{
	TreeNodes.Reset();
	FTimerNodePtr RootTimerNodePtr;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			RootTimerNodePtr = CreateTimerNodeRec(Root, *TimingProfilerProvider);
			if (RootTimerNodePtr)
			{
				// Mark the hot path. The child nodes are already sorted by InclTime (descending), so we just follow the first child.
				FTimerNodePtr TimerNodePtr = RootTimerNodePtr;
				while (TimerNodePtr.IsValid())
				{
					TimerNodePtr->SetIsHotPath(true);
					const TArray<FBaseTreeNodePtr>& Children = TimerNodePtr->GetChildren();
					if (Children.Num() > 0)
					{
						check(Children[0]->Is<FTimerNode>());
						TimerNodePtr = StaticCastSharedPtr<FTimerNode>(Children[0]);
					}
					else
					{
						TimerNodePtr = nullptr;
					}
				}

				TreeNodes.Add(RootTimerNodePtr);
			}
		}
	}

	SortTreeNodes();

	TreeView_Refresh();

	if (RootTimerNodePtr)
	{
		ExpandNodesRec(RootTimerNodePtr, 0);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNodePtr STimerTreeView::CreateTimerNodeRec(const TraceServices::FTimingProfilerButterflyNode& Node, const TraceServices::ITimingProfilerProvider& TimingProfilerProvider)
{
	if (Node.Timer == nullptr)
	{
		return nullptr;
	}

	const ETimerNodeType Type =
		Node.Timer->Type == TraceServices::ETimingProfilerTimerType::GpuScope ? ETimerNodeType::GpuScope :
		Node.Timer->Type == TraceServices::ETimingProfilerTimerType::VerseSampling ? ETimerNodeType::VerseSampling :
		Node.Timer->Type == TraceServices::ETimingProfilerTimerType::CpuScope ? ETimerNodeType::CpuScope :
		Node.Timer->Type == TraceServices::ETimingProfilerTimerType::CpuSampling ? ETimerNodeType::CpuSampling :
		ETimerNodeType::InvalidOrMax;

	TSharedPtr<STimingProfilerWindow> ProfilerWindow = FTimingProfilerManager::Get()->GetProfilerWindow();
	const bool bIsTimerAddedToGraphs = ProfilerWindow.IsValid() ? ProfilerWindow->IsTimerAddedToGraphs(Node.Timer->Id) : false;

	TStringBuilder<1024> StringBuilder;
	if (FInsightsManager::Get()->GetSettings().GetShowFormatInTimerNames())
	{
		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			TimingProfilerProvider.MakeTimerDisplayName(*Node.Timer, StringBuilder);
		}
	}
	else
	{
		StringBuilder.Append(Node.Timer->Name);
	}

	FTimerNodePtr TimerNodePtr = MakeShared<FTimerNode>(Node.Timer->Id, *StringBuilder, Type, true);
	FLinearColor ColorOverride;

	if (ProfilerWindow.IsValid() && ProfilerWindow->GetTimerColor(Node.Timer->Id, ColorOverride))
	{
		TimerNodePtr->SetColor(ColorOverride);
	}

	TimerNodePtr->SetAddedToGraphsFlag(bIsTimerAddedToGraphs);

	TraceServices::FTimingProfilerAggregatedStats AggregatedStats;
	AggregatedStats.InstanceCount = Node.Count;
	AggregatedStats.TotalInclusiveTime = Node.InclusiveTime;
	AggregatedStats.TotalExclusiveTime = Node.ExclusiveTime;
	AggregatedStats.AverageInclusiveTime = Node.Count != 0 ? Node.InclusiveTime / (double)Node.Count : 0.0;
	AggregatedStats.AverageExclusiveTime = Node.Count != 0 ? Node.ExclusiveTime / (double)Node.Count : 0.0;
	constexpr double NanTimeValue = std::numeric_limits<double>::quiet_NaN();
	AggregatedStats.MinInclusiveTime = NanTimeValue;
	AggregatedStats.MinExclusiveTime = NanTimeValue;
	AggregatedStats.MaxInclusiveTime = NanTimeValue;
	AggregatedStats.MaxExclusiveTime = NanTimeValue;
	AggregatedStats.MedianInclusiveTime = NanTimeValue;
	AggregatedStats.MedianExclusiveTime = NanTimeValue;
	TimerNodePtr->SetAggregatedStats(AggregatedStats);

	for (const TraceServices::FTimingProfilerButterflyNode* ChildNodePtr : Node.Children)
	{
		if (ChildNodePtr != nullptr)
		{
			FTimerNodePtr ChildTimerNodePtr = CreateTimerNodeRec(*ChildNodePtr, TimingProfilerProvider);
			if (ChildTimerNodePtr)
			{
				TimerNodePtr->AddChildAndSetParent(ChildTimerNodePtr);
			}
		}
	}

	// Sort children by InclTime (descending).
	TimerNodePtr->SortChildren([](const FBaseTreeNodePtr& A, const FBaseTreeNodePtr& B) -> bool
	{
		checkSlow(A->Is<FTimerNode>());
		const double InclTimeA = StaticCastSharedPtr<FTimerNode>(A)->GetAggregatedStats().TotalInclusiveTime;
		checkSlow(B->Is<FTimerNode>());
		const double InclTimeB = StaticCastSharedPtr<FTimerNode>(B)->GetAggregatedStats().TotalInclusiveTime;
		return InclTimeA >= InclTimeB;
	});

	return TimerNodePtr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ExpandNodesRec(FTimerNodePtr NodePtr, int32 Depth)
{
	//constexpr int32 MaxDepth = 3;

	TreeView->SetItemExpansion(NodePtr, NodePtr->IsHotPath()); // expand only the hot path

	//if (Depth < MaxDepth)
	{
		for (const FBaseTreeNodePtr& ChildPtr : NodePtr->GetChildren())
		{
			check(ChildPtr->Is<FTimerNode>());
			ExpandNodesRec(StaticCastSharedPtr<FTimerNode>(ChildPtr), Depth + 1);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNodePtr STimerTreeView::GetTimerNode(uint32 TimerId) const
{
	for (FTimerNodePtr TimerNode : TreeNodes)
	{
		FTimerNodePtr FoundNode = GetTimerNodeRec(TimerId, TimerNode);
		if (FoundNode)
		{
			return FoundNode;
		}
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNodePtr STimerTreeView::GetTimerNodeRec(uint32 TimerId, const FTimerNodePtr TimerNode) const
{
	if (TimerNode->GetTimerId() == TimerId)
	{
		return TimerNode;
	}

	for (const FBaseTreeNodePtr& ChildPtr : TimerNode->GetChildren())
	{
		check(ChildPtr->Is<FTimerNode>());
		FTimerNodePtr FoundNode = GetTimerNodeRec(TimerId, StaticCastSharedPtr<FTimerNode>(ChildPtr));
		if (FoundNode)
		{
			return FoundNode;
		}
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ToggleTimingViewEventFilter(FTimerNodePtr TimerNode) const
{
	TSharedPtr<STimingView> TimingView = GetTimingView();
	if (TimingView.IsValid())
	{
		const uint64 EventType = static_cast<uint64>(TimerNode->GetTimerId());
		TimingView->ToggleEventFilterByEventType(EventType);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTimingGraphTrack> STimerTreeView::GetTimingViewMainGraphTrack() const
{
	TSharedPtr<STimingView> TimingView = GetTimingView();
	return TimingView.IsValid() ? TimingView->GetMainTimingGraphTrack() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SFrameTrack> STimerTreeView::GetFrameTrack() const
{
	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	return Wnd.IsValid() ? Wnd->GetFrameView() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ToggleGraphInstanceSeries(TSharedRef<FTimingGraphTrack> GraphTrack, FTimerNodeRef NodePtr) const
{
	const uint32 TimerId = NodePtr->GetTimerId();

	TSharedPtr<FTimingGraphSeries> Series = GraphTrack->GetTimerSeries(TimerId);
	if (Series.IsValid())
	{
		GraphTrack->RemoveTimerSeries(TimerId);
		GraphTrack->SetDirtyFlag();
	}
	else
	{
		GraphTrack->Show();
		Series = GraphTrack->AddTimerSeries(TimerId, NodePtr->GetColor());
		Series->SetName(NodePtr->GetDisplayName());
		GraphTrack->SetDirtyFlag();
	}

	TSharedPtr<STimingProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		ProfilerWindow->OnTimerAddedToGraphsChanged(TimerId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::IsInstanceSeriesInTimingViewMainGraph(FTimerNodePtr TimerNode) const
{
	TSharedPtr<FTimingGraphTrack> GraphTrack = GetTimingViewMainGraphTrack();

	if (GraphTrack.IsValid())
	{
		const uint32 TimerId = TimerNode->GetTimerId();
		TSharedPtr<FTimingGraphSeries> Series = GraphTrack->GetTimerSeries(TimerId);

		return Series.IsValid();
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ToggleTimingViewMainGraphEventInstanceSeries(FTimerNodePtr TimerNode) const
{
	TSharedPtr<FTimingGraphTrack> GraphTrack = GetTimingViewMainGraphTrack();
	if (GraphTrack.IsValid())
	{
		ToggleGraphInstanceSeries(GraphTrack.ToSharedRef(), TimerNode.ToSharedRef());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ToggleGraphFrameStatsSeries(TSharedRef<FTimingGraphTrack> GraphTrack, FTimerNodeRef NodePtr, ETraceFrameType FrameType) const
{
	const uint32 TimerId = NodePtr->GetTimerId();

	TSharedPtr<FTimingGraphSeries> Series = GraphTrack->GetFrameStatsTimerSeries(TimerId, FrameType);
	if (Series.IsValid())
	{
		GraphTrack->RemoveFrameStatsTimerSeries(TimerId, FrameType);
		GraphTrack->SetDirtyFlag();
	}
	else
	{
		GraphTrack->Show();
		Series = GraphTrack->AddFrameStatsTimerSeries(TimerId, FrameType, NodePtr->GetColor());
		FText SeriesName = FText::Format(LOCTEXT("FrameStatsTimerSeriesName_Fmt", "{0} ({1})"),
			NodePtr->GetDisplayName(),
			FFrameTrackDrawHelper::FrameTypeToText(FrameType));
		Series->SetName(SeriesName);
		GraphTrack->SetDirtyFlag();
	}

	TSharedPtr<STimingProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		ProfilerWindow->OnTimerAddedToGraphsChanged(TimerId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::IsFrameStatsSeriesInTimingViewMainGraph(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const
{
	if (!TimerNode.IsValid())
	{
		return false;
	}

	TSharedPtr<FTimingGraphTrack> GraphTrack = GetTimingViewMainGraphTrack();
	if (!GraphTrack.IsValid())
	{
		return false;
	}

	const uint32 TimerId = TimerNode->GetTimerId();
	TSharedPtr<FTimingGraphSeries> Series = GraphTrack->GetFrameStatsTimerSeries(TimerId, FrameType);
	return Series.IsValid();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ToggleTimingViewMainGraphEventFrameStatsSeries(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const
{
	if (!TimerNode.IsValid())
	{
		return;
	}

	TSharedPtr<FTimingGraphTrack> GraphTrack = GetTimingViewMainGraphTrack();
	if (!GraphTrack.IsValid())
	{
		return;
	}

	ToggleGraphFrameStatsSeries(GraphTrack.ToSharedRef(), TimerNode.ToSharedRef(), FrameType);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::IsSeriesInFrameTrack(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const
{
	if (!TimerNode.IsValid())
	{
		return false;
	}

	TSharedPtr<SFrameTrack> FrameTrack = GetFrameTrack();
	if (!FrameTrack.IsValid())
	{
		return false;
	}

	return FrameTrack->HasFrameStatSeries(FrameType, TimerNode->GetTimerId());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ToggleFrameTrackSeries(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const
{
	if (!TimerNode.IsValid())
	{
		return;
	}

	TSharedPtr<SFrameTrack> FrameTrack = GetFrameTrack();
	if (!FrameTrack.IsValid())
	{
		return;
	}

	const uint32 TimerId = TimerNode->GetTimerId();

	if (FrameTrack->HasFrameStatSeries(FrameType, TimerId))
	{
		FrameTrack->RemoveTimerFrameStatSeries(FrameType, TimerId);
	}
	else
	{
		FText SeriesName = FText::Format(LOCTEXT("FrameStatsTimerSeriesName_Fmt", "{0} ({1})"),
			TimerNode->GetDisplayName(),
			FFrameTrackDrawHelper::FrameTypeToText(FrameType));
		FrameTrack->AddTimerFrameStatSeries(FrameType, TimerId, TimerNode->GetColor(), SeriesName);
	}

	TSharedPtr<STimingProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		ProfilerWindow->OnTimerAddedToGraphsChanged(TimerId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Copy to Clipboard
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::ContextMenu_CopyToClipboard_CanExecute() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ContextMenu_CopyToClipboard_Execute()
{
	if (Table->IsValid())
	{
		const ESortMode SortMode = ColumnSortMode == EColumnSortMode::Ascending ? Insights::ESortMode::Ascending : Insights::ESortMode::Descending;
		UE::Insights::CopyToClipboard(Table.ToSharedRef(), TreeView->GetSelectedItems(), CurrentSorter, SortMode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::ContextMenu_CopyTimerNameToClipboard_CanExecute() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ContextMenu_CopyTimerNameToClipboard_Execute()
{
	if (Table->IsValid())
	{
		const ESortMode SortMode = ColumnSortMode == EColumnSortMode::Ascending ? Insights::ESortMode::Ascending : Insights::ESortMode::Descending;
		UE::Insights::CopyNameToClipboard(Table.ToSharedRef(), TreeView->GetSelectedItems(), CurrentSorter, SortMode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Open Source File in IDE
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::ContextMenu_OpenSource_CanExecute() const
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();

	if (!SourceCodeAccessor.CanAccessSourceCode())
	{
		return false;
	}

	FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
	if (!SelectedNode.IsValid())
	{
		return false;
	}

	FString File;
	uint32 Line = 0;
	return SelectedNode->GetSourceFileAndLine(File, Line);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ContextMenu_OpenSource_Execute() const
{
	FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
	if (SelectedNode.IsValid())
	{
		OpenSourceFileInIDE(SelectedNode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::OpenSourceFileInIDE(FTimerNodePtr InNode) const
{
	if (!InNode.IsValid() || InNode->GetType() == ETimerNodeType::Group)
	{
		return;
	}

	FString File;
	uint32 Line = 0;
	if (!InNode->GetSourceFileAndLine(File, Line))
	{
		return;
	}

	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");

	FString UsablePath;
	const bool bPathExists = FInsightsManager::Get()->GetSourceFilePathHelper().GetUsableFilePath(File, UsablePath);
	if (bPathExists)
	{
		ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();
		SourceCodeAccessor.OpenFileAtLine(UsablePath, Line);
	}
	else
	{
		SourceCodeAccessModule.OnOpenFileFailed().Broadcast(UsablePath);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Find Min/Max Instance
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::ContextMenu_FindInstance_CanExecute() const
{
	FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
	return SelectedNode.IsValid() && SelectedNode->GetType() != ETimerNodeType::Group;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ContextMenu_FindInstance_Execute(ESelectEventType EventType) const
{
	FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
	if (!SelectedNode.IsValid() || SelectedNode->GetType() == ETimerNodeType::Group)
	{
		return;
	}

	TSharedPtr<STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return;
	}

	TimingView->SelectEventInstance(SelectedNode->GetTimerId(), EventType, false);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimerTreeView::ContextMenu_FindInstanceInSelection_CanExecute() const
{
	FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
	if (!SelectedNode.IsValid() || SelectedNode->GetType() == ETimerNodeType::Group)
	{
		return false;
	}

	TSharedPtr<STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return false;
	}

	return TimingView->GetSelectionEndTime() > TimingView->GetSelectionStartTime();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimerTreeView::ContextMenu_FindInstanceInSelection_Execute(ESelectEventType EventType) const
{
	FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
	if (!SelectedNode.IsValid() || SelectedNode->GetType() == ETimerNodeType::Group)
	{
		return;
	}

	TSharedPtr<STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return;
	}

	TimingView->SelectEventInstance(SelectedNode->GetTimerId(), EventType, true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Misc
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<STimingProfilerWindow> STimerTreeView::GetProfilerWindow() const
{
	return FTimingProfilerManager::Get()->GetProfilerWindow();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<STimingView> STimerTreeView::GetTimingView() const
{
	TSharedPtr<STimingProfilerWindow> ProfilerWindow = GetProfilerWindow();
	return ProfilerWindow.IsValid() ? ProfilerWindow->GetTimingView() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimerTreeView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return CommandList->ProcessCommandBindings(InKeyEvent) == true ? FReply::Handled() : FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
