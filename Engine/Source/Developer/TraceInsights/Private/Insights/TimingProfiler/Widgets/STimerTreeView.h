// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

// TraceServices
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/TimerGroupingAndSorting.h"
#include "Insights/TimingProfiler/ViewModels/TimerNode.h"

class FMenuBuilder;
class FUICommandList;

namespace UE::Insights
{
	class FTable;
	class FTableColumn;
	class ITableCellValueSorter;
	class SAsyncOperationStatus;
}

namespace UE::Insights::TimingProfiler
{

enum class ESelectEventType : uint32;
class SFrameTrack;
class FTimingGraphTrack;
class STimingProfilerWindow;
class STimingView;

////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A custom widget used to display the timers in a tree view (ex. Callers and Callees).
 */
class STimerTreeView : public SCompoundWidget
{
public:
	enum class ETreeType
	{
		Timers,
		Callers,
		Callees,
	};

public:
	/** Default constructor. */
	STimerTreeView();

	/** Virtual destructor. */
	virtual ~STimerTreeView();

	SLATE_BEGIN_ARGS(STimerTreeView) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs, const FText& InViewName, ETreeType InTreeType);

	void Reset();

	void SetTree(const TraceServices::FTimingProfilerButterflyNode& Root);

	TArray<FTimerNodePtr>& GetTreeNodes() { return TreeNodes; };

	FTimerNodePtr GetTimerNode(uint32 TimerId) const;

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	void InitCommandList();

	FTimerNodePtr CreateTimerNodeRec(const TraceServices::FTimingProfilerButterflyNode& Node, const TraceServices::ITimingProfilerProvider& TimingProfilerProvider);
	void ExpandNodesRec(FTimerNodePtr NodePtr, int32 Depth);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Context Menu

	TSharedPtr<SWidget> TreeView_GetMenuContent();
	void TreeView_BuildSortByMenu(FMenuBuilder& MenuBuilder);
	void TreeView_BuildViewColumnMenu(FMenuBuilder& MenuBuilder);

	//////////////////////////////////////////////////
	// Copy to Clipboard

	bool ContextMenu_CopyToClipboard_CanExecute() const;
	void ContextMenu_CopyToClipboard_Execute();

	bool ContextMenu_CopyTimerNameToClipboard_CanExecute() const;
	void ContextMenu_CopyTimerNameToClipboard_Execute();

	//////////////////////////////////////////////////
	// Open Source File in IDE

	bool ContextMenu_OpenSource_CanExecute() const;
	void ContextMenu_OpenSource_Execute() const;

	void OpenSourceFileInIDE(FTimerNodePtr InNode) const;

	//////////////////////////////////////////////////
	// Generate and Edit Timer Colors

	bool ContextMenu_GenerateNewColor_CanExecute() const;
	void ContextMenu_GenerateNewColor_Execute() const;

	//////////////////////////////////////////////////
	// Plot (Graph Series)

	void TreeView_BuildPlotTimerMenu(FMenuBuilder& MenuBuilder);

	//////////////////////////////////////////////////
	// Find Instance

	void TreeView_FindMenu(FMenuBuilder& MenuBuilder);

	bool ContextMenu_FindInstance_CanExecute() const;
	void ContextMenu_FindInstance_Execute(ESelectEventType EventType) const;

	bool ContextMenu_FindInstanceInSelection_CanExecute() const;
	void ContextMenu_FindInstanceInSelection_Execute(ESelectEventType EventType) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Columns' Header

	void InitializeAndShowHeaderColumns();

	FText GetColumnHeaderText(const FName ColumnId) const;

	TSharedRef<SWidget> TreeViewHeaderRow_GenerateColumnMenu(const FTableColumn& Column);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Misc

	void TreeView_Refresh();

	/**
	 * Called by STreeView to retrieves the children for the specified parent item.
	 * @param InParent    - The parent node to retrieve the children from.
	 * @param OutChildren - List of children for the parent node.
	 */
	void TreeView_OnGetChildren(FTimerNodePtr InParent, TArray<FTimerNodePtr>& OutChildren);

	/** Called by STreeView when selection has changed. */
	void TreeView_OnSelectionChanged(FTimerNodePtr SelectedItem, ESelectInfo::Type SelectInfo);

	FTimerNodePtr GetSingleSelectedTimerNode() const;

	/** Called by STreeView when a tree item is double clicked. */
	void TreeView_OnMouseButtonDoubleClick(FTimerNodePtr TreeNode);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Table Row

	/** Called by STreeView to generate a table row for the specified item. */
	TSharedRef<ITableRow> TreeView_OnGenerateRow(FTimerNodePtr TreeNode, const TSharedRef<STableViewBase>& OwnerTable);

	bool TableRow_ShouldBeEnabled(FTimerNodePtr NodePtr) const;

	void TableRow_SetHoveredCell(TSharedPtr<FTable> TablePtr, TSharedPtr<FTableColumn> ColumnPtr, FTimerNodePtr NodePtr);
	EHorizontalAlignment TableRow_GetColumnOutlineHAlignment(const FName ColumnId) const;

	FText TableRow_GetHighlightText() const;
	
	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Sorting

	static const FName GetDefaultColumnBeingSorted();
	static const EColumnSortMode::Type GetDefaultColumnSortMode();

	void CreateSortings();

	void UpdateCurrentSortingByColumn();
	void SortTreeNodes();
	void SortTreeNodesRec(FTimerNode& Node, const ITableCellValueSorter& Sorter);

	EColumnSortMode::Type GetSortModeForColumn(const FName ColumnId) const;
	void SetSortModeForColumn(const FName& ColumnId, EColumnSortMode::Type SortMode);
	void OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Sorting Actions

	// SortMode (HeaderMenu)
	bool HeaderMenu_SortMode_IsChecked(const FName ColumnId, const EColumnSortMode::Type InSortMode);
	bool HeaderMenu_SortMode_CanExecute(const FName ColumnId, const EColumnSortMode::Type InSortMode) const;
	void HeaderMenu_SortMode_Execute(const FName ColumnId, const EColumnSortMode::Type InSortMode);

	// SortMode (ContextMenu)
	bool ContextMenu_SortMode_IsChecked(const EColumnSortMode::Type InSortMode);
	bool ContextMenu_SortMode_CanExecute(const EColumnSortMode::Type InSortMode) const;
	void ContextMenu_SortMode_Execute(const EColumnSortMode::Type InSortMode);

	// SortByColumn (ContextMenu)
	bool ContextMenu_SortByColumn_IsChecked(const FName ColumnId);
	bool ContextMenu_SortByColumn_CanExecute(const FName ColumnId) const;
	void ContextMenu_SortByColumn_Execute(const FName ColumnId);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Column Visibility Actions

	// ShowColumn
	bool CanShowColumn(const FName ColumnId) const;
	void ShowColumn(const FName ColumnId);

	// HideColumn
	bool CanHideColumn(const FName ColumnId) const;
	void HideColumn(const FName ColumnId);

	// ToggleColumnVisibility
	bool IsColumnVisible(const FName ColumnId) const;
	bool CanToggleColumnVisibility(const FName ColumnId) const;
	void ToggleColumnVisibility(const FName ColumnId);

	// ShowAllColumns (ContextMenu)
	bool ContextMenu_ShowAllColumns_CanExecute() const;
	void ContextMenu_ShowAllColumns_Execute();

	// ResetColumns (ContextMenu)
	bool ContextMenu_ResetColumns_CanExecute() const;
	void ContextMenu_ResetColumns_Execute();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Event Highlight/Filtering in Timing View

	void ToggleTimingViewEventFilter(FTimerNodePtr TimerNode) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Plot (Graph Series)

	TSharedPtr<FTimingGraphTrack> GetTimingViewMainGraphTrack() const;
	TSharedPtr<SFrameTrack> GetFrameTrack() const;

	void ToggleGraphInstanceSeries(TSharedRef<FTimingGraphTrack> GraphTrack, FTimerNodeRef NodePtr) const;
	bool IsInstanceSeriesInTimingViewMainGraph(FTimerNodePtr TimerNode) const;
	void ToggleTimingViewMainGraphEventInstanceSeries(FTimerNodePtr TimerNode) const;

	void ToggleGraphFrameStatsSeries(TSharedRef<FTimingGraphTrack> GraphTrack, FTimerNodeRef NodePtr, ETraceFrameType FrameType) const;
	bool IsFrameStatsSeriesInTimingViewMainGraph(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const;
	void ToggleTimingViewMainGraphEventFrameStatsSeries(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const;

	bool IsSeriesInFrameTrack(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const;
	void ToggleFrameTrackSeries(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedPtr<STimingProfilerWindow> GetProfilerWindow() const;
	TSharedPtr<STimingView> GetTimingView() const;

	FTimerNodePtr GetTimerNodeRec(uint32 TimerId, const FTimerNodePtr TimerNode) const;

private:
	/** Table view model. */
	TSharedPtr<FTable> Table;

	ETreeType TreeType;

	/** The view name (ex.: "Callers" or "Callees"). */
	FText ViewName;

	TSharedPtr<FUICommandList> CommandList;

	//////////////////////////////////////////////////
	// Tree View, Columns

	/** The tree widget which holds the list of groups and timers corresponding with each group. */
	TSharedPtr<STreeView<FTimerNodePtr>> TreeView;

	/** Holds the tree view header row widget which display all columns in the tree view. */
	TSharedPtr<SHeaderRow> TreeViewHeaderRow;

	/** External scrollbar used to synchronize tree view position. */
	TSharedPtr<SScrollBar> ExternalScrollbar;

	//////////////////////////////////////////////////
	// Hovered Column, Hovered Timer Node

	/** Name of the column currently being hovered by the mouse. */
	FName HoveredColumnId;

	/** A shared pointer to the timer node currently being hovered by the mouse. */
	FTimerNodePtr HoveredNodePtr;

	//////////////////////////////////////////////////
	// Timer Nodes

	/** The root node(s) of the tree. */
	TArray<FTimerNodePtr> TreeNodes;

	//////////////////////////////////////////////////
	// Sorting

	/** All available sorters. */
	TArray<TSharedPtr<ITableCellValueSorter>> AvailableSorters;

	/** Current sorter. It is nullptr if sorting is disabled. */
	TSharedPtr<ITableCellValueSorter> CurrentSorter;

	/** Name of the column currently being sorted. Can be NAME_None if sorting is disabled (CurrentSorting == nullptr) or if a complex sorting is used (CurrentSorting != nullptr). */
	FName ColumnBeingSorted;

	/** How we sort the nodes? Ascending or Descending. */
	EColumnSortMode::Type ColumnSortMode = EColumnSortMode::Type::Ascending;

	//////////////////////////////////////////////////

	TSharedPtr<SAsyncOperationStatus> AsyncOperationStatus;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
