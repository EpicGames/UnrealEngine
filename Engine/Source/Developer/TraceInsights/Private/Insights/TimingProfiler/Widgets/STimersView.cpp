// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimersView.h"

#include "DesktopPlatformModule.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GenericPlatform/GenericPlatformTime.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformFileManager.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"
#include "Logging/MessageLog.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Views/STableViewBase.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/StackSamples.h"
#include "TraceServices/Model/Threads.h"
#include "TraceServices/Model/TimingProfiler.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"
#include "InsightsCore/Table/Widgets/SAsyncOperationStatus.h"

// TraceInsights
#include "Insights/Common/Utf8FileWriter.h"
#include "Insights/InsightsStyle.h"
#include "Insights/Log.h"
#include "Insights/Table/ViewModels/TableCommands.h"
#include "Insights/TimingProfiler/GraphTracks/TimingGraphSeries.h"
#include "Insights/TimingProfiler/GraphTracks/TimingGraphTrack.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Tracks/GpuTimingTrack.h"
#include "Insights/TimingProfiler/Tracks/VerseTimingTrack.h"
#include "Insights/TimingProfiler/ViewModels/TimerAggregator.h"
#include "Insights/TimingProfiler/ViewModels/TimerNodeHelper.h"
#include "Insights/TimingProfiler/ViewModels/TimersViewColumnFactory.h"
#include "Insights/TimingProfiler/ViewModels/TimingExporter.h"
#include "Insights/TimingProfiler/Widgets/SFrameTrack.h"
#include "Insights/TimingProfiler/Widgets/STimersViewTooltip.h"
#include "Insights/TimingProfiler/Widgets/STimerTableRow.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/TimingProfilerCommon.h"
#include "Insights/Widgets/STimingView.h"

#include <limits>

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::STimersView"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimersViewCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimersViewCommands : public TCommands<FTimersViewCommands>
{
public:
	FTimersViewCommands()
	: TCommands<FTimersViewCommands>(
		TEXT("TimersViewCommands"),
		NSLOCTEXT("Contexts", "TimersViewCommands", "Insights - Timers View"),
		NAME_None,
		FInsightsStyle::GetStyleSetName())
	{
	}

	virtual ~FTimersViewCommands()
	{
	}

	// UI_COMMAND takes long for the compiler to optimize
	UE_DISABLE_OPTIMIZATION_SHIP
	virtual void RegisterCommands() override
	{
		UI_COMMAND(Command_CopyToClipboard,
			"Copy To Clipboard",
			"Copies the selection (timers and their aggregated statistics) to clipboard.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Control, EKeys::C));

		UI_COMMAND(Command_CopyNameToClipboard,
			"Copy Name To Clipboard",
			"Copies the name of the selected timer to the clipboard.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::C));

		UI_COMMAND(Command_Export,
			"Export...",
			"Exports the selection (timers and their aggregated statistics) to a text file (tab-separated values or comma-separated values).",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Control, EKeys::S));

		UI_COMMAND(Command_ExportTimingEventsSelection,
			"Export Timing Events (Selection)...",
			"Exports the timing events to a text file (tab-separated values or comma-separated values).\nOnly exports the timing events of the selected timer(s), for the visible CPU/GPU Thread tracks and in the selected time region (if any).",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_ExportTimingEvents,
			"Export Timing Events (All)...",
			"Exports all the timing events (for all CPU/GPU threads) to a text file (tab-separated values or comma-separated values).",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_ExportThreads,
			"Export Threads...",
			"Exports the list of threads to a text file (tab-separated values or comma-separated values).",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_ExportTimers,
			"Export Timers...",
			"Exports the list of timers to a text file (tab-separated values or comma-separated values).",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_OpenSource,
			"Open Source",
			"Opens the source file of the selected timer in the registered IDE.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_GenerateNewColor,
			"Generate New Color",
			"Generates a new random color for the selected timers.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_EditColor,
			"Edit Color...",
			"Changes color for the selected timers.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_FindFirstInstance,
			"First Instance",
			"Navigates to and selects the first timing event instance for the selected timer.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_FindPreviousInstance,
			"Previous Instance",
			"Navigates to and selects the previous timing event instance for the selected timer.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Shift, EKeys::Comma));

		UI_COMMAND(Command_FindNextInstance,
			"Next Instance",
			"Navigates to and selects the next timing event instance for the selected timer.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Shift, EKeys::Period));

		UI_COMMAND(Command_FindLastInstance,
			"Last Instance",
			"Navigates to and selects the last timing event instance for the selected timer.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_FindMaxInstance,
			"Maximum Duration Instance",
			"Navigates to and selects the timing event instance with the maximum duration, for the selected timer.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Shift, EKeys::Slash));

		UI_COMMAND(Command_FindMinInstance,
			"Minimum Duration Instance",
			"Navigates to and selects the timing event instance with the minimum duration, for the selected timer.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_FindFirstInstanceInSelection,
			"First Instance in Selection",
			"Navigates to and selects the first timing event instance for the selected timer, in the selected time range.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_FindPreviousInstanceInSelection,
			"Previous Instance in Selection",
			"Navigates to and selects the previous timing event instance for the selected timer, in the selected time range.",
			EUserInterfaceActionType::Button,
			FInputChord(EKeys::Comma));

		UI_COMMAND(Command_FindNextInstanceInSelection,
			"Next Instance in Selection",
			"Navigates to and selects the next timing event instance for the selected timer, in the selected time range.",
			EUserInterfaceActionType::Button,
			FInputChord(EKeys::Period));

		UI_COMMAND(Command_FindLastInstanceInSelection,
			"Last Instance in Selection",
			"Navigates to and selects the last timing event instance for the selected timer, in the selected time range.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_FindMaxInstanceInSelection,
			"Maximum Duration Instance in Selection",
			"Navigates to and selects the timing event instance with the maximum duration, for the selected timer, in the selected time range.",
			EUserInterfaceActionType::Button,
			FInputChord(EKeys::Slash));

		UI_COMMAND(Command_FindMinInstanceInSelection,
			"Minimum Duration Instance in Selection",
			"Navigates to and selects the timing event instance with the minimum duration, for the selected timer, in the selected time range.",
			EUserInterfaceActionType::Button,
			FInputChord());
	}
	UE_ENABLE_OPTIMIZATION_SHIP

	TSharedPtr<FUICommandInfo> Command_CopyToClipboard;
	TSharedPtr<FUICommandInfo> Command_CopyNameToClipboard;
	TSharedPtr<FUICommandInfo> Command_Export;
	TSharedPtr<FUICommandInfo> Command_ExportTimingEventsSelection;
	TSharedPtr<FUICommandInfo> Command_ExportTimingEvents;
	TSharedPtr<FUICommandInfo> Command_ExportThreads;
	TSharedPtr<FUICommandInfo> Command_ExportTimers;
	TSharedPtr<FUICommandInfo> Command_OpenSource;
	TSharedPtr<FUICommandInfo> Command_GenerateNewColor;
	TSharedPtr<FUICommandInfo> Command_EditColor;
	TSharedPtr<FUICommandInfo> Command_FindFirstInstance;
	TSharedPtr<FUICommandInfo> Command_FindPreviousInstance;
	TSharedPtr<FUICommandInfo> Command_FindNextInstance;
	TSharedPtr<FUICommandInfo> Command_FindLastInstance;
	TSharedPtr<FUICommandInfo> Command_FindMaxInstance;
	TSharedPtr<FUICommandInfo> Command_FindMinInstance;
	TSharedPtr<FUICommandInfo> Command_FindFirstInstanceInSelection;
	TSharedPtr<FUICommandInfo> Command_FindPreviousInstanceInSelection;
	TSharedPtr<FUICommandInfo> Command_FindNextInstanceInSelection;
	TSharedPtr<FUICommandInfo> Command_FindLastInstanceInSelection;
	TSharedPtr<FUICommandInfo> Command_FindMaxInstanceInSelection;
	TSharedPtr<FUICommandInfo> Command_FindMinInstanceInSelection;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// STimersView
////////////////////////////////////////////////////////////////////////////////////////////////////

STimersView::STimersView()
	: Table(MakeShared<FTable>())
	, ColumnBeingSorted(GetDefaultColumnBeingSorted())
	, ColumnSortMode(GetDefaultColumnSortMode())
	, Aggregator(MakeShared<FTimerAggregator>())
{
	FMemory::Memset(bTimerTypeIsVisible, 1);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

STimersView::~STimersView()
{
	SaveSettings();
	SaveVisibleColumnsSettings();

	// Remove ourselves from the Insights manager.
	if (FInsightsManager::Get().IsValid())
	{
		FInsightsManager::Get()->GetSessionChangedEvent().RemoveAll(this);
		FInsightsManager::Get()->GetSessionAnalysisCompletedEvent().RemoveAll(this);
	}

	FTimersViewCommands::Unregister();

	Session.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::InitCommandList()
{
	FTimersViewCommands::Register();
	CommandList = MakeShared<FUICommandList>();
	CommandList->MapAction(FTimersViewCommands::Get().Command_CopyToClipboard, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_CopyToClipboard_Execute), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_CopyToClipboard_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_CopyNameToClipboard, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_CopyTimerNameToClipboard_Execute), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_CopyTimerNameToClipboard_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_Export, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_Export_Execute), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_Export_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_ExportTimingEventsSelection, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_ExportTimingEventsSelection_Execute), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_ExportTimingEventsSelection_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_ExportTimingEvents, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_ExportTimingEvents_Execute), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_ExportTimingEvents_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_ExportThreads, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_ExportThreads_Execute), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_ExportThreads_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_ExportTimers, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_ExportTimers_Execute), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_ExportTimers_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_OpenSource, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_OpenSource_Execute), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_OpenSource_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_GenerateNewColor, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_GenerateNewColor_Execute), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_GenerateNewColor_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_EditColor, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_EditColor_Execute), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_EditColor_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_FindFirstInstance, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstance_Execute, ESelectEventType::First), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstance_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_FindPreviousInstance, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstance_Execute, ESelectEventType::Previous), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstance_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_FindNextInstance, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstance_Execute, ESelectEventType::Next), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstance_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_FindLastInstance, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstance_Execute, ESelectEventType::Last), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstance_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_FindMaxInstance, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstance_Execute, ESelectEventType::Max), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstance_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_FindMinInstance, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstance_Execute, ESelectEventType::Min), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstance_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_FindFirstInstanceInSelection, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstanceInSelection_Execute, ESelectEventType::First), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstanceInSelection_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_FindPreviousInstanceInSelection, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstanceInSelection_Execute, ESelectEventType::Previous), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstanceInSelection_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_FindNextInstanceInSelection, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstanceInSelection_Execute, ESelectEventType::Next), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstanceInSelection_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_FindLastInstanceInSelection, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstanceInSelection_Execute, ESelectEventType::Last), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstanceInSelection_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_FindMaxInstanceInSelection, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstanceInSelection_Execute, ESelectEventType::Max), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstanceInSelection_CanExecute));
	CommandList->MapAction(FTimersViewCommands::Get().Command_FindMinInstanceInSelection, FExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstanceInSelection_Execute, ESelectEventType::Min), FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_FindInstanceInSelection_CanExecute));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STimersView::Construct(const FArguments& InArgs)
{
	SAssignNew(ExternalScrollbar, SScrollBar)
	.AlwaysShowScrollbar(true);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoHeight()
		.Padding(2.0f, 2.0f, 2.0f, 2.0f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				// Search box
				+ SHorizontalBox::Slot()
				.Padding(2.0f)
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SAssignNew(SearchBox, SSearchBox)
					.HintText(LOCTEXT("SearchBoxHint", "Search timers or groups"))
					.OnTextChanged(this, &STimersView::SearchBox_OnTextChanged)
					.IsEnabled(this, &STimersView::SearchBox_IsEnabled)
					.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search timer or group."))
				]

				// Filter by type (GPU Scope)
				+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 4.0f, 0.0f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					GetToggleButtonForTimerType(ETimerNodeType::GpuScope)
				]

				// Filter by type (Verse Sampling)
				+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 4.0f, 0.0f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					GetToggleButtonForTimerType(ETimerNodeType::VerseSampling)
				]

				// Filter by type (CPU Scope)
				+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 4.0f, 0.0f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					GetToggleButtonForTimerType(ETimerNodeType::CpuScope)
				]

				// Filter by type (CPU Sampling)
				+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 4.0f, 0.0f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					GetToggleButtonForTimerType(ETimerNodeType::CpuSampling)
				]

				// Filter out timers with zero instance count
				+ SHorizontalBox::Slot()
				.Padding(2.0f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
					.HAlign(HAlign_Center)
					.Padding(3.0f)
					.OnCheckStateChanged(this, &STimersView::FilterOutZeroCountTimers_OnCheckStateChanged)
					.IsChecked(this, &STimersView::FilterOutZeroCountTimers_IsChecked)
					.ToolTipText(LOCTEXT("FilterOutZeroCountTimers_Tooltip", "Filter out the timers having zero total instance count (aggregated stats)."))
					[
						SNew(SImage)
						.Image(FInsightsStyle::Get().GetBrush("Icons.ZeroCountFilter"))
					]
				]
			]

			// Group by
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("GroupByText", "Group by"))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.MinDesiredWidth(128.0f)
					[
						SAssignNew(GroupByComboBox, SComboBox<TSharedPtr<ETimerGroupingMode>>)
						.ToolTipText(this, &STimersView::GroupBy_GetSelectedTooltipText)
						.OptionsSource(&GroupByOptionsSource)
						.OnSelectionChanged(this, &STimersView::GroupBy_OnSelectionChanged)
						.OnGenerateWidget(this, &STimersView::GroupBy_OnGenerateWidget)
						[
							SNew(STextBlock)
							.Text(this, &STimersView::GroupBy_GetSelectedText)
						]
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 0.0f, 4.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Mode", "Mode"))
						.ToolTipText(LOCTEXT("ModeToolTip", "Aggregation Mode"))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SBox)
						.MinDesiredWidth(128.0f)
						[
							SAssignNew(ModeComboBox, SComboBox<TSharedPtr<ETimerAggregationMode>>)
							.ToolTipText(this, &STimersView::Mode_GetSelectedTooltipText)
							.OptionsSource(&ModeOptionsSource)
							.OnSelectionChanged(this, &STimersView::Mode_OnSelectionChanged)
							.OnGenerateWidget(this, &STimersView::Mode_OnGenerateWidget)
							[
								SNew(STextBlock)
								.Text(this, &STimersView::Mode_GetSelectedText)
							]
						]
					]
				]
			]
		]

		// Tree view
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.0f, 2.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(0.0f)
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(TreeView, STreeView<FTimerNodePtr>)
					.ExternalScrollbar(ExternalScrollbar)
					.SelectionMode(ESelectionMode::Multi)
					.TreeItemsSource(&FilteredGroupNodes)
					.OnGetChildren(this, &STimersView::TreeView_OnGetChildren)
					.OnGenerateRow(this, &STimersView::TreeView_OnGenerateRow)
					.OnSelectionChanged(this, &STimersView::TreeView_OnSelectionChanged)
					.OnMouseButtonDoubleClick(this, &STimersView::TreeView_OnMouseButtonDoubleClick)
					.OnContextMenuOpening(FOnContextMenuOpening::CreateSP(this, &STimersView::TreeView_GetMenuContent))
					.HeaderRow
					(
						SAssignNew(TreeViewHeaderRow, SHeaderRow)
						.Visibility(EVisibility::Visible)
					)
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				.Padding(16.0f)
				[
					SAssignNew(AsyncOperationStatus, SAsyncOperationStatus, Aggregator)
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f)
			[
				SNew(SBox)
				.WidthOverride(FOptionalSize(13.0f))
				[
					ExternalScrollbar.ToSharedRef()
				]
			]
		]

		// Status bar
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(0.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
			.BorderBackgroundColor(FLinearColor(0.05f, 0.1f, 0.2f, 1.0f))
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Margin(FMargin(4.0f, 1.0f, 4.0f, 1.0f))
				.Text(LOCTEXT("EmptyAggregationNote", "Please select a time range to update the aggregated statistics!"))
				.ColorAndOpacity(FLinearColor(1.0f, 0.75f, 0.5f, 1.0f))
				.Visibility_Lambda([this]()
				{
					return Aggregator->IsEmptyTimeInterval() && !Aggregator->IsRunning() ?
						EVisibility::Visible : EVisibility::Collapsed;
				})
			]
		]
	];

	InitializeAndShowHeaderColumns();
	LoadSettings();

	Aggregator->SetAggregationMode(AggregationMode);
	SetTimingViewFrameType();

	// Create the search filters: text based, type based etc.
	TextFilter = MakeShared<FTimerNodeTextFilter>(FTimerNodeTextFilter::FItemToStringArray::CreateSP(this, &STimersView::HandleItemToStringArray));
	Filters = MakeShared<FTimerNodeFilterCollection>();
	Filters->Add(TextFilter);

	CreateGroupByOptionsSources();
	CreateModeOptionsSources();
	CreateSortings();

	InitCommandList();

	// Register ourselves with the Insights manager.
	FInsightsManager::Get()->GetSessionChangedEvent().AddSP(this, &STimersView::InsightsManager_OnSessionChanged);
	FInsightsManager::Get()->GetSessionAnalysisCompletedEvent().AddSP(this, &STimersView::InsightsManager_OnSessionAnalysisCompleted);

	// Update the Session (i.e. when analysis session was already started).
	InsightsManager_OnSessionChanged();

	if (Session.IsValid())
	{
		TraceServices::IAnalysisSession& EditableSession = *const_cast<TraceServices::IAnalysisSession*>(Session.Get());
		TraceServices::IEditableTimingProfilerProvider* TimingProfilerProvider = TraceServices::EditTimingProfilerProvider(EditableSession);
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionEditScope SessionEditScope(EditableSession);

			FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
			TimingProfilerProvider->SetRatioOfThreadsToUse(Settings.GetThreadRatio());
		}
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> STimersView::TreeView_GetMenuContent()
{
	int32 NumSelectedNodes = TreeView->GetNumItemsSelected();
	FTimerNodePtr SingleSelectedNode;
	bool bIsValidNode = false;

	if (NumSelectedNodes == 1)
	{
		TArray<FTimerNodePtr> SelectedNodes = TreeView->GetSelectedItems();
		check(SelectedNodes.Num() == 1);
		SingleSelectedNode = SelectedNodes[0];
		if (SingleSelectedNode.IsValid())
		{
			bIsValidNode = (SingleSelectedNode->GetType() != ETimerNodeType::Group);
		}
		else
		{
			NumSelectedNodes = 0;
		}
	}

	FText SelectionStr;
	if (NumSelectedNodes == 0)
	{
		SelectionStr = LOCTEXT("NothingSelected", "Nothing selected");
	}
	else if (NumSelectedNodes == 1)
	{
		check(SingleSelectedNode.IsValid());
		FString ItemName = SingleSelectedNode->GetDisplayName().ToString();
		const int32 MaxStringLen = 64;
		if (ItemName.Len() > MaxStringLen)
		{
			ItemName = ItemName.Left(MaxStringLen) + TEXT("...");
		}
		SelectionStr = FText::FromString(ItemName);
	}
	else
	{
		SelectionStr = FText::Format(LOCTEXT("MultipleSelection_Fmt", "{0} selected items"), FText::AsNumber(NumSelectedNodes));
	}

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList.ToSharedRef());

	// Selection menu
	MenuBuilder.BeginSection("Selection", LOCTEXT("ContextMenu_Section_Selection", "Selection"));
	{
		struct FLocal
		{
			static bool ReturnFalse()
			{
				return false;
			}
		};

		FUIAction DummyUIAction;
		DummyUIAction.CanExecuteAction = FCanExecuteAction::CreateStatic(&FLocal::ReturnFalse);
		MenuBuilder.AddMenuEntry
		(
			SelectionStr,
			LOCTEXT("ContextMenu_Selection", "Currently selected items"),
			FSlateIcon(),
			DummyUIAction,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	// Timer options section
	MenuBuilder.BeginSection("TimerOptions", LOCTEXT("ContextMenu_Section_TimerOptions", "Timer Options"));
	{
		auto CanExecute = [this, bIsValidNode]()
		{
			return bIsValidNode && GetTimingView().IsValid();
		};

		// Highlight event
		{
			FUIAction Action_ToggleHighlight;
			Action_ToggleHighlight.CanExecuteAction = FCanExecuteAction::CreateLambda(CanExecute);
			Action_ToggleHighlight.ExecuteAction = FExecuteAction::CreateSP(this, &STimersView::ToggleTimingViewEventFilter, SingleSelectedNode);

			TSharedPtr<STimingView> TimingView = GetTimingView();

			if (bIsValidNode &&
				TimingView.IsValid() &&
				TimingView->IsFilterByEventType(SingleSelectedNode->GetTimerId()))
			{
				MenuBuilder.AddMenuEntry
				(
					LOCTEXT("ContextMenu_StopHighlightEvent", "Stop Highlighting Event"),
					LOCTEXT("ContextMenu_StopHighlightEvent_Desc", "Stops highlighting timing event instances for the selected timer."),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Visible"),
					Action_ToggleHighlight,
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}
			else
			{
				MenuBuilder.AddMenuEntry
				(
					LOCTEXT("ContextMenu_HighlightEvent", "Highlight Event"),
					LOCTEXT("ContextMenu_HighlightEvent_Desc", "Highlights all timing event instances for the selected timer."),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Visible"),
					Action_ToggleHighlight,
					NAME_None,
					EUserInterfaceActionType::Button
				);
			}
		}

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_PlotTimer_SubMenu", "Plot Timer"),
			LOCTEXT("ContextMenu_PlotTimer_SubMenu_Desc", "Options to add the timer series to graph or frame tracks."),
			FNewMenuDelegate::CreateSP(this, &STimersView::TreeView_BuildPlotTimerMenu),
			false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.AddGraphSeries")
		);

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_FindInstance_SubMenu", "Find Instance"),
			LOCTEXT("ContextMenu_PlotInstance_SubMenu_Desc", "Find the instance of this timer with the minimum or maximum duration."),
			FNewMenuDelegate::CreateSP(this, &STimersView::TreeView_FindMenu),
			false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FindInstance")
		);

		// Open Source in IDE
		{
			ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
			ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();

			FString File;
			uint32 Line = 0;
			bool bIsValidSource = bIsValidNode && SingleSelectedNode->GetSourceFileAndLine(File, Line);
			FString UsablePath;
			const bool bPathExists = FInsightsManager::Get()->GetSourceFilePathHelper().GetUsableFilePath(File, UsablePath);

			FText ItemLabel;
			FText ItemToolTip;

			if (SourceCodeAccessor.CanAccessSourceCode())
			{
				ItemLabel = FText::Format(LOCTEXT("ContextMenu_OpenSource", "Open Source in {0}"), SourceCodeAccessor.GetNameText());
				if (bIsValidSource)
				{
					ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSource_Desc1", "Opens the source file of the selected timer in {0}.\n{1} ({2})"),
						SourceCodeAccessor.GetNameText(),
						FText::FromString(UsablePath),
						FText::AsNumber(Line, &FNumberFormattingOptions::DefaultNoGrouping()));
				}
				else
				{
					ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSource_Desc2", "Opens the source file of the selected timer in {0}."),
						SourceCodeAccessor.GetNameText());
				}
			}
			else
			{
				ItemLabel = LOCTEXT("ContextMenu_OpenSourceNA", "Open Source");
				if (bIsValidSource)
				{
					ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSourceNA_Desc1", "{0} ({1})\nSource Code Accessor is not available."),
						FText::FromString(UsablePath),
						FText::AsNumber(Line, &FNumberFormattingOptions::DefaultNoGrouping()));
				}
				else
				{
					ItemToolTip = LOCTEXT("ContextMenu_OpenSourceNA_Desc2", "Source Code Accessor is not available.");
				}
			}

			MenuBuilder.AddMenuEntry(
				FTimersViewCommands::Get().Command_OpenSource,
				NAME_None,
				ItemLabel,
				ItemToolTip,
				FSlateIcon(SourceCodeAccessor.GetStyleSet(), SourceCodeAccessor.GetOpenIconName()));
		}

		MenuBuilder.AddMenuEntry
		(
			FTimersViewCommands::Get().Command_GenerateNewColor,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Refresh")
		);

		MenuBuilder.AddMenuEntry
		(
			FTimersViewCommands::Get().Command_EditColor,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.EyeDropper")
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Misc", LOCTEXT("ContextMenu_Section_Misc", "Miscellaneous"));
	{
		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_Settings_SubMenu", "Settings"),
			LOCTEXT("ContextMenu_Settings_SubMenu_Desc", "Options to adjust timer-related settings."),
			FNewMenuDelegate::CreateSP(this, &STimersView::TreeView_BuildSettingsMenu),
			false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Toolbar.Settings")
		);

		MenuBuilder.AddMenuEntry
		(
			FTimersViewCommands::Get().Command_CopyToClipboard,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);

		MenuBuilder.AddMenuEntry
		(
			FTimersViewCommands::Get().Command_CopyNameToClipboard,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);

		MenuBuilder.AddMenuEntry
		(
			FTimersViewCommands::Get().Command_Export,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);

		MenuBuilder.AddMenuEntry
		(
			FTimersViewCommands::Get().Command_ExportTimingEventsSelection,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_ExportOptions_SubMenu", "More Export Options"),
			LOCTEXT("ContextMenu_ExportOptions_SubMenu_Desc", "Exports threads, timers and timing events to text files."),
			FNewMenuDelegate::CreateSP(this, &STimersView::TreeView_BuildExportMenu),
			false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::TreeView_BuildSortByMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("SortColumn", LOCTEXT("ContextMenu_Section_SortColumn", "Sort Column"));

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (Column.IsVisible() && Column.CanBeSorted())
		{
			FUIAction Action_SortByColumn
			(
				FExecuteAction::CreateSP(this, &STimersView::ContextMenu_SortByColumn_Execute, Column.GetId()),
				FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_SortByColumn_CanExecute, Column.GetId()),
				FIsActionChecked::CreateSP(this, &STimersView::ContextMenu_SortByColumn_IsChecked, Column.GetId())
			);
			MenuBuilder.AddMenuEntry
			(
				Column.GetTitleName(),
				Column.GetDescription(),
				FSlateIcon(),
				Action_SortByColumn,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}
	}

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("SortMode", LOCTEXT("ContextMenu_Section_SortMode", "Sort Mode"));
	{
		FUIAction Action_SortAscending
		(
			FExecuteAction::CreateSP(this, &STimersView::ContextMenu_SortMode_Execute, EColumnSortMode::Ascending),
			FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_SortMode_CanExecute, EColumnSortMode::Ascending),
			FIsActionChecked::CreateSP(this, &STimersView::ContextMenu_SortMode_IsChecked, EColumnSortMode::Ascending)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_SortAscending", "Sort Ascending"),
			LOCTEXT("ContextMenu_SortAscending_Desc", "Sorts ascending."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortUp"),
			Action_SortAscending,
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		FUIAction Action_SortDescending
		(
			FExecuteAction::CreateSP(this, &STimersView::ContextMenu_SortMode_Execute, EColumnSortMode::Descending),
			FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_SortMode_CanExecute, EColumnSortMode::Descending),
			FIsActionChecked::CreateSP(this, &STimersView::ContextMenu_SortMode_IsChecked, EColumnSortMode::Descending)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_SortDescending", "Sort Descending"),
			LOCTEXT("ContextMenu_SortDescending_Desc", "Sorts descending."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortDown"),
			Action_SortDescending,
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::TreeView_BuildViewColumnMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Columns", LOCTEXT("ContextMenu_Section_Columns", "Columns"));

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		FUIAction Action_ToggleColumn
		(
			FExecuteAction::CreateSP(this, &STimersView::ToggleColumnVisibility, Column.GetId()),
			FCanExecuteAction::CreateSP(this, &STimersView::CanToggleColumnVisibility, Column.GetId()),
			FIsActionChecked::CreateSP(this, &STimersView::IsColumnVisible, Column.GetId())
		);
		MenuBuilder.AddMenuEntry
		(
			Column.GetTitleName(),
			Column.GetDescription(),
			FSlateIcon(),
			Action_ToggleColumn,
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::TreeView_BuildExportMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.SetSearchable(false);

	MenuBuilder.BeginSection("Export");
	{
		MenuBuilder.AddMenuEntry
		(
			FTimersViewCommands::Get().Command_ExportThreads,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);

		MenuBuilder.AddMenuEntry
		(
			FTimersViewCommands::Get().Command_ExportTimers,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);

		MenuBuilder.AddMenuEntry
		(
			FTimersViewCommands::Get().Command_ExportTimingEvents,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::TreeView_BuildPlotTimerMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.SetSearchable(false);

	FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
	const bool bIsValidNode = SelectedNode.IsValid() && SelectedNode->GetType() != ETimerNodeType::Group;

	auto CanExecuteAddToGraphTrack = [this, bIsValidNode]()
	{
		return bIsValidNode && GetTimingView().IsValid();
	};

	auto CanExecuteAddToFramesTrack = [this, bIsValidNode]()
	{
		return bIsValidNode && GetFrameTrack().IsValid();
	};

	MenuBuilder.BeginSection("Instance", LOCTEXT("Plot_Series_Instance_Section", "Instance"));

	// Add/remove series to/from graph track
	{
		FUIAction Action_ToggleTimerInGraphTrack;
		Action_ToggleTimerInGraphTrack.CanExecuteAction = FCanExecuteAction::CreateLambda(CanExecuteAddToGraphTrack);
		Action_ToggleTimerInGraphTrack.ExecuteAction = FExecuteAction::CreateSP(this, &STimersView::ToggleTimingViewMainGraphEventInstanceSeries, SelectedNode);

		if (bIsValidNode &&
			IsInstanceSeriesInTimingViewMainGraph(SelectedNode))
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_RemoveFromGraphTrack", "Remove instance series from graph track"),
				LOCTEXT("ContextMenu_RemoveFromGraphTrack_Desc", "Removes the series containing event instances of the selected timer from the Main Graph track."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.RemoveGraphSeries"),
				Action_ToggleTimerInGraphTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		else
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_AddToGraphTrack", "Add instance series to graph track"),
				LOCTEXT("ContextMenu_AddToGraphTrack_Desc", "Adds a series containing event instances of the selected timer to the Main Graph track."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.AddGraphSeries"),
				Action_ToggleTimerInGraphTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
	}

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Game Frame", LOCTEXT("Plot_Series_GameFrame_Section", "Game Frame"));

	// Add/remove game frame stats series to/from graph track
	{
		FUIAction Action_ToggleFrameStatsTimerInGraphTrack;
		Action_ToggleFrameStatsTimerInGraphTrack.CanExecuteAction = FCanExecuteAction::CreateLambda(CanExecuteAddToGraphTrack);
		Action_ToggleFrameStatsTimerInGraphTrack.ExecuteAction = FExecuteAction::CreateSP(this, &STimersView::ToggleTimingViewMainGraphEventFrameStatsSeries, SelectedNode, ETraceFrameType::TraceFrameType_Game);

		if (bIsValidNode &&
			IsFrameStatsSeriesInTimingViewMainGraph(SelectedNode, ETraceFrameType::TraceFrameType_Game))
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_RemoveGameFrameStatsFromGraphTrack", "Remove game frame stats series from graph track"),
				LOCTEXT("ContextMenu_RemoveGameFrameStatsFromGraphTrack_Desc", "Remove the game frame stats series for this timer."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.RemoveGraphSeries"),
				Action_ToggleFrameStatsTimerInGraphTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		else
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_AddGameFrameStatsSeriesToGraphTrack", "Add game frame stats series to graph track"),
				LOCTEXT("ContextMenu_AddGameFrameStatsSeriesToGraphTrack_Desc", "Adds a game frame stats series for this timer. Each data entry is computed as the sum of all instances of this timer in a game frame."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.AddGraphSeries"),
				Action_ToggleFrameStatsTimerInGraphTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
	}

	// Add/remove game frame stats series to/from frame track
	{
		FUIAction Action_ToggleFrameStatsTimerInFrameTrack;
		Action_ToggleFrameStatsTimerInFrameTrack.CanExecuteAction = FCanExecuteAction::CreateLambda(CanExecuteAddToFramesTrack);
		Action_ToggleFrameStatsTimerInFrameTrack.ExecuteAction = FExecuteAction::CreateSP(this, &STimersView::ToggleFrameTrackSeries, SelectedNode, ETraceFrameType::TraceFrameType_Game);

		if (bIsValidNode &&
			IsSeriesInFrameTrack(SelectedNode, ETraceFrameType::TraceFrameType_Game))
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_RemoveGameFrameStatsSeriesFromFrameTrack", "Remove game frame stats series from frame track"),
				LOCTEXT("ContextMenu_RemoveGameFrameStatsSeriesFromFrameTrack_Desc", "Remove the game frame stats series for this timer from the frame track."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.RemoveGraphSeries"),
				Action_ToggleFrameStatsTimerInFrameTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		else
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_AddGameFrameStatsSeriesToFrameTrack", "Add game frame stats series to the frame track"),
				LOCTEXT("ContextMenu_AddGameFrameStatsSeriesToFrameTrack_Desc", "Adds a game frame stats series for this timer to the frame track. Each data entry is computed as the sum of all instances of this timer in a game frame."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.AddGraphSeries"),
				Action_ToggleFrameStatsTimerInFrameTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
	}

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Rendering Frame", LOCTEXT("Plot_Series_RenderingFrame_Section", "Rendering frame"));

	// Add/remove rendering frame stats series to/from graph track
	{
		FUIAction Action_ToggleFrameStatsTimerInGraphTrack;
		Action_ToggleFrameStatsTimerInGraphTrack.CanExecuteAction = FCanExecuteAction::CreateLambda(CanExecuteAddToGraphTrack);
		Action_ToggleFrameStatsTimerInGraphTrack.ExecuteAction = FExecuteAction::CreateSP(this, &STimersView::ToggleTimingViewMainGraphEventFrameStatsSeries, SelectedNode, ETraceFrameType::TraceFrameType_Rendering);

		if (bIsValidNode &&
			IsFrameStatsSeriesInTimingViewMainGraph(SelectedNode, ETraceFrameType::TraceFrameType_Rendering))
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_RemoveRenderingFrameStatsFromGraphTrack", "Remove rendering frame stats series from graph track"),
				LOCTEXT("ContextMenu_RemoveRenderingFrameStatsFromGraphTrack_Desc", "Remove the rendering frame stats series for this timer."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.RemoveGraphSeries"),
				Action_ToggleFrameStatsTimerInGraphTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		else
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_AddRenderingFrameStatsSeriesToGraphTrack", "Add rendering frame stats series to graph track"),
				LOCTEXT("ContextMenu_AddRenderingFrameStatsSeriesToGraphTrack_Desc", "Adds a rendering frame stats series for this timer. Each data entry is computed as the sum of all instances of this timer in a rendering frame."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.AddGraphSeries"),
				Action_ToggleFrameStatsTimerInGraphTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
	}

	// Add/remove rendering frame stats series to/from frame track
	{
		FUIAction Action_ToggleFrameStatsTimerInFrameTrack;
		Action_ToggleFrameStatsTimerInFrameTrack.CanExecuteAction = FCanExecuteAction::CreateLambda(CanExecuteAddToFramesTrack);
		Action_ToggleFrameStatsTimerInFrameTrack.ExecuteAction = FExecuteAction::CreateSP(this, &STimersView::ToggleFrameTrackSeries, SelectedNode, ETraceFrameType::TraceFrameType_Rendering);

		if (bIsValidNode &&
			IsSeriesInFrameTrack(SelectedNode, ETraceFrameType::TraceFrameType_Rendering))
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_RemoveRenderingFrameStatsFromFrameTrac", "Remove rendering frame stats series from frame track"),
				LOCTEXT("ContextMenu_RemoveRenderingFrameStatsFromFrameTrack_Desc", "Remove the rendering frame stats series for this timer from the frame track."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.RemoveGraphSeries"),
				Action_ToggleFrameStatsTimerInFrameTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		else
		{
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_AddRenderingFrameStatsSeriesToFrameTrack", "Add rendering frame stats series to the frame track"),
				LOCTEXT("ContextMenu_AddRenderingFrameStatsSeriesToFrameTrack_Desc", "Adds a rendering frame stats series for this timer to the frame track. Each data entry is computed as the sum of all instances of this timer in a rendering frame."),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.AddGraphSeries"),
				Action_ToggleFrameStatsTimerInFrameTrack,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
	}

	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::TreeView_FindMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.SetSearchable(false);

	MenuBuilder.AddMenuEntry
	(
		FTimersViewCommands::Get().Command_FindFirstInstance,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::Get().GetStyleSetName(), "Icons.FindFirst")
	);

	MenuBuilder.AddMenuEntry
	(
		FTimersViewCommands::Get().Command_FindPreviousInstance,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::Get().GetStyleSetName(), "Icons.FindPrevious")
	);

	MenuBuilder.AddMenuEntry
	(
		FTimersViewCommands::Get().Command_FindNextInstance,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::Get().GetStyleSetName(), "Icons.FindNext")
	);

	MenuBuilder.AddMenuEntry
	(
		FTimersViewCommands::Get().Command_FindLastInstance,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::Get().GetStyleSetName(), "Icons.FindLast")
	);

	MenuBuilder.AddMenuEntry
	(
		FTimersViewCommands::Get().Command_FindMaxInstance,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FindMaxInstance")
	);

	MenuBuilder.AddMenuEntry
	(
		FTimersViewCommands::Get().Command_FindMinInstance,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FindMinInstance")
	);

	MenuBuilder.AddMenuSeparator();

	MenuBuilder.AddMenuEntry
	(
		FTimersViewCommands::Get().Command_FindFirstInstanceInSelection,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::Get().GetStyleSetName(), "Icons.FindFirst")
	);

	MenuBuilder.AddMenuEntry
	(
		FTimersViewCommands::Get().Command_FindPreviousInstanceInSelection,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::Get().GetStyleSetName(), "Icons.FindPrevious")
	);

	MenuBuilder.AddMenuEntry
	(
		FTimersViewCommands::Get().Command_FindNextInstanceInSelection,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::Get().GetStyleSetName(), "Icons.FindNext")
	);

	MenuBuilder.AddMenuEntry
	(
		FTimersViewCommands::Get().Command_FindLastInstanceInSelection,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FInsightsStyle::Get().GetStyleSetName(), "Icons.FindLast")
	);

	MenuBuilder.AddMenuEntry
	(
		FTimersViewCommands::Get().Command_FindMaxInstanceInSelection,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FindMaxInstance")
	);

	MenuBuilder.AddMenuEntry
	(
		FTimersViewCommands::Get().Command_FindMinInstanceInSelection,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FindMinInstance")
	);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::TreeView_BuildSettingsMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.SetSearchable(false);

	MenuBuilder.BeginSection("Settings");
	{
		{
			MenuBuilder.AddEditableText
			(
				LOCTEXT("ThreadsAggregation", "Number of Threads"),
				LOCTEXT("ThreadsAggregation_Tooltip",
					"Number of threads being used for Aggregation.\n"
					"Using higher values may cause systems to freeze.\n"
					"This value is persistent to the UnrealInsightsSettings.ini file."),
				FSlateIcon(),
				TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &STimersView::GetThreadRatio)),
				FOnTextCommitted::CreateLambda([this](const FText& NewValue, ETextCommit::Type)
					{
						double Ratio = FCString::Atof(*NewValue.ToString());
						OnThreadRatioCommited(Ratio);
					})
			);
		}

		{
			FUIAction Action_SetSourceFilesSearchPath
			(
				FExecuteAction::CreateSP(this, &STimersView::SetSourceFileSearchPaths_Execute),
				FCanExecuteAction::CreateLambda([]() { return true; })
			);

			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("SetSourceFilesSearchPath_Fmt", "Source File Path"),
				TAttribute<FText>::CreateSP(this, &STimersView::SetSourceFileSearchPaths_GetTooltip),
				FSlateIcon(),
				Action_SetSourceFilesSearchPath,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}

		{
			FUIAction Action_ShowFormatInTimerNames
			(
				FExecuteAction::CreateSP(this, &STimersView::ToggleShowFormatInTimerNames_Execute),
				FCanExecuteAction::CreateLambda([]() { return true; }),
				FIsActionChecked::CreateLambda([]() { return FInsightsManager::Get()->GetSettings().GetShowFormatInTimerNames(); })
			);

			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("SetShowFormatInTimerNames_Fmt", "Show Format In Timer Names"),
				LOCTEXT("SetShowFormatInTimerNamesTooltip_Fmt", "Append the format to timers names for events with metadata."),
				FSlateIcon(),
				Action_ShowFormatInTimerNames,
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::InitializeAndShowHeaderColumns()
{
	// Create columns.
	TArray<TSharedRef<FTableColumn>> Columns;
	FTimersViewColumnFactory::CreateTimersViewColumns(Columns);
	Table->SetColumns(Columns);

	// Show columns.
	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		if (ColumnRef->ShouldBeVisible())
		{
			ShowColumn(ColumnRef->GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STimersView::GetColumnHeaderText(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.GetShortName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimersView::TreeViewHeaderRow_GenerateColumnMenu(const FTableColumn& Column)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, NULL);

	MenuBuilder.BeginSection("Sorting", LOCTEXT("ContextMenu_Section_Sorting", "Sorting"));
	{
		if (Column.CanBeSorted())
		{
			FUIAction Action_SortAscending
			(
				FExecuteAction::CreateSP(this, &STimersView::HeaderMenu_SortMode_Execute, Column.GetId(), EColumnSortMode::Ascending),
				FCanExecuteAction::CreateSP(this, &STimersView::HeaderMenu_SortMode_CanExecute, Column.GetId(), EColumnSortMode::Ascending),
				FIsActionChecked::CreateSP(this, &STimersView::HeaderMenu_SortMode_IsChecked, Column.GetId(), EColumnSortMode::Ascending)
			);
			MenuBuilder.AddMenuEntry
			(
				FText::Format(LOCTEXT("ContextMenu_SortAscending_Fmt", "Sort Ascending (by {0})"), Column.GetTitleName()),
				FText::Format(LOCTEXT("ContextMenu_SortAscending_Desc_Fmt", "Sorts ascending by {0}."), Column.GetTitleName()),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortUp"),
				Action_SortAscending,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);

			FUIAction Action_SortDescending
			(
				FExecuteAction::CreateSP(this, &STimersView::HeaderMenu_SortMode_Execute, Column.GetId(), EColumnSortMode::Descending),
				FCanExecuteAction::CreateSP(this, &STimersView::HeaderMenu_SortMode_CanExecute, Column.GetId(), EColumnSortMode::Descending),
				FIsActionChecked::CreateSP(this, &STimersView::HeaderMenu_SortMode_IsChecked, Column.GetId(), EColumnSortMode::Descending)
			);
			MenuBuilder.AddMenuEntry
			(
				FText::Format(LOCTEXT("ContextMenu_SortDescending_Fmt", "Sort Descending (by {0})"), Column.GetTitleName()),
				FText::Format(LOCTEXT("ContextMenu_SortDescending_Desc_Fmt", "Sorts descending by {0}."), Column.GetTitleName()),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortDown"),
				Action_SortDescending,
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_SortBy_SubMenu", "Sort By"),
			LOCTEXT("ContextMenu_SortBy_SubMenu_Desc", "Sorts by a column."),
			FNewMenuDelegate::CreateSP(this, &STimersView::TreeView_BuildSortByMenu),
			false,
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.SortBy")
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("ColumnVisibility", LOCTEXT("ContextMenu_Section_ColumnVisibility", "Column Visibility"));
	{
		if (Column.CanBeHidden())
		{
			FUIAction Action_HideColumn
			(
				FExecuteAction::CreateSP(this, &STimersView::HideColumn, Column.GetId()),
				FCanExecuteAction::CreateSP(this, &STimersView::CanHideColumn, Column.GetId())
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_HideColumn", "Hide"),
				LOCTEXT("ContextMenu_HideColumn_Desc", "Hides the selected column."),
				FSlateIcon(),
				Action_HideColumn,
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("ContextMenu_ViewColumn_SubMenu", "View Column"),
			LOCTEXT("ContextMenu_ViewColumn_SubMenu_Desc", "Hides or shows columns."),
			FNewMenuDelegate::CreateSP(this, &STimersView::TreeView_BuildViewColumnMenu),
			false,
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ViewColumn")
		);

		FUIAction Action_ShowAllColumns
		(
			FExecuteAction::CreateSP(this, &STimersView::ContextMenu_ShowAllColumns_Execute),
			FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_ShowAllColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_ShowAllColumns", "Show All Columns"),
			LOCTEXT("ContextMenu_ShowAllColumns_Desc", "Resets tree view to show all columns."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_ShowAllColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);

		FUIAction Action_ShowMinMaxMedColumns
		(
			FExecuteAction::CreateSP(this, &STimersView::ContextMenu_ShowMinMaxMedColumns_Execute),
			FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_ShowMinMaxMedColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_ShowMinMaxMedColumns", "Reset Columns to Min/Max/Median Preset"),
			LOCTEXT("ContextMenu_ShowMinMaxMedColumns_Desc", "Resets columns to Min/Max/Median preset."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_ShowMinMaxMedColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);

		FUIAction Action_ResetColumns
		(
			FExecuteAction::CreateSP(this, &STimersView::ContextMenu_ResetColumns_Execute),
			FCanExecuteAction::CreateSP(this, &STimersView::ContextMenu_ResetColumns_CanExecute)
		);
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ContextMenu_ResetColumns", "Reset Columns to Default"),
			LOCTEXT("ContextMenu_ResetColumns_Desc", "Resets columns to default."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ResetColumn"),
			Action_ResetColumns,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::InsightsManager_OnSessionChanged()
{
	TSharedPtr<const TraceServices::IAnalysisSession> NewSession = FInsightsManager::Get()->GetSession();
	if (NewSession != Session)
	{
		Session = NewSession;
		Reset();
	}
	else
	{
		UpdateTree();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::InsightsManager_OnSessionAnalysisCompleted()
{
	// Re-sync the list of timers to update the "<unknown>" timer names.
	RebuildTree(true);

	// Aggregate stats automatically for the entire session (but only if user didn't made a time selection yet).
	if (Aggregator->IsEmptyTimeInterval() && !Aggregator->IsRunning())
	{
		TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();
		InsightsManager->UpdateSessionDuration();
		const double SessionDuration = InsightsManager->GetSessionDuration();

		constexpr double Delta = 0.0; // session padding

		Aggregator->Cancel();
		Aggregator->SetAggregationMode(AggregationMode);
		Aggregator->SetTimeInterval(0.0 - Delta, SessionDuration + Delta);
		Aggregator->Start();

		if (ColumnBeingSorted == NAME_None)
		{
			// Restore sorting...
			SetSortModeForColumn(GetDefaultColumnBeingSorted(), GetDefaultColumnSortMode());
			TreeView_Refresh();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::UpdateTree()
{
	CreateGroups();
	SortTreeNodes();
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ApplyFiltering()
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	FilteredGroupNodes.Reset();

	// Apply filter to all groups and its children.
	const int32 NumGroups = GroupNodes.Num();
	for (int32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
	{
		FTimerNodePtr& GroupPtr = GroupNodes[GroupIndex];
		GroupPtr->ClearFilteredChildren();
		const bool bIsGroupVisible = Filters->PassesAllFilters(GroupPtr);

		const TArray<FBaseTreeNodePtr>& GroupChildren = GroupPtr->GetChildren();
		int32 NumVisibleChildren = 0;
		for (const FBaseTreeNodePtr& ChildPtr : GroupChildren)
		{
			check(ChildPtr->Is<FTimerNode>());
			const FTimerNodePtr& NodePtr = StaticCastSharedPtr<FTimerNode>(ChildPtr);

			const bool bIsChildVisible =
				(!bFilterOutZeroCountTimers || NodePtr->GetAggregatedStats().InstanceCount > 0) &&
				bTimerTypeIsVisible[static_cast<int>(NodePtr->GetType())] &&
				Filters->PassesAllFilters(NodePtr);

			if (bIsChildVisible)
			{
				// Add a child.
				GroupPtr->AddFilteredChild(NodePtr);
				NumVisibleChildren++;
			}
		}

		if (bIsGroupVisible || NumVisibleChildren > 0)
		{
			// Add a group.
			FilteredGroupNodes.Add(GroupPtr);
			GroupPtr->SetExpansion(true);
		}
		else
		{
			GroupPtr->SetExpansion(false);
		}
	}

	// Only expand tree nodes if we have a text filter.
	const bool bNonEmptyTextFilter = !TextFilter->GetRawFilterText().IsEmpty();
	if (bNonEmptyTextFilter)
	{
		if (!bExpansionSaved)
		{
			ExpandedNodes.Empty();
			TreeView->GetExpandedItems(ExpandedNodes);
			bExpansionSaved = true;
		}

		for (const FTimerNodePtr& GroupPtr : FilteredGroupNodes)
		{
			TreeView->SetItemExpansion(GroupPtr, GroupPtr->IsExpanded());
		}
	}
	else
	{
		if (bExpansionSaved)
		{
			// Restore previously expanded nodes when the text filter is disabled.
			TreeView->ClearExpandedItems();
			for (auto It = ExpandedNodes.CreateConstIterator(); It; ++It)
			{
				TreeView->SetItemExpansion(*It, true);
			}
			bExpansionSaved = false;
		}
	}

	// Update aggregations for groups.
	for (FTimerNodePtr& GroupPtr : FilteredGroupNodes)
	{
		TraceServices::FTimingProfilerAggregatedStats& AggregatedStats = GroupPtr->GetAggregatedStats();

		GroupPtr->ResetAggregatedStats();

		constexpr double NanTimeValue = std::numeric_limits<double>::quiet_NaN();
		AggregatedStats.MedianInclusiveTime = NanTimeValue;
		AggregatedStats.MedianExclusiveTime = NanTimeValue;

		const TArray<FBaseTreeNodePtr>& GroupChildren = GroupPtr->GetFilteredChildren();
		for (const FBaseTreeNodePtr& ChildPtr : GroupChildren)
		{
			check(ChildPtr->Is<FTimerNode>());
			const FTimerNodePtr& NodePtr = StaticCastSharedPtr<FTimerNode>(ChildPtr);
			const TraceServices::FTimingProfilerAggregatedStats& NodeAggregatedStats = NodePtr->GetAggregatedStats();

			if (NodeAggregatedStats.InstanceCount > 0)
			{
				AggregatedStats.InstanceCount += NodeAggregatedStats.InstanceCount;
				AggregatedStats.AverageInstanceCount += NodeAggregatedStats.AverageInstanceCount;
				AggregatedStats.AverageInclusiveTime += NodeAggregatedStats.AverageInclusiveTime;
				AggregatedStats.AverageExclusiveTime += NodeAggregatedStats.AverageExclusiveTime;
				AggregatedStats.TotalInclusiveTime += NodeAggregatedStats.TotalInclusiveTime;
				AggregatedStats.MinInclusiveTime = FMath::Min(AggregatedStats.MinInclusiveTime, NodeAggregatedStats.MinInclusiveTime);
				AggregatedStats.MaxInclusiveTime = FMath::Max(AggregatedStats.MaxInclusiveTime, NodeAggregatedStats.MaxInclusiveTime);
				AggregatedStats.TotalExclusiveTime += NodeAggregatedStats.TotalExclusiveTime;
				AggregatedStats.MinExclusiveTime = FMath::Min(AggregatedStats.MinExclusiveTime, NodeAggregatedStats.MinExclusiveTime);
				AggregatedStats.MaxExclusiveTime = FMath::Max(AggregatedStats.MaxExclusiveTime, NodeAggregatedStats.MaxExclusiveTime);
			}
		}
	}

	// Request tree refresh
	TreeView->RequestTreeRefresh();

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.1)
	{
		UE_LOGF(LogTimingProfiler, Log, "[Timers] Tree view filtered in %.3fs (%d timers)",
			TotalTime, TimerNodes.Num());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::HandleItemToStringArray(const FTimerNodePtr& FTimerNodePtr, TArray<FString>& OutSearchStrings) const
{
	OutSearchStrings.Add(FTimerNodePtr->GetDisplayName().ToString());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimersView::GetToggleButtonForTimerType(const ETimerNodeType NodeType)
{
	return SNew(SCheckBox)
		.Visibility_Lambda([this, NodeType]()
			{
				return static_cast<uint32>(NodeType) < static_cast<uint32>(ETimerNodeType::InvalidOrMax) &&
					bTimerTypeIsEnabled[static_cast<uint32>(NodeType)] ? EVisibility::Visible : EVisibility::Collapsed;
			})
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.Padding(FMargin(4.0f, 2.0f, 4.0f, 2.0f))
		.HAlign(HAlign_Center)
		.OnCheckStateChanged(this, &STimersView::FilterByTimerType_OnCheckStateChanged, NodeType)
		.IsChecked(this, &STimersView::FilterByTimerType_IsChecked, NodeType)
		.ToolTipText(TimerNodeTypeHelper::ToDescription(NodeType))
		[
			SNew(SHorizontalBox)

			//+ SHorizontalBox::Slot()
			//.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			//.AutoWidth()
			//.VAlign(VAlign_Center)
			//[
			//	SNew(SImage)
			//	.Image(TimerNodeTypeHelper::GetIconForTimerNodeType(NodeType))
			//]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(TimerNodeTypeHelper::ToText(NodeType))
			]
		];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::FilterOutZeroCountTimers_OnCheckStateChanged(ECheckBoxState NewRadioState)
{
	bFilterOutZeroCountTimers = (NewRadioState == ECheckBoxState::Checked);
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState STimersView::FilterOutZeroCountTimers_IsChecked() const
{
	return bFilterOutZeroCountTimers ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::FilterByTimerType_OnCheckStateChanged(ECheckBoxState NewRadioState, const ETimerNodeType InNodeType)
{
	bTimerTypeIsVisible[static_cast<int>(InNodeType)] = (NewRadioState == ECheckBoxState::Checked);
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState STimersView::FilterByTimerType_IsChecked(const ETimerNodeType InNodeType) const
{
	return bTimerTypeIsVisible[static_cast<int>(InNodeType)] ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// TreeView
////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::TreeView_Refresh()
{
	if (TreeView.IsValid())
	{
		TreeView->RequestTreeRefresh();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::TreeView_OnSelectionChanged(FTimerNodePtr SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
		if (SelectedNode.IsValid() && SelectedNode->GetType() != ETimerNodeType::Group)
		{
			FTimingProfilerManager::Get()->SetSelectedTimer(SelectedNode->GetTimerId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNodePtr STimersView::GetSingleSelectedTimerNode() const
{
	if (TreeView->GetNumItemsSelected() != 1)
	{
		return nullptr;
	}
	TArray<FTimerNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	check(SelectedNodes.Num() == 1);
	return SelectedNodes[0];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::TreeView_OnGetChildren(FTimerNodePtr InParent, TArray<FTimerNodePtr>& OutChildren)
{
	const TArray<FBaseTreeNodePtr>& Children = InParent->GetFilteredChildren();
	OutChildren.Reset(Children.Num());
	for (const FBaseTreeNodePtr& Child : Children)
	{
		check(Child->Is<FTimerNode>());
		OutChildren.Add(StaticCastSharedPtr<FTimerNode>(Child));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::TreeView_OnMouseButtonDoubleClick(FTimerNodePtr NodePtr)
{
	if (NodePtr->GetType() == ETimerNodeType::Group)
	{
		const bool bIsGroupExpanded = TreeView->IsItemExpanded(NodePtr);
		TreeView->SetItemExpansion(NodePtr, !bIsGroupExpanded);
	}
	else
	{
		if (FSlateApplication::Get().GetModifierKeys().IsControlDown())
		{
			ToggleTimingViewEventFilter(NodePtr);
		}
		else
		{
			ToggleTimingViewMainGraphEventSeries(NodePtr);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Tree View's Table Row
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableRow> STimersView::TreeView_OnGenerateRow(FTimerNodePtr NodePtr, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<ITableRow> TableRow =
		SNew(STimerTableRow, OwnerTable)
		.OnShouldBeEnabled(this, &STimersView::TableRow_ShouldBeEnabled)
		.OnIsColumnVisible(this, &STimersView::IsColumnVisible)
		.OnSetHoveredCell(this, &STimersView::TableRow_SetHoveredCell)
		.OnGetColumnOutlineHAlignmentDelegate(this, &STimersView::TableRow_GetColumnOutlineHAlignment)
		.HighlightText(this, &STimersView::TableRow_GetHighlightText)
		.TablePtr(Table)
		.TimerNodePtr(NodePtr);

	return TableRow;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::TableRow_ShouldBeEnabled(FTimerNodePtr NodePtr) const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::TableRow_SetHoveredCell(TSharedPtr<FTable> InTablePtr, TSharedPtr<FTableColumn> InColumnPtr, FTimerNodePtr InNodePtr)
{
	HoveredColumnId = InColumnPtr ? InColumnPtr->GetId() : FName();

	const bool bIsAnyMenusVisible = FSlateApplication::Get().AnyMenusVisible();
	if (!HasMouseCapture() && !bIsAnyMenusVisible)
	{
		HoveredNodePtr = InNodePtr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EHorizontalAlignment STimersView::TableRow_GetColumnOutlineHAlignment(const FName ColumnId) const
{
	const TIndirectArray<SHeaderRow::FColumn>& Columns = TreeViewHeaderRow->GetColumns();

	// First column
	if (Columns.Num() == 0 ||
		Columns[0].ColumnId == ColumnId)
	{
		return HAlign_Left;
	}
	// Last column
	else if (Columns[Columns.Num() - 1].ColumnId == ColumnId)
	{
		return HAlign_Right;
	}
	// Middle columns
	{
		return HAlign_Center;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STimersView::TableRow_GetHighlightText() const
{
	return SearchBox->GetText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SearchBox
////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::SearchBox_OnTextChanged(const FText& InFilterText)
{
	TextFilter->SetRawFilterText(InFilterText);
	SearchBox->SetError(TextFilter->GetFilterErrorText());
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Aggregation Performance
////////////////////////////////////////////////////////////////////////////////////////////////////

double STimersView::GetMinimumThreadRatio() const
{
	return 1.0f / (double)GThreadPool->GetNumThreads();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STimersView::GetThreadRatio() const
{
	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			return FText::AsNumber((int)(TimingProfilerProvider->GetRatioOfThreadsToUse() * GThreadPool->GetNumThreads()));
		}
	}
	return FText::AsNumber((int)(0.75 * GThreadPool->GetNumThreads()));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::OnThreadRatioCommited(double InNewValue)
{
	double ClampedNewValue = FMath::Clamp(
		InNewValue / GThreadPool->GetNumThreads(),
		GetMinimumThreadRatio(),
		1.0);

	if (Session.IsValid())
	{
		TraceServices::IAnalysisSession& EditableSession = *const_cast<TraceServices::IAnalysisSession*>(Session.Get());
		TraceServices::IEditableTimingProfilerProvider* TimingProfilerProvider = TraceServices::EditTimingProfilerProvider(EditableSession);
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionEditScope SessionEditScope(EditableSession);
			TimingProfilerProvider->SetRatioOfThreadsToUse(ClampedNewValue);
		}
	}

	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
	Settings.SetAndSaveThreadRatio(ClampedNewValue);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::SearchBox_IsEnabled() const
{
	return TimerNodes.Num() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Grouping
////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::CreateGroups()
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	if (GroupingMode == ETimerGroupingMode::Flat)
	{
		GroupNodes.Reset();

		FTimerNodePtr GroupPtr = MakeShared<FTimerNode>(LOCTEXT("GroupName_All", "All"));
		GroupNodes.Add(GroupPtr);

		for (const FTimerNodePtr& NodePtr : TimerNodes)
		{
			GroupPtr->AddChildAndSetParent(NodePtr);
		}
		TreeView->SetItemExpansion(GroupPtr, true);
	}
	// Creates groups based on stat metadata groups.
	else if (GroupingMode == ETimerGroupingMode::ByMetaGroupName)
	{
		TMap<FName, FTimerNodePtr> GroupNodeSet;
		for (const FTimerNodePtr& NodePtr : TimerNodes)
		{
			const FName GroupName = NodePtr->GetMetaGroupName();
			FTimerNodePtr GroupPtr = GroupNodeSet.FindRef(GroupName);
			if (!GroupPtr)
			{
				const FText DisplayName = FText::FromName(GroupName);
				GroupPtr = GroupNodeSet.Add(GroupName, MakeShared<FTimerNode>(DisplayName));
			}
			GroupPtr->AddChildAndSetParent(NodePtr);
			TreeView->SetItemExpansion(GroupPtr, true);
		}
		GroupNodeSet.KeySort([](const FName& A, const FName& B) { return A.Compare(B) < 0; }); // sort groups by name
		GroupNodeSet.GenerateValueArray(GroupNodes);
	}
	// Creates one group for each stat type.
	else if (GroupingMode == ETimerGroupingMode::ByType)
	{
		TMap<ETimerNodeType, FTimerNodePtr> GroupNodeSet;
		for (const FTimerNodePtr& NodePtr : TimerNodes)
		{
			const ETimerNodeType NodeType = NodePtr->GetType();
			FTimerNodePtr GroupPtr = GroupNodeSet.FindRef(NodeType);
			if (!GroupPtr)
			{
				const FText GroupName = TimerNodeTypeHelper::ToText(NodeType);
				GroupPtr = GroupNodeSet.Add(NodeType, MakeShared<FTimerNode>(GroupName));
			}
			GroupPtr->AddChildAndSetParent(NodePtr);
			TreeView->SetItemExpansion(GroupPtr, true);
		}
		GroupNodeSet.KeySort([](const ETimerNodeType& A, const ETimerNodeType& B) { return A < B; }); // sort groups by type
		GroupNodeSet.GenerateValueArray(GroupNodes);
	}
	// Creates one group for one letter.
	else if (GroupingMode == ETimerGroupingMode::ByName)
	{
		TMap<TCHAR, FTimerNodePtr> GroupNodeSet;
		for (const FTimerNodePtr& NodePtr : TimerNodes)
		{
			FString FirstLetterStr(NodePtr->GetDisplayName().ToString().Left(1).ToUpper());
			const TCHAR FirstLetter = FirstLetterStr.Len() > 0 ? FirstLetterStr[0] : TEXT('\0');
			FTimerNodePtr GroupPtr = GroupNodeSet.FindRef(FirstLetter);
			if (!GroupPtr)
			{
				const FText GroupName = FText::FromString(FirstLetterStr);
				GroupPtr = GroupNodeSet.Add(FirstLetter, MakeShared<FTimerNode>(GroupName));
			}
			GroupPtr->AddChildAndSetParent(NodePtr);
		}
		GroupNodeSet.KeySort([](const TCHAR& A, const TCHAR& B) { return A < B; }); // sort groups alphabetically
		GroupNodeSet.GenerateValueArray(GroupNodes);
	}
	// Creates one group for each logarithmic range ie. 0, [1 .. 10), [10 .. 100), [100 .. 1K), etc.
	else if (GroupingMode == ETimerGroupingMode::ByInstanceCount)
	{
		const TCHAR* Orders[] =
		{
			TEXT("1"), TEXT("10"), TEXT("100"),
			TEXT("1K"), TEXT("10K"), TEXT("100K"),
			TEXT("1M"), TEXT("10M"), TEXT("100M"),
			TEXT("1G"), TEXT("10G"), TEXT("100G"),
			TEXT("1T")
		};
		const uint32 MaxOrder = UE_ARRAY_COUNT(Orders);
		TMap<uint32, FTimerNodePtr> GroupNodeSet;
		for (const FTimerNodePtr& NodePtr : TimerNodes)
		{
			uint64 InstanceCount = NodePtr->GetAggregatedStats().InstanceCount;
			uint32 Order = 0;
			while (InstanceCount)
			{
				InstanceCount /= 10;
				Order++;
			}
			FTimerNodePtr GroupPtr = GroupNodeSet.FindRef(Order);
			if (!GroupPtr)
			{
				const FText GroupName =
					(Order == 0) ?
						LOCTEXT("GroupName_ZeroCount", "Count == 0") :
					(Order < MaxOrder) ?
						FText::Format(LOCTEXT("GroupName_CountFmt1", "Count: [{0} .. {1})"), FText::FromString(Orders[Order - 1]), FText::FromString(Orders[Order])) :
						FText::Format(LOCTEXT("GroupName_CountFmt2", "Count >= {0}"), FText::FromString(Orders[MaxOrder - 1]));
				GroupPtr = GroupNodeSet.Add(Order, MakeShared<FTimerNode>(GroupName));
			}
			GroupPtr->AddChildAndSetParent(NodePtr);
		}
		GroupNodeSet.KeySort([](const uint32& A, const uint32& B) { return A > B; }); // sort groups by order
		GroupNodeSet.GenerateValueArray(GroupNodes);
	}
	else
	{
		unimplemented();
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.1)
	{
		UE_LOGF(LogTimingProfiler, Log, "[Timers] Tree view grouping updated in %.3fs (%d timers)",
			TotalTime, TimerNodes.Num());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::CreateGroupByOptionsSources()
{
	GroupByOptionsSource.Reset(3);

	// Must be added in order of elements in the ETimerGroupingMode.
	GroupByOptionsSource.Add(MakeShared<ETimerGroupingMode>(ETimerGroupingMode::Flat));
	GroupByOptionsSource.Add(MakeShared<ETimerGroupingMode>(ETimerGroupingMode::ByName));
	//GroupByOptionsSource.Add(MakeShared<ETimerGroupingMode>(ETimerGroupingMode::ByMetaGroupName));
	GroupByOptionsSource.Add(MakeShared<ETimerGroupingMode>(ETimerGroupingMode::ByType));
	GroupByOptionsSource.Add(MakeShared<ETimerGroupingMode>(ETimerGroupingMode::ByInstanceCount));

	ETimerGroupingModePtr* GroupingModePtrPtr = GroupByOptionsSource.FindByPredicate([&](const ETimerGroupingModePtr InGroupingModePtr) { return *InGroupingModePtr == GroupingMode; });
	if (GroupingModePtrPtr != nullptr)
	{
		GroupByComboBox->SetSelectedItem(*GroupingModePtrPtr);
	}

	GroupByComboBox->RefreshOptions();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::GroupBy_OnSelectionChanged(TSharedPtr<ETimerGroupingMode> NewGroupingMode, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct &&
		NewGroupingMode.IsValid() &&
		*NewGroupingMode != GroupingMode)
	{
		GroupingMode = *NewGroupingMode;

		CreateGroups();
		SortTreeNodes();
		ApplyFiltering();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimersView::GroupBy_OnGenerateWidget(TSharedPtr<ETimerGroupingMode> InGroupingMode) const
{
	check(InGroupingMode.IsValid());
	return SNew(STextBlock)
		.Text(TimerNodeGroupingHelper::ToText(*InGroupingMode))
		.ToolTipText(TimerNodeGroupingHelper::ToDescription(*InGroupingMode));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STimersView::GroupBy_GetSelectedText() const
{
	return TimerNodeGroupingHelper::ToText(GroupingMode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STimersView::GroupBy_GetSelectedTooltipText() const
{
	return TimerNodeGroupingHelper::ToDescription(GroupingMode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Aggregation Mode
////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::CreateModeOptionsSources()
{
	ModeOptionsSource.Reset(uint32(ETimerAggregationMode::Count));
	for (uint32 Mode = 0; Mode < uint32(ETimerAggregationMode::Count); ++Mode)
	{
		ModeOptionsSource.Add(MakeShared<ETimerAggregationMode>(ETimerAggregationMode(Mode)));
	}

	for (TSharedPtr<ETimerAggregationMode> Option : ModeOptionsSource)
	{
		if (*Option == AggregationMode)
		{
			ModeComboBox->SetSelectedItem(Option);
		}
	}

	ModeComboBox->RefreshOptions();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::Mode_OnSelectionChanged(TSharedPtr<ETimerAggregationMode> InAggregationMode, ESelectInfo::Type SelectInfo)
{
	if (!InAggregationMode.IsValid() || *InAggregationMode == AggregationMode)
	{
		return;
	}

	SaveVisibleColumnsSettings();
	AggregationMode = *InAggregationMode;
	LoadVisibleColumnsSettings();

	SetTimingViewFrameType();

	Aggregator->SetAggregationMode(AggregationMode);
	Aggregator->Cancel();
	Aggregator->Start();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimersView::Mode_OnGenerateWidget(TSharedPtr<ETimerAggregationMode> InAggregationMode) const
{
	check(InAggregationMode.IsValid());
	return SNew(STextBlock)
		.Text(Mode_GetText(*InAggregationMode))
		.ToolTipText(Mode_GetTooltipText(*InAggregationMode));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STimersView::Mode_GetSelectedText() const
{
	return Mode_GetText(AggregationMode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STimersView::Mode_GetText(ETimerAggregationMode InAggregationMode) const
{
	switch (InAggregationMode)
	{
	case ETimerAggregationMode::Instance:
		return LOCTEXT("InstanceMode", "Instance");
	case ETimerAggregationMode::GameFrame:
		return LOCTEXT("GameFrameMode", "Game Frame");
	case ETimerAggregationMode::RenderingFrame:
		return LOCTEXT("RenderingFrameMode", "Rendering Frame");
	}

	return FText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STimersView::Mode_GetSelectedTooltipText() const
{
	return Mode_GetTooltipText(AggregationMode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STimersView::Mode_GetTooltipText(ETimerAggregationMode InAggregationMode) const
{
	switch (InAggregationMode)
	{
	case ETimerAggregationMode::Instance:
		return LOCTEXT("ModeInstanceTooltip", "'Instance' Aggregation Mode\nCalculates the stats per event instance.");
	case ETimerAggregationMode::GameFrame:
		return LOCTEXT("ModeGameFrameTooltip", "'Game Frame' Aggregation Mode\nCalculates the timer stats per game frame.");
	case ETimerAggregationMode::RenderingFrame:
		return LOCTEXT("ModeRenderingFrameTooltip", "'Rendering Frame' Aggregation Mode\nCalculates the timer stats per rendering frame.");
	}

	return FText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Sorting
////////////////////////////////////////////////////////////////////////////////////////////////////

const FName STimersView::GetDefaultColumnBeingSorted()
{
	return FTimersViewColumns::TotalInclusiveTimeColumnID;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const EColumnSortMode::Type STimersView::GetDefaultColumnSortMode()
{
	return EColumnSortMode::Descending;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::CreateSortings()
{
	AvailableSorters.Reset();
	CurrentSorter = nullptr;

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		if (ColumnRef->CanBeSorted())
		{
			TSharedPtr<ITableCellValueSorter> SorterPtr = ColumnRef->GetValueSorter();
			if (ensure(SorterPtr.IsValid()))
			{
				AvailableSorters.Add(SorterPtr);
			}
		}
	}

	UpdateCurrentSortingByColumn();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::UpdateCurrentSortingByColumn()
{
	TSharedPtr<FTableColumn> ColumnPtr = Table->FindColumn(ColumnBeingSorted);
	CurrentSorter = ColumnPtr.IsValid() ? ColumnPtr->GetValueSorter() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::SortTreeNodes()
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	if (CurrentSorter.IsValid())
	{
		for (FTimerNodePtr& Root : GroupNodes)
		{
			SortTreeNodesRec(*Root, *CurrentSorter);
		}
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.1)
	{
		UE_LOGF(LogTimingProfiler, Log, "[Timers] Tree view sorted (%ls, %c) in %.3fs (%d timers)",
			CurrentSorter.IsValid() ? *CurrentSorter->GetShortName().ToString() : TEXT("N/A"),
			(ColumnSortMode == EColumnSortMode::Type::Descending) ? TEXT('D') : TEXT('A'),
			TotalTime, TimerNodes.Num());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::SortTreeNodesRec(FTimerNode& Node, const ITableCellValueSorter& Sorter)
{
	ESortMode SortMode = (ColumnSortMode == EColumnSortMode::Type::Descending) ? ESortMode::Descending : ESortMode::Ascending;
	Node.SortChildren(Sorter, SortMode);

#if 0 // Current groupings creates only one level.
	for (FBaseTreeNodePtr ChildPtr : Node.GetChildren())
	{
		if (ChildPtr->GetChildrenCount() > 0)
		{
			check(ChildPtr->Is<FTimerNode>());
			SortTreeNodesRec(*StaticCastSharedPtr<FTimerNode>(ChildPtr), Sorter);
		}
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EColumnSortMode::Type STimersView::GetSortModeForColumn(const FName ColumnId) const
{
	if (ColumnBeingSorted != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return ColumnSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::SetSortModeForColumn(const FName& ColumnId, const EColumnSortMode::Type SortMode)
{
	ColumnBeingSorted = ColumnId;
	ColumnSortMode = SortMode;
	UpdateCurrentSortingByColumn();

	SortTreeNodes();
	ApplyFiltering();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode)
{
	SetSortModeForColumn(ColumnId, SortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortMode action (HeaderMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::HeaderMenu_SortMode_IsChecked(const FName ColumnId, const EColumnSortMode::Type InSortMode)
{
	return ColumnBeingSorted == ColumnId && ColumnSortMode == InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::HeaderMenu_SortMode_CanExecute(const FName ColumnId, const EColumnSortMode::Type InSortMode) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.CanBeSorted();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::HeaderMenu_SortMode_Execute(const FName ColumnId, const EColumnSortMode::Type InSortMode)
{
	SetSortModeForColumn(ColumnId, InSortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortMode action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_SortMode_IsChecked(const EColumnSortMode::Type InSortMode)
{
	return ColumnSortMode == InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_SortMode_CanExecute(const EColumnSortMode::Type InSortMode) const
{
	return true; //ColumnSortMode != InSortMode;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_SortMode_Execute(const EColumnSortMode::Type InSortMode)
{
	SetSortModeForColumn(ColumnBeingSorted, InSortMode);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// SortByColumn action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_SortByColumn_IsChecked(const FName ColumnId)
{
	return ColumnId == ColumnBeingSorted;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_SortByColumn_CanExecute(const FName ColumnId) const
{
	return true; //ColumnId != ColumnBeingSorted;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_SortByColumn_Execute(const FName ColumnId)
{
	SetSortModeForColumn(ColumnId, EColumnSortMode::Descending);
	TreeView_Refresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ShowColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::CanShowColumn(const FName ColumnId) const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ShowColumn(const FName ColumnId)
{
	FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	Column.Show();

	SHeaderRow::FColumn::FArguments ColumnArgs;
	ColumnArgs
		.ColumnId(Column.GetId())
		.DefaultLabel(Column.GetShortName())
		.ToolTip(STimersViewTooltip::GetColumnTooltip(Column))
		.HAlignHeader(Column.GetHorizontalAlignment())
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Fill)
		.InitialSortMode(Column.GetInitialSortMode())
		.SortMode(this, &STimersView::GetSortModeForColumn, Column.GetId())
		.OnSort(this, &STimersView::OnSortModeChanged)
		.FillWidth(Column.GetInitialWidth())
		//.FixedWidth(Column.IsFixedWidth() ? Column.GetInitialWidth() : TOptional<float>())
		.HeaderContent()
		[
			SNew(SBox)
			.HeightOverride(24.0f)
			.Padding(FMargin(0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &STimersView::GetColumnHeaderText, Column.GetId())
			]
		]
		.MenuContent()
		[
			TreeViewHeaderRow_GenerateColumnMenu(Column)
		];

	int32 ColumnIndex = 0;
	const int32 NewColumnPosition = Table->GetColumnPositionIndex(ColumnId);
	const int32 NumColumns = TreeViewHeaderRow->GetColumns().Num();
	for (; ColumnIndex < NumColumns; ColumnIndex++)
	{
		const SHeaderRow::FColumn& CurrentColumn = TreeViewHeaderRow->GetColumns()[ColumnIndex];
		const int32 CurrentColumnPosition = Table->GetColumnPositionIndex(CurrentColumn.ColumnId);
		if (NewColumnPosition < CurrentColumnPosition)
		{
			break;
		}
	}

	TreeViewHeaderRow->InsertColumn(ColumnArgs, ColumnIndex);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// HideColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::CanHideColumn(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.CanBeHidden();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::HideColumn(const FName ColumnId)
{
	FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	Column.Hide();

	TreeViewHeaderRow->RemoveColumn(ColumnId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ToggleColumn action
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::IsColumnVisible(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return Column.IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::CanToggleColumnVisibility(const FName ColumnId) const
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	return !Column.IsVisible() || Column.CanBeHidden();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ToggleColumnVisibility(const FName ColumnId)
{
	const FTableColumn& Column = *Table->FindColumnChecked(ColumnId);
	if (Column.IsVisible())
	{
		HideColumn(ColumnId);
	}
	else
	{
		ShowColumn(ColumnId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// "Show All Columns" action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_ShowAllColumns_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_ShowAllColumns_Execute()
{
	ColumnBeingSorted = GetDefaultColumnBeingSorted();
	ColumnSortMode = GetDefaultColumnSortMode();
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (!Column.IsVisible())
		{
			ShowColumn(Column.GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// "Show Min/Max/Median Columns" action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_ShowMinMaxMedColumns_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_ShowMinMaxMedColumns_Execute()
{
	TSet<FName> Preset =
	{
		FTimersViewColumns::NameColumnID,
		//FTimersViewColumns::MetaGroupNameColumnID,
		//FTimersViewColumns::TypeColumnID,
		FTimersViewColumns::InstanceCountColumnID,

		FTimersViewColumns::TotalInclusiveTimeColumnID,
		FTimersViewColumns::MaxInclusiveTimeColumnID,
		FTimersViewColumns::MedianInclusiveTimeColumnID,
		FTimersViewColumns::MinInclusiveTimeColumnID,

		FTimersViewColumns::TotalExclusiveTimeColumnID,
		FTimersViewColumns::MaxExclusiveTimeColumnID,
		FTimersViewColumns::MedianExclusiveTimeColumnID,
		FTimersViewColumns::MinExclusiveTimeColumnID,
	};

	ColumnBeingSorted = FTimersViewColumns::TotalInclusiveTimeColumnID;
	ColumnSortMode = EColumnSortMode::Descending;
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		const bool bShouldBeVisible = Preset.Contains(Column.GetId());

		if (bShouldBeVisible && !Column.IsVisible())
		{
			ShowColumn(Column.GetId());
		}
		else if (!bShouldBeVisible && Column.IsVisible())
		{
			HideColumn(Column.GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ResetColumns action (ContextMenu)
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_ResetColumns_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_ResetColumns_Execute()
{
	ColumnBeingSorted = GetDefaultColumnBeingSorted();
	ColumnSortMode = GetDefaultColumnSortMode();
	UpdateCurrentSortingByColumn();

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		const FTableColumn& Column = *ColumnRef;

		if (Column.ShouldBeVisible() && !Column.IsVisible())
		{
			ShowColumn(Column.GetId());
		}
		else if (!Column.ShouldBeVisible() && Column.IsVisible())
		{
			HideColumn(Column.GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::Reset()
{
	Aggregator->Cancel();
	Aggregator->SetTimeInterval(0.0, 0.0);

	RebuildTree(true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Check if we need to update the lists of timers, but not too often.
	static uint64 NextTimestamp = 0;
	const uint64 Time = FPlatformTime::Cycles64();
	if (Time > NextTimestamp)
	{
		RebuildTree(false);

		// 1000 timers --> check each 150ms
		// 10000 timers --> check each 600ms
		// 100000 timers --> check each 5.1s
		const double WaitTimeSec = 0.1 + TimerNodes.Num() / 20000.0;
		const uint64 WaitTime = static_cast<uint64>(WaitTimeSec / FPlatformTime::GetSecondsPerCycle64());
		NextTimestamp = Time + WaitTime;
	}

	Aggregator->Tick(Session, InCurrentTime, InDeltaTime, [this]() { FinishAggregation(); });

	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			bTimerTypeIsEnabled[static_cast<int>(ETimerNodeType::GpuScope)] = TimingProfilerProvider->HasGpuTiming();
			bTimerTypeIsEnabled[static_cast<int>(ETimerNodeType::VerseSampling)] = TimingProfilerProvider->HasVerseTiming();
			bTimerTypeIsEnabled[static_cast<int>(ETimerNodeType::CpuScope)] = TimingProfilerProvider->HasCpuTiming();
		}
		else
		{
			bTimerTypeIsEnabled[static_cast<int>(ETimerNodeType::GpuScope)] = false;
			bTimerTypeIsEnabled[static_cast<int>(ETimerNodeType::VerseSampling)] = false;
			bTimerTypeIsEnabled[static_cast<int>(ETimerNodeType::CpuScope)] = false;
		}

		const TraceServices::IStackSamplesProvider* StackSamplesProvider = TraceServices::ReadStackSamplesProvider(*Session.Get());
		if (StackSamplesProvider)
		{
			TraceServices::FProviderReadScopeLock _(*StackSamplesProvider);

			bTimerTypeIsEnabled[static_cast<int>(ETimerNodeType::CpuSampling)] = (StackSamplesProvider->GetTimelineCount() > 0);
		}
		else
		{
			bTimerTypeIsEnabled[static_cast<int>(ETimerNodeType::CpuSampling)] = false;
		}
	}
	else
	{
		bTimerTypeIsEnabled[static_cast<int>(ETimerNodeType::GpuScope)] = false;
		bTimerTypeIsEnabled[static_cast<int>(ETimerNodeType::VerseSampling)] = false;
		bTimerTypeIsEnabled[static_cast<int>(ETimerNodeType::CpuScope)] = false;
		bTimerTypeIsEnabled[static_cast<int>(ETimerNodeType::CpuSampling)] = false;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::RebuildTree(bool bResync)
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	FStopwatch SyncStopwatch;
	SyncStopwatch.Start();

	if (bResync)
	{
		TimerNodes.Empty();
	}

	uint32 AddedNodeCount = 0;

	if (Session.IsValid())
	{
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();

			const uint32 PreviousNodeCount = TimerNodes.Num();
			const uint32 TimerCount = TimerReader.GetTimerCount();
			if (TimerCount != PreviousNodeCount)
			{
				check(TimerCount > PreviousNodeCount);

				TimerNodes.Reserve(TimerCount);

				TSharedPtr<SFrameTrack> FrameTrack = GetFrameTrack();
				TSharedPtr<FTimingGraphTrack> GraphTrack = GetTimingViewMainGraphTrack();

				// Add nodes only for new timers.
				for (uint32 TimerIndex = PreviousNodeCount; TimerIndex < TimerCount; ++TimerIndex)
				{
					const TraceServices::FTimingProfilerTimer* TimerPtr = TimerReader.GetTimer(TimerIndex);
					check(TimerPtr != nullptr);
					const TraceServices::FTimingProfilerTimer& Timer = *TimerPtr;

					ensure(Timer.Id == TimerIndex);

					const ETimerNodeType Type =
						Timer.Type == TraceServices::ETimingProfilerTimerType::GpuScope ? ETimerNodeType::GpuScope :
						Timer.Type == TraceServices::ETimingProfilerTimerType::VerseSampling ? ETimerNodeType::VerseSampling :
						Timer.Type == TraceServices::ETimingProfilerTimerType::CpuScope ? ETimerNodeType::CpuScope :
						Timer.Type == TraceServices::ETimingProfilerTimerType::CpuSampling ? ETimerNodeType::CpuSampling :
						ETimerNodeType::InvalidOrMax;

					FTimerNodePtr TimerNodePtr;
					TStringBuilder<1024> StringBuilder;
					if (FInsightsManager::Get()->GetSettings().GetShowFormatInTimerNames())
					{
						TimingProfilerProvider->MakeTimerDisplayName(Timer, StringBuilder);
					}
					else
					{
						StringBuilder.Append(Timer.Name);
					}

					TimerNodePtr = MakeShared<FTimerNode>(Timer.Id, *StringBuilder, Type, false);

					const bool bIsTimerAddedToGraphs =
						(FrameTrack.IsValid() && FrameTrack->HasAnySeriesForTimer(Timer.Id)) ||
						(GraphTrack.IsValid() && GraphTrack->HasAnySeriesForTimer(Timer.Id));
					TimerNodePtr->SetAddedToGraphsFlag(bIsTimerAddedToGraphs);

					FLinearColor ColorOverride;
					TSharedPtr<STimingProfilerWindow> ProfilerWindow = GetProfilerWindow();
					if (ProfilerWindow.IsValid() && ProfilerWindow->GetTimerColor(Timer.Id, ColorOverride))
					{
						TimerNodePtr->SetColor(ColorOverride);
					}

					TimerNodes.Add(TimerNodePtr);
					++AddedNodeCount;
				}
				ensure(TimerNodes.Num() == TimerCount);
			}
		}
	}

	SyncStopwatch.Stop();

	if (bResync || AddedNodeCount > 0)
	{
		// Disable sorting if too many items.
		if (TimerNodes.Num() > 10000)
		{
			ColumnBeingSorted = NAME_None;
			ColumnSortMode = GetDefaultColumnSortMode();
			UpdateCurrentSortingByColumn();
		}

		UpdateTree();
		Aggregator->Cancel();
		Aggregator->Start();

		// Save selection.
		TArray<FTimerNodePtr> SelectedItems;
		TreeView->GetSelectedItems(SelectedItems);

		TreeView->RebuildList();

		// Restore selection.
		if (SelectedItems.Num() > 0)
		{
			TreeView->ClearSelection();
			for (FTimerNodePtr& NodePtr : SelectedItems)
			{
				NodePtr = GetTimerNode(NodePtr->GetTimerId());
			}
			SelectedItems.RemoveAll([](const FTimerNodePtr& NodePtr) { return !NodePtr.IsValid(); });
			if (SelectedItems.Num() > 0)
			{
				TreeView->SetItemSelection(SelectedItems, true);
				TreeView->RequestScrollIntoView(SelectedItems.Last());
			}
		}
	}

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	if (TotalTime > 0.01)
	{
		const double SyncTime = SyncStopwatch.GetAccumulatedTime();
		UE_LOGF(LogTimingProfiler, Log, "[Timers] Tree view rebuilt in %.4fs (sync: %.4fs + update: %.4fs) --> %d timers (%d added)",
			TotalTime, SyncTime, TotalTime - SyncTime, TimerNodes.Num(), AddedNodeCount);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ResetStats()
{
	Aggregator->Cancel();
	Aggregator->SetTimeInterval(0.0, 0.0);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::UpdateStats(double StartTime, double EndTime)
{
	if (Aggregator->GetIntervalStartTime() != StartTime ||
		Aggregator->GetIntervalEndTime() != EndTime)
	{
		Aggregator->Cancel();
		Aggregator->SetTimeInterval(StartTime, EndTime);
		Aggregator->Start();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::FinishAggregation()
{
	for (const FTimerNodePtr& NodePtr : TimerNodes)
	{
		NodePtr->ResetAggregatedStats();
	}

	ApplyAggregation(Aggregator->GetResultTable());
	Aggregator->ResetResults();

	// Invalidate all tree table rows.
	for (const FTimerNodePtr& NodePtr : TimerNodes)
	{
		TSharedPtr<ITableRow> TableRowPtr = TreeView->WidgetFromItem(NodePtr);
		if (TableRowPtr.IsValid())
		{
			TSharedPtr<STimerTableRow> RowPtr = StaticCastSharedPtr<STimerTableRow, ITableRow>(TableRowPtr);
			RowPtr->InvalidateContent();
		}
	}

	UpdateTree(); // grouping + sorting + filtering

	// Ensure the last selected item is visible.
	TArray<FTimerNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	if (SelectedNodes.Num() > 0)
	{
		TreeView->RequestScrollIntoView(SelectedNodes.Last());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ApplyAggregation(TraceServices::ITable<TraceServices::FTimingProfilerAggregatedStats>* AggregatedStatsTable)
{
	if (AggregatedStatsTable)
	{
		TUniquePtr<TraceServices::ITableReader<TraceServices::FTimingProfilerAggregatedStats>> TableReader(AggregatedStatsTable->CreateReader());
		while (TableReader->IsValid())
		{
			const TraceServices::FTimingProfilerAggregatedStats* Row = TableReader->GetCurrentRow();
			FTimerNodePtr TimerNodePtr = GetTimerNode(Row->Timer->Id);
			if (TimerNodePtr)
			{
				TimerNodePtr->SetAggregatedStats(*Row);
			}
			TableReader->NextRow();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimerNodePtr STimersView::GetTimerNode(uint32 TimerId) const
{
	return (TimerId < (uint32)TimerNodes.Num()) ? TimerNodes[TimerId] : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::SelectTimerNode(uint32 TimerId)
{
	FTimerNodePtr NodePtr = GetTimerNode(TimerId);
	if (NodePtr)
	{
		FBaseTreeNodePtr ParentNodePtr = NodePtr->GetParent();
		while (ParentNodePtr.IsValid())
		{
			check(ParentNodePtr->Is<FTimerNode>());
			TreeView->SetItemExpansion(StaticCastSharedPtr<FTimerNode>(ParentNodePtr), true);
			ParentNodePtr = ParentNodePtr->GetParent();
		}
		TreeView->SetSelection(NodePtr);
		TreeView->RequestScrollIntoView(NodePtr);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ToggleTimingViewEventFilter(FTimerNodePtr TimerNode) const
{
	TSharedPtr<STimingView> TimingView = GetTimingView();
	if (TimingView.IsValid())
	{
		const uint64 EventType = static_cast<uint64>(TimerNode->GetTimerId());
		TimingView->ToggleEventFilterByEventType(EventType);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTimingGraphTrack> STimersView::GetTimingViewMainGraphTrack() const
{
	TSharedPtr<STimingView> TimingView = GetTimingView();
	return TimingView.IsValid() ? TimingView->GetMainTimingGraphTrack() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SFrameTrack> STimersView::GetFrameTrack() const
{
	TSharedPtr<STimingProfilerWindow> Wnd = GetProfilerWindow();
	return Wnd.IsValid() ? Wnd->GetFrameView() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ToggleTimingViewMainGraphEventSeries(FTimerNodePtr TimerNode) const
{
	switch (AggregationMode)
	{
	case ETimerAggregationMode::Instance:
		ToggleTimingViewMainGraphEventInstanceSeries(TimerNode);
		break;
	case ETimerAggregationMode::GameFrame:
		ToggleTimingViewMainGraphEventFrameStatsSeries(TimerNode, ETraceFrameType::TraceFrameType_Game);
		break;
	case ETimerAggregationMode::RenderingFrame:
		ToggleTimingViewMainGraphEventFrameStatsSeries(TimerNode, ETraceFrameType::TraceFrameType_Rendering);
		break;
	default:
		ensure(0);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ToggleGraphInstanceSeries(TSharedRef<FTimingGraphTrack> GraphTrack, FTimerNodeRef NodePtr) const
{
	const uint32 TimerId = NodePtr->GetTimerId();

	TSharedPtr<FTimingGraphSeries> Series = GraphTrack->GetTimerSeries(TimerId);
	if (Series.IsValid())
	{
		GraphTrack->RemoveTimerSeries(TimerId);
		GraphTrack->SetDirtyFlag();
	}
	else
	{
		GraphTrack->Show();
		Series = GraphTrack->AddTimerSeries(TimerId, NodePtr->GetColor());
		Series->SetName(NodePtr->GetDisplayName());
		GraphTrack->SetDirtyFlag();
	}

	TSharedPtr<STimingProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		ProfilerWindow->OnTimerAddedToGraphsChanged(TimerId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::IsInstanceSeriesInTimingViewMainGraph(FTimerNodePtr TimerNode) const
{
	TSharedPtr<FTimingGraphTrack> GraphTrack = GetTimingViewMainGraphTrack();

	if (GraphTrack.IsValid())
	{
		const uint32 TimerId = TimerNode->GetTimerId();
		TSharedPtr<FTimingGraphSeries> Series = GraphTrack->GetTimerSeries(TimerId);

		return Series.IsValid();
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ToggleTimingViewMainGraphEventInstanceSeries(FTimerNodePtr TimerNode) const
{
	TSharedPtr<FTimingGraphTrack> GraphTrack = GetTimingViewMainGraphTrack();
	if (GraphTrack.IsValid())
	{
		ToggleGraphInstanceSeries(GraphTrack.ToSharedRef(), TimerNode.ToSharedRef());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ToggleGraphFrameStatsSeries(TSharedRef<FTimingGraphTrack> GraphTrack, FTimerNodeRef NodePtr, ETraceFrameType FrameType) const
{
	const uint32 TimerId = NodePtr->GetTimerId();

	TSharedPtr<FTimingGraphSeries> Series = GraphTrack->GetFrameStatsTimerSeries(TimerId, FrameType);
	if (Series.IsValid())
	{
		GraphTrack->RemoveFrameStatsTimerSeries(TimerId, FrameType);
		GraphTrack->SetDirtyFlag();
	}
	else
	{
		GraphTrack->Show();
		Series = GraphTrack->AddFrameStatsTimerSeries(TimerId, FrameType, NodePtr->GetColor());
		FText SeriesName = FText::Format(LOCTEXT("FrameStatsTimerSeriesName_Fmt", "{0} ({1})"),
			NodePtr->GetDisplayName(),
			FFrameTrackDrawHelper::FrameTypeToText(FrameType));
		Series->SetName(SeriesName);
		GraphTrack->SetDirtyFlag();
	}

	TSharedPtr<STimingProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		ProfilerWindow->OnTimerAddedToGraphsChanged(TimerId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::IsFrameStatsSeriesInTimingViewMainGraph(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const
{
	if (!TimerNode.IsValid())
	{
		return false;
	}

	TSharedPtr<FTimingGraphTrack> GraphTrack = GetTimingViewMainGraphTrack();
	if (!GraphTrack.IsValid())
	{
		return false;
	}

	const uint32 TimerId = TimerNode->GetTimerId();
	TSharedPtr<FTimingGraphSeries> Series = GraphTrack->GetFrameStatsTimerSeries(TimerId, FrameType);
	return Series.IsValid();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ToggleTimingViewMainGraphEventFrameStatsSeries(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const
{
	if (!TimerNode.IsValid())
	{
		return;
	}

	TSharedPtr<FTimingGraphTrack> GraphTrack = GetTimingViewMainGraphTrack();
	if (!GraphTrack.IsValid())
	{
		return;
	}

	ToggleGraphFrameStatsSeries(GraphTrack.ToSharedRef(), TimerNode.ToSharedRef(), FrameType);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::IsSeriesInFrameTrack(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const
{
	if (!TimerNode.IsValid())
	{
		return false;
	}

	TSharedPtr<SFrameTrack> FrameTrack = GetFrameTrack();
	if (!FrameTrack.IsValid())
	{
		return false;
	}

	return FrameTrack->HasFrameStatSeries(FrameType, TimerNode->GetTimerId());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ToggleFrameTrackSeries(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const
{
	if (!TimerNode.IsValid())
	{
		return;
	}

	TSharedPtr<SFrameTrack> FrameTrack = GetFrameTrack();
	if (!FrameTrack.IsValid())
	{
		return;
	}

	const uint32 TimerId = TimerNode->GetTimerId();

	if (FrameTrack->HasFrameStatSeries(FrameType, TimerId))
	{
		FrameTrack->RemoveTimerFrameStatSeries(FrameType, TimerId);
	}
	else
	{
		FText SeriesName = FText::Format(LOCTEXT("FrameStatsTimerSeriesName_Fmt", "{0} ({1})"),
			TimerNode->GetDisplayName(),
			FFrameTrackDrawHelper::FrameTypeToText(FrameType));
		FrameTrack->AddTimerFrameStatSeries(FrameType, TimerId, TimerNode->GetColor(), SeriesName);
	}

	TSharedPtr<STimingProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		ProfilerWindow->OnTimerAddedToGraphsChanged(TimerId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Copy to Clipboard
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_CopyToClipboard_CanExecute() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_CopyToClipboard_Execute()
{
	if (Table->IsValid())
	{
		const ESortMode SortMode = ColumnSortMode == EColumnSortMode::Ascending ? Insights::ESortMode::Ascending : Insights::ESortMode::Descending;
		UE::Insights::CopyToClipboard(Table.ToSharedRef(), TreeView->GetSelectedItems(), CurrentSorter, SortMode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_CopyTimerNameToClipboard_CanExecute() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_CopyTimerNameToClipboard_Execute()
{
	if (Table->IsValid())
	{
		const ESortMode SortMode = ColumnSortMode == EColumnSortMode::Ascending ? Insights::ESortMode::Ascending : Insights::ESortMode::Descending;
		UE::Insights::CopyNameToClipboard(Table.ToSharedRef(), TreeView->GetSelectedItems(), CurrentSorter, SortMode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Export
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_Export_CanExecute() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_Export_Execute()
{
	if (!Table->IsValid())
	{
		return;
	}

	TArray<FTimerNodePtr> SelectedTimerNodes = TreeView->GetSelectedItems();
	if (SelectedTimerNodes.Num() == 0)
	{
		return;
	}

	TArray<FBaseTreeNodePtr> SelectedNodes;
	for (FTimerNodePtr Node : SelectedTimerNodes)
	{
		SelectedNodes.Add(Node);
	}

	const FString DialogTitle = LOCTEXT("Export_Title", "Export Aggregated Timer Stats").ToString();
	const FString DefaultFile = TEXT("TimerStats");
	FString Filename;
	if (!OpenSaveTextFileDialog(DialogTitle, DefaultFile, Filename))
	{
		return;
	}

	TUniquePtr<IFileHandle> ExportFileHandle{ OpenExportFile(*Filename) };
	if (!ExportFileHandle)
	{
		return;
	}

	FStopwatch Stopwatch;
	Stopwatch.Start();

	bool bIsCSV = Filename.EndsWith(TEXT(".csv"));
	FUtf8FileWriter Writer(ExportFileHandle.Get(), bIsCSV);

	TArray<TSharedRef<FTableColumn>> VisibleColumns;
	Table->GetVisibleColumns(VisibleColumns);

	// Write header.
	{
		bool bIsFirstColumn = true;
		for (const TSharedRef<FTableColumn>& ColumnRef : VisibleColumns)
		{
			if (bIsFirstColumn)
			{
				bIsFirstColumn = false;
			}
			else
			{
				Writer.AppendSeparator();
			}

			FString ColumnName = ColumnRef->GetShortName().ToString();
			Writer.AppendValue(ColumnName);
		}
		Writer.AppendLineEnd();
	}

	if (CurrentSorter.IsValid())
	{
		CurrentSorter->Sort(SelectedNodes, ColumnSortMode == EColumnSortMode::Ascending ? ESortMode::Ascending : ESortMode::Descending);
	}

	const int32 NodeCount = SelectedNodes.Num();
	for (int32 Index = 0; Index < NodeCount; Index++)
	{
		const FBaseTreeNodePtr& Node = SelectedNodes[Index];

		bool bIsFirstColumn = true;
		for (const TSharedRef<FTableColumn>& ColumnRef : VisibleColumns)
		{
			if (bIsFirstColumn)
			{
				bIsFirstColumn = false;
			}
			else
			{
				Writer.AppendSeparator();
			}

			FString Value = ColumnRef->GetValueAsSerializableString(*Node);
			Writer.AppendValue(Value);
		}
		Writer.AppendLineEnd();
	}

	Writer.Flush();
	ExportFileHandle->Flush();
	ExportFileHandle.Reset();

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TraceInsights, Log, "Exported aggregated timer stats to file in %.3fs (\"%ls\").", TotalTime, *Filename);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_ExportTimingEventsSelection_CanExecute() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::AddTimerNodeRecursive(FTimerNodePtr InNode, TSet<uint32>& InOutIncludedTimers) const
{
	if (InNode->GetType() == ETimerNodeType::Group)
	{
		for (FBaseTreeNodePtr ChildNode : InNode->GetFilteredChildren())
		{
			check(ChildNode->Is<FTimerNode>());
			AddTimerNodeRecursive(StaticCastSharedPtr<FTimerNode>(ChildNode), InOutIncludedTimers);
		}
	}
	else
	{
		InOutIncludedTimers.Add(InNode->GetTimerId());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_ExportTimingEventsSelection_Execute() const
{
	if (!Session.IsValid())
	{
		return;
	}

	const FString DialogTitle = LOCTEXT("ExportTimingEventsSelection_Title", "Export Timing Events (Selection)").ToString();
	const FString DefaultFile = TEXT("TimingEvents");
	FString Filename;
	if (!OpenSaveTextFileDialog(DialogTitle, DefaultFile, Filename))
	{
		return;
	}

	FTimingExporter Exporter(*Session.Get());
	FTimingExporter::FExportTimingEventsParams Params; // default columns, all timing events

	////////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedPtr<STimingView> TimingView = GetTimingView();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Filter by thread (visible GPU/CPU tracks in TimingView).

	// These variables needs to be in the same scope with the call to Exporter.ExportTimingEventsAsText()
	// (because they are referenced in the ThreadFilter lambda function).
	TSet<uint32> IncludedThreads;
	TSet<uint32> ExcludedThreads;

	if (TimingView.IsValid())
	{
		// Add available GPU threads to the ExcludedThreads list.
		{
			ExcludedThreads.Add(FGpuTimingTrack::Gpu1ThreadId);
			ExcludedThreads.Add(FGpuTimingTrack::Gpu2ThreadId);
		}

		// Add GPU Queues for the new GPU Profiler
		const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
		if (TimingProfilerProvider)
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			TimingProfilerProvider->EnumerateGpuQueues(
				[&ExcludedThreads](const TraceServices::FGpuQueueInfo& QueueInfo)
				{
					ExcludedThreads.Add(QueueInfo.Id + (1 << 16));
				});
		}

		// Add the Verse Sampling fake thread to the ExcludedThreads list.
		{
			ExcludedThreads.Add(FVerseTimingTrack::VerseThreadId);
		}

		// Add available CPU threads to the ExcludedThreads list.
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

			const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(*Session.Get());

			ThreadProvider.EnumerateThreads(
				[&ExcludedThreads](const TraceServices::FThreadInfo& ThreadInfo)
				{
					ExcludedThreads.Add(ThreadInfo.Id);
				});
		}

		// Move the threads corresponding to visible CPU/GPU tracks to the IncludedThreads list.
		TimingView->EnumerateAllTracks(
			[&IncludedThreads, &ExcludedThreads]
			(TSharedPtr<FBaseTimingTrack>& Track) -> bool
			{
				if (Track->IsVisible())
				{
					if (Track->Is<FGpuQueueTimingTrack>())
					{
						const uint32 ThreadId = Track->As<FGpuQueueTimingTrack>().GetQueueId() + (1 << 16);
						ExcludedThreads.Remove(ThreadId);
						IncludedThreads.Add(ThreadId);
					}
					else if (Track->Is<FThreadTimingTrack>())
					{
						const uint32 ThreadId = Track->As<FThreadTimingTrack>().GetThreadId();
						ExcludedThreads.Remove(ThreadId);
						IncludedThreads.Add(ThreadId);
					}
				}
				return true;
			});

		if (IncludedThreads.Num() < ExcludedThreads.Num())
		{
			Params.ThreadFilter = FTimingExporter::MakeThreadFilterInclusive(IncludedThreads);
		}
		else
		{
			Params.ThreadFilter = FTimingExporter::MakeThreadFilterExclusive(ExcludedThreads);
		}
	}

	// Debug/test filters.
	//Params.ThreadFilter = [](uint32 ThreadId) -> bool { return ThreadId == 2; };

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Filter by timing event (e.g.: by timer, by duration, by depth).

	// This variable needs to be in the same scope with the call to Exporter.ExportTimingEventsAsText()
	// (because it is referenced in the TimingEventFilter lambda function).
	TSet<uint32> IncludedTimers;

	TArray<FTimerNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	for (FTimerNodePtr Node : SelectedNodes)
	{
		AddTimerNodeRecursive(Node, IncludedTimers);
	}

	Params.TimingEventFilter = FTimingExporter::MakeTimingEventFilterByTimersInclusive(IncludedTimers);

	// Debug/test filters.
	//Params.TimingEventFilter = [](double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event) -> bool { return Event.TimerIndex < 100; };
	//Params.TimingEventFilter = [](double EventStartTime, double EventEndTime, uint32 EventDepth, const TraceServices::FTimingProfilerEvent& Event) -> bool { return EventEndTime - EventStartTime > 0.001; };

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Limit the time interval for enumeration (if a time range selection is made in Timing view).

	Params.IntervalStartTime = -std::numeric_limits<double>::infinity();
	Params.IntervalEndTime = +std::numeric_limits<double>::infinity();
	if (TimingView.IsValid())
	{
		const double SelectionStartTime = TimingView->GetSelectionStartTime();
		const double SelectionEndTime = TimingView->GetSelectionEndTime();
		if (SelectionStartTime < SelectionEndTime)
		{
			Params.IntervalStartTime = SelectionStartTime;
			Params.IntervalEndTime = SelectionEndTime;
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	Exporter.ExportTimingEventsAsText(*Filename, Params);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_ExportTimingEvents_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_ExportTimingEvents_Execute() const
{
	if (!Session.IsValid())
	{
		return;
	}

	const FString DialogTitle = LOCTEXT("ExportTimingEvents_Title", "Export Timing Events (All)").ToString();
	const FString DefaultFile = TEXT("TimingEvents");
	FString Filename;
	if (!OpenSaveTextFileDialog(DialogTitle, DefaultFile, Filename))
	{
		return;
	}

	FTimingExporter Exporter(*Session.Get());
	FTimingExporter::FExportTimingEventsParams Params; // default columns, all timing events
	Exporter.ExportTimingEventsAsText(*Filename, Params);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_ExportThreads_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_ExportThreads_Execute() const
{
	if (!Session.IsValid())
	{
		return;
	}

	const FString DialogTitle = LOCTEXT("ExportThreads_Title", "Export Threads").ToString();
	const FString DefaultFile = TEXT("Threads");
	FString Filename;
	if (!OpenSaveTextFileDialog(DialogTitle, DefaultFile, Filename))
	{
		return;
	}

	FTimingExporter Exporter(*Session.Get());
	FTimingExporter::FExportThreadsParams Params; // default
	Exporter.ExportThreadsAsText(*Filename, Params);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_ExportTimers_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_ExportTimers_Execute() const
{
	if (!Session.IsValid())
	{
		return;
	}

	const FString DialogTitle = LOCTEXT("ExportTimers_Title", "Export Timers").ToString();
	const FString DefaultFile = TEXT("Timers");
	FString Filename;
	if (!OpenSaveTextFileDialog(DialogTitle, DefaultFile, Filename))
	{
		return;
	}

	FTimingExporter Exporter(*Session.Get());
	FTimingExporter::FExportTimersParams Params; // default
	Params.bShowFormatInTimerNames = FInsightsManager::Get()->GetSettings().GetShowFormatInTimerNames();

	Exporter.ExportTimersAsText(*Filename, Params);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::OpenSaveTextFileDialog(const FString& InDialogTitle, const FString& InDefaultFile, FString& OutFilename) const
{
	TArray<FString> SaveFilenames;
	bool bDialogResult = false;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		const FString DefaultPath = FPaths::ProjectSavedDir();
		bDialogResult = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			InDialogTitle,
			DefaultPath,
			InDefaultFile,
			TEXT("Comma-Separated Values (*.csv)|*.csv|Tab-Separated Values (*.tsv)|*.tsv|Text Files (*.txt)|*.txt|All Files (*.*)|*.*"),
			EFileDialogFlags::None,
			SaveFilenames
		);
	}

	if (!bDialogResult || SaveFilenames.Num() == 0)
	{
		return false;
	}

	OutFilename = SaveFilenames[0];
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

IFileHandle* STimersView::OpenExportFile(const TCHAR* InFilename) const
{
	IFileHandle* ExportFileHandle = FPlatformFileManager::Get().GetPlatformFile().OpenWrite(InFilename);

	if (ExportFileHandle == nullptr)
	{
		FName LogListingName = FTimingProfilerManager::Get()->GetLogListingName();
		FMessageLog ReportMessageLog((LogListingName != NAME_None) ? LogListingName : TEXT("Other"));
		ReportMessageLog.Error(LOCTEXT("FailedToOpenFile", "Export failed. Failed to open file for write."));
		ReportMessageLog.Notify();
		return nullptr;
	}

	return ExportFileHandle;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Open Source File in IDE
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_OpenSource_CanExecute() const
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();

	if (!SourceCodeAccessor.CanAccessSourceCode())
	{
		return false;
	}

	FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
	if (!SelectedNode.IsValid())
	{
		return false;
	}

	FString File;
	uint32 Line = 0;
	return SelectedNode->GetSourceFileAndLine(File, Line);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_OpenSource_Execute() const
{
	FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
	if (SelectedNode.IsValid())
	{
		OpenSourceFileInIDE(SelectedNode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::OpenSourceFileInIDE(FTimerNodePtr InNode) const
{
	if (!InNode.IsValid() || InNode->GetType() == ETimerNodeType::Group)
	{
		return;
	}

	FString File;
	uint32 Line = 0;
	if (!InNode->GetSourceFileAndLine(File, Line))
	{
		return;
	}

	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");

	FString UsablePath;
	const bool bPathExists = FInsightsManager::Get()->GetSourceFilePathHelper().GetUsableFilePath(File, UsablePath);
	if (bPathExists)
	{
		ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();
		SourceCodeAccessor.OpenFileAtLine(UsablePath, Line);
	}
	else
	{
		SourceCodeAccessModule.OnOpenFileFailed().Broadcast(UsablePath);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Generate and Edit Colors
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_GenerateNewColor_CanExecute() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_GenerateNewColor_Execute() const
{
	TArray<FTimerNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	for (const FTimerNodePtr& SelectedNode : SelectedNodes)
	{
		if (SelectedNode->Is<FTimerNode>())
		{
			FTimerNode& TimerNode = SelectedNode->As<FTimerNode>();
			SetRandomColorToNode(TimerNode);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::SetColorToNode(FTimerNode& TimerNode, FLinearColor CustomColor) const
{
	TimerNode.SetColor(CustomColor);

	TSharedPtr<STimingProfilerWindow> ProfilerWindow = GetProfilerWindow();
	if (ProfilerWindow.IsValid())
	{
		ProfilerWindow->SetTimerColor(TimerNode.GetTimerId(), CustomColor);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::SetRandomColorToNode(FTimerNode& TimerNode) const
{
	uint64 Time = FPlatformTime::Cycles64();
	uint32 Hash = (Time & 0xFFFFFFFF) ^ (Time >> 32);
	Hash = ((Hash << 5) + Hash) * 0xfb23618f;

	const uint8 H = Hash & 0xFF;
	const uint8 S = 128 + ((Hash >> 8) & 0x7F);
	const uint8 V = 128 + ((Hash >> 16) & 0x7F);
	FLinearColor CustomColor = FLinearColor::MakeFromHSV8(H, S, V);
	CustomColor.A = 1.0f;

	SetColorToNode(TimerNode, CustomColor);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_EditColor_CanExecute() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_EditColor_Execute()
{
	EditableColorValue = FLinearColor(0.5f, 0.5f, 0.5f, 1.0f);

	TArray<FTimerNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	for (const FTimerNodePtr& SelectedNode : SelectedNodes)
	{
		if (SelectedNode->Is<FTimerNode>())
		{
			FTimerNode& TimerNode = SelectedNode->As<FTimerNode>();
			EditableColorValue = TimerNode.GetColor();
			break;
		}
	}

	FColorPickerArgs PickerArgs;
	{
		PickerArgs.bUseAlpha = true;
		PickerArgs.bOnlyRefreshOnMouseUp = false;
		PickerArgs.bOnlyRefreshOnOk = false;
		PickerArgs.bExpandAdvancedSection = false;
		PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &STimersView::SetEditableColor);
		PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &STimersView::ColorPickerCancelled);
		PickerArgs.InitialColor = EditableColorValue;
		PickerArgs.ParentWidget = SharedThis(this);
	}

	OpenColorPicker(PickerArgs);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor STimersView::GetEditableColor() const
{
	return EditableColorValue;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::SetEditableColor(FLinearColor NewColor)
{
	EditableColorValue = NewColor;

	TArray<FTimerNodePtr> SelectedNodes = TreeView->GetSelectedItems();
	for (const FTimerNodePtr& SelectedNode : SelectedNodes)
	{
		if (SelectedNode->Is<FTimerNode>())
		{
			FTimerNode& TimerNode = SelectedNode->As<FTimerNode>();
			SetColorToNode(TimerNode, EditableColorValue);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ColorPickerCancelled(FLinearColor OriginalColor)
{
	SetEditableColor(OriginalColor);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Find First/Previous/Next/Last/Min/Max Instance
////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_FindInstance_CanExecute() const
{
	FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
	return SelectedNode.IsValid() && SelectedNode->GetType() != ETimerNodeType::Group;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_FindInstance_Execute(ESelectEventType EventType) const
{
	FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
	if (!SelectedNode.IsValid() || SelectedNode->GetType() == ETimerNodeType::Group)
	{
		return;
	}

	TSharedPtr<STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return;
	}

	TimingView->SelectEventInstance(SelectedNode->GetTimerId(), EventType, false);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersView::ContextMenu_FindInstanceInSelection_CanExecute() const
{
	FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
	if (!SelectedNode.IsValid() || SelectedNode->GetType() == ETimerNodeType::Group)
	{
		return false;
	}

	TSharedPtr<STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return false;
	}

	return TimingView->GetSelectionEndTime() > TimingView->GetSelectionStartTime();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ContextMenu_FindInstanceInSelection_Execute(ESelectEventType EventType) const
{
	FTimerNodePtr SelectedNode = GetSingleSelectedTimerNode();
	if (!SelectedNode.IsValid() || SelectedNode->GetType() == ETimerNodeType::Group)
	{
		return;
	}

	TSharedPtr<STimingView> TimingView = GetTimingView();
	if (!TimingView.IsValid())
	{
		return;
	}

	TimingView->SelectEventInstance(SelectedNode->GetTimerId(), EventType, true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Persistent Settings
////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::SaveVisibleColumnsSettings()
{
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();

	TArray<FString> Columns;
	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		if (ColumnRef->IsVisible() && ColumnRef->CanBeHidden())
		{
			Columns.Add(ColumnRef->GetId().ToString());
		}
	}

	switch (AggregationMode)
	{
		case ETimerAggregationMode::Instance:
			Settings.SetTimersViewInstanceVisibleColumns(Columns);
			break;
		case ETimerAggregationMode::GameFrame:
			Settings.SetTimersViewGameFrameVisibleColumns(Columns);
			break;
		case ETimerAggregationMode::RenderingFrame:
			Settings.SetTimersViewRenderingFrameVisibleColumns(Columns);
			break;
	}

	Settings.SaveToConfig();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::SaveSettings()
{
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();

	Settings.SetTimersViewAggregationMode(static_cast<int32>(AggregationMode));
	Settings.SetTimersViewGroupingMode(static_cast<int32>(GroupingMode));
	Settings.SetTimersViewShowGpuTimers(static_cast<int32>(bTimerTypeIsVisible[static_cast<int32>(ETimerNodeType::GpuScope)]));
	Settings.SetTimersViewShowVerseSamplingTimers(static_cast<int32>(bTimerTypeIsVisible[static_cast<int32>(ETimerNodeType::VerseSampling)]));
	Settings.SetTimersViewShowCpuTimers(static_cast<int32>(bTimerTypeIsVisible[static_cast<int32>(ETimerNodeType::CpuScope)]));
	Settings.SetTimersViewShowCpuSamplingTimers(static_cast<int32>(bTimerTypeIsVisible[static_cast<int32>(ETimerNodeType::CpuSampling)]));
	Settings.SetTimersViewShowZeroCountTimers(!bFilterOutZeroCountTimers);

	Settings.SaveToConfig();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::LoadVisibleColumnsSettings()
{
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
	TArray<FString> Columns;

	switch (AggregationMode)
	{
		case ETimerAggregationMode::Instance:
			Columns = Settings.GetTimersViewInstanceVisibleColumns();
			break;
		case ETimerAggregationMode::GameFrame:
			Columns = Settings.GetTimersViewGameFrameVisibleColumns();
			break;
		case ETimerAggregationMode::RenderingFrame:
			Columns = Settings.GetTimersViewRenderingFrameVisibleColumns();
			break;
	}

	for (const TSharedRef<FTableColumn>& ColumnRef : Table->GetColumns())
	{
		if (ColumnRef->CanBeHidden())
		{
			HideColumn(ColumnRef->GetId());
		}
	}

	for (const FString& Column : Columns)
	{
		ShowColumn(FName(Column));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::LoadSettings()
{
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();

	AggregationMode = static_cast<ETimerAggregationMode>(Settings.GetTimersViewAggregationMode());
	GroupingMode = static_cast<ETimerGroupingMode>(Settings.GetTimersViewGroupingMode());
	bTimerTypeIsVisible[static_cast<int32>(ETimerNodeType::GpuScope)] = Settings.GetTimersViewShowGpuTimers();
	bTimerTypeIsVisible[static_cast<int32>(ETimerNodeType::VerseSampling)] = Settings.GetTimersViewShowVerseSamplingTimers();
	bTimerTypeIsVisible[static_cast<int32>(ETimerNodeType::CpuScope)] = Settings.GetTimersViewShowCpuTimers();
	bTimerTypeIsVisible[static_cast<int32>(ETimerNodeType::CpuSampling)] = Settings.GetTimersViewShowCpuSamplingTimers();
	bFilterOutZeroCountTimers = !Settings.GetTimersViewShowZeroCountTimers();

	LoadVisibleColumnsSettings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Misc
////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::SetTimingViewFrameType()
{
	TSharedPtr<STimingView> TimingView = GetTimingView();
	if (TimingView.IsValid())
	{
		TimingView->SelectTimeInterval(TimingView->GetSelectionStartTime(), TimingView->GetSelectionEndTime() - TimingView->GetSelectionStartTime());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::OnTimingViewTrackListChanged()
{
	if (!Aggregator->IsEmptyTimeInterval())
	{
		Aggregator->Cancel();
		Aggregator->SetAggregationMode(AggregationMode);
		Aggregator->Start();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<STimingProfilerWindow> STimersView::GetProfilerWindow() const
{
	return FTimingProfilerManager::Get()->GetProfilerWindow();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<STimingView> STimersView::GetTimingView() const
{
	TSharedPtr<STimingProfilerWindow> ProfilerWindow = GetProfilerWindow();
	return ProfilerWindow.IsValid() ? ProfilerWindow->GetTimingView() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimersView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return CommandList->ProcessCommandBindings(InKeyEvent) == true ? FReply::Handled() : FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STimersView::SetSourceFileSearchPaths_GetTooltip() const
{
	const FString& CurrentPath = FInsightsManager::Get()->GetSourceFilePathHelper().GetLocalRootDirectoryPath();
	return FText::Format(LOCTEXT("CurrentPathFmt", "Current path: {0}"), CurrentPath.Len() > 0 ? FText::FromString(CurrentPath) : LOCTEXT("None", "None"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::SetSourceFileSearchPaths_Execute()
{
	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		bool bHasSelected = false;
		FString SelectedPath;
		const FString& CurrentPath = FInsightsManager::Get()->GetSourceFilePathHelper().GetLocalRootDirectoryPath();
		const FString Title = LOCTEXT("SetSourceFileSeachPaths", "Select repository root folder...").ToString();

		bHasSelected = DesktopPlatform->OpenDirectoryDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			Title,
			CurrentPath,
			SelectedPath
		);

		if (bHasSelected)
		{
			FInsightsManager::Get()->GetSourceFilePathHelper().SetUserDefinedPath(SelectedPath);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersView::ToggleShowFormatInTimerNames_Execute()
{
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
	Settings.SetAndSaveShowFormatInTimerNames(!Settings.GetShowFormatInTimerNames());

	RebuildTree(true);

	FTimingProfilerManager::Get()->UpdateCallersAndCallees();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
