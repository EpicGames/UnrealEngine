// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Misc/FilterCollection.h"
#include "Misc/TextFilter.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/TimerAggregator.h"
#include "Insights/TimingProfiler/ViewModels/TimerGroupingAndSorting.h"
#include "Insights/TimingProfiler/ViewModels/TimerNode.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMenuBuilder;
class FUICommandList;

namespace TraceServices
{
	class IAnalysisSession;
}

namespace UE::Insights
{
	class FTable;
	class FTableColumn;
	class ITableCellValueSorter;
	class SAsyncOperationStatus;
}

namespace UE::Insights::TimingProfiler
{

enum class ESelectEventType : uint32;
class SFrameTrack;
class FTimerAggregator;
class FTimingGraphTrack;
class STimingProfilerWindow;
class STimingView;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** The filter collection - used for updating the list of timer nodes. */
typedef TFilterCollection<const FTimerNodePtr&> FTimerNodeFilterCollection;

/** The text based filter - used for updating the list of timer nodes. */
typedef TTextFilter<const FTimerNodePtr&> FTimerNodeTextFilter;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * A custom widget used to display the list of timers.
 */
class STimersView : public SCompoundWidget
{
public:
	/** Default constructor. */
	STimersView();

	/** Virtual destructor. */
	virtual ~STimersView();

	SLATE_BEGIN_ARGS(STimersView) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs);

	TSharedPtr<FTable> GetTable() const { return Table; }

	void Reset();

	/**
	 * Rebuilds the tree (if necessary).
	 * @param bResync - If true, it forces a resync with list of timers from Analysis, even if the list did not changed since last sync.
	 */
	void RebuildTree(bool bResync);

	void ResetStats();
	void UpdateStats(double StartTime, double EndTime);

	FTimerNodePtr GetTimerNode(uint32 TimerId) const;
	void SelectTimerNode(uint32 TimerId);

	ETimerAggregationMode GetAggregationMode() { return AggregationMode; }

	void OnTimingViewTrackListChanged();

	void ToggleTimingViewMainGraphEventSeries(FTimerNodePtr TimerNode) const;

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	void InitCommandList();

	void UpdateTree();

	void FinishAggregation();
	void ApplyAggregation(TraceServices::ITable<TraceServices::FTimingProfilerAggregatedStats>* AggregatedStatsTable);

	/** Called when the session has changed. */
	void InsightsManager_OnSessionChanged();

	/** Called when the analysis was completed. */
	void InsightsManager_OnSessionAnalysisCompleted();

	/**
	 * Populates OutSearchStrings with the strings that should be used in searching.
	 *
	 * @param GroupOrStatNodePtr - the group and stat node to get a text description from.
	 * @param OutSearchStrings   - an array of strings to use in searching.
	 *
	 */
	void HandleItemToStringArray(const FTimerNodePtr& GroupOrStatNodePtr, TArray<FString>& OutSearchStrings) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Context Menu

	TSharedPtr<SWidget> TreeView_GetMenuContent();
	void TreeView_BuildSortByMenu(FMenuBuilder& MenuBuilder);
	void TreeView_BuildViewColumnMenu(FMenuBuilder& MenuBuilder);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Copy to Clipboard

	bool ContextMenu_CopyToClipboard_CanExecute() const;
	void ContextMenu_CopyToClipboard_Execute();

	bool ContextMenu_CopyTimerNameToClipboard_CanExecute() const;
	void ContextMenu_CopyTimerNameToClipboard_Execute();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Export

	void TreeView_BuildExportMenu(FMenuBuilder& MenuBuilder);

	bool ContextMenu_Export_CanExecute() const;
	void ContextMenu_Export_Execute();

	void AddTimerNodeRecursive(FTimerNodePtr InNode, TSet<uint32>& InOutIncludedTimers) const;

	bool ContextMenu_ExportTimingEventsSelection_CanExecute() const;
	void ContextMenu_ExportTimingEventsSelection_Execute() const;

	bool ContextMenu_ExportTimingEvents_CanExecute() const;
	void ContextMenu_ExportTimingEvents_Execute() const;

	bool ContextMenu_ExportThreads_CanExecute() const;
	void ContextMenu_ExportThreads_Execute() const;

	bool ContextMenu_ExportTimers_CanExecute() const;
	void ContextMenu_ExportTimers_Execute() const;

	bool OpenSaveTextFileDialog(const FString& InDialogTitle, const FString& InDefaultFile, FString& OutFilename) const;
	class IFileHandle* OpenExportFile(const TCHAR* InFilename) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Open Source File in IDE

	bool ContextMenu_OpenSource_CanExecute() const;
	void ContextMenu_OpenSource_Execute() const;

	void OpenSourceFileInIDE(FTimerNodePtr InNode) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Generate and Edit Timer Colors

	bool ContextMenu_GenerateNewColor_CanExecute() const;
	void ContextMenu_GenerateNewColor_Execute() const;

	bool ContextMenu_EditColor_CanExecute() const;
	void ContextMenu_EditColor_Execute();

	void SetColorToNode(FTimerNode& TimerNode, FLinearColor CustomColor) const;
	void SetRandomColorToNode(FTimerNode& TimerNode) const;

	FLinearColor GetEditableColor() const;
	void SetEditableColor(FLinearColor NewColor);
	void ColorPickerCancelled(FLinearColor OriginalColor);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Plot (Graph Series)

	void TreeView_BuildPlotTimerMenu(FMenuBuilder& MenuBuilder);

	//////////////////////////////////////////////////
	// Find Instance

	void TreeView_FindMenu(FMenuBuilder& MenuBuilder);

	bool ContextMenu_FindInstance_CanExecute() const;
	void ContextMenu_FindInstance_Execute(ESelectEventType EventType) const;

	bool ContextMenu_FindInstanceInSelection_CanExecute() const;
	void ContextMenu_FindInstanceInSelection_Execute(ESelectEventType EventType) const;

	//////////////////////////////////////////////////
	// Settings

	void TreeView_BuildSettingsMenu(FMenuBuilder& MenuBuilder);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Columns' Header

	void InitializeAndShowHeaderColumns();

	FText GetColumnHeaderText(const FName ColumnId) const;

	TSharedRef<SWidget> TreeViewHeaderRow_GenerateColumnMenu(const FTableColumn& Column);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Misc

	void TreeView_Refresh();

	/**
	 * Called by STreeView to retrieves the children for the specified parent item.
	 * @param InParent    - The parent node to retrieve the children from.
	 * @param OutChildren - List of children for the parent node.
	 */
	void TreeView_OnGetChildren(FTimerNodePtr InParent, TArray<FTimerNodePtr>& OutChildren);

	/** Called by STreeView when selection has changed. */
	void TreeView_OnSelectionChanged(FTimerNodePtr SelectedItem, ESelectInfo::Type SelectInfo);

	FTimerNodePtr GetSingleSelectedTimerNode() const;

	/** Called by STreeView when a tree item is double clicked. */
	void TreeView_OnMouseButtonDoubleClick(FTimerNodePtr TreeNode);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Tree View - Table Row

	/** Called by STreeView to generate a table row for the specified item. */
	TSharedRef<ITableRow> TreeView_OnGenerateRow(FTimerNodePtr TreeNode, const TSharedRef<STableViewBase>& OwnerTable);

	bool TableRow_ShouldBeEnabled(FTimerNodePtr NodePtr) const;

	void TableRow_SetHoveredCell(TSharedPtr<FTable> TablePtr, TSharedPtr<FTableColumn> ColumnPtr, FTimerNodePtr NodePtr);
	EHorizontalAlignment TableRow_GetColumnOutlineHAlignment(const FName ColumnId) const;

	FText TableRow_GetHighlightText() const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Filtering

	/** Populates the group and stat tree with items based on the current data. */
	void ApplyFiltering();

	void FilterOutZeroCountTimers_OnCheckStateChanged(ECheckBoxState NewRadioState);
	ECheckBoxState FilterOutZeroCountTimers_IsChecked() const;

	TSharedRef<SWidget> GetToggleButtonForTimerType(const ETimerNodeType InNodeType);
	void FilterByTimerType_OnCheckStateChanged(ECheckBoxState NewRadioState, const ETimerNodeType InNodeType);
	ECheckBoxState FilterByTimerType_IsChecked(const ETimerNodeType InNodeType) const;

	bool SearchBox_IsEnabled() const;
	void SearchBox_OnTextChanged(const FText& InFilterText);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Aggregation Performance

	double GetMinimumThreadRatio() const;
	FText GetThreadRatio() const;
	void OnThreadRatioCommited(double InNewValue);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Grouping

	void CreateGroups();
	void CreateGroupByOptionsSources();

	void GroupBy_OnSelectionChanged(TSharedPtr<ETimerGroupingMode> NewGroupingMode, ESelectInfo::Type SelectInfo);

	TSharedRef<SWidget> GroupBy_OnGenerateWidget(TSharedPtr<ETimerGroupingMode> InGroupingMode) const;

	FText GroupBy_GetSelectedText() const;

	FText GroupBy_GetSelectedTooltipText() const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Aggregation Mode

	void CreateModeOptionsSources();
	void Mode_OnSelectionChanged(TSharedPtr<ETimerAggregationMode> InAggregationMode, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> Mode_OnGenerateWidget(TSharedPtr<ETimerAggregationMode> InAggregationMode) const;
	FText Mode_GetSelectedText() const;
	FText Mode_GetText(ETimerAggregationMode InAggregationMode) const;
	FText Mode_GetSelectedTooltipText() const;
	FText Mode_GetTooltipText(ETimerAggregationMode InAggregationMode) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Sorting

	static const FName GetDefaultColumnBeingSorted();
	static const EColumnSortMode::Type GetDefaultColumnSortMode();

	void CreateSortings();

	void UpdateCurrentSortingByColumn();
	void SortTreeNodes();
	void SortTreeNodesRec(FTimerNode& Node, const ITableCellValueSorter& Sorter);

	EColumnSortMode::Type GetSortModeForColumn(const FName ColumnId) const;
	void SetSortModeForColumn(const FName& ColumnId, EColumnSortMode::Type SortMode);
	void OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Sorting Actions

	// SortMode (HeaderMenu)
	bool HeaderMenu_SortMode_IsChecked(const FName ColumnId, const EColumnSortMode::Type InSortMode);
	bool HeaderMenu_SortMode_CanExecute(const FName ColumnId, const EColumnSortMode::Type InSortMode) const;
	void HeaderMenu_SortMode_Execute(const FName ColumnId, const EColumnSortMode::Type InSortMode);

	// SortMode (ContextMenu)
	bool ContextMenu_SortMode_IsChecked(const EColumnSortMode::Type InSortMode);
	bool ContextMenu_SortMode_CanExecute(const EColumnSortMode::Type InSortMode) const;
	void ContextMenu_SortMode_Execute(const EColumnSortMode::Type InSortMode);

	// SortByColumn (ContextMenu)
	bool ContextMenu_SortByColumn_IsChecked(const FName ColumnId);
	bool ContextMenu_SortByColumn_CanExecute(const FName ColumnId) const;
	void ContextMenu_SortByColumn_Execute(const FName ColumnId);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Column Visibility Actions

	// ShowColumn
	bool CanShowColumn(const FName ColumnId) const;
	void ShowColumn(const FName ColumnId);

	// HideColumn
	bool CanHideColumn(const FName ColumnId) const;
	void HideColumn(const FName ColumnId);

	// ToggleColumnVisibility
	bool IsColumnVisible(const FName ColumnId) const;
	bool CanToggleColumnVisibility(const FName ColumnId) const;
	void ToggleColumnVisibility(const FName ColumnId);

	// ShowAllColumns (ContextMenu)
	bool ContextMenu_ShowAllColumns_CanExecute() const;
	void ContextMenu_ShowAllColumns_Execute();

	// MinMaxMedColumns (ContextMenu)
	bool ContextMenu_ShowMinMaxMedColumns_CanExecute() const;
	void ContextMenu_ShowMinMaxMedColumns_Execute();

	// ResetColumns (ContextMenu)
	bool ContextMenu_ResetColumns_CanExecute() const;
	void ContextMenu_ResetColumns_Execute();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Event Highlight/Filtering in Timing View

	void ToggleTimingViewEventFilter(FTimerNodePtr TimerNode) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Plot (Graph Series)

	TSharedPtr<FTimingGraphTrack> GetTimingViewMainGraphTrack() const;
	TSharedPtr<SFrameTrack> GetFrameTrack() const;

	void ToggleGraphInstanceSeries(TSharedRef<FTimingGraphTrack> GraphTrack, FTimerNodeRef NodePtr) const;
	bool IsInstanceSeriesInTimingViewMainGraph(FTimerNodePtr TimerNode) const;
	void ToggleTimingViewMainGraphEventInstanceSeries(FTimerNodePtr TimerNode) const;

	void ToggleGraphFrameStatsSeries(TSharedRef<FTimingGraphTrack> GraphTrack, FTimerNodeRef NodePtr, ETraceFrameType FrameType) const;
	bool IsFrameStatsSeriesInTimingViewMainGraph(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const;
	void ToggleTimingViewMainGraphEventFrameStatsSeries(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const;

	bool IsSeriesInFrameTrack(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const;
	void ToggleFrameTrackSeries(FTimerNodePtr TimerNode, ETraceFrameType FrameType) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void SaveSettings();
	void SaveVisibleColumnsSettings();
	void LoadSettings();
	void LoadVisibleColumnsSettings();

	void SetTimingViewFrameType();

	FText SetSourceFileSearchPaths_GetTooltip() const;
	void SetSourceFileSearchPaths_Execute();

	void ToggleShowFormatInTimerNames_Execute();

	TSharedPtr<STimingProfilerWindow> GetProfilerWindow() const;
	TSharedPtr<STimingView> GetTimingView() const;

private:
	/** Table view model. */
	TSharedPtr<FTable> Table;

	/** The analysis session used to populate this widget. */
	TSharedPtr<const TraceServices::IAnalysisSession> Session;

	TSharedPtr<FUICommandList> CommandList;

	/** Eye-Dropper selected color for graphing.*/
	FLinearColor EditableColorValue;

	//////////////////////////////////////////////////
	// Tree View, Columns

	/** The tree widget which holds the list of groups and timers corresponding with each group. */
	TSharedPtr<STreeView<FTimerNodePtr>> TreeView;

	/** Holds the tree view header row widget which display all columns in the tree view. */
	TSharedPtr<SHeaderRow> TreeViewHeaderRow;

	/** External scrollbar used to synchronize tree view position. */
	TSharedPtr<SScrollBar> ExternalScrollbar;

	//////////////////////////////////////////////////
	// Hovered Column, Hovered Timer Node

	/** Name of the column currently being hovered by the mouse. */
	FName HoveredColumnId;

	/** A shared pointer to the timer node currently being hovered by the mouse. */
	FTimerNodePtr HoveredNodePtr;

	//////////////////////////////////////////////////
	// Timer Nodes

	/** An array of group and timer nodes generated from the metadata. */
	TArray<FTimerNodePtr> GroupNodes;

	/** A filtered array of group and timer nodes to be displayed in the tree widget. */
	TArray<FTimerNodePtr> FilteredGroupNodes;

	/** All timer nodes. An index in this array is a TimerId. */
	TArray<FTimerNodePtr> TimerNodes;

	/** Currently expanded group nodes. */
	TSet<FTimerNodePtr> ExpandedNodes;

	/** If true, the expanded nodes have been saved before applying a text filter. */
	bool bExpansionSaved = false;

	//////////////////////////////////////////////////
	// Search box and filters

	//bool bUseFiltering;

	/** The search box widget used to filter items displayed in the tree. */
	TSharedPtr<SSearchBox> SearchBox;

	/** The text based filter. */
	TSharedPtr<FTimerNodeTextFilter> TextFilter;

	/** The filter collection. */
	TSharedPtr<FTimerNodeFilterCollection> Filters;

	/** Holds the enabled state of each timer type. */
	bool bTimerTypeIsEnabled[static_cast<int>(ETimerNodeType::InvalidOrMax)];

	/** Holds the visibility of each timer type. */
	bool bTimerTypeIsVisible[static_cast<int>(ETimerNodeType::InvalidOrMax)];

	/** Filter out the timers having zero total instance count (aggregated stats). */
	bool bFilterOutZeroCountTimers = false;

	//////////////////////////////////////////////////
	// Grouping

	//bool bUseGrouping;

	TArray<TSharedPtr<ETimerGroupingMode>> GroupByOptionsSource;

	TSharedPtr<SComboBox<TSharedPtr<ETimerGroupingMode>>> GroupByComboBox;

	/** How we group the timers? */
	ETimerGroupingMode GroupingMode = ETimerGroupingMode::ByType;

	//////////////////////////////////////////////////
	// Aggregation Mode

	TArray<TSharedPtr<ETimerAggregationMode>> ModeOptionsSource;

	TSharedPtr<SComboBox<TSharedPtr<ETimerAggregationMode>>> ModeComboBox;

	ETimerAggregationMode AggregationMode = ETimerAggregationMode::Instance;

	//////////////////////////////////////////////////
	// Sorting

	/** All available sorters. */
	TArray<TSharedPtr<ITableCellValueSorter>> AvailableSorters;

	/** Current sorter. It is nullptr if sorting is disabled. */
	TSharedPtr<ITableCellValueSorter> CurrentSorter;

	/** Name of the column currently being sorted. Can be NAME_None if sorting is disabled (CurrentSorting == nullptr) or if a complex sorting is used (CurrentSorting != nullptr). */
	FName ColumnBeingSorted;

	/** How we sort the nodes? Ascending or Descending. */
	EColumnSortMode::Type ColumnSortMode = EColumnSortMode::Type::Ascending;

	//////////////////////////////////////////////////

	TSharedRef<FTimerAggregator> Aggregator;
	TSharedPtr<SAsyncOperationStatus> AsyncOperationStatus;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
