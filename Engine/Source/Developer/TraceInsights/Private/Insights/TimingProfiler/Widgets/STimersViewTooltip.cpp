// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimersViewTooltip.h"

#include "SlateOptMacros.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

// TraceServices
#include "TraceServices/Model/AnalysisSession.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/ViewModels/TimerNode.h"
#include "Insights/TimingProfiler/ViewModels/TimerNodeHelper.h"
#include "Insights/TimingProfiler/ViewModels/TimersViewColumnFactory.h"
#include "Insights/TimingProfiler/Widgets/STimersView.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"

#include <cmath>

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::STimersView"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedPtr<SToolTip> STimersViewTooltip::GetTableTooltip(const FTable& Table)
{
	TSharedPtr<SToolTip> ColumnTooltip =
		SNew(SToolTip)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Table.GetDisplayName())
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Table.GetDescription())
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			]
		];

	return ColumnTooltip;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SToolTip> STimersViewTooltip::GetColumnTooltip(const FTableColumn& Column)
{
	const FTimersTableColumn& TimersColumn = static_cast<const FTimersTableColumn&>(Column);
	FText InstanceDescription = TimersColumn.GetDescription(ETimerAggregationMode::Instance);
	FText GameFrameDescription = TimersColumn.GetDescription(ETimerAggregationMode::GameFrame);
	FText RenderingDescription = TimersColumn.GetDescription(ETimerAggregationMode::RenderingFrame);

	auto GetDescriptionLambda = [InstanceDescription, GameFrameDescription, RenderingDescription]()
	{
		ETimerAggregationMode AggregationMode = ETimerAggregationMode::Instance;

		TSharedPtr<STimingProfilerWindow> Window = FTimingProfilerManager::Get()->GetProfilerWindow();
		if (Window.IsValid())
		{
			TSharedPtr<STimersView> TimersView = Window->GetTimersView();
			if (TimersView.IsValid())
			{
				AggregationMode = TimersView->GetAggregationMode();
			}
		}

		switch (AggregationMode)
		{
		case ETimerAggregationMode::Instance:
			return InstanceDescription;
		case ETimerAggregationMode::GameFrame:
			return GameFrameDescription;
		case ETimerAggregationMode::RenderingFrame:
			return RenderingDescription;
		default:
			ensure(0);
		}

		return InstanceDescription;
	};

	TSharedPtr<SToolTip> ColumnTooltip =
		SNew(SToolTip)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Column.GetTitleName())
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(TAttribute<FText>::CreateLambda(GetDescriptionLambda))
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			]
		];

	return ColumnTooltip;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SToolTip> STimersViewTooltip::GetColumnTooltipForMode(const FTableColumn& Column, ETimerAggregationMode InAggregationMode)
{
	const FTimersTableColumn& TimersColumn = static_cast<const FTimersTableColumn&>(Column);
	FText Description = TimersColumn.GetDescription(InAggregationMode);

	TSharedPtr<SToolTip> ColumnTooltip =
		SNew(SToolTip)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Column.GetTitleName())
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(Description)
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
			]
		];

	return ColumnTooltip;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SToolTip> STimersViewTooltip::GetRowTooltip(const TSharedPtr<FTimerNode> TimerNodePtr)
{
	const TraceServices::FTimingProfilerAggregatedStats& Stats = TimerNodePtr->GetAggregatedStats();

	const FText InstanceCountText = FText::AsNumber(Stats.InstanceCount);

	TSharedPtr<SGridPanel> GridPanel;
	TSharedPtr<SHorizontalBox> HBox;

	FText SourcePrefix;
	FText SourceSuffix;
	GetSource(TimerNodePtr, SourcePrefix, SourceSuffix);

	TSharedPtr<SVerticalBox> SourceWidget = SNew(SVerticalBox);
	if (!SourcePrefix.IsEmptyOrWhitespace())
	{
		SourceWidget->AddSlot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(SourcePrefix)
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			];
	}
	if (!SourceSuffix.IsEmptyOrWhitespace())
	{
		SourceWidget->AddSlot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(SourceSuffix)
				.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			];
	}

	TSharedPtr<SToolTip> TableCellTooltip =
		SNew(SToolTip)
		[
			SAssignNew(HBox, SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SGridPanel)

					// Id: [Id]
					+ SGridPanel::Slot(0, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_Id", "Id:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(1, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(FText::AsNumber(TimerNodePtr->GetTimerId()))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					]

					// Name: [Name]
					+ SGridPanel::Slot(0, 1)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_Name", "Name:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(1, 1)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.WrapTextAt(1024.0f)
						.WrappingPolicy(ETextWrappingPolicy::AllowPerCharacterWrapping)
						.Text(TimerNodePtr->GetDisplayName())
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					]

					// Item Type: [Type]
					+ SGridPanel::Slot(0, 2)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_Type", "Item Type:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(1, 2)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(TimerNodeTypeHelper::ToText(TimerNodePtr->GetType()))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					]

					// Source: [Source]
					+ SGridPanel::Slot(0, 3)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_Source", "Source:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(1, 3)
					.Padding(2.0f)
					[
						SourceWidget.ToSharedRef()
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SGridPanel)

					+ SGridPanel::Slot(0, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_NumInstances", "Num Instances:"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(1, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(InstanceCountText)
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SAssignNew(GridPanel, SGridPanel)

					+ SGridPanel::Slot(1, 0)
					.Padding(2.0f)
					.HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_InclusiveTime", "Inclusive"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]
					+ SGridPanel::Slot(2, 0)
					.Padding(FMargin(8.0f, 2.0f, 2.0f, 2.0f))
					.HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TT_ExclusiveTime", "Exclusive"))
						.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
					]

					// Stats are added here.
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]
			]
		];

	const int32 NumDigits = 5;
	int32 Row = 1;

	if (true)
	{
		static constexpr TCHAR FormatString[] = TEXT("%.*fs (%s)");
		const FText TotalInclusiveTimeText = FText::FromString(FString::Printf(FormatString, NumDigits, Stats.TotalInclusiveTime, *FormatTimeAuto(Stats.TotalInclusiveTime, 2)));
		const FText TotalExclusiveTimeText = FText::FromString(FString::Printf(FormatString, NumDigits, Stats.TotalExclusiveTime, *FormatTimeAuto(Stats.TotalExclusiveTime, 2)));
		AddStatsRow(GridPanel, Row, LOCTEXT("TT_TotalTime", "Total Time:"), TotalInclusiveTimeText, TotalExclusiveTimeText);
	}
	if (!std::isnan(Stats.MaxInclusiveTime) || !std::isnan(Stats.MaxExclusiveTime))
	{
		const FText MaxInclusiveTimeText = FText::FromString(FormatTimeMs(Stats.MaxInclusiveTime, NumDigits, true));
		const FText MaxExclusiveTimeText = FText::FromString(FormatTimeMs(Stats.MaxExclusiveTime, NumDigits, true));
		AddStatsRow(GridPanel, Row, LOCTEXT("TT_MaxTime", "Max Time:"), MaxInclusiveTimeText, MaxExclusiveTimeText);
	}
	if (true)
	{
		const FText AvgInclusiveTimeText = FText::FromString(FormatTimeMs(Stats.AverageInclusiveTime, NumDigits, true));
		const FText AvgExclusiveTimeText = FText::FromString(FormatTimeMs(Stats.AverageExclusiveTime, NumDigits, true));
		AddStatsRow(GridPanel, Row, LOCTEXT("TT_AverageTime", "Average Time:"), AvgInclusiveTimeText, AvgExclusiveTimeText);
	}
	if (!std::isnan(Stats.MedianInclusiveTime) || !std::isnan(Stats.MedianExclusiveTime))
	{
		const FText MedInclusiveTimeText = FText::FromString(FormatTimeMs(Stats.MedianInclusiveTime, NumDigits, true));
		const FText MedExclusiveTimeText = FText::FromString(FormatTimeMs(Stats.MedianExclusiveTime, NumDigits, true));
		AddStatsRow(GridPanel, Row, LOCTEXT("TT_MedianTime", "Median Time:"), MedInclusiveTimeText, MedExclusiveTimeText);
	}
	if (!std::isnan(Stats.MinInclusiveTime) || !std::isnan(Stats.MinExclusiveTime))
	{
		const FText MinInclusiveTimeText = FText::FromString(FormatTimeMs(Stats.MinInclusiveTime, NumDigits, true));
		const FText MinExclusiveTimeText = FText::FromString(FormatTimeMs(Stats.MinExclusiveTime, NumDigits, true));
		AddStatsRow(GridPanel, Row, LOCTEXT("TT_MinTime", "Min Time:"), MinInclusiveTimeText, MinExclusiveTimeText);
	}

	return TableCellTooltip;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimersViewTooltip::AddStatsRow(TSharedPtr<SGridPanel> Grid, int32& Row, const FText& Name, const FText& Value1, const FText& Value2)
{
	Grid->AddSlot(0, Row)
		.Padding(2.0f)
		[
			SNew(STextBlock)
			.Text(Name)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.TooltipBold"))
		];

	Grid->AddSlot(1, Row)
		.Padding(2.0f)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(Value1)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
		];

	Grid->AddSlot(2, Row)
		.Padding(FMargin(8.0f, 2.0f, 2.0f, 2.0f))
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(Value2)
			.TextStyle(FInsightsStyle::Get(), TEXT("TreeTable.Tooltip"))
		];

	Row++;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimersViewTooltip::GetSource(const TSharedPtr<FTimerNode> TreeNodePtr, FText& OutSourcePrefix, FText& OutSourceSuffix)
{
	bool bIsSourceFileValid = false;
	if (TreeNodePtr.IsValid())
	{
		FString File;
		uint32 Line;
		bIsSourceFileValid = TreeNodePtr->GetSourceFileAndLine(File, Line);
		if (bIsSourceFileValid)
		{
			int32 Index = -1;
			if (!File.FindLastChar('\\', Index))
			{
				File.FindLastChar('/', Index);
			}
			++Index;
			OutSourcePrefix = FText::FromString(*File.Left(Index));
			OutSourceSuffix = FText::FromString(*FString::Printf(TEXT("%s (%u)"), *File.RightChop(Index), Line));
		}
	}
	if (!bIsSourceFileValid)
	{
		OutSourcePrefix = LOCTEXT("Source_NA", "N/A");
		OutSourceSuffix = FText::GetEmpty();
	}
	return bIsSourceFileValid;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
