// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Widgets/IToolTip.h"
#include "Widgets/SToolTip.h"

// TraceInsights
#include "Insights/TimingProfiler/ViewModels/TimerAggregator.h"

class SGridPanel;

namespace UE::Insights
{
	class FTable;
	class FTableColumn;
}

namespace UE::Insights::TimingProfiler
{

class FTimerNode;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Timers View Tooltip */
class STimersViewTooltip
{
public:
	STimersViewTooltip() = delete;

	static TSharedPtr<SToolTip> GetTableTooltip(const FTable& Table);
	static TSharedPtr<SToolTip> GetColumnTooltip(const FTableColumn& Column);
	static TSharedPtr<SToolTip> GetColumnTooltipForMode(const FTableColumn& Column, ETimerAggregationMode InAggregationMode);
	static TSharedPtr<SToolTip> GetRowTooltip(const TSharedPtr<FTimerNode> TreeNodePtr);

	static bool GetSource(const TSharedPtr<FTimerNode> TreeNodePtr, FText& OutSourcePrefix, FText& OutSourceSuffix);

private:
	static void AddStatsRow(TSharedPtr<SGridPanel> Grid, int32& Row, const FText& Name, const FText& Value1, const FText& Value2);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class STimerTableRowToolTip : public IToolTip
{
public:
	STimerTableRowToolTip(const TSharedPtr<FTimerNode> InTreeNodePtr) : TreeNodePtr(InTreeNodePtr) {}
	virtual ~STimerTableRowToolTip() = default;

	virtual TSharedRef<class SWidget> AsWidget()
	{
		CreateToolTipWidget();
		return ToolTipWidget.ToSharedRef();
	}

	virtual TSharedRef<SWidget> GetContentWidget()
	{
		CreateToolTipWidget();
		return ToolTipWidget->GetContentWidget();
	}

	virtual void SetContentWidget(const TSharedRef<SWidget>& InContentWidget)
	{
		CreateToolTipWidget();
		ToolTipWidget->SetContentWidget(InContentWidget);
	}

	void InvalidateWidget()
	{
		ToolTipWidget.Reset();
	}

	virtual bool IsEmpty() const { return false; }
	virtual bool IsInteractive() const { return false; }
	virtual void OnOpening() {}
	virtual void OnClosed() {}

private:
	void CreateToolTipWidget()
	{
		if (!ToolTipWidget.IsValid())
		{
			ToolTipWidget = STimersViewTooltip::GetRowTooltip(TreeNodePtr);
		}
	}

private:
	TSharedPtr<SToolTip> ToolTipWidget;
	const TSharedPtr<FTimerNode> TreeNodePtr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
