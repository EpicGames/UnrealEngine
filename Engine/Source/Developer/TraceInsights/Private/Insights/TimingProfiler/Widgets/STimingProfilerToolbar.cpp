// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimingProfilerToolbar.h"

#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "ToolMenus.h"
#include "ToolMenuDelegates.h"
#include "Widgets/SBoxPanel.h"

// TraceInsights
#include "Insights/InsightsCommands.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/TimingProfiler/TimingProfilerCommands.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

STimingProfilerToolbar::STimingProfilerToolbar()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

STimingProfilerToolbar::~STimingProfilerToolbar()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerToolbar::Construct(const FArguments& InArgs, const FInsightsMajorTabConfig& Config)
{
	struct Local
	{
		static void MakeTabSpawnerMenu(FToolMenuSection& Section, const FInsightsMajorTabConfig& Config)
		{
			if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::FramesTrackID))
			{
				Section.AddMenuEntry(FTimingProfilerCommands::Get().ToggleFramesTrackVisibility, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.FramesTrack.ToolBar"));
			}
			if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::TimingViewID))
			{
				Section.AddMenuEntry(FTimingProfilerCommands::Get().ToggleTimingViewVisibility, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TimingView.ToolBar"));
			}
			if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::TimersID))
			{
				Section.AddMenuEntry(FTimingProfilerCommands::Get().ToggleTimersViewVisibility, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TimersView.ToolBar"));
			}
			if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::CallersID))
			{
				Section.AddMenuEntry(FTimingProfilerCommands::Get().ToggleCallersTreeViewVisibility, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.CallersView.ToolBar"));
			}
			if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::CalleesID))
			{
				Section.AddMenuEntry(FTimingProfilerCommands::Get().ToggleCalleesTreeViewVisibility, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.CalleesView.ToolBar"));
			}
			if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::StatsCountersID))
			{
				Section.AddMenuEntry(FTimingProfilerCommands::Get().ToggleStatsCountersViewVisibility, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.CountersView.ToolBar"));
			}
			if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::LogViewID))
			{
				Section.AddMenuEntry(FTimingProfilerCommands::Get().ToggleLogViewVisibility, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.LogView.ToolBar"));
			}
			if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::ModulesViewID))
			{
				Section.AddMenuEntry(FTimingProfilerCommands::Get().ToggleModulesViewVisibility, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ModulesView.ToolBar"));
			}
			if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::UserAnnotationsID))
			{
				Section.AddMenuEntry(FTimingProfilerCommands::Get().ToggleUserAnnotationsViewVisibility, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.Annotation.ToolBar"));
			}
		}

		static void FillViewToolbar(UToolMenu* MainToolBar, const FArguments& InArgs, const FInsightsMajorTabConfig& Config)
		{
			if (Config.bUseCompactTabSpawners)
			{
				const FName MenuName = FName("TimingProfiler.MainToolBar.PanelSelectorMenu");
				const UToolMenus* ToolMenus = UToolMenus::Get();

				if (ToolMenus->IsMenuRegistered(MenuName))
				{
					return;
				}

				UToolMenu* PanelsMenu = UToolMenus::Get()->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);
				if (PanelsMenu == nullptr)
				{
					return;
				}
				FToolMenuSection& Section = PanelsMenu->AddSection("InsightsTabs");
				Local::MakeTabSpawnerMenu(Section, Config);

				FToolMenuSection& ToolbarSection = MainToolBar->AddSection(FName("Main"));
				ToolbarSection.AddEntry(FToolMenuEntry::InitComboButton(
					"PanelSelector",
					FUIAction(),
					FNewToolMenuChoice(FNewToolMenuWidget::CreateLambda([MenuName](const FToolMenuContext& Ctx) {
						return UToolMenus::Get()->GenerateWidget(MenuName, Ctx);
						})),
					LOCTEXT("PanelsText", "Panels"),
					LOCTEXT("PanelsToolTip", "Open/Close the available panels."),
					FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.PanelMenu.Toolbar"),
					false));

				FSlimHorizontalToolBarBuilder ToolbarBuilder(nullptr, FMultiBoxCustomization::None);
				ToolbarBuilder.SetStyle(&FInsightsStyle::Get(), "LegacyExtendersToolbarStyle");

				if (InArgs._ToolbarExtender.IsValid())
				{
					InArgs._ToolbarExtender->Apply("MainToolbar", EExtensionHook::First, ToolbarBuilder);
				}

				ToolbarSection.AddEntry(FToolMenuEntry::InitWidget(
					"LegacyExtenders",
					ToolbarBuilder.MakeWidget(),
					FText()));
			}
			else
			{
				FToolMenuSection& ToolbarSection = MainToolBar->AddSection(FName("Main"));
				Local::MakeTabSpawnerMenu(ToolbarSection, Config);

				FSlimHorizontalToolBarBuilder ToolbarBuilder(nullptr, FMultiBoxCustomization::None);
				ToolbarBuilder.SetStyle(&FInsightsStyle::Get(), "LegacyExtendersToolbarStyle");

				if (InArgs._ToolbarExtender.IsValid())
				{
					InArgs._ToolbarExtender->Apply("MainToolbar", EExtensionHook::First, ToolbarBuilder);
				}

				ToolbarSection.AddEntry(FToolMenuEntry::InitWidget(
					"LegacyExtenders",
					ToolbarBuilder.MakeWidget(),
					FText()));
			}
		}

		static void FillRightSideToolbar(UToolMenu* ToolBar, const FArguments& InArgs, const FInsightsMajorTabConfig& Config)
		{
			FToolMenuSection& Section = ToolBar->AddSection("Debug");

			Section.AddEntry(FToolMenuEntry::InitToolBarButton(FInsightsCommands::Get().ToggleDebugInfo,
				FText(), FText(), FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.Debug.ToolBar")));
		}
	};

	TSharedPtr<FUICommandList> CommandList = FInsightsManager::Get()->GetCommandList();

	const FName ToolBarName = FName("TimingProfiler.MainToolBar");
	const FName RightSideToolBarName = FName("TimingProfiler.RightToolBar");
	const UToolMenus* ToolMenus = UToolMenus::Get();
		
	if (!ToolMenus->IsMenuRegistered(ToolBarName))
	{
		FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
		UToolMenu* MainToolBar = UToolMenus::Get()->RegisterMenu(ToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		Local::FillViewToolbar(MainToolBar, InArgs, Config);

		UToolMenu* RightSideToolBar = UToolMenus::Get()->RegisterMenu(RightSideToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		Local::FillRightSideToolbar(RightSideToolBar, InArgs, Config);
	}

	FToolMenuContext MenuContext(CommandList, InArgs._ToolbarExtender);

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.FillWidth(1.0)
		.Padding(0.0f)
		[
			UToolMenus::Get()->GenerateWidget(ToolBarName, MenuContext)
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0.0f)
		[
			UToolMenus::Get()->GenerateWidget(RightSideToolBarName, MenuContext)
		]
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
