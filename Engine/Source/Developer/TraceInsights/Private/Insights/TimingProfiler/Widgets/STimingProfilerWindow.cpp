// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimingProfilerWindow.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Docking/SDockTab.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/TimingProfiler/GraphTracks/TimingGraphSeries.h"
#include "Insights/TimingProfiler/GraphTracks/TimingGraphTrack.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Widgets/SFrameTrack.h"
#include "Insights/TimingProfiler/Widgets/SStatsView.h"
#include "Insights/TimingProfiler/Widgets/STimersView.h"
#include "Insights/TimingProfiler/Widgets/STimerTreeView.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerToolbar.h"
#include "Insights/TimingProfiler/Widgets/SUserAnnotationsPanel.h"
#include "Insights/Widgets/SLogView.h"
#include "Insights/Widgets/SModulesView.h"
#include "Insights/Widgets/STimingView.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

const FName FTimingProfilerTabs::ToolbarID(TEXT("Toolbar")); // DEPRECATED
const FName FTimingProfilerTabs::FramesTrackID(TEXT("Frames"));
const FName FTimingProfilerTabs::TimingViewID(TEXT("TimingView"));
const FName FTimingProfilerTabs::TimersID(TEXT("Timers"));
const FName FTimingProfilerTabs::CallersID(TEXT("Callers"));
const FName FTimingProfilerTabs::CalleesID(TEXT("Callees"));
const FName FTimingProfilerTabs::StatsCountersID(TEXT("StasCounters"));
const FName FTimingProfilerTabs::LogViewID(TEXT("LogView"));
const FName FTimingProfilerTabs::ModulesViewID(TEXT("Modules"));
const FName FTimingProfilerTabs::UserAnnotationsID(TEXT("UserAnnotations"));

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// STimingProfilerWindow
////////////////////////////////////////////////////////////////////////////////////////////////////

STimingProfilerWindow::STimingProfilerWindow()
	: SMajorTabWindow(FInsightsManagerTabs::TimingProfilerTabId)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

STimingProfilerWindow::~STimingProfilerWindow()
{
	CloseAllOpenTabs();

	check(UserAnnotationsPanel == nullptr);
	check(ModulesView == nullptr);
	check(LogView == nullptr);
	check(StatsView == nullptr);
	check(CalleesTreeView == nullptr);
	check(CallersTreeView == nullptr);
	check(TimersView == nullptr);
	check(TimingView == nullptr);
	check(FrameTrack == nullptr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* STimingProfilerWindow::GetAnalyticsEventName() const
{
	return TEXT("Insights.Usage.TimingProfiler");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::Reset()
{
	if (FrameTrack)
	{
		FrameTrack->Reset();
	}

	if (TimingView)
	{
		TimingView->Reset();
	}

	if (TimersView)
	{
		TimersView->Reset();
	}

	if (CallersTreeView)
	{
		CallersTreeView->Reset();
	}

	if (CalleesTreeView)
	{
		CalleesTreeView->Reset();
	}

	if (StatsView)
	{
		StatsView->Reset();
	}

	if (LogView)
	{
		LogView->Reset();
	}

	if (ModulesView)
	{
		ModulesView->Reset();
	}

	// UserAnnotationsPanel does not need a Reset call; it auto-refreshes via Tick.

	TimerColors.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> STimingProfilerWindow::SpawnTab_FramesTrack(const FSpawnTabArgs& Args)
{
	FTimingProfilerManager::Get()->SetFramesTrackVisible(true);

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(FrameTrack, SFrameTrack)
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &STimingProfilerWindow::OnFramesTrackTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::OnFramesTrackTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FTimingProfilerManager::Get()->SetFramesTrackVisible(false);
	FrameTrack = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> STimingProfilerWindow::SpawnTab_TimingView(const FSpawnTabArgs& Args)
{
	FTimingProfilerManager::Get()->SetTimingViewVisible(true);

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(TimingView, STimingView, FInsightsManagerTabs::TimingProfilerTabId)
		];

	// Listen to TimingView keyboard shortcuts from anywhere in this window. The command list is kept internally as 
	// a weak reference, so when the TimingView is closed and destroyed, its command list will be ignored and cleaned
	// up on next mutation of this window's command list. Initial motivation to add this was for keyboard shortcuts 
	// for Go to Frame and Go to Time, which only make sense when TimingView is shown, but are nice to capture from any panel.
	GetCommandList()->Append(TimingView->GetCommandList().ToSharedRef());

	TimingView->Reset(true);
	TimingView->OnSelectionChanged().AddSP(this, &STimingProfilerWindow::OnTimeSelectionChanged);
	const double SelectionStartTime = FTimingProfilerManager::Get()->GetSelectionStartTime();
	const double SelectionEndTime = FTimingProfilerManager::Get()->GetSelectionEndTime();
	TimingView->SelectTimeInterval(SelectionStartTime, SelectionEndTime - SelectionStartTime);

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &STimingProfilerWindow::OnTimingViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::OnTimingViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FTimingProfilerManager::Get()->SetTimingViewVisible(false);
	if (TimingView)
	{
		TimingView->OnSelectionChanged().RemoveAll(this);
		TimingView = nullptr;
	}

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> STimingProfilerWindow::SpawnTab_Timers(const FSpawnTabArgs& Args)
{
	FTimingProfilerManager::Get()->SetTimersViewVisible(true);

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(TimersView, STimersView)
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &STimingProfilerWindow::OnTimersTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::OnTimersTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FTimingProfilerManager::Get()->SetTimersViewVisible(false);
	TimersView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> STimingProfilerWindow::SpawnTab_Callers(const FSpawnTabArgs& Args)
{
	FTimingProfilerManager::Get()->SetCallersTreeViewVisible(true);

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(CallersTreeView, STimerTreeView, LOCTEXT("CallersTreeViewName", "Callers"), STimerTreeView::ETreeType::Callers)
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &STimingProfilerWindow::OnCallersTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::OnCallersTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FTimingProfilerManager::Get()->SetCallersTreeViewVisible(false);
	CallersTreeView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> STimingProfilerWindow::SpawnTab_Callees(const FSpawnTabArgs& Args)
{
	FTimingProfilerManager::Get()->SetCalleesTreeViewVisible(true);

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(CalleesTreeView, STimerTreeView, LOCTEXT("CalleesTreeViewName", "Callees"), STimerTreeView::ETreeType::Callees)
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &STimingProfilerWindow::OnCalleesTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::OnCalleesTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FTimingProfilerManager::Get()->SetCalleesTreeViewVisible(false);
	CalleesTreeView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> STimingProfilerWindow::SpawnTab_StatsCounters(const FSpawnTabArgs& Args)
{
	FTimingProfilerManager::Get()->SetStatsCountersViewVisible(true);

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(StatsView, SStatsView)
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &STimingProfilerWindow::OnStatsCountersTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::OnStatsCountersTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FTimingProfilerManager::Get()->SetStatsCountersViewVisible(false);
	StatsView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> STimingProfilerWindow::SpawnTab_LogView(const FSpawnTabArgs& Args)
{
	FTimingProfilerManager::Get()->SetLogViewVisible(true);

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(LogView, SLogView)
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &STimingProfilerWindow::OnLogViewTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::OnLogViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FTimingProfilerManager::Get()->SetLogViewVisible(false);
	LogView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> STimingProfilerWindow::SpawnTab_ModulesView(const FSpawnTabArgs& Args)
{
	FTimingProfilerManager::Get()->SetModulesViewVisible(true);

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(ModulesView, SModulesView)
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &STimingProfilerWindow::OnModulesViewClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::OnModulesViewClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FTimingProfilerManager::Get()->SetModulesViewVisible(false);
	ModulesView = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SDockTab> STimingProfilerWindow::SpawnTab_UserAnnotations(const FSpawnTabArgs& Args)
{
	FTimingProfilerManager::Get()->SetUserAnnotationsViewVisible(true);

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(UserAnnotationsPanel, SUserAnnotationsPanel)
		.HostTimingView(TimingView)
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &STimingProfilerWindow::OnUserAnnotationsTabClosed));
	AddOpenTab(DockTab);

	return DockTab;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::OnUserAnnotationsTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FTimingProfilerManager::Get()->SetUserAnnotationsViewVisible(false);
	UserAnnotationsPanel = nullptr;

	RemoveOpenTab(TabBeingClosed);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	TSharedPtr<FTimingProfilerManager> TimingProfilerManager = FTimingProfilerManager::Get();
	ensure(TimingProfilerManager.IsValid());

	SetCommandList(TimingProfilerManager->GetCommandList());

	SMajorTabWindow::FArguments Args;
	SMajorTabWindow::Construct(Args, ConstructUnderMajorTab, ConstructUnderWindow);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FWorkspaceItem> STimingProfilerWindow::CreateWorkspaceMenuGroup()
{
	return GetTabManager()->AddLocalWorkspaceMenuCategory(LOCTEXT("TimingProfilerMenuGroupName", "Timing Insights"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::RegisterTabSpawners()
{
	check(GetTabManager().IsValid());
	FTabManager* TabManagerPtr = GetTabManager().Get();
	check(GetWorkspaceMenuGroup().IsValid());
	const TSharedRef<FWorkspaceItem> Group = GetWorkspaceMenuGroup().ToSharedRef();

	IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(FInsightsManagerTabs::TimingProfilerTabId);

	if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::FramesTrackID))
	{
		TabManagerPtr->RegisterTabSpawner(FTimingProfilerTabs::FramesTrackID, FOnSpawnTab::CreateRaw(this, &STimingProfilerWindow::SpawnTab_FramesTrack))
			.SetDisplayName(LOCTEXT("FramesTrackTabTitle", "Frames"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.FramesTrack"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::TimingViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FTimingProfilerTabs::TimingViewID, FOnSpawnTab::CreateRaw(this, &STimingProfilerWindow::SpawnTab_TimingView))
			.SetDisplayName(LOCTEXT("TimingViewTabTitle", "Timing View"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TimingView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::TimersID))
	{
		TabManagerPtr->RegisterTabSpawner(FTimingProfilerTabs::TimersID, FOnSpawnTab::CreateRaw(this, &STimingProfilerWindow::SpawnTab_Timers))
			.SetDisplayName(LOCTEXT("TimersTabTitle", "Timers"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TimersView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::CallersID))
	{
		TabManagerPtr->RegisterTabSpawner(FTimingProfilerTabs::CallersID, FOnSpawnTab::CreateRaw(this, &STimingProfilerWindow::SpawnTab_Callers))
			.SetDisplayName(LOCTEXT("CallersTabTitle", "Callers"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.CallersView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::CalleesID))
	{
		TabManagerPtr->RegisterTabSpawner(FTimingProfilerTabs::CalleesID, FOnSpawnTab::CreateRaw(this, &STimingProfilerWindow::SpawnTab_Callees))
			.SetDisplayName(LOCTEXT("CalleesTabTitle", "Callees"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.CalleesView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::StatsCountersID))
	{
		TabManagerPtr->RegisterTabSpawner(FTimingProfilerTabs::StatsCountersID, FOnSpawnTab::CreateRaw(this, &STimingProfilerWindow::SpawnTab_StatsCounters))
			.SetDisplayName(LOCTEXT("StatsCountersTabTitle", "Counters"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.CountersView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::LogViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FTimingProfilerTabs::LogViewID, FOnSpawnTab::CreateRaw(this, &STimingProfilerWindow::SpawnTab_LogView))
			.SetDisplayName(LOCTEXT("LogViewTabTitle", "Log View"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.LogView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::ModulesViewID))
	{
		TabManagerPtr->RegisterTabSpawner(FTimingProfilerTabs::ModulesViewID, FOnSpawnTab::CreateRaw(this, &STimingProfilerWindow::SpawnTab_ModulesView))
			.SetDisplayName(LOCTEXT("ModulesViewTabTitle", "Modules"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ModulesView"))
			.SetGroup(Group);
	}

	if (Config.ShouldRegisterMinorTab(FTimingProfilerTabs::UserAnnotationsID))
	{
		TabManagerPtr->RegisterTabSpawner(FTimingProfilerTabs::UserAnnotationsID, FOnSpawnTab::CreateRaw(this, &STimingProfilerWindow::SpawnTab_UserAnnotations))
			.SetDisplayName(LOCTEXT("UserAnnotationsTabTitle", "Annotations"))
			.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.Annotation"))
			.SetGroup(Group);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<FTabManager::FLayout> STimingProfilerWindow::CreateDefaultTabLayout() const
{
	return FTabManager::NewLayout("InsightsTimingProfilerLayout_v1.3")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.65f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.1f)
					->SetHideTabWell(true)
					->AddTab(FTimingProfilerTabs::FramesTrackID, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->SetHideTabWell(true)
					->AddTab(FTimingProfilerTabs::TimingViewID, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.2f)
					->SetHideTabWell(true)
					->AddTab(FTimingProfilerTabs::LogViewID, ETabState::OpenedTab)
					->AddTab(FTimingProfilerTabs::ModulesViewID, ETabState::OpenedTab)
				)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.35f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.67f)
					->AddTab(FTimingProfilerTabs::TimersID, ETabState::OpenedTab)
					->AddTab(FTimingProfilerTabs::StatsCountersID, ETabState::OpenedTab)
					->AddTab(FTimingProfilerTabs::UserAnnotationsID, ETabState::OpenedTab)
					->SetForegroundTab(FTimingProfilerTabs::TimersID)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.165f)
					->SetHideTabWell(true)
					->AddTab(FTimingProfilerTabs::CallersID, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.165f)
					->SetHideTabWell(true)
					->AddTab(FTimingProfilerTabs::CalleesID, ETabState::OpenedTab)
				)
			)
		);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimingProfilerWindow::CreateToolbar(TSharedPtr<FExtender> Extender)
{
	IUnrealInsightsModule& InsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	const FInsightsMajorTabConfig& Config = InsightsModule.FindMajorTabConfig(FInsightsManagerTabs::TimingProfilerTabId);

	return SNew(STimingProfilerToolbar, Config).ToolbarExtender(Extender);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::OnTimeSelectionChanged(Timing::ETimeChangedFlags InFlags, double InStartTime, double InEndTime)
{
	if (InFlags != Timing::ETimeChangedFlags::Interactive)
	{
		FTimingProfilerManager::Get()->SetSelectedTimeRange(InStartTime, InEndTime);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingProfilerWindow::GetTimerColor(uint32 TimerId, FLinearColor& OutColor)
{
	if (TimerColors.Contains(TimerId))
	{
		OutColor = *TimerColors.Find(TimerId);
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::SetTimerColor(uint32 TimerId, FLinearColor Color)
{
	TimerColors.Add(TimerId, Color);
	const FLinearColor BorderColor(Color.R + 0.4f, Color.G + 0.4f, Color.B + 0.4f, 1.0f);

	if (TimingView.IsValid())
	{
		TSharedPtr<FTimingGraphTrack> GraphTrack = TimingView->GetMainTimingGraphTrack();
		if (GraphTrack.IsValid())
		{
			for (const TSharedPtr<FGraphSeries>& Series : GraphTrack->GetSeries())
			{
				if (Series->Is<FTimingGraphSeries>())
				{
					if (Series->As<FTimingGraphSeries>().IsTimer(TimerId))
					{
						Series->SetColor(Color, BorderColor);
					}
				}
			}
		}
	}

	if (FrameTrack.IsValid())
	{
		for (const TSharedPtr<FFrameTrackSeries>& Series : FrameTrack->GetSeries())
		{
			if (Series->Is<FTimerFrameStatsTrackSeries>())
			{
				if (Series->As<FTimerFrameStatsTrackSeries>().GetTimerId() == TimerId)
				{
					Series->SetColor(Color);
				}
			}
		}
	}

	if (CallersTreeView.IsValid())
	{
		FTimerNodePtr TimerNode = CallersTreeView->GetTimerNode(TimerId);
		if (TimerNode.IsValid())
		{
			TimerNode->SetColor(Color);
		}
	}

	if (CalleesTreeView.IsValid())
	{
		FTimerNodePtr TimerNode = CalleesTreeView->GetTimerNode(TimerId);
		if (TimerNode.IsValid())
		{
			TimerNode->SetColor(Color);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingProfilerWindow::IsTimerAddedToGraphs(uint32 TimerId) const
{
	if (FrameTrack.IsValid() &&
		FrameTrack->HasAnySeriesForTimer(TimerId))
	{
		return true;
	}

	if (TimingView.IsValid())
	{
		TSharedPtr<FTimingGraphTrack> GraphTrack = TimingView->GetMainTimingGraphTrack();
		if (GraphTrack.IsValid() &&
			GraphTrack->HasAnySeriesForTimer(TimerId))
		{
			return true;
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingProfilerWindow::OnTimerAddedToGraphsChanged(uint32 TimerId)
{
	const bool bIsAddedToGraphs = IsTimerAddedToGraphs(TimerId);
	FLinearColor ColorOverride;

	if (TimersView.IsValid())
	{
		FTimerNodePtr TimerNode = TimersView->GetTimerNode(TimerId);
		if (TimerNode.IsValid())
		{
			TimerNode->SetAddedToGraphsFlag(bIsAddedToGraphs);
		}
	}

	if (CallersTreeView.IsValid())
	{
		FTimerNodePtr TimerNode = CallersTreeView->GetTimerNode(TimerId);
		if (TimerNode.IsValid())
		{
			TimerNode->SetAddedToGraphsFlag(bIsAddedToGraphs);
			if (GetTimerColor(TimerId, ColorOverride))
			{
				TimerNode->SetColor(ColorOverride);
			}
		}
	}

	if (CalleesTreeView.IsValid())
	{
		FTimerNodePtr TimerNode = CalleesTreeView->GetTimerNode(TimerId);
		if (TimerNode.IsValid())
		{
			TimerNode->SetAddedToGraphsFlag(bIsAddedToGraphs);
			if (GetTimerColor(TimerId, ColorOverride))
			{
				TimerNode->SetColor(ColorOverride);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
