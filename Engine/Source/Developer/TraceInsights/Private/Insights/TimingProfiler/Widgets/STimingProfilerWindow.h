// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/ITimingViewSession.h" // for ETimeChangedFlags
#include "Insights/Widgets/SMajorTabWindow.h"

namespace UE::Insights
{
	class SLogView;
	class SModulesView;
}

namespace UE::Insights::TimingProfiler { class SUserAnnotationsPanel; }

namespace UE::Insights::TimingProfiler
{

class SFrameTrack;
class SStatsView;
class STimersView;
class STimerTreeView;
class STimingView;

/** Implements the Timing Insights major tab window. */
class STimingProfilerWindow : public ::Insights::SMajorTabWindow
{
public:
	/** Default constructor. */
	STimingProfilerWindow();

	/** Virtual destructor. */
	virtual ~STimingProfilerWindow();

	SLATE_BEGIN_ARGS(STimingProfilerWindow) {}
	SLATE_END_ARGS()

	virtual void Reset() override;

	/** Constructs this widget. */
	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);

	TSharedPtr<STimingView> GetTimingView() const { return TimingView; }
	TSharedPtr<STimersView> GetTimersView() const { return TimersView; }
	TSharedPtr<STimerTreeView> GetCallersTreeView() const { return CallersTreeView; }
	TSharedPtr<STimerTreeView> GetCalleesTreeView() const { return CalleesTreeView; }
	TSharedPtr<SStatsView> GetStatsView() const { return StatsView; }
	TSharedPtr<SLogView> GetLogView() const { return LogView; }
	TSharedPtr<SFrameTrack> GetFrameView() const { return FrameTrack; }

	/** Gets the color of a given timer, returns true if successful, false if the timer was not found. */
	bool GetTimerColor(uint32 TimerId, FLinearColor& OutColor);
	/** Updates the TMap and propagates the color to any relevant graphs and nodes. */
	void SetTimerColor(uint32 TimerId, FLinearColor Color);
	
	bool IsTimerAddedToGraphs(uint32 TimerId) const;
	void OnTimerAddedToGraphsChanged(uint32 TimerId);

protected:
	virtual const TCHAR* GetAnalyticsEventName() const override;
	virtual TSharedRef<FWorkspaceItem> CreateWorkspaceMenuGroup() override;
	virtual void RegisterTabSpawners() override;
	virtual TSharedRef<FTabManager::FLayout> CreateDefaultTabLayout() const override;
	virtual TSharedRef<SWidget> CreateToolbar(TSharedPtr<FExtender> Extender);

private:
	TSharedRef<SDockTab> SpawnTab_FramesTrack(const FSpawnTabArgs& Args);
	void OnFramesTrackTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_TimingView(const FSpawnTabArgs& Args);
	void OnTimingViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_Timers(const FSpawnTabArgs& Args);
	void OnTimersTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_Callers(const FSpawnTabArgs& Args);
	void OnCallersTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_Callees(const FSpawnTabArgs& Args);
	void OnCalleesTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_StatsCounters(const FSpawnTabArgs& Args);
	void OnStatsCountersTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_LogView(const FSpawnTabArgs& Args);
	void OnLogViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_ModulesView(const FSpawnTabArgs& Args);
	void OnModulesViewClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedRef<SDockTab> SpawnTab_UserAnnotations(const FSpawnTabArgs& Args);
	void OnUserAnnotationsTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	void OnTimeSelectionChanged(Timing::ETimeChangedFlags InFlags, double InStartTime, double InEndTime);

private:
	/** The Frame track widget */
	TSharedPtr<SFrameTrack> FrameTrack;

	/** The Timing view (multi-track) widget */
	TSharedPtr<STimingView> TimingView;

	/** The Timers view widget */
	TSharedPtr<STimersView> TimersView;

	/** The Callers tree view widget */
	TSharedPtr<STimerTreeView> CallersTreeView;

	/** The Callees tree view widget */
	TSharedPtr<STimerTreeView> CalleesTreeView;

	/** The Stats (Counters) view widget */
	TSharedPtr<SStatsView> StatsView;

	/** The Log view widget */
	TSharedPtr<SLogView> LogView;

	/** The Modules view widget. */
	TSharedPtr<SModulesView> ModulesView;

	/** The User Annotations panel widget. */
	TSharedPtr<SUserAnnotationsPanel> UserAnnotationsPanel;

	/** Color-TimerId Mapping for easy pairing */
	TMap<uint32, FLinearColor> TimerColors;
	
};

} // namespace UE::Insights::TimingProfiler
