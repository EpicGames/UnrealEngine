// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUserAnnotationsPanel.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/TimingProfiler/Models/UserAnnotation.h"
#include "Insights/TimingProfiler/Models/UserAnnotationStore.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/UserAnnotationsTimingViewExtender.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/TraceInsightsModule.h"
#include "Insights/ViewModels/BaseTimingTrack.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingViewLayout.h"
#include "Insights/Widgets/STimingView.h"
#include "Insights/TimingProfiler/Widgets/STimersView.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::SUserAnnotationsPanel"

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Column IDs
////////////////////////////////////////////////////////////////////////////////////////////////////

namespace
{
	/** Surface a save-failure toast so users don't see silent reverts on a read-only sidecar. */
	void NotifyAnnotationSaveFailed()
	{
		FNotificationInfo Info(NSLOCTEXT("UE::Insights::TimingProfiler",
			"AnnotationUpdateFailed",
			"Failed to update annotation. The sidecar .ini file may be read-only."));
		Info.ExpireDuration = 5.0f;
		TSharedPtr<SNotificationItem> Item = FSlateNotificationManager::Get().AddNotification(Info);
		if (Item.IsValid())
		{
			Item->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}
}

namespace UserAnnotationColumns
{
	static const FName VisibleColumnID("Visible");
	static const FName ColorColumnID("Color");
	static const FName TypeColumnID("Type");
	static const FName TextColumnID("Text");
	static const FName DescriptionColumnID("Description");
	static const FName TimeColumnID("Time");
	static const FName FramesColumnID("Frames");
	static const FName TrackColumnID("Track");
	static const FName AuthorColumnID("Author");
	static const FName SourceColumnID("Source");
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Row widget
////////////////////////////////////////////////////////////////////////////////////////////////////

class SUserAnnotationTableRow : public SMultiColumnTableRow<TSharedPtr<FUserAnnotation>>
{
public:
	SLATE_BEGIN_ARGS(SUserAnnotationTableRow) {}
		SLATE_ARGUMENT(TSharedPtr<FUserAnnotation>, AnnotationPtr)
		SLATE_ARGUMENT(ETimerAggregationMode, AggregationMode)
		SLATE_ARGUMENT(TWeakPtr<FUserAnnotationStore>, AnnotationStore)
		SLATE_ARGUMENT(TWeakPtr<STimingView>, HostTimingView)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		AnnotationPtr = InArgs._AnnotationPtr;
		AggregationMode = InArgs._AggregationMode;
		WeakStore = InArgs._AnnotationStore;
		WeakHostTimingView = InArgs._HostTimingView;
		SMultiColumnTableRow<TSharedPtr<FUserAnnotation>>::Construct(
			SMultiColumnTableRow<TSharedPtr<FUserAnnotation>>::FArguments(), InOwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (!AnnotationPtr.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		const FUserAnnotation& Annotation = *AnnotationPtr;

		if (ColumnName == UserAnnotationColumns::VisibleColumnID)
		{
			const FGuid AnnotationId = Annotation.Id;
			TWeakPtr<FUserAnnotationStore> StoreWeak = WeakStore;
			return SNew(SCheckBox)
				.IsChecked(Annotation.bVisible ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
				.OnCheckStateChanged_Lambda([AnnotationId, StoreWeak](ECheckBoxState NewState)
				{
					TSharedPtr<FUserAnnotationStore> Store = StoreWeak.Pin();
					if (!Store.IsValid())
					{
						return;
					}
					const FUserAnnotation* Live = Store->FindAnnotation(AnnotationId);
					if (!Live)
					{
						return;
					}
					FUserAnnotation Updated = *Live;
					Updated.bVisible = (NewState == ECheckBoxState::Checked);
					Updated.ModifiedAt = FDateTime::UtcNow();
					if (!Store->UpdateAnnotation(Updated))
					{
						NotifyAnnotationSaveFailed();
					}
				});
		}

		if (ColumnName == UserAnnotationColumns::ColorColumnID)
		{
			const FGuid AnnotationId = Annotation.Id;
			TWeakPtr<FUserAnnotationStore> StoreWeak = WeakStore;
			// Outer SBox must use default Fill alignment — Center collapses SBorder to its DesiredSize.
			return SNew(SBox)
				.Padding(FMargin(0.0f, 2.0f))
				[
					SNew(SBox)
					.WidthOverride(15.0f)
					.HeightOverride(15.0f)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
					.BorderBackgroundColor_Lambda([AnnotationId, StoreWeak]()
					{
						TSharedPtr<FUserAnnotationStore> Store = StoreWeak.Pin();
						if (Store.IsValid())
						{
							if (const FUserAnnotation* Live = Store->FindAnnotation(AnnotationId))
							{
								return FSlateColor(Live->Color);
							}
						}
						return FSlateColor(FLinearColor::White);
					})
					.OnMouseButtonDown_Lambda([AnnotationId, StoreWeak](const FGeometry&, const FPointerEvent&) -> FReply
					{
						TSharedPtr<FUserAnnotationStore> Store = StoreWeak.Pin();
						if (!Store.IsValid())
						{
							return FReply::Unhandled();
						}
						const FUserAnnotation* Live = Store->FindAnnotation(AnnotationId);
						if (!Live)
						{
							return FReply::Unhandled();
						}
						FColorPickerArgs Args;
						Args.InitialColor = Live->Color;
						Args.bUseAlpha = false;
						Args.bIsModal = true;
						Args.OnColorCommitted = FOnLinearColorValueChanged::CreateLambda(
							[AnnotationId, StoreWeak](FLinearColor NewColor)
							{
								TSharedPtr<FUserAnnotationStore> Store = StoreWeak.Pin();
								if (!Store.IsValid())
								{
									return;
								}
								const FUserAnnotation* CommitLive = Store->FindAnnotation(AnnotationId);
								if (!CommitLive)
								{
									return;
								}
								FUserAnnotation Updated = *CommitLive;
								Updated.Color = NewColor;
								Updated.ModifiedAt = FDateTime::UtcNow();
								if (!Store->UpdateAnnotation(Updated))
								{
									NotifyAnnotationSaveFailed();
								}
							});
						OpenColorPicker(Args);
						return FReply::Handled();
					})
				]
			];
		}

		if (ColumnName == UserAnnotationColumns::TypeColumnID)
		{
			const FText TypeText = Annotation.HasEventAnchor()
				? LOCTEXT("TypeEvent", "Event")
				: Annotation.IsRange()
					? LOCTEXT("TypeRange", "Range")
					: LOCTEXT("TypePoint", "Time");
			return SNew(SBox).VAlign(VAlign_Center).Padding(FMargin(4.0f, 0.0f))
				[
					SNew(STextBlock).Text(TypeText)
				];
		}

		if (ColumnName == UserAnnotationColumns::TextColumnID)
		{
			return SNew(SBox).VAlign(VAlign_Center).Padding(FMargin(4.0f, 0.0f))
				[
					SNew(STextBlock).Text(FText::FromString(Annotation.Text))
				];
		}

		if (ColumnName == UserAnnotationColumns::DescriptionColumnID)
		{
			return SNew(SBox).VAlign(VAlign_Center).Padding(FMargin(4.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text(FText::FromString(Annotation.Description))
					.ToolTipText(Annotation.Description.IsEmpty()
						? FText::GetEmpty()
						: FText::FromString(Annotation.Description))
				];
		}

		if (ColumnName == UserAnnotationColumns::TimeColumnID)
		{
			// Single-row cell: short primary text, full detail in tooltip. Vcentered.
			const FString StartStr = FormatTime(Annotation.Time, FTimeValue::Microsecond);
			FString TooltipStr = StartStr;
			if (Annotation.IsRange())
			{
				const double Duration = Annotation.EndTime - Annotation.Time;
				TooltipStr = FString::Printf(TEXT("%s \u2014 %s (%s)"),
					*StartStr,
					*FormatTime(Annotation.EndTime, FTimeValue::Microsecond),
					*FormatTime(Duration, FTimeValue::Microsecond));
			}
			return SNew(SBox)
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text(FText::FromString(StartStr))
					.ToolTipText(FText::FromString(TooltipStr))
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				];
		}

		if (ColumnName == UserAnnotationColumns::FramesColumnID)
		{
			FString FrameStr;
			const bool bGameOnly = (AggregationMode == ETimerAggregationMode::GameFrame);
			const bool bRenderOnly = (AggregationMode == ETimerAggregationMode::RenderingFrame);

			if (Annotation.IsRange())
			{
				// Show range format only when frames actually differ.
				const FString GameStr = (Annotation.GameFrameNumber == Annotation.GameFrameNumberEnd)
					? FString::Printf(TEXT("G:%u"), Annotation.GameFrameNumber)
					: FString::Printf(TEXT("G:%u\u2014%u"), Annotation.GameFrameNumber, Annotation.GameFrameNumberEnd);
				const FString RenderStr = (Annotation.RenderFrameNumber == Annotation.RenderFrameNumberEnd)
					? FString::Printf(TEXT("R:%u"), Annotation.RenderFrameNumber)
					: FString::Printf(TEXT("R:%u\u2014%u"), Annotation.RenderFrameNumber, Annotation.RenderFrameNumberEnd);

				if (bGameOnly)
				{
					FrameStr = GameStr;
				}
				else if (bRenderOnly)
				{
					FrameStr = RenderStr;
				}
				else
				{
					FrameStr = GameStr + TEXT(" ") + RenderStr;
				}
			}
			else
			{
				if (bGameOnly)
				{
					FrameStr = FString::Printf(TEXT("G:%u"), Annotation.GameFrameNumber);
				}
				else if (bRenderOnly)
				{
					FrameStr = FString::Printf(TEXT("R:%u"), Annotation.RenderFrameNumber);
				}
				else
				{
					FrameStr = FString::Printf(TEXT("G:%u R:%u"), Annotation.GameFrameNumber, Annotation.RenderFrameNumber);
				}
			}
			return SNew(SBox).VAlign(VAlign_Center).Padding(FMargin(4.0f, 0.0f))
				[
					SNew(STextBlock).Text(FText::FromString(FrameStr))
				];
		}

		if (ColumnName == UserAnnotationColumns::TrackColumnID)
		{
			// Bind the label + color + tooltip dynamically so toggling track visibility
			// via the All Tracks / CPU-GPU / Other dropdowns updates the panel row
			// without needing to rebuild it.
			const FString ThreadName = Annotation.ThreadName;
			// Show the "(hidden)" indicator for any annotation tied to a track (point, range, or event-anchored).
			const bool bHasTargetTrack = !ThreadName.IsEmpty();

			// Resolve the extender ONCE at row construction. The TraceInsights module is a
			// process-lifetime singleton, so raw-pointer capture is safe and avoids a
			// per-paint LoadModuleChecked (which would run 3x per visible row per frame).
			FUserAnnotationsTimingViewExtender* ExtenderPtr = nullptr;
			if (bHasTargetTrack)
			{
				FTraceInsightsModule& Module = static_cast<FTraceInsightsModule&>(
					FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights"));
				ExtenderPtr = &Module.GetUserAnnotationsExtender();
			}

			// Scoped to this panel's host window — tracks have independent visibility per window.
			TWeakPtr<STimingView> WeakHost = WeakHostTimingView;
			auto IsTargetHidden = [ExtenderPtr, ThreadName, WeakHost]() -> bool
			{
				if (ExtenderPtr == nullptr)
				{
					return false;
				}
				// Suppress until analysis completes so the warning doesn't pop on mid-load.
				const TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();
				if (!InsightsManager.IsValid() || !InsightsManager->IsAnalysisComplete())
				{
					return false;
				}
				const UE::Insights::Timing::ITimingViewSession* HostSession = WeakHost.Pin().Get();
				return !ExtenderPtr->IsTargetThreadTrackVisible(HostSession, ThreadName);
			};

			TSharedRef<STextBlock> TextBlock = SNew(STextBlock)
				.Text_Lambda([ThreadName, IsTargetHidden]()
				{
					if (IsTargetHidden())
					{
						return FText::Format(LOCTEXT("TrackHiddenFmt", "(hidden) {0}"),
							FText::FromString(ThreadName));
					}
					return FText::FromString(ThreadName);
				})
				.ColorAndOpacity_Lambda([IsTargetHidden]()
				{
					return IsTargetHidden()
						? FSlateColor(FLinearColor(1.0f, 0.6f, 0.2f, 1.0f))
						: FSlateColor::UseForeground();
				})
				.ToolTipText_Lambda([IsTargetHidden]()
				{
					return IsTargetHidden()
						? LOCTEXT("TrackHiddenTooltip",
							"The target track for this annotation is currently hidden. "
							"The annotation still exists but no highlight will render. "
							"Enable the track via the timing view's Other / All Tracks menu.")
						: FText::GetEmpty();
				});
			return SNew(SBox).VAlign(VAlign_Center).Padding(FMargin(4.0f, 0.0f))
				[
					TextBlock
				];
		}

		if (ColumnName == UserAnnotationColumns::AuthorColumnID)
		{
			return SNew(SBox).VAlign(VAlign_Center).Padding(FMargin(4.0f, 0.0f))
				[
					SNew(STextBlock).Text(FText::FromString(Annotation.Author))
				];
		}

		if (ColumnName == UserAnnotationColumns::SourceColumnID)
		{
			return SNew(SBox).VAlign(VAlign_Center).Padding(FMargin(4.0f, 0.0f))
				[
					SNew(STextBlock).Text(FText::FromString(Annotation.Source))
				];
		}

		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FUserAnnotation> AnnotationPtr;
	ETimerAggregationMode AggregationMode = ETimerAggregationMode::Instance;
	TWeakPtr<FUserAnnotationStore> WeakStore;
	TWeakPtr<STimingView> WeakHostTimingView;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// SUserAnnotationsPanel
////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SUserAnnotationsPanel::Construct(const FArguments& InArgs)
{
	WeakHostTimingView = InArgs._HostTimingView;

	// STextBlock.Text_Lambda on a listview row does not re-evaluate when a track is toggled
	// elsewhere — the row's invalidation chain is independent of the timing view. Rebuild rows
	// on visibility change so the "(hidden) TrackName" indicator flips immediately.
	if (TSharedPtr<STimingView> Host = WeakHostTimingView.Pin())
	{
		TrackVisibilityChangedHandle = Host->OnTrackVisibilityChanged().AddLambda([WeakThis = TWeakPtr<SUserAnnotationsPanel>(SharedThis(this))]()
		{
			if (TSharedPtr<SUserAnnotationsPanel> Pinned = WeakThis.Pin())
			{
				if (Pinned->ListView.IsValid())
				{
					Pinned->ListView->RebuildList();
				}
			}
		});
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		// Read-only banner — re-queries each frame so it appears as soon as writability is known.
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 4.0f, 4.0f, 0.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.Padding(6.0f)
			.Visibility_Lambda([]() -> EVisibility
			{
				FTraceInsightsModule& Module = static_cast<FTraceInsightsModule&>(
					FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights"));
				return Module.GetUserAnnotationsExtender().IsAnyCurrentSessionReadOnly()
					? EVisibility::Visible : EVisibility::Collapsed;
			})
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ReadOnlyBanner",
					"Annotations are disabled for this trace — the sidecar .ini (or its folder) is read-only. "
					"Existing annotations remain viewable, but adding, editing, or deleting is blocked. "
					"Make the folder writable and reopen the trace to re-enable."))
				.AutoWrapText(true)
				.ColorAndOpacity(FLinearColor(1.0f, 0.75f, 0.75f, 1.0f))
			]
		]

		// Search box
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f)
		[
			SNew(SSearchBox)
			.OnTextChanged(this, &SUserAnnotationsPanel::OnSearchTextChanged)
			.HintText(LOCTEXT("SearchHint", "Filter annotations..."))
		]

		// List view
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(ListView, SListView<TSharedPtr<FUserAnnotation>>)
			.ListItemsSource(&FilteredAnnotations)
			.OnGenerateRow(this, &SUserAnnotationsPanel::OnGenerateRow)
			.OnSelectionChanged(this, &SUserAnnotationsPanel::OnSelectionChanged)
			.OnMouseButtonDoubleClick(this, &SUserAnnotationsPanel::OnListItemDoubleClicked)
			.OnContextMenuOpening(this, &SUserAnnotationsPanel::OnContextMenuOpening)
			.SelectionMode(ESelectionMode::Multi)
			.HeaderRow
			(
				SAssignNew(HeaderRowPtr, SHeaderRow)

				+ SHeaderRow::Column(UserAnnotationColumns::VisibleColumnID)
				.DefaultLabel(FText::GetEmpty())
				.ManualWidth(40.0f)
				.HAlignHeader(HAlign_Center)
				.HAlignCell(HAlign_Center)
				.VAlignCell(VAlign_Center)
				.HeaderContent()
				[
					SNew(SCheckBox)
					.IsChecked_Lambda([this]()
					{
						TSharedPtr<FUserAnnotationStore> Store = WeakAnnotationStore.Pin();
						if (!Store.IsValid() || Store->GetAllAnnotations().Num() == 0)
						{
							return ECheckBoxState::Unchecked;
						}
						bool bAllVisible = true;
						bool bAnyVisible = false;
						for (const FUserAnnotation& Annotation : Store->GetAllAnnotations())
						{
							if (Annotation.bVisible) { bAnyVisible = true; }
							else { bAllVisible = false; }
						}
						if (bAllVisible) { return ECheckBoxState::Checked; }
						if (bAnyVisible) { return ECheckBoxState::Undetermined; }
						return ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
					{
						TSharedPtr<FUserAnnotationStore> Store = WeakAnnotationStore.Pin();
						if (!Store.IsValid())
						{
							return;
						}
						const bool bNewVisible = (NewState == ECheckBoxState::Checked);
						bool bAnyFailed = false;
						for (const FUserAnnotation& Annotation : Store->GetAllAnnotations())
						{
							if (Annotation.bVisible != bNewVisible)
							{
								FUserAnnotation Updated = Annotation;
								Updated.bVisible = bNewVisible;
								Updated.ModifiedAt = FDateTime::UtcNow();
								if (!Store->UpdateAnnotation(Updated))
								{
									bAnyFailed = true;
								}
							}
						}
						if (bAnyFailed)
						{
							NotifyAnnotationSaveFailed();
						}
					})
				]

				+ SHeaderRow::Column(UserAnnotationColumns::ColorColumnID)
				.DefaultLabel(FText::GetEmpty())
				.ManualWidth(21.0f)
				.HAlignHeader(HAlign_Center)
				.HAlignCell(HAlign_Center)

				+ SHeaderRow::Column(UserAnnotationColumns::TypeColumnID)
				.DefaultLabel(LOCTEXT("TypeColumn", "Type"))
				.FillWidth(0.08f)
				.SortMode(this, &SUserAnnotationsPanel::GetSortModeForColumn, UserAnnotationColumns::TypeColumnID)
				.OnSort(this, &SUserAnnotationsPanel::OnSortColumnHeader)

				+ SHeaderRow::Column(UserAnnotationColumns::TextColumnID)
				.DefaultLabel(LOCTEXT("TextColumn", "Text"))
				.FillWidth(0.22f)
				.SortMode(this, &SUserAnnotationsPanel::GetSortModeForColumn, UserAnnotationColumns::TextColumnID)
				.OnSort(this, &SUserAnnotationsPanel::OnSortColumnHeader)

				+ SHeaderRow::Column(UserAnnotationColumns::DescriptionColumnID)
				.DefaultLabel(LOCTEXT("DescriptionColumn", "Description"))
				.FillWidth(0.15f)
				.SortMode(this, &SUserAnnotationsPanel::GetSortModeForColumn, UserAnnotationColumns::DescriptionColumnID)
				.OnSort(this, &SUserAnnotationsPanel::OnSortColumnHeader)

				+ SHeaderRow::Column(UserAnnotationColumns::TimeColumnID)
				.DefaultLabel(LOCTEXT("TimeColumn", "Time"))
				.FillWidth(0.22f)
				.SortMode(this, &SUserAnnotationsPanel::GetSortModeForColumn, UserAnnotationColumns::TimeColumnID)
				.OnSort(this, &SUserAnnotationsPanel::OnSortColumnHeader)

				+ SHeaderRow::Column(UserAnnotationColumns::FramesColumnID)
				.DefaultLabel(LOCTEXT("FramesColumn", "Frames"))
				.FillWidth(0.13f)

				+ SHeaderRow::Column(UserAnnotationColumns::TrackColumnID)
				.DefaultLabel(LOCTEXT("TrackColumn", "Track"))
				.FillWidth(0.10f)

				+ SHeaderRow::Column(UserAnnotationColumns::AuthorColumnID)
				.DefaultLabel(LOCTEXT("AuthorColumn", "Author"))
				.FillWidth(0.10f)
				.SortMode(this, &SUserAnnotationsPanel::GetSortModeForColumn, UserAnnotationColumns::AuthorColumnID)
				.OnSort(this, &SUserAnnotationsPanel::OnSortColumnHeader)

				+ SHeaderRow::Column(UserAnnotationColumns::SourceColumnID)
				.DefaultLabel(LOCTEXT("SourceColumn", "Source"))
				.FillWidth(0.08f)
				.SortMode(this, &SUserAnnotationsPanel::GetSortModeForColumn, UserAnnotationColumns::SourceColumnID)
				.OnSort(this, &SUserAnnotationsPanel::OnSortColumnHeader)
			)
		]

		// Keyboard shortcuts reference
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 2.0f, 4.0f, 4.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.DarkGroupBorder"))
			.Padding(FMargin(6.0f, 3.0f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AnnotationShortcuts",
					"Ctrl+B: Add Annotation | Ctrl+Shift+B: Add Range | Ctrl+Alt+B: Add Event | Ctrl+N / Ctrl+Shift+N: Next / Prev | Del: Delete | F2: Edit"))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
			]
		]
	];

	// Frames / Author / Source hidden by default; toggle via the row right-click "Columns" submenu.
	// Track stays visible-by-default because it carries the "(hidden) TrackName" indicator
	// for annotations whose target track has been hidden in the timing view.
	if (HeaderRowPtr.IsValid())
	{
		HeaderRowPtr->SetShowGeneratedColumn(UserAnnotationColumns::FramesColumnID, false);
		HeaderRowPtr->SetShowGeneratedColumn(UserAnnotationColumns::AuthorColumnID, false);
		HeaderRowPtr->SetShowGeneratedColumn(UserAnnotationColumns::SourceColumnID, false);
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

SUserAnnotationsPanel::~SUserAnnotationsPanel()
{
	if (TrackVisibilityChangedHandle.IsValid())
	{
		if (TSharedPtr<STimingView> Host = WeakHostTimingView.Pin())
		{
			Host->OnTrackVisibilityChanged().Remove(TrackVisibilityChangedHandle);
		}
		TrackVisibilityChangedHandle.Reset();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUserAnnotationsPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	// Cache the extender once — module is process-lifetime and the extender lives as a member of it.
	if (!CachedExtender)
	{
		FTraceInsightsModule* Module = static_cast<FTraceInsightsModule*>(
			FModuleManager::GetModulePtr<IUnrealInsightsModule>("TraceInsights"));
		if (Module)
		{
			CachedExtender = &Module->GetUserAnnotationsExtender();
		}
	}
	if (!CachedExtender)
	{
		return;
	}
	// Drive the load ourselves — STimingView's extender-Tick gate skips Memory/Asset
	// Loading on early frames and the panel would otherwise stay empty until Timing is clicked.
	CachedExtender->EnsureAllSessionsLoaded();

	// Re-resolve the current store every tick — EnsureSessionLoaded may swap a session's
	// store to a sibling's when the same sidecar is opened in multiple windows, leaving
	// a previously pinned store orphaned. Compare by pointer identity, rebind on mismatch.
	{
		TSharedPtr<FUserAnnotationStore> CurrentStore = CachedExtender->GetAnnotationStoreForCurrentSession();
		TSharedPtr<FUserAnnotationStore> PinnedStore = WeakAnnotationStore.Pin();
		if (CurrentStore.Get() != PinnedStore.Get())
		{
			WeakAnnotationStore = CurrentStore;
			LastChangeNumber = 0;
		}
	}

	// Check for changes to refresh the list.
	TSharedPtr<FUserAnnotationStore> Store = WeakAnnotationStore.Pin();
	if (Store.IsValid())
	{
		const uint64 CurrentChangeNumber = Store->GetChangeNumber();
		if (CurrentChangeNumber != LastChangeNumber)
		{
			LastChangeNumber = CurrentChangeNumber;
			RefreshAnnotationList();
		}
	}

	// Detect aggregation mode changes and refresh the frame column display.
	ETimerAggregationMode CurrentAggMode = ETimerAggregationMode::Instance;
	TSharedPtr<FTimingProfilerManager> Manager = FTimingProfilerManager::Get();
	if (Manager.IsValid())
	{
		TSharedPtr<STimingProfilerWindow> Wnd = Manager->GetProfilerWindow();
		if (Wnd.IsValid())
		{
			TSharedPtr<STimersView> TimersView = Wnd->GetTimersView();
			if (TimersView.IsValid())
			{
				CurrentAggMode = TimersView->GetAggregationMode();
			}
		}
	}
	if (CurrentAggMode != CachedAggregationMode)
	{
		CachedAggregationMode = CurrentAggMode;
		if (ListView.IsValid())
		{
			ListView->RequestListRefresh();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUserAnnotationsPanel::RefreshAnnotationList()
{
	// Capture the currently-selected annotation IDs before rebuilding the source array.
	// RefreshAnnotationList creates fresh TSharedPtr<FUserAnnotation> instances; without an
	// Id-based restore, SListView loses selection because it tracks by shared-ptr identity.
	TSet<FGuid> PreviouslySelectedIds;
	if (ListView.IsValid())
	{
		for (const TSharedPtr<FUserAnnotation>& Selected : ListView->GetSelectedItems())
		{
			if (Selected.IsValid())
			{
				PreviouslySelectedIds.Add(Selected->Id);
			}
		}
	}

	AllAnnotations.Empty();

	TSharedPtr<FUserAnnotationStore> Store = WeakAnnotationStore.Pin();
	if (Store.IsValid())
	{
		for (const FUserAnnotation& Annotation : Store->GetAllAnnotations())
		{
			AllAnnotations.Add(MakeShared<FUserAnnotation>(Annotation));
		}
	}

	ApplyFilter();

	// Re-apply selection on the new TSharedPtr instances by matching Ids.
	if (ListView.IsValid() && PreviouslySelectedIds.Num() > 0)
	{
		ListView->ClearSelection();
		for (const TSharedPtr<FUserAnnotation>& Item : FilteredAnnotations)
		{
			if (Item.IsValid() && PreviouslySelectedIds.Contains(Item->Id))
			{
				ListView->SetItemSelection(Item, true, ESelectInfo::Direct);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUserAnnotationsPanel::ApplyFilter()
{
	FilteredAnnotations.Empty();

	if (SearchText.IsEmpty())
	{
		FilteredAnnotations = AllAnnotations;
	}
	else
	{
		for (const TSharedPtr<FUserAnnotation>& Annotation : AllAnnotations)
		{
			if (Annotation.IsValid() &&
				(Annotation->Text.Contains(SearchText, ESearchCase::IgnoreCase) ||
				 Annotation->Description.Contains(SearchText, ESearchCase::IgnoreCase)))
			{
				FilteredAnnotations.Add(Annotation);
			}
		}
	}

	// Apply sort if a column is active.
	if (SortMode != EColumnSortMode::None && SortByColumn != NAME_None)
	{
		const bool bAscending = (SortMode == EColumnSortMode::Ascending);

		if (SortByColumn == UserAnnotationColumns::TypeColumnID)
		{
			// Sort order: Event (0), Range (1), Time Annotation (2).
			auto GetTypeOrder = [](const FUserAnnotation& Annotation) -> int32
			{
				if (Annotation.HasEventAnchor()) return 0;
				if (Annotation.IsRange()) return 1;
				return 2;
			};

			FilteredAnnotations.Sort([&](const TSharedPtr<FUserAnnotation>& A, const TSharedPtr<FUserAnnotation>& B)
			{
				const int32 OrderA = GetTypeOrder(*A);
				const int32 OrderB = GetTypeOrder(*B);
				return bAscending ? (OrderA < OrderB) : (OrderA > OrderB);
			});
		}
		else if (SortByColumn == UserAnnotationColumns::TextColumnID)
		{
			FilteredAnnotations.Sort([bAscending](const TSharedPtr<FUserAnnotation>& A, const TSharedPtr<FUserAnnotation>& B)
			{
				const int32 Cmp = A->Text.Compare(B->Text, ESearchCase::IgnoreCase);
				return bAscending ? (Cmp < 0) : (Cmp > 0);
			});
		}
		else if (SortByColumn == UserAnnotationColumns::DescriptionColumnID)
		{
			FilteredAnnotations.Sort([bAscending](const TSharedPtr<FUserAnnotation>& A, const TSharedPtr<FUserAnnotation>& B)
			{
				const int32 Cmp = A->Description.Compare(B->Description, ESearchCase::IgnoreCase);
				return bAscending ? (Cmp < 0) : (Cmp > 0);
			});
		}
		else if (SortByColumn == UserAnnotationColumns::TimeColumnID)
		{
			FilteredAnnotations.Sort([bAscending](const TSharedPtr<FUserAnnotation>& A, const TSharedPtr<FUserAnnotation>& B)
			{
				return bAscending ? (A->Time < B->Time) : (A->Time > B->Time);
			});
		}
		else if (SortByColumn == UserAnnotationColumns::AuthorColumnID)
		{
			FilteredAnnotations.Sort([bAscending](const TSharedPtr<FUserAnnotation>& A, const TSharedPtr<FUserAnnotation>& B)
			{
				const int32 Cmp = A->Author.Compare(B->Author, ESearchCase::IgnoreCase);
				return bAscending ? (Cmp < 0) : (Cmp > 0);
			});
		}
		else if (SortByColumn == UserAnnotationColumns::SourceColumnID)
		{
			FilteredAnnotations.Sort([bAscending](const TSharedPtr<FUserAnnotation>& A, const TSharedPtr<FUserAnnotation>& B)
			{
				const int32 Cmp = A->Source.Compare(B->Source, ESearchCase::IgnoreCase);
				return bAscending ? (Cmp < 0) : (Cmp > 0);
			});
		}
	}

	if (ListView.IsValid())
	{
		ListView->RequestListRefresh();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUserAnnotationsPanel::OnSortColumnHeader(EColumnSortPriority::Type /*SortPriority*/, const FName& ColumnId, EColumnSortMode::Type NewSortMode)
{
	SortByColumn = ColumnId;
	SortMode = NewSortMode;
	ApplyFilter();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EColumnSortMode::Type SUserAnnotationsPanel::GetSortModeForColumn(FName ColumnId) const
{
	if (SortByColumn == ColumnId)
	{
		return SortMode;
	}
	return EColumnSortMode::None;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUserAnnotationsPanel::NavigateToAnnotation(const FUserAnnotation& Annotation)
{
	// Prefer the hosting window's timing view (passed via HostTimingView at construction).
	// Fall back to the Timing Insights window for legacy call sites that don't pass one.
	TSharedPtr<STimingView> TimingView = WeakHostTimingView.Pin();
	if (!TimingView.IsValid())
	{
		TSharedPtr<FTimingProfilerManager> Manager = FTimingProfilerManager::Get();
		if (!Manager.IsValid())
		{
			return;
		}
		TSharedPtr<STimingProfilerWindow> Wnd = Manager->GetProfilerWindow();
		if (!Wnd.IsValid())
		{
			return;
		}
		TimingView = Wnd->GetTimingView();
		if (!TimingView.IsValid())
		{
			return;
		}
	}

	if (Annotation.HasEventAnchor())
	{
		// Zoom to show the event with 10% margin on each side (matches engine zoom-to-event pattern).
		const double Duration = Annotation.EventEndTime - Annotation.EventStartTime;
		TimingView->ZoomOnTimeInterval(Annotation.EventStartTime - Duration * 0.1, Duration * 1.2);
	}
	else if (Annotation.IsRange())
	{
		// Zoom to show the range with 10% margin on each side.
		const double Duration = Annotation.EndTime - Annotation.Time;
		TimingView->ZoomOnTimeInterval(Annotation.Time - Duration * 0.1, Duration * 1.2);
	}
	else
	{
		// Center on the point annotation.
		TimingView->SetAndCenterOnTimeMarker(Annotation.Time);
	}

	if (!Annotation.ThreadName.IsEmpty())
	{
		TSharedPtr<FBaseTimingTrack> TargetTrack;
		TimingView->EnumerateTracks([&TargetTrack, &Annotation](TSharedPtr<FBaseTimingTrack> Track)
		{
			if (!TargetTrack.IsValid() && Track.IsValid() && Track->GetName() == Annotation.ThreadName)
			{
				TargetTrack = Track;
			}
		});

		if (TargetTrack.IsValid())
		{
			// Reveal the track if hidden so the navigated-to annotation is visible.
			if (!TargetTrack->IsVisible())
			{
				TargetTrack->SetVisibilityFlag(true);
			}

			TimingView->BringScrollableTrackIntoView(*TargetTrack);

			if (Annotation.HasEventAnchor())
			{
				const FTimingTrackViewport& Viewport = TimingView->GetViewport();
				const float EventMidX = Viewport.TimeToSlateUnitsRounded(
					(Annotation.EventStartTime + Annotation.EventEndTime) * 0.5);
				const FTimingViewLayout& Layout = Viewport.GetLayout();
				const float EventY = TargetTrack->GetPosY()
					+ TargetTrack->GetChildTracksTopHeight(Layout)
					+ Layout.GetLaneY(static_cast<int32>(Annotation.EventDepth))
					+ Layout.EventH * 0.5f;

				const TSharedPtr<const ITimingEvent> Event =
					TargetTrack->GetEvent(EventMidX, EventY, Viewport);
				if (Event.IsValid())
				{
					TimingView->SelectTimingEvent(Event, true, true);
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableRow> SUserAnnotationsPanel::OnGenerateRow(TSharedPtr<FUserAnnotation> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	ETimerAggregationMode CurrentMode = ETimerAggregationMode::Instance;
	TSharedPtr<FTimingProfilerManager> Manager = FTimingProfilerManager::Get();
	if (Manager.IsValid())
	{
		TSharedPtr<STimingProfilerWindow> Wnd = Manager->GetProfilerWindow();
		if (Wnd.IsValid())
		{
			TSharedPtr<STimersView> TimersView = Wnd->GetTimersView();
			if (TimersView.IsValid())
			{
				CurrentMode = TimersView->GetAggregationMode();
			}
		}
	}

	return SNew(SUserAnnotationTableRow, OwnerTable)
		.AnnotationPtr(Item)
		.AggregationMode(CurrentMode)
		.AnnotationStore(WeakAnnotationStore)
		.HostTimingView(WeakHostTimingView);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUserAnnotationsPanel::OnSelectionChanged(TSharedPtr<FUserAnnotation> Item, ESelectInfo::Type SelectInfo)
{
	// Single click only selects the row. Navigation + range selection happen on double-click
	// (OnListItemDoubleClicked) to match the established timing-event interaction convention:
	//   double-click  -> navigate + select the annotation's time range
	//   Ctrl+click    -> extend selection (not implemented for annotations yet)
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUserAnnotationsPanel::OnListItemDoubleClicked(TSharedPtr<FUserAnnotation> Item)
{
	if (!Item.IsValid())
	{
		return;
	}

	if (ListView.IsValid())
	{
		ListView->SetSelection(Item);
	}

	// Double-click convention (matches UE timing-event tracks):
	//   double-click -> select the event's time range + navigate
	//   Use F2 or right-click "Edit..." for editing.
	NavigateToAnnotation(*Item);

	// Prefer host timing view; fall back to Timing Insights.
	TSharedPtr<STimingView> TimingView = WeakHostTimingView.Pin();
	if (!TimingView.IsValid())
	{
		TSharedPtr<FTimingProfilerManager> Manager = FTimingProfilerManager::Get();
		if (!Manager.IsValid())
		{
			return;
		}
		TSharedPtr<STimingProfilerWindow> Wnd = Manager->GetProfilerWindow();
		if (!Wnd.IsValid())
		{
			return;
		}
		TimingView = Wnd->GetTimingView();
		if (!TimingView.IsValid())
		{
			return;
		}
	}

	// Select the annotation's time range: anchor for event-anchored, [Time, EndTime] for range,
	// skip for point annotations (no duration to select).
	if (Item->HasEventAnchor())
	{
		const double Duration = Item->EventEndTime - Item->EventStartTime;
		if (Duration > 0.0)
		{
			TimingView->SelectTimeInterval(Item->EventStartTime, Duration);
		}
	}
	else if (Item->IsRange())
	{
		const double Duration = Item->EndTime - Item->Time;
		if (Duration > 0.0)
		{
			TimingView->SelectTimeInterval(Item->Time, Duration);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> SUserAnnotationsPanel::OnContextMenuOpening()
{
	if (!ListView.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<FUserAnnotationStore> Store = WeakAnnotationStore.Pin();
	if (!Store.IsValid())
	{
		return nullptr;
	}

	TArray<TSharedPtr<FUserAnnotation>> SelectedItems = ListView->GetSelectedItems();
	TWeakPtr<FUserAnnotationStore> WeakStore = Store;

	const bool bMultipleSelected = (SelectedItems.Num() > 1);

	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection(TEXT("AnnotationActions"), LOCTEXT("AnnotationActions", "Annotation Actions"));
	{
		// Single-item actions (only when exactly one annotation is selected)
		if (SelectedItems.Num() == 1)
		{
			TSharedPtr<FUserAnnotation> SelectedAnnotation = SelectedItems[0];
			if (!SelectedAnnotation.IsValid())
			{
				return nullptr;
			}

			FString AnnotationText = SelectedAnnotation->Text;

			MenuBuilder.AddMenuEntry(
				LOCTEXT("NavigateToAnnotation", "Navigate To"),
				LOCTEXT("NavigateToAnnotation_Desc", "Navigate the timing view to this annotation"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Search"),
				FUIAction(FExecuteAction::CreateLambda([this, SelectedAnnotation]()
				{
					if (SelectedAnnotation.IsValid())
					{
						NavigateToAnnotation(*SelectedAnnotation);
					}
				}))
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("EditAnnotation", "Edit..."),
				LOCTEXT("EditAnnotation_Desc", "Edit this annotation (same as F2)"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Edit"),
				FUIAction(FExecuteAction::CreateSP(this, &SUserAnnotationsPanel::EditSelectedAnnotation))
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("CopyText", "Copy Text"),
				LOCTEXT("CopyText_Desc", "Copy the annotation text to the clipboard"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(FExecuteAction::CreateLambda([AnnotationText]()
				{
					FPlatformApplicationMisc::ClipboardCopy(*AnnotationText);
				}))
			);

			FString AnnotationDesc = SelectedAnnotation->Description;
			MenuBuilder.AddMenuEntry(
				LOCTEXT("PanelCopyDescription", "Copy Description"),
				LOCTEXT("PanelCopyDescription_Desc", "Copy the annotation description to the clipboard"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(
					FExecuteAction::CreateLambda([AnnotationDesc]()
					{
						FPlatformApplicationMisc::ClipboardCopy(*AnnotationDesc);
					}),
					FCanExecuteAction::CreateLambda([AnnotationDesc]()
					{
						return !AnnotationDesc.IsEmpty();
					})
				)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("PanelCopyAll", "Copy Text + Description"),
				LOCTEXT("PanelCopyAll_Desc", "Copy both text and description to the clipboard"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(FExecuteAction::CreateLambda([AnnotationText, AnnotationDesc]()
				{
					FString Combined = AnnotationText;
					if (!AnnotationDesc.IsEmpty())
					{
						Combined += TEXT("\n") + AnnotationDesc;
					}
					FPlatformApplicationMisc::ClipboardCopy(*Combined);
				}))
			);

			MenuBuilder.AddMenuSeparator();

			MenuBuilder.AddMenuEntry(
				LOCTEXT("DeleteAnnotation", "Delete"),
				LOCTEXT("DeleteAnnotation_Desc", "Remove this annotation"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Delete"),
				FUIAction(FExecuteAction::CreateSP(this, &SUserAnnotationsPanel::DeleteSelectedAnnotations))
			);
		}

		// Multi-selection actions
		if (bMultipleSelected)
		{
			const int32 SelectedCount = SelectedItems.Num();

			// Collect text for batch copy
			TArray<TSharedPtr<FUserAnnotation>> ItemsCopy = SelectedItems;

			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("CopySelectedTextFmt", "Copy {0} Annotation Texts"), FText::AsNumber(SelectedCount)),
				LOCTEXT("CopySelectedText_Desc", "Copy all selected annotation texts to the clipboard"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(FExecuteAction::CreateLambda([ItemsCopy]()
				{
					FString Combined;
					for (const TSharedPtr<FUserAnnotation>& Item : ItemsCopy)
					{
						if (Item.IsValid())
						{
							if (!Combined.IsEmpty())
							{
								Combined += TEXT("\n");
							}
							Combined += Item->Text;
						}
					}
					FPlatformApplicationMisc::ClipboardCopy(*Combined);
				}))
			);

			MenuBuilder.AddMenuSeparator();

			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("DeleteSelectedFmt", "Delete {0} Annotations"), FText::AsNumber(SelectedCount)),
				LOCTEXT("DeleteSelected_Desc", "Remove all selected annotations"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Delete"),
				FUIAction(FExecuteAction::CreateSP(this, &SUserAnnotationsPanel::DeleteSelectedAnnotations))
			);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("AnnotationVisibility"), LOCTEXT("AnnotationVisibility", "Visibility"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("HideAllAnnotations", "Hide All Annotations"),
			LOCTEXT("HideAllAnnotations_Desc", "Hide all annotations from the timing view"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Visible"),
			FUIAction(FExecuteAction::CreateLambda([WeakStore]()
			{
				TSharedPtr<FUserAnnotationStore> PinnedStore = WeakStore.Pin();
				if (PinnedStore.IsValid())
				{
					bool bAnyFailed = false;
					for (const FUserAnnotation& Annotation : PinnedStore->GetAllAnnotations())
					{
						if (Annotation.bVisible)
						{
							FUserAnnotation Updated = Annotation;
							Updated.bVisible = false;
							Updated.ModifiedAt = FDateTime::UtcNow();
							if (!PinnedStore->UpdateAnnotation(Updated))
							{
								bAnyFailed = true;
							}
						}
					}
					if (bAnyFailed)
					{
						NotifyAnnotationSaveFailed();
					}
				}
			}))
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowAllAnnotations", "Show All Annotations"),
			LOCTEXT("ShowAllAnnotations_Desc", "Show all annotations in the timing view"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Visible"),
			FUIAction(FExecuteAction::CreateLambda([WeakStore]()
			{
				TSharedPtr<FUserAnnotationStore> PinnedStore = WeakStore.Pin();
				if (PinnedStore.IsValid())
				{
					bool bAnyFailed = false;
					for (const FUserAnnotation& Annotation : PinnedStore->GetAllAnnotations())
					{
						if (!Annotation.bVisible)
						{
							FUserAnnotation Updated = Annotation;
							Updated.bVisible = true;
							Updated.ModifiedAt = FDateTime::UtcNow();
							if (!PinnedStore->UpdateAnnotation(Updated))
							{
								bAnyFailed = true;
							}
						}
					}
					if (bAnyFailed)
					{
						NotifyAnnotationSaveFailed();
					}
				}
			}))
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("AnnotationDisplay"), LOCTEXT("AnnotationDisplay", "Display"));
	{
		FTraceInsightsModule& Module = static_cast<FTraceInsightsModule&>(
			FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights"));
		FUserAnnotationsTimingViewExtender& Extender = Module.GetUserAnnotationsExtender();

		MenuBuilder.AddMenuEntry(
			LOCTEXT("PanelFloatingCallouts", "Floating Callouts"),
			LOCTEXT("PanelFloatingCallouts_Desc",
				"Show or hide floating annotation callouts on events"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([&Extender]()
				{
					Extender.ToggleFloatingAnnotations();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([&Extender]()
				{
					return Extender.GetShowFloatingAnnotations();
				})
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("AnnotationColumns"), LOCTEXT("AnnotationColumns", "Columns"));
	{
		TWeakPtr<SHeaderRow> WeakHeader = HeaderRowPtr;
		auto AddColumnToggle = [&MenuBuilder, WeakHeader](const FName& ColumnId, const FText& Label, const FText& Desc)
		{
			MenuBuilder.AddMenuEntry(
				Label, Desc, FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([WeakHeader, ColumnId]()
					{
						TSharedPtr<SHeaderRow> Header = WeakHeader.Pin();
						if (Header.IsValid())
						{
							Header->SetShowGeneratedColumn(ColumnId, !Header->IsColumnVisible(ColumnId));
						}
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([WeakHeader, ColumnId]()
					{
						TSharedPtr<SHeaderRow> Header = WeakHeader.Pin();
						return Header.IsValid() && Header->IsColumnVisible(ColumnId);
					})
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		};

		AddColumnToggle(UserAnnotationColumns::FramesColumnID,
			LOCTEXT("ShowFramesColumn", "Show Frames Column"),
			LOCTEXT("ShowFramesColumn_Desc", "Show the Frames column (hidden by default)."));
		AddColumnToggle(UserAnnotationColumns::TrackColumnID,
			LOCTEXT("ShowTrackColumn", "Show Track Column"),
			LOCTEXT("ShowTrackColumn_Desc", "Show the Track column. Shown by default because it surfaces \"(hidden) TrackName\" when an annotation's target track has been hidden in the timing view."));
		AddColumnToggle(UserAnnotationColumns::AuthorColumnID,
			LOCTEXT("ShowAuthorColumn", "Show Author Column"),
			LOCTEXT("ShowAuthorColumn_Desc", "Show the Author column (hidden by default)."));
		AddColumnToggle(UserAnnotationColumns::SourceColumnID,
			LOCTEXT("ShowSourceColumn", "Show Source Column"),
			LOCTEXT("ShowSourceColumn_Desc", "Show the Source column (hidden by default)."));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUserAnnotationsPanel::OnSearchTextChanged(const FText& NewText)
{
	SearchText = NewText.ToString();
	ApplyFilter();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SUserAnnotationsPanel::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Delete)
	{
		DeleteSelectedAnnotations();
		return FReply::Handled();
	}

	if (InKeyEvent.GetKey() == EKeys::F2)
	{
		EditSelectedAnnotation();
		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUserAnnotationsPanel::DeleteSelectedAnnotations()
{
	if (!ListView.IsValid())
	{
		return;
	}

	TSharedPtr<FUserAnnotationStore> Store = WeakAnnotationStore.Pin();
	if (!Store.IsValid())
	{
		return;
	}

	TArray<TSharedPtr<FUserAnnotation>> SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.Num() == 0)
	{
		return;
	}

	// Snapshot spans before delete so we only clear the ruler highlight if it matches one
	// of the deleted annotations (not an unrelated selection the user made).
	TArray<FGuid> IdsToRemove;
	TArray<TPair<double, double>> SpansRemoved;
	IdsToRemove.Reserve(SelectedItems.Num());
	SpansRemoved.Reserve(SelectedItems.Num());
	for (const TSharedPtr<FUserAnnotation>& Item : SelectedItems)
	{
		if (Item.IsValid())
		{
			IdsToRemove.Add(Item->Id);
			if (Item->HasEventAnchor())
			{
				SpansRemoved.Emplace(Item->EventStartTime, Item->EventEndTime);
			}
			else if (Item->IsRange())
			{
				SpansRemoved.Emplace(Item->Time, Item->EndTime);
			}
		}
	}

	int32 NumFailed = 0;
	for (const FGuid& Id : IdsToRemove)
	{
		if (!Store->RemoveAnnotation(Id))
		{
			++NumFailed;
		}
	}

	// Surface the failure explicitly — the banner at the top is a passive hint, but the
	// user just attempted an action and needs immediate feedback when it silently fails
	// (e.g. sidecar .ini or its folder is read-only).
	if (NumFailed > 0)
	{
		FNotificationInfo Info(FText::Format(
			LOCTEXT("AnnotationDeleteFailedFmt",
				"Failed to delete {0} annotation(s). The sidecar .ini file may be read-only."),
			FText::AsNumber(NumFailed)));
		Info.ExpireDuration = 5.0f;
		TSharedPtr<SNotificationItem> Item = FSlateNotificationManager::Get().AddNotification(Info);
		if (Item.IsValid())
		{
			Item->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}

	// Clear the ruler highlight only if it matches a deleted annotation's span.
	TSharedPtr<STimingView> TimingView = WeakHostTimingView.Pin();
	if (!TimingView.IsValid())
	{
		if (TSharedPtr<FTimingProfilerManager> Manager = FTimingProfilerManager::Get())
		{
			if (TSharedPtr<STimingProfilerWindow> Wnd = Manager->GetProfilerWindow())
			{
				TimingView = Wnd->GetTimingView();
			}
		}
	}
	if (TimingView.IsValid())
	{
		const double SelStart = TimingView->GetSelectionStartTime();
		const double SelEnd = TimingView->GetSelectionEndTime();
		const double Epsilon = 1e-6;
		for (const TPair<double, double>& Span : SpansRemoved)
		{
			if (FMath::IsNearlyEqual(SelStart, Span.Key, Epsilon)
				&& FMath::IsNearlyEqual(SelEnd, Span.Value, Epsilon))
			{
				TimingView->SelectTimeInterval(0.0, 0.0);
				break;
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SUserAnnotationsPanel::EditSelectedAnnotation()
{
	if (!ListView.IsValid())
	{
		return;
	}

	TArray<TSharedPtr<FUserAnnotation>> SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.Num() != 1 || !SelectedItems[0].IsValid())
	{
		return;
	}

	TSharedPtr<FUserAnnotationStore> Store = WeakAnnotationStore.Pin();
	if (!Store.IsValid())
	{
		return;
	}

	const FUserAnnotation& SelectedAnnotation = *SelectedItems[0];

	FTraceInsightsModule& Module = static_cast<FTraceInsightsModule&>(
		FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights"));
	FUserAnnotationsTimingViewExtender& Extender = Module.GetUserAnnotationsExtender();

	FUserAnnotationsTimingViewExtender::FAnnotationContext EditCtx =
		FUserAnnotationsTimingViewExtender::FAnnotationContext::FromAnnotation(SelectedAnnotation);

	const FGuid AnnotationId = SelectedAnnotation.Id;
	Extender.ShowAnnotationDialog(Store, EditCtx, &AnnotationId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef LOCTEXT_NAMESPACE
