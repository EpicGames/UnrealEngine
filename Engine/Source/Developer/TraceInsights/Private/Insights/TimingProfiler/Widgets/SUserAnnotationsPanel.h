// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

#include "Insights/TimingProfiler/ViewModels/TimerAggregator.h"

namespace UE::Insights::TimingProfiler
{

class FUserAnnotationStore;
class FUserAnnotationsTimingViewExtender;
class STimingView;
struct FUserAnnotation;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * SUserAnnotationsPanel
 *
 * Displays all user annotations in a list view with search, sort, and navigation.
 * Clicking a row navigates the timing view to the annotation location.
 */
class SUserAnnotationsPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SUserAnnotationsPanel) {}
		/** The hosting window's STimingView — navigate() drives this one, not the first
		 *  STimingView found globally. Leave unset to fall back to Timing Insights. */
		SLATE_ARGUMENT(TWeakPtr<STimingView>, HostTimingView)
	SLATE_END_ARGS()

	/** Constructs this widget. */
	void Construct(const FArguments& InArgs);

	virtual ~SUserAnnotationsPanel();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }

	/** Deletes all currently selected annotations. */
	void DeleteSelectedAnnotations();

	/** Opens the edit dialog for the single selected annotation. */
	void EditSelectedAnnotation();

private:
	/** Refreshes the filtered annotation list from the store. */
	void RefreshAnnotationList();

	/** Applies the current search filter to the annotation list. */
	void ApplyFilter();

	/** Navigates the timing view to the given annotation. */
	void NavigateToAnnotation(const FUserAnnotation& Annotation);

	/** List view callbacks. */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FUserAnnotation> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void OnSelectionChanged(TSharedPtr<FUserAnnotation> Item, ESelectInfo::Type SelectInfo);
	void OnListItemDoubleClicked(TSharedPtr<FUserAnnotation> Item);

	/** Context menu. */
	TSharedPtr<SWidget> OnContextMenuOpening();

	/** Search box callback. */
	void OnSearchTextChanged(const FText& NewText);

	/** Column sort callback. */
	void OnSortColumnHeader(EColumnSortPriority::Type SortPriority, const FName& ColumnId, EColumnSortMode::Type NewSortMode);

	/** Returns the current sort mode for the given column. */
	EColumnSortMode::Type GetSortModeForColumn(FName ColumnId) const;

private:
	/** Currently active sort column. */
	FName SortByColumn;

	/** Current sort direction. */
	EColumnSortMode::Type SortMode = EColumnSortMode::None;
	/** Weak reference to the annotation store. */
	TWeakPtr<FUserAnnotationStore> WeakAnnotationStore;

	/** Last known change number from the store, to detect updates. */
	uint64 LastChangeNumber = 0;

	/** All annotations (snapshot from store). */
	TArray<TSharedPtr<FUserAnnotation>> AllAnnotations;

	/** Filtered annotations (after search). */
	TArray<TSharedPtr<FUserAnnotation>> FilteredAnnotations;

	/** The list view widget. */
	TSharedPtr<SListView<TSharedPtr<FUserAnnotation>>> ListView;

	/** The list view's header row; used to toggle optional columns at runtime. */
	TSharedPtr<SHeaderRow> HeaderRowPtr;

	/** Current search text. */
	FString SearchText;

	/** Cached aggregation mode to detect changes and refresh frame display. */
	ETimerAggregationMode CachedAggregationMode = ETimerAggregationMode::Instance;

	/** Timing view in the same window as this panel. NavigateToAnnotation targets this. */
	TWeakPtr<STimingView> WeakHostTimingView;

	/** Subscription to host timing view's track visibility change; rebuilds rows so the
	 *  "(hidden) TrackName" indicator updates immediately when a track is toggled. */
	FDelegateHandle TrackVisibilityChangedHandle;

	/** Cached extender pointer — avoids LoadModuleChecked + GetUserAnnotationsExtender per Tick.
	 *  Safe under the same module-lifetime assumption as STimingView::CachedAnnotationExtender:
	 *  TraceInsights module is process-lifetime, extender is a value member, address stable. */
	FUserAnnotationsTimingViewExtender* CachedExtender = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler
