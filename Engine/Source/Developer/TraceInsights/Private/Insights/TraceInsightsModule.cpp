// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceInsightsModule.h"

#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"
#include "Logging/MessageLog.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

// TraceAnalysis
#include "Trace/StoreClient.h"
#include "Trace/StoreService.h"

// TraceServices
#include "TraceServices/ITraceServicesModule.h"
#include "TraceServices/Model/AnalysisSession.h"

// TraceInsightsCore
#include "InsightsCore/ITraceInsightsCoreModule.h"

// TraceInsights
#include "Insights/ContextSwitches/ContextSwitchesProfilerManager.h"
#include "Insights/CookProfiler/CookProfilerManager.h"
#include "Insights/ImportTool/TableImportTool.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/LoadingProfiler/LoadingProfilerManager.h"
#include "Insights/Log.h"
#include "Insights/MemoryProfiler/MemoryProfilerManager.h"
#include "Insights/MemoryProfiler/ViewModels/MemorySharedState.h"
#include "Insights/MemoryProfiler/Widgets/SMemAllocTableTreeView.h"
#include "Insights/MemoryProfiler/Widgets/SMemoryProfilerWindow.h"
#include "Insights/NetworkingProfiler/NetworkingProfilerManager.h"
#include "Insights/ObjectProfiler/ObjectProfilerManager.h"
#include "Insights/ObjectProfiler/Widgets/SObjectProfilerWindow.h"
#include "Insights/SpatialProfiler/SpatialProfilerManager.h"
#include "Insights/SpatialProfiler/Widgets/SSpatialInsightsWindow.h"
#include "Insights/TaskGraphProfiler/TaskGraphProfilerManager.h"
#include "Insights/Tests/InsightsTestRunner.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/UserAnnotationsTimingViewExtender.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY(TraceInsights);

namespace UE::Insights
{

IMPLEMENT_MODULE(FTraceInsightsModule, TraceInsights);

FString FTraceInsightsModule::UnrealInsightsLayoutIni;

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTraceInsightsModule
////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::StartupModule()
{
	LLM_SCOPE_BYTAG(Insights);

	ITraceInsightsCoreModule& TraceInsightsCoreModule = FModuleManager::LoadModuleChecked<ITraceInsightsCoreModule>("TraceInsightsCore");

	ITraceServicesModule& TraceServicesModule = FModuleManager::LoadModuleChecked<ITraceServicesModule>("TraceServices");
	TraceAnalysisService = TraceServicesModule.GetAnalysisService();
	TraceModuleService = TraceServicesModule.GetModuleService();

	FInsightsStyle::Initialize();
#if !WITH_EDITOR
	FAppStyle::SetAppStyleSet(FInsightsStyle::Get());
#endif

	// Register FInsightsManager first, as the main component (first to init, last to shutdown).
	RegisterComponent(FInsightsManager::CreateInstance(TraceAnalysisService.ToSharedRef(), TraceModuleService.ToSharedRef()));

	// Register other default components.
	RegisterComponent(UE::Insights::TimingProfiler::FTimingProfilerManager::CreateInstance());
	RegisterComponent(UE::Insights::LoadingProfiler::FLoadingProfilerManager::CreateInstance());
	RegisterComponent(UE::Insights::NetworkingProfiler::FNetworkingProfilerManager::CreateInstance());
	RegisterComponent(UE::Insights::MemoryProfiler::FMemoryProfilerManager::CreateInstance());
	RegisterComponent(UE::Insights::TaskGraphProfiler::FTaskGraphProfilerManager::CreateInstance());
	RegisterComponent(UE::Insights::ContextSwitches::FContextSwitchesProfilerManager::CreateInstance());
	RegisterComponent(UE::Insights::CookProfiler::FCookProfilerManager::CreateInstance());
	RegisterComponent(UE::Insights::ObjectProfiler::FObjectProfilerManager::CreateInstance());
	RegisterComponent(UE::Insights::SpatialProfiler::FSpatialProfilerManager::CreateInstance());
	RegisterComponent(Insights::FTableImportTool::CreateInstance());

	// Register User Annotations extender as a modular feature
	IModularFeatures::Get().RegisterModularFeature(
		UE::Insights::Timing::TimingViewExtenderFeatureName,
		&UserAnnotationsExtender);

#if !WITH_EDITOR
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	SourceCodeAccessModule.OnOpenFileFailed().AddRaw(this, &FTraceInsightsModule::HandleCodeAccessorOpenFileFailed);
#endif

	UnrealInsightsLayoutIni = GConfig->GetConfigFilename(TEXT("UnrealInsightsLayout"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::ShutdownModule()
{
	LLM_SCOPE_BYTAG(Insights);

#if !WITH_EDITOR
	if (FModuleManager::Get().IsModuleLoaded("SourceCodeAccess"))
	{
		ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::GetModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
		SourceCodeAccessModule.OnOpenFileFailed().RemoveAll(this);
	}
#endif

	// Unregister User Annotations extender
	IModularFeatures::Get().UnregisterModularFeature(
		UE::Insights::Timing::TimingViewExtenderFeatureName,
		&UserAnnotationsExtender);

	if (PersistentLayout.IsValid())
	{
		// Save application layout.
		FLayoutSaveRestore::SaveToConfig(UnrealInsightsLayoutIni, PersistentLayout.ToSharedRef());
		GConfig->Flush(false, UnrealInsightsLayoutIni);
	}

	UnregisterTabSpawners();

#define INSIGHTS_CHECK_SHARED_REFERENCES 1
#if INSIGHTS_CHECK_SHARED_REFERENCES
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	auto TimingInsightsWindow = UE::Insights::TimingProfiler::FTimingProfilerManager::Get()->GetProfilerWindow();
	auto AssetLoadingInsightsWindow = UE::Insights::LoadingProfiler::FLoadingProfilerManager::Get()->GetProfilerWindow();
	auto NetworkingInsightsWindow0 = UE::Insights::NetworkingProfiler::FNetworkingProfilerManager::Get()->GetProfilerWindow(0);
	auto MemoryInsightsWindow = UE::Insights::MemoryProfiler::FMemoryProfilerManager::Get()->GetProfilerWindow();
#endif

	// Unregister components. Shutdown in the reverse order they were registered.
	for (int32 ComponentIndex = Components.Num() - 1; ComponentIndex >= 0; --ComponentIndex)
	{
		Components[ComponentIndex]->Shutdown();
	}
	Components.Reset();

#if INSIGHTS_CHECK_SHARED_REFERENCES
	ensure(MemoryInsightsWindow.GetSharedReferenceCount() <= 1);
	MemoryInsightsWindow.Reset();

	ensure(NetworkingInsightsWindow0.GetSharedReferenceCount() <= 1);
	NetworkingInsightsWindow0.Reset();

	ensure(AssetLoadingInsightsWindow.GetSharedReferenceCount() <= 1);
	AssetLoadingInsightsWindow.Reset();

	ensure(TimingInsightsWindow.GetSharedReferenceCount() <= 1);
	TimingInsightsWindow.Reset();

	if (ensure(Session.GetSharedReferenceCount() <= 1))
	{
		Session.Reset();
	}
	else // Some component(s) failed to release the references to Session shared ptr!
	{
		UE_LOGF(TraceInsights, Warning, "The analysis Session is still referenced! Force delete!");
		const TraceServices::IAnalysisSession* SessionPtr = Session.Get();
		Session.Reset();
		delete SessionPtr;
	}
#endif
#undef INSIGHTS_CHECK_SHARED_REFERENCES

	FInsightsStyle::Shutdown();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::RegisterComponent(TSharedPtr<IInsightsComponent> Component)
{
	if (Component.IsValid())
	{
		LLM_SCOPE_BYTAG(Insights);
		Components.Add(Component.ToSharedRef());
		Component->Initialize(*this);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::UnregisterComponent(TSharedPtr<IInsightsComponent> Component)
{
	if (Component.IsValid())
	{
		LLM_SCOPE_BYTAG(Insights);
		Component->Shutdown();
		Components.Remove(Component.ToSharedRef());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<::Insights::IInsightsManager> FTraceInsightsModule::GetInsightsManager()
{
	return FInsightsManager::Get();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::CreateDefaultStore()
{
	ConnectToStore(TEXT("127.0.0.1"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FTraceInsightsModule::GetDefaultStoreDir()
{
	using UE::Trace::FStoreClient;

	TUniquePtr<FStoreClient> StoreClient(FStoreClient::Create(TEXT("localhost")));

	if (!StoreClient)
	{
		UE_LOGF(TraceInsights, Error, "Failed to connect to the store client.");
		return FString("");
	}

	const FStoreClient::FStatus* Status = StoreClient->GetStatus();
	if (!Status)
	{
		UE_LOGF(TraceInsights, Error, "Failed to get the status of the store client.");
		return FString("");
	}

	return FString(Status->GetStoreDir());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

UE::Trace::FStoreClient* FTraceInsightsModule::GetStoreClient()
{
	return FInsightsManager::Get()->GetStoreClient();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTraceInsightsModule::ConnectToStore(const TCHAR* InStoreHost, uint32 InStorePort)
{
	return FInsightsManager::Get()->ConnectToStore(InStoreHost, InStorePort);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::RegisterTabSpawners()
{
	// Allow components to register major tabs.
	for (TSharedRef<IInsightsComponent>& Component : Components)
	{
		Component->RegisterMajorTabs(*this);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::UnregisterTabSpawners()
{
	// Unregister major tabs in the reverse order they were registered.
	for (int32 ComponentIndex = Components.Num() - 1; ComponentIndex >= 0; --ComponentIndex)
	{
		Components[ComponentIndex]->UnregisterMajorTabs();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::CreateSessionViewer(bool bAllowDebugTools)
{
	RegisterTabSpawners();

#if !WITH_EDITOR || defined(__INTELLISENSE__)

	//////////////////////////////////////////////////
	// Create the main window.

#if PLATFORM_MAC
	const bool bEmbedTitleAreaContent = true;
#else
	const bool bEmbedTitleAreaContent = false;
#endif

	// Get desktop metrics. It also ensures the correct metrics will be used later in SWindow.
	FDisplayMetrics DisplayMetrics;
	FSlateApplication::Get().GetDisplayMetrics(DisplayMetrics);
	const float DPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(
		static_cast<float>(DisplayMetrics.PrimaryDisplayWorkAreaRect.Left),
		static_cast<float>(DisplayMetrics.PrimaryDisplayWorkAreaRect.Top));

	const FVector2D ClientSize(1280.f * DPIScaleFactor, 720.0f * DPIScaleFactor);

	TSharedRef<SWindow> RootWindow = SNew(SWindow)
		.Title(NSLOCTEXT("TraceInsightsModule", "UnrealInsightsAppName", "Unreal Insights"))
		.CreateTitleBar(!bEmbedTitleAreaContent)
		.SupportsMaximize(true)
		.SupportsMinimize(true)
		.IsInitiallyMaximized(false)
		.IsInitiallyMinimized(false)
		.SizingRule(ESizingRule::UserSized)
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.ClientSize(ClientSize)
		.AdjustInitialSizeAndPositionForDPIScale(false);

	const bool bShowRootWindowImmediately = false;
	FSlateApplication::Get().AddWindow(RootWindow, bShowRootWindowImmediately);

	FGlobalTabmanager::Get()->SetRootWindow(RootWindow);
	FGlobalTabmanager::Get()->SetAllowWindowMenuBar(true);

	FSlateNotificationManager::Get().SetRootWindow(RootWindow);

	//////////////////////////////////////////////////
	// Setup the window's content.

	TSharedRef<FTabManager::FLayout> DefaultLayout = FTabManager::NewLayout("UnrealInsightsLayout_v1.2");

	AddAreaForSessionViewer(DefaultLayout);

	AddAreaForWidgetReflector(DefaultLayout, false);

	// Load layout from ini file.
	PersistentLayout = FLayoutSaveRestore::LoadFromConfig(UnrealInsightsLayoutIni, DefaultLayout);

	// Restore application layout.
	const EOutputCanBeNullptr OutputCanBeNullptr = EOutputCanBeNullptr::Never;
	TSharedPtr<SWidget> Content = FGlobalTabmanager::Get()->RestoreFrom(PersistentLayout.ToSharedRef(), RootWindow, bEmbedTitleAreaContent, OutputCanBeNullptr);

	RootWindow->SetContent(
		SNew(SOverlay)

		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			Content.ToSharedRef()
		]

		+ SOverlay::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Top)
		.Padding(0.0f, 15.0f, 4.0f, 0.0f)
		[
			VersionWidgetVM->CreateWidget()
		]
	);
	RootWindow->GetOnWindowClosedEvent().AddRaw(this, &FTraceInsightsModule::OnWindowClosedEvent);

	//////////////////////////////////////////////////
	// Show the window.

	RootWindow->ShowWindow();
	const bool bForceWindowToFront = true;
	RootWindow->BringToFront(bForceWindowToFront);

#endif // !WITH_EDITOR
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::AddAreaForSessionViewer(TSharedRef<FTabManager::FLayout> Layout)
{
	TSharedRef<FTabManager::FStack> Stack = FTabManager::NewStack();

#if WITH_EDITOR
	// In editor, we default to all tabs closed.
	Stack->AddTab(FInsightsManagerTabs::SessionInfoTabId, ETabState::ClosedTab);
	Stack->AddTab(FInsightsManagerTabs::TimingProfilerTabId, ETabState::ClosedTab);
	Stack->AddTab(FInsightsManagerTabs::LoadingProfilerTabId, ETabState::ClosedTab);
	Stack->AddTab(FInsightsManagerTabs::NetworkingProfilerTabId, ETabState::ClosedTab);
	Stack->AddTab(FInsightsManagerTabs::MemoryProfilerTabId, ETabState::ClosedTab);
	Stack->AddTab(UE::Insights::ObjectProfiler::ObjectProfilerTabId, ETabState::ClosedTab);
	Stack->AddTab(UE::Insights::SpatialProfiler::SpatialProfilerTabId, ETabState::ClosedTab);
	//Stack->SetForegroundTab(FTabId(FInsightsManagerTabs::TimingProfilerTabId));

	// Create area for the main window.
	const float DPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(10.0f, 10.0f);
	Layout->AddArea
	(
		FTabManager::NewArea(1280.f * DPIScaleFactor, 720.0f * DPIScaleFactor)
		->Split(Stack)
	);
#else
	Stack->AddTab(FInsightsManagerTabs::SessionInfoTabId, ETabState::OpenedTab);
	Stack->AddTab(FInsightsManagerTabs::TimingProfilerTabId, ETabState::OpenedTab);
	Stack->AddTab(FInsightsManagerTabs::LoadingProfilerTabId, ETabState::ClosedTab);
	Stack->AddTab(FInsightsManagerTabs::NetworkingProfilerTabId, ETabState::ClosedTab);
	Stack->AddTab(FInsightsManagerTabs::MemoryProfilerTabId, ETabState::ClosedTab);
	Stack->AddTab(FInsightsManagerTabs::MessageLogTabId, ETabState::ClosedTab);
	Stack->AddTab(UE::Insights::ObjectProfiler::ObjectProfilerTabId, ETabState::ClosedTab);
	Stack->AddTab(UE::Insights::SpatialProfiler::SpatialProfilerTabId, ETabState::ClosedTab);
	Stack->SetForegroundTab(FTabId(FInsightsManagerTabs::TimingProfilerTabId));

	Layout->AddArea
	(
		FTabManager::NewPrimaryArea()
		->Split(Stack)
	);
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::AddAreaForWidgetReflector(TSharedRef<FTabManager::FLayout> Layout, bool bIsOpenedTab)
{
	const float DPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(10.0f, 10.0f);

	// Create area and tab for Slate's WidgetReflector.
	Layout->AddArea
	(
		FTabManager::NewArea(800.0f * DPIScaleFactor, 400.0f * DPIScaleFactor)
		->SetWindow(FVector2D(10.0f * DPIScaleFactor, 10.0f * DPIScaleFactor), false)
		->Split
		(
			FTabManager::NewStack()->AddTab(FTabId("WidgetReflector"), bIsOpenedTab ? ETabState::OpenedTab : ETabState::ClosedTab)
		)
	);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::ShutdownUserInterface()
{
	if (PersistentLayout.IsValid())
	{
		// Save application layout.
		FLayoutSaveRestore::SaveToConfig(UnrealInsightsLayoutIni, PersistentLayout.ToSharedRef());
		GConfig->Flush(false, UnrealInsightsLayoutIni);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::RegisterMajorTabConfig(const FName& InMajorTabId, const FInsightsMajorTabConfig& InConfig)
{
	TabConfigs.Add(InMajorTabId, InConfig);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::UnregisterMajorTabConfig(const FName& InMajorTabId)
{
	TabConfigs.Remove(InMajorTabId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FOnRegisterMajorTabExtensions& FTraceInsightsModule::OnRegisterMajorTabExtension(const FName& InMajorTabId)
{
	return MajorTabExtensionDelegates.FindOrAdd(InMajorTabId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FInsightsMajorTabConfig& FTraceInsightsModule::FindMajorTabConfig(const FName& InMajorTabId) const
{
	const FInsightsMajorTabConfig* FoundConfig = TabConfigs.Find(InMajorTabId);
	if (FoundConfig != nullptr)
	{
		return *FoundConfig;
	}

	static FInsightsMajorTabConfig DefaultConfig;
	return DefaultConfig;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FOnRegisterMajorTabExtensions* FTraceInsightsModule::FindMajorTabLayoutExtension(const FName& InMajorTabId)
{
	return MajorTabExtensionDelegates.Find(InMajorTabId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FString& FTraceInsightsModule::GetUnrealInsightsLayoutIni()
{
	return UnrealInsightsLayoutIni;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::SetUnrealInsightsLayoutIni(const FString& InIniPath)
{
	UnrealInsightsLayoutIni = InIniPath;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<const TraceServices::IAnalysisSession> FTraceInsightsModule::GetAnalysisSession() const
{
	return FInsightsManager::Get()->GetSession();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::SetAutoQuit(bool bInAutoQuit, bool bInWaitForSymbolResolver)
{
	FInsightsManager::Get()->SetAutoQuit(bInAutoQuit, bInWaitForSymbolResolver);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::StartAnalysisForTrace(uint32 InTraceId)
{
	if (InTraceId != 0)
	{
		FInsightsManager::Get()->LoadTrace(InTraceId);
	}
	UpdateAppTitle();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::StartAnalysisForLastLiveSession(float InRetryTime)
{
	FInsightsManager::Get()->LoadLastLiveSession(InRetryTime);
	UpdateAppTitle();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::StartAnalysisForTraceFile(const TCHAR* InTraceFile)
{
	if (InTraceFile != nullptr)
	{
		FInsightsManager::Get()->LoadTraceFile(FString(InTraceFile));
	}
	UpdateAppTitle();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint16 FTraceInsightsModule::StartAnalysisWithDirectTrace(const TCHAR* InStreamName, uint16 InPort)
{
	uint16 Port = FInsightsManager::Get()->ListenForDirectTrace(InStreamName, InPort);
	UpdateAppTitle();
	return Port;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
void FTraceInsightsModule::StartAnalysisWithSecureTrace(FStringView Host, uint16 Port, TConstArrayView<uint8> AuthorizationToken, FStringView CertificateFile, TConstArrayView<uint8> CertificateAuthority, FAnsiStringView Identity)
{
	FInsightsManager::Get()->ConnectToSecureTrace(Host, Port, AuthorizationToken, CertificateFile, CertificateAuthority, Identity);
	UpdateAppTitle();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::StartAnalysisWithStream(TUniquePtr<UE::Trace::IInDataStream>&& InStream, const TCHAR* InStreamName)
{
	FInsightsManager::Get()->StartAnalysisWithStream(MoveTemp(InStream), InStreamName);
	UpdateAppTitle();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::OnWindowClosedEvent(const TSharedRef<SWindow>&)
{
	for (TSharedRef<IInsightsComponent>& Component : Components)
	{
		Component->OnWindowClosedEvent();
	}

	FGlobalTabmanager::Get()->SaveAllVisualState();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::InitializeTesting(bool bInInitAutomationModules, bool bInAutoQuit)
{
#if !UE_BUILD_SHIPPING && !WITH_EDITOR
	auto TestRunner = FInsightsTestRunner::CreateInstance();

	TestRunner->SetInitAutomationModules(bInInitAutomationModules);
	TestRunner->SetAutoQuit(bInAutoQuit);

	RegisterComponent(TestRunner);
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::ScheduleCommand(const FString& InCmd)
{
	FString ActualCmd = InCmd;
	ActualCmd.TrimCharInline(TEXT('\"'), nullptr);
	ActualCmd.TrimCharInline(TEXT('\''), nullptr);

#if !UE_BUILD_SHIPPING && !WITH_EDITOR
	if (ActualCmd.StartsWith(TEXT("Automation RunTests")))
	{
		FInsightsTestRunner::Get()->ScheduleCommand(ActualCmd);
		return;
	}
#endif

	FInsightsManager::Get()->ScheduleCommand(ActualCmd);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTraceInsightsModule::Exec(const TCHAR* Cmd, FOutputDevice& Ar)
{
	for (TSharedRef<IInsightsComponent>& Component : Components)
	{
		if (Component->Exec(Cmd, Ar))
		{
			return true;
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::UpdateAppTitle()
{
	FInsightsManager::Get()->UpdateAppTitle();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::HandleCodeAccessorOpenFileFailed(const FString& Filename)
{
	FMessageLog ReportMessageLog(FInsightsManager::Get()->GetLogListingName());
	FText Text = FText::Format(NSLOCTEXT("TraceInsightsModule", "FailedToOpenSourceFile", "Failed to open source file!\n\"{0}\""), FText::FromString(Filename));
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning, Text);
	ReportMessageLog.AddMessage(Message);
	ReportMessageLog.Notify();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsModule::SetAssetInfoProvider(
	TSharedPtr<UE::Insights::ObjectProfiler::IAssetInfoProvider> InProvider)
{
	AssetInfoProvider = InProvider;

	// Apply immediately if the window is already open.
	if (TSharedPtr<UE::Insights::ObjectProfiler::FObjectProfilerManager> Manager =
		UE::Insights::ObjectProfiler::FObjectProfilerManager::Get())
	{
		if (TSharedPtr<UE::Insights::ObjectProfiler::SObjectProfilerWindow> Window =
			Manager->GetProfilerWindow())
		{
			Window->SetAssetInfoProvider(InProvider);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<UE::Insights::ObjectProfiler::IAssetInfoProvider>
FTraceInsightsModule::GetAssetInfoProvider() const
{
	return AssetInfoProvider;
}

void FTraceInsightsModule::OpenMemoryAllocationsView(double InTimeA, const FText& InTabDisplayName, const TSet<FString>& InPackageFilter)
{
	using namespace UE::Insights::MemoryProfiler;

	check(IsInGameThread());

	TSharedPtr<FMemoryProfilerManager> Manager = FMemoryProfilerManager::Get();
	if (!Manager.IsValid())
	{
		UE_LOGF(TraceInsights, Warning, "Cannot open Memory allocations view: Memory Profiler is not loaded.");
		return;
	}

	TSharedPtr<SMemoryProfilerWindow> Window = Manager->GetProfilerWindow();
	if (!Window.IsValid())
	{
		UE_LOGF(TraceInsights, Warning, "Cannot open Memory allocations view: Memory Profiler window is not open.");
		return;
	}

	FMemorySharedState& SharedState = Window->GetSharedState();

	TSharedPtr<FMemoryRuleSpec> ActiveAllocsRule;
	for (const TSharedPtr<FMemoryRuleSpec>& Rule : SharedState.GetMemoryRules())
	{
		if (Rule.IsValid() && Rule->GetValue() == TraceServices::IAllocationsProvider::EQueryRule::aAf)
		{
			ActiveAllocsRule = Rule;
			break;
		}
	}

	if (!ActiveAllocsRule.IsValid())
	{
		UE_LOGF(TraceInsights, Warning, "Cannot open Memory allocations view: 'aAf' rule not registered.");
		return;
	}

	TSharedPtr<SMemAllocTableTreeView> MemAllocTableTreeView = Window->OpenNewMemAllocTableTreeViewTab(InTabDisplayName);
	if (!MemAllocTableTreeView.IsValid())
	{
		UE_LOGF(TraceInsights, Warning, "Cannot open Memory allocations view: failed to create allocations tab.");
		return;
	}

	SMemAllocTableTreeView::FQueryParams QueryParams;
	QueryParams.Rule = ActiveAllocsRule;
	QueryParams.TimeMarkers[0] = InTimeA;
	QueryParams.PackageFilter = InPackageFilter;
	MemAllocTableTreeView->SetQueryParams(QueryParams);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
