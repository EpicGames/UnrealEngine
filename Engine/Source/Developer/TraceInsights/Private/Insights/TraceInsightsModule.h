// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Insights/IUnrealInsightsModule.h"
#include "Framework/Docking/TabManager.h"

// TraceInsightsCore
#include "InsightsCore/Common/MiscUtils.h"

// TraceInsights
#include "Insights/TimingProfiler/UserAnnotationsTimingViewExtender.h"

namespace Insights
{
	class IInsightsManager;
}

namespace TraceServices
{
	class IAnalysisService;
	class IModuleService;
}

class SDockTab;
class FSpawnTabArgs;
class SWindow;

namespace UE::Insights
{

/**
 * Implements the Trace Insights module.
 */
class FTraceInsightsModule : public IUnrealInsightsModule
{
public:
	FTraceInsightsModule()
		: VersionWidgetVM(MakeShared<FVersionWidget>())
	{
	}
	virtual ~FTraceInsightsModule() = default;

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	virtual void CreateDefaultStore() override;
	FString GetDefaultStoreDir();

	virtual UE::Trace::FStoreClient* GetStoreClient() override;
	virtual bool ConnectToStore(const TCHAR* InStoreHost, uint32 InStorePort=0) override;

	virtual void CreateSessionViewer(bool bAllowDebugTools = false) override;

	virtual TSharedPtr<::Insights::IInsightsManager> GetInsightsManager() override;
	virtual TSharedPtr<const TraceServices::IAnalysisSession> GetAnalysisSession() const override;

	virtual void SetAutoQuit(bool bInAutoQuit, bool bInWaitForSymbolResolver = false) override;

	virtual void StartAnalysisForTrace(uint32 InTraceId) override;
	virtual void StartAnalysisForLastLiveSession(float InRetryTime = 1.0f) override;
	virtual void StartAnalysisForTraceFile(const TCHAR* InTraceFile) override;
	virtual uint16 StartAnalysisWithDirectTrace(const TCHAR* InStreamName = nullptr, uint16 InPort = 0) override;
	virtual void StartAnalysisWithSecureTrace(FStringView Host, uint16 Port, TConstArrayView<uint8> AuthorizationToken, FStringView CertificateFile = FStringView(), TConstArrayView<uint8> CertificateAuthority = TConstArrayView<uint8>(), FAnsiStringView Identity = FAnsiStringView()) override;
	virtual void StartAnalysisWithStream(TUniquePtr<UE::Trace::IInDataStream>&& InStream, const TCHAR* InStreamName = nullptr) override;

	virtual void ShutdownUserInterface() override;

	virtual void RegisterComponent(TSharedPtr<IInsightsComponent> Component) override;
	virtual void UnregisterComponent(TSharedPtr<IInsightsComponent> Component) override;

	virtual void RegisterMajorTabConfig(const FName& InMajorTabId, const FInsightsMajorTabConfig& InConfig) override;
	virtual void UnregisterMajorTabConfig(const FName& InMajorTabId) override;
	virtual FOnInsightsMajorTabCreated& OnMajorTabCreated() override { return OnInsightsMajorTabCreatedDelegate; }
	virtual FOnRegisterMajorTabExtensions& OnRegisterMajorTabExtension(const FName& InMajorTabId) override;

	virtual const FInsightsMajorTabConfig& FindMajorTabConfig(const FName& InMajorTabId) const override;

	FOnRegisterMajorTabExtensions* FindMajorTabLayoutExtension(const FName& InMajorTabId);

	/** Retrieve ini path for saving persistent layout data. */
	static const FString& GetUnrealInsightsLayoutIni();

	/** Set the ini path for saving persistent layout data. */
	virtual void SetUnrealInsightsLayoutIni(const FString& InIniPath) override;

	virtual void InitializeTesting(bool InInitAutomationModules, bool InAutoQuit) override;
	virtual void ScheduleCommand(const FString& InCmd) override;
	virtual bool Exec(const TCHAR* Cmd, FOutputDevice& Ar) override;

	virtual void SetAssetInfoProvider(TSharedPtr<UE::Insights::ObjectProfiler::IAssetInfoProvider> InProvider) override;
	virtual TSharedPtr<UE::Insights::ObjectProfiler::IAssetInfoProvider> GetAssetInfoProvider() const override;

	virtual void OpenMemoryAllocationsView(double InTimeA, const FText& InTabDisplayName, const TSet<FString>& InPackageFilter) override;

	/** Returns the user annotations extender (for accessing annotation stores). */
	UE::Insights::TimingProfiler::FUserAnnotationsTimingViewExtender& GetUserAnnotationsExtender() { return UserAnnotationsExtender; }

private:
	void RegisterTabSpawners();
	void UnregisterTabSpawners();

	void AddAreaForSessionViewer(TSharedRef<FTabManager::FLayout> Layout);
	void AddAreaForWidgetReflector(TSharedRef<FTabManager::FLayout> Layout, bool bAllowDebugTools);

	/** Session Info */
	TSharedRef<SDockTab> SpawnSessionInfoTab(const FSpawnTabArgs& Args);

	void OnWindowClosedEvent(const TSharedRef<SWindow>&);

	void UpdateAppTitle();

	void HandleCodeAccessorOpenFileFailed(const FString& Filename);

private:
	TSharedPtr<TraceServices::IAnalysisService> TraceAnalysisService;
	TSharedPtr<TraceServices::IModuleService> TraceModuleService;

	TMap<FName, FInsightsMajorTabConfig> TabConfigs;
	TMap<FName, FOnRegisterMajorTabExtensions> MajorTabExtensionDelegates;

	FOnInsightsMajorTabCreated OnInsightsMajorTabCreatedDelegate;

	TSharedPtr<FTabManager::FLayout> PersistentLayout;
	static FString UnrealInsightsLayoutIni;

	TArray<TSharedRef<IInsightsComponent>> Components;

	TSharedPtr<UE::Insights::ObjectProfiler::IAssetInfoProvider> AssetInfoProvider;

	TSharedRef<FVersionWidget> VersionWidgetVM;

	/** User Annotations extender registered as a modular feature. */
	UE::Insights::TimingProfiler::FUserAnnotationsTimingViewExtender UserAnnotationsExtender;
};

} // namespace UE::Insights
