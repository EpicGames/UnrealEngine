// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/ViewModels/AxisViewportDouble.h"

#include "Widgets/Layout/SScrollBar.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FAxisViewportDouble::SetSize(const float InSize)
{
	if (!FMath::IsNearlyEqual(Size, InSize, SLATE_UNITS_TOLERANCE))
	{
		Size = InSize;
		OnSizeChanged();
		return true;
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FAxisViewportDouble::SetScale(const double NewScale)
{
	const double LocalNewScale = FMath::Clamp(NewScale, MinScale, MaxScale);
	if (LocalNewScale != Scale)
	{
		Scale = LocalNewScale;
		OnScaleChanged();
		return true;
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FAxisViewportDouble::ZoomWithFixedOffset(const double NewScale, const float Offset)
{
	const double LocalNewScale = FMath::Clamp(NewScale, MinScale, MaxScale);
	if (LocalNewScale != Scale)
	{
		// Value at Offset should remain the same. So we resolve equation:
		//   (Position + Offset) / Scale == (NewPosition + Offset) / NewScale
		//   ==> NewPosition = (Position + Offset) / Scale * NewScale - Offset
		Position = (Position + Offset) * static_cast<float>(LocalNewScale / Scale) - Offset;
		Scale = LocalNewScale;

		OnPositionChanged();
		OnScaleChanged();
		return true;
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FAxisViewportDouble::RelativeZoomWithFixedOffset(const float Delta, const float Offset)
{
	constexpr double ZoomStep = 0.25; // as percent

	double NewScale;

	if (Delta > 0)
	{
		NewScale = Scale * FMath::Pow(1.0 + ZoomStep, static_cast<double>(Delta));
	}
	else
	{
		NewScale = Scale * FMath::Pow(1.0 / (1.0 + ZoomStep), static_cast<double>(-Delta));
	}

	return ZoomWithFixedOffset(NewScale, Offset);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FAxisViewportDouble::UpdatePosWithinLimits()
{
	float MinPos, MaxPos;
	GetScrollLimits(MinPos, MaxPos);
	return EnforceScrollLimits(MinPos, MaxPos, 0.5);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAxisViewportDouble::GetScrollLimits(float& OutMinPos, float& OutMaxPos)
{
	if (MaxPosition - MinPosition < Size)
	{
		OutMinPos = MaxPosition - Size;
		OutMaxPos = MinPosition;
	}
	else
	{
		constexpr float ExtraSizeFactor = 0.15f; // allow extra 15% on sides
		OutMinPos = MinPosition - ExtraSizeFactor * Size;
		OutMaxPos = MaxPosition - (1.0f - ExtraSizeFactor) * Size;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FAxisViewportDouble::EnforceScrollLimits(const float InMinPos, const float InMaxPos, const float InterpolationFactor)
{
	float Pos = Position;

	if (Pos < InMinPos)
	{
		Pos = InterpolationFactor * Pos + (1.0f - InterpolationFactor) * InMinPos;

		if (FMath::IsNearlyEqual(Pos, InMinPos, 0.5f))
		{
			Pos = InMinPos;
		}
	}
	else if (Pos > InMaxPos)
	{
		Pos = InterpolationFactor * Pos + (1.0f - InterpolationFactor) * InMaxPos;

		if (FMath::IsNearlyEqual(Pos, InMaxPos, 0.5f))
		{
			Pos = InMaxPos;
		}

		if (Pos < InMinPos)
		{
			Pos = InMinPos;
		}
	}

	return ScrollAtPos(Pos);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAxisViewportDouble::OnUserScrolled(TSharedPtr<SScrollBar> ScrollBar, float ScrollOffset)
{
	const float SX = 1.0f / (MaxPosition - MinPosition);
	const float ThumbSizeFraction = FMath::Clamp<float>(Size * SX, 0.0f, 1.0f);
	const float OffsetFraction = FMath::Clamp<float>(ScrollOffset, 0.0f, 1.0f - ThumbSizeFraction);

	const float Pos = MinPosition + OffsetFraction * (MaxPosition - MinPosition);
	ScrollAtPos(Pos);

	ScrollBar->SetState(OffsetFraction, ThumbSizeFraction);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAxisViewportDouble::UpdateScrollBar(TSharedPtr<SScrollBar> ScrollBar) const
{
	const float SX = 1.0f / (MaxPosition - MinPosition);
	const float ThumbSizeFraction = FMath::Clamp<float>(Size * SX, 0.0f, 1.0f);
	const float ScrollOffset = (Position - MinPosition) * SX;
	const float OffsetFraction = FMath::Clamp<float>(ScrollOffset, 0.0f, 1.0f - ThumbSizeFraction);

	ScrollBar->SetState(OffsetFraction, ThumbSizeFraction);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
