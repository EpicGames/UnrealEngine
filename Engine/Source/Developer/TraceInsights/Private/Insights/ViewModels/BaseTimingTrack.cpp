// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/ViewModels/BaseTimingTrack.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Insights/ViewModels/TimingViewLayout.h"

#define LOCTEXT_NAMESPACE "BaseTimingTrack"

INSIGHTS_IMPLEMENT_RTTI(FBaseTimingTrack)

// start auto generated ids from a big number (MSB set to 1) to avoid collisions with ids for GPU/CPU tracks based on 32bit timeline index
uint64 FBaseTimingTrack::IdGenerator = (1ULL << 63);

// The default destructor generates deprecation warnings for any member variables marked as
// deprecated, so we need to suppress that.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FBaseTimingTrack::~FBaseTimingTrack() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FBaseTimingTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
}

void FBaseTimingTrack::AddChildTrack(TSharedRef<FBaseTimingTrack> Track)
{
	ChildTracks.AddUnique(Track);
}

void FBaseTimingTrack::AddChildTrack(TSharedRef<FBaseTimingTrack> Track, int32 Index)
{
	ChildTracks.Insert(Track, Index);
}

void FBaseTimingTrack::RemoveChildTrack(TSharedRef<FBaseTimingTrack> Track)
{
	ChildTracks.Remove(Track);
}

TSharedPtr<FBaseTimingTrack> FBaseTimingTrack::FindChildTrackOfType(FName TrackTypeName)
{
	const TSharedRef<FBaseTimingTrack>* FoundTrack = ChildTracks.FindByPredicate(
		[&TrackTypeName](const TSharedRef<FBaseTimingTrack>& Track)
		{
			return Track->GetTypeName() == TrackTypeName;
		});

	return FoundTrack ? *FoundTrack : TSharedPtr<FBaseTimingTrack>{};
}

float FBaseTimingTrack::GetChildTracksTopHeight(const FTimingViewLayout& Layout) const
{
	float TotalHeight = 0.0f;
	for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
	{
		float TrackHeight = Track->GetHeight();
		TotalHeight += TrackHeight;
		if (TrackHeight > 0.0f)
		{
			TotalHeight += Layout.ChildTimelineDY;
		}
	}
	return TotalHeight;
}

void FBaseTimingTrack::UpdateChildTracksPosY(const FTimingViewLayout& Layout)
{
	float RelativeChildTrackY = 0.0f;
	for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
	{
		Track->SetPosY(GetPosY() + RelativeChildTrackY);
		Track->UpdateChildTracksPosY(Layout);
		RelativeChildTrackY += Track->GetHeight() + Layout.ChildTimelineDY;
	}
}

void FBaseTimingTrack::SetLocation(ETimingTrackLocation InLocation)
{ 
	for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
	{
		Track->SetLocation(InLocation);
	}

	Location = InLocation;
	OnLocationChanged();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TSharedPtr<FBaseTimingTrack> FBaseTimingTrack::GetChildTrack() const
{
	return ChildTrack;
}

void FBaseTimingTrack::SetChildTrack(TSharedPtr<FBaseTimingTrack> InTrack)
{
	if (ChildTrack.IsValid())
	{
		RemoveChildTrack(ChildTrack.ToSharedRef());
	}
	ChildTrack = InTrack;
	if (InTrack.IsValid())
	{
		AddChildTrack(InTrack.ToSharedRef());
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE