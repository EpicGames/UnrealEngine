// Copyright Epic Games, Inc. All Rights Reserved.

#include "DrawHelpers.h"

#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/ViewModels/TimingTrackViewport.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDrawHelpers::DrawBackground(
	const UE::Insights::FDrawContext& DrawContext,
	const FSlateBrush* BackgroundAreaBrush,
	const FLinearColor& ValidAreaColor,
	const FLinearColor& InvalidAreaColor,
	const FLinearColor& EdgeColor,
	const float X0,
	const float X1,
	const float X2,
	const float X3,
	const float Y,
	const float H,
	float& OutValidAreaX,
	float& OutValidAreaW)
{
	//  <------- W ------->
	//  X0    X1    X2     X3
	//  ++++++|*****|++++++
	//  ++++++|*****|++++++
	//  ++++++|*****|++++++

	if (X1 >= X3 || X2 <= X0)
	{
		OutValidAreaX = X0;
		OutValidAreaW = X3 - X0;

		// Draw invalid area (entire view).
		DrawContext.DrawBox(X0, Y, X3 - X0, H, BackgroundAreaBrush, InvalidAreaColor);
	}
	else // X1 < X3 && X2 > X0
	{
		if (X1 > X0)
		{
			// Draw invalid area (left).
			DrawContext.DrawBox(X0, Y, X1 - X0, H, BackgroundAreaBrush, InvalidAreaColor);
		}

		if (X2 < X3)
		{
			// Draw invalid area (right).
			DrawContext.DrawBox(X2 + 1.0f, Y, X3 - X2 - 1.0f, H, BackgroundAreaBrush, InvalidAreaColor);

			// Draw the right edge (end time).
			DrawContext.DrawBox(X2, Y, 1.0f, H, BackgroundAreaBrush, EdgeColor);
		}

		float ValidAreaX = FMath::Max(X1, X0);
		float ValidAreaW = FMath::Min(X2, X3) - ValidAreaX;

		if (X1 >= X0)
		{
			// Draw the left edge (start time).
			DrawContext.DrawBox(X1, Y, 1.0f, H, BackgroundAreaBrush, EdgeColor);

			// Adjust valid area to not overlap the left edge.
			ValidAreaX += 1.0f;
			ValidAreaW -= 1.0f;
		}

		if (ValidAreaW > 0.0f)
		{
			// Draw valid area.
			DrawContext.DrawBox(ValidAreaX, Y, ValidAreaW, H, BackgroundAreaBrush, ValidAreaColor);
		}

		OutValidAreaX = ValidAreaX;
		OutValidAreaW = ValidAreaW;
	}

	DrawContext.LayerId++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDrawHelpers::DrawBackground(const UE::Insights::FDrawContext& DrawContext,
								  const FSlateBrush* BackgroundAreaBrush,
								  const float X0,
								  const float X1,
								  const float X2,
								  const float X3,
								  const float Y,
								  const float H)
{
	const FLinearColor ValidAreaColor(0.07f, 0.07f, 0.07f, 1.0f);
	const FLinearColor InvalidAreaColor(0.1f, 0.07f, 0.07f, 1.0f);
	const FLinearColor EdgeColor(0.05f, 0.05f, 0.05f, 1.0f);

	float ValidAreaX, ValidAreaW;
	FDrawHelpers::DrawBackground(DrawContext, BackgroundAreaBrush, ValidAreaColor, InvalidAreaColor, EdgeColor, X0, X1, X2, X3, Y, H, ValidAreaX, ValidAreaW);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDrawHelpers::DrawBackground(const UE::Insights::FDrawContext& DrawContext,
								  const FSlateBrush* BackgroundAreaBrush,
								  const FTimingTrackViewport& Viewport,
								  const float Y,
								  const float H)
{
	const FLinearColor ValidAreaColor(0.07f, 0.07f, 0.07f, 1.0f);
	const FLinearColor InvalidAreaColor(0.1f, 0.07f, 0.07f, 1.0f);
	const FLinearColor EdgeColor(0.05f, 0.05f, 0.05f, 1.0f);

	const float X0 = 0.0f;
	const float X1 = Viewport.TimeToSlateUnitsRounded(Viewport.GetMinValidTime());
	const float X2 = Viewport.TimeToSlateUnitsRounded(Viewport.GetMaxValidTime());
	const float X3 = FMath::CeilToFloat(Viewport.GetWidth());

	float ValidAreaX, ValidAreaW;
	FDrawHelpers::DrawBackground(DrawContext, BackgroundAreaBrush, ValidAreaColor, InvalidAreaColor, EdgeColor, X0, X1, X2, X3, Y, H, ValidAreaX, ValidAreaW);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDrawHelpers::DrawBackground(const UE::Insights::FDrawContext& DrawContext,
								  const FSlateBrush* BackgroundAreaBrush,
								  const FTimingTrackViewport& Viewport,
								  const float Y,
								  const float H,
								  float& OutValidAreaX,
								  float& OutValidAreaW)
{
	const FLinearColor ValidAreaColor(0.07f, 0.07f, 0.07f, 1.0f);
	const FLinearColor InvalidAreaColor(0.1f, 0.07f, 0.07f, 1.0f);
	const FLinearColor EdgeColor(0.05f, 0.05f, 0.05f, 1.0f);

	const float X0 = 0.0f;
	const float X1 = Viewport.TimeToSlateUnitsRounded(Viewport.GetMinValidTime());
	const float X2 = Viewport.TimeToSlateUnitsRounded(Viewport.GetMaxValidTime());
	const float X3 = FMath::CeilToFloat(Viewport.GetWidth());

	FDrawHelpers::DrawBackground(DrawContext, BackgroundAreaBrush, ValidAreaColor, InvalidAreaColor, EdgeColor, X0, X1, X2, X3, Y, H, OutValidAreaX, OutValidAreaW);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDrawHelpers::DrawTimeRangeSelection(
	const UE::Insights::FDrawContext& DrawContext,
	const FTimingTrackViewport& Viewport,
	const double StartTime,
	const double EndTime,
	const FSlateBrush* Brush,
	const FSlateFontInfo& Font)
{
	if (EndTime > StartTime)
	{
		const float MinX = 0.0f;
		const float MaxX = Viewport.GetWidth();

		float SelectionX1 = Viewport.TimeToSlateUnitsRounded(StartTime);
		float SelectionX2 = Viewport.TimeToSlateUnitsRounded(EndTime);

		if (SelectionX1 <= MaxX && SelectionX2 >= MinX)
		{
			const float Y = Viewport.GetPosY();
			const float H = Viewport.GetHeight();

			const double Duration = EndTime - StartTime;
			const FString Text = UE::Insights::FormatTimeAuto(Duration);

			DrawSelection(DrawContext, MinX, MaxX, SelectionX1, SelectionX2, Y, H, 6.0f, Text, Brush, Font);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FDrawHelpers::DrawSelection(
	const UE::Insights::FDrawContext& DrawContext,
	const float MinX,
	const float MaxX,
	float SelectionX1,
	float SelectionX2,
	const float Y,
	const float H,
	const float TextY,
	const FString Text,
	const FSlateBrush* Brush,
	const FSlateFontInfo& Font)
{
	float ClipLeft = 0.0f;
	float ClipRight = 0.0f;
	if (SelectionX1 < MinX)
	{
		ClipLeft = MinX - SelectionX1;
		SelectionX1 = MinX;
	}
	if (SelectionX2 > MaxX)
	{
		ClipRight = SelectionX2 - MaxX;
		SelectionX2 = MaxX;
	}

	const FLinearColor BackgroundColor(0.25f, 0.5f, 1.0f, 0.25f);

	// Draw selection area.
	DrawContext.DrawBox(SelectionX1, Y, SelectionX2 - SelectionX1, H, Brush, BackgroundColor);
	DrawContext.LayerId++;

	const FColor ArrowFillColor(32, 64, 128, 255);
	const FLinearColor ArrowColor(ArrowFillColor);

	if (SelectionX1 > MinX)
	{
		// Draw left side (vertical line).
		DrawContext.DrawBox(SelectionX1 - 1.0f, Y, 1.0f, H, Brush, ArrowColor);
	}

	if (SelectionX2 < MaxX)
	{
		// Draw right side (vertical line).
		DrawContext.DrawBox(SelectionX2, Y, 1.0f, H, Brush, ArrowColor);
	}

	DrawContext.LayerId++;

	const float ArrowSize = 6.0f;
	const float ArrowY = Y + TextY;

	if (SelectionX2 - SelectionX1 > 2 * ArrowSize)
	{
		// Draw horizontal line.
		float HorizLineX1 = SelectionX1;
		if (ClipLeft == 0.0f)
		{
			HorizLineX1 += 1.0f;
		}
		float HorizLineX2 = SelectionX2;
		if (ClipRight == 0.0f)
		{
			HorizLineX2 -= 1.0f;
		}
		DrawContext.DrawBox(HorizLineX1, ArrowY - 1.0f, HorizLineX2 - HorizLineX1, 3.0f, Brush, ArrowColor);

		if (ClipLeft < ArrowSize)
		{
			// Draw left arrow.
			for (float AH = 0.0f; AH < ArrowSize; AH += 1.0f)
			{
				DrawContext.DrawBox(SelectionX1 - ClipLeft + AH, ArrowY - AH, 1.0f, 2.0f * AH + 1.0f, Brush, ArrowColor);
			}
		}

		if (ClipRight < ArrowSize)
		{
			// Draw right arrow.
			for (float AH = 0.0f; AH < ArrowSize; AH += 1.0f)
			{
				DrawContext.DrawBox(SelectionX2 + ClipRight - AH - 1.0f, ArrowY - AH, 1.0f, 2.0f * AH + 1.0f, Brush, ArrowColor);
			}
		}

		DrawContext.LayerId++;

#if 0
		//im: This should be a more efficeint way top draw the arrows, but it renders them with artifacts (missing vertical lines; shader bug?)!

		const FSlateBrush* MyBrush = WhiteBrush;
		FSlateResourceHandle ResourceHandle = FSlateApplication::Get().GetRenderer()->GetResourceHandle(*MyBrush);
		const FSlateShaderResourceProxy* ResourceProxy = ResourceHandle.GetResourceProxy();

		FVector2D AtlasOffset = ResourceProxy ? ResourceProxy->StartUV : FVector2D(0.0f, 0.0f);
		FVector2D AtlasUVSize = ResourceProxy ? ResourceProxy->SizeUV : FVector2D(1.0f, 1.0f);

		const FVector2D Pos = AllottedGeometry.GetAbsolutePosition() + FVector2D(0.0f, 40.0f);
		const float Scale = AllottedGeometry.Scale;

		FSlateRenderTransform RenderTransform;

		TArray<FSlateVertex> Verts;
		Verts.Reserve(6);
		Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, Pos + FVector2D(0.5f + SelectionX1 + ArrowSize, 0.5f + ArrowY + ArrowSize) * Scale, AtlasOffset + FVector2D(0.0f, 1.0f) * AtlasUVSize, ArrowFillColor));
		Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, Pos + FVector2D(0.5f + SelectionX1, 0.5f + ArrowY) * Scale, AtlasOffset + FVector2D(1.0f, 0.5f) * AtlasUVSize, ArrowFillColor));
		Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, Pos + FVector2D(0.5f + SelectionX1 + ArrowSize, 0.5f + ArrowY - ArrowSize) * Scale, AtlasOffset + FVector2D(0.0f, 0.0f) * AtlasUVSize, ArrowFillColor));
		Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, Pos + FVector2D(0.5f + SelectionX2 - ArrowSize, 0.5f + ArrowY - ArrowSize) * Scale, AtlasOffset + FVector2D(0.0f, 0.0f) * AtlasUVSize, ArrowFillColor));
		Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, Pos + FVector2D(0.5f + SelectionX2, 0.5f + ArrowY) * Scale, AtlasOffset + FVector2D(1.0f, 0.5f) * AtlasUVSize, ArrowFillColor));
		Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, Pos + FVector2D(0.5f + SelectionX2 - ArrowSize, 0.5f + ArrowY + ArrowSize) * Scale, AtlasOffset + FVector2D(0.0f, 1.0f) * AtlasUVSize, ArrowFillColor));

		TArray<SlateIndex> Indices;
		Indices.Reserve(6);
		if (ClipLeft < ArrowSize)
		{
			Indices.Add(0);
			Indices.Add(1);
			Indices.Add(2);
		}
		if (ClipRight < ArrowSize)
		{
			Indices.Add(3);
			Indices.Add(4);
			Indices.Add(5);
		}

		FSlateDrawElement::MakeCustomVerts(
			OutDrawElements,
			LayerId,
			ResourceHandle,
			Verts,
			Indices,
			nullptr,
			0,
			0,
			ESlateDrawEffect::PreMultipliedAlpha);

		DrawContext.LayerId++;
#endif
	}

	//////////////////////////////////////////////////
	// Draw text info for selected interval.

	if (!Text.IsEmpty())
	{
		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const float FontScale = DrawContext.Geometry.Scale;
		const float TextWidth = static_cast<float>(FontMeasureService->Measure(Text, Font, FontScale).X / FontScale);

		const float CenterX = (SelectionX1 + SelectionX2) / 2.0f;

		DrawContext.DrawBox(CenterX - TextWidth / 2.0f - 2.0f, ArrowY - 6.0f, TextWidth + 4.0f, 13.0f, Brush, ArrowColor);
		DrawContext.LayerId++;

		DrawContext.DrawText(CenterX - TextWidth / 2.0f, ArrowY - 6.0f, Text, Font, FLinearColor::White);
		DrawContext.LayerId++;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
