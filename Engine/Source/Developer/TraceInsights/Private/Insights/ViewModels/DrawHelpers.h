// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Fonts/SlateFontInfo.h"
#include "Math/Color.h"

struct FSlateBrush;

namespace UE::Insights { class FDrawContext; }

class FTimingTrackViewport;

class FDrawHelpers
{
public:
	/**
	 * Draw background.
	 * X1-X2 specifies the valid area. X0-X1 and X2-X3 specifies the invalid areas.
	 *
	 *  X0    X1    X2     X3
	 *  ++++++|*****|++++++
	 *  ++++++|*****|++++++
	 *  ++++++|*****|++++++
	 */
	static void DrawBackground(const UE::Insights::FDrawContext& DrawContext,
							   const FSlateBrush* BackgroundAreaBrush,
							   const FLinearColor& ValidAreaColor,
							   const FLinearColor& InvalidAreaColor,
							   const FLinearColor& EdgeColor,
							   const float X0,
							   const float X1,
							   const float X2,
							   const float X3,
							   const float Y,
							   const float H,
							   float& OutValidAreaX,
							   float& OutValidAreaW);

	static void DrawBackground(const UE::Insights::FDrawContext& DrawContext,
							   const FSlateBrush* BackgroundAreaBrush,
							   const float X0,
							   const float X1,
							   const float X2,
							   const float X3,
							   const float Y,
							   const float H);

	static void DrawBackground(const UE::Insights::FDrawContext& DrawContext,
							   const FSlateBrush* BackgroundAreaBrush,
							   const FTimingTrackViewport& Viewport,
							   const float Y,
							   const float H);

	static void DrawBackground(const UE::Insights::FDrawContext& DrawContext,
							   const FSlateBrush* BackgroundAreaBrush,
							   const FTimingTrackViewport& Viewport,
							   const float Y,
							   const float H,
							   float& OutValidAreaX,
							   float& OutValidAreaW);

	/**
	 * Draw time range selection.
	 */
	static void DrawTimeRangeSelection(const UE::Insights::FDrawContext& DrawContext,
									   const FTimingTrackViewport& Viewport,
									   const double StartTime,
									   const double EndTime,
									   const FSlateBrush* Brush,
									   const FSlateFontInfo& Font);

	static void DrawSelection(const UE::Insights::FDrawContext& DrawContext,
							  const float MinX,
							  const float MaxX,
							  float SelectionX1,
							  float SelectionX2,
							  const float Y,
							  const float H,
							  const float TextY,
							  const FString Text,
							  const FSlateBrush* Brush,
							  const FSlateFontInfo& Font);
};
