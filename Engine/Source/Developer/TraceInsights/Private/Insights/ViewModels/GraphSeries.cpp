// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/ViewModels/GraphSeries.h"

// TraceInsights
#include "Insights/ViewModels/GraphTrackEvent.h"
#include "Insights/ViewModels/TimingTrackViewport.h"

#define LOCTEXT_NAMESPACE "GraphTrack"

////////////////////////////////////////////////////////////////////////////////////////////////////
// FGraphSeries
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FGraphSeries)

////////////////////////////////////////////////////////////////////////////////////////////////////

FGraphSeries::FGraphSeries()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FGraphSeries::~FGraphSeries()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FGraphSeriesEvent* FGraphSeries::GetEvent(const float X, const float Y, const FTimingTrackViewport& Viewport, bool bCheckLine, bool bCheckBox) const
{
	const float LocalBaselineY = static_cast<float>(GetBaselineY());

	for (const FGraphSeriesEvent& Event : Events)
	{
		const float EventX1 = Viewport.TimeToSlateUnitsRounded(Event.Time);
		const float EventX2 = Viewport.TimeToSlateUnitsRounded(Event.Time + Event.Duration);

		const float EventY = GetRoundedYForValue(Event.Value);

		// Check bounding box of the visual point.
		constexpr float PointTolerance = 5.0f;
		if (X >= EventX1 - PointTolerance && X <= EventX1 + PointTolerance &&
			Y >= EventY - PointTolerance && Y <= EventY + PointTolerance)
		{
			return &Event;
		}

		if (bCheckLine)
		{
			// Check bounding box of the horizontal line.
			constexpr float LineTolerance = 2.0f;
			if (X >= EventX1 - LineTolerance && X <= EventX2 + LineTolerance &&
				Y >= EventY - LineTolerance && Y <= EventY + LineTolerance)
			{
				return &Event;
			}
		}

		if (bCheckBox)
		{
			// Check bounding box of the visual box.
			constexpr float BoxTolerance = 1.0f;
			if (X >= EventX1 - BoxTolerance && X <= EventX2 + BoxTolerance)
			{
				if (EventY < LocalBaselineY)
				{
					if (Y >= EventY - BoxTolerance && Y <= LocalBaselineY + BoxTolerance)
					{
						return &Event;
					}
				}
				else
				{
					if (Y >= LocalBaselineY - BoxTolerance && Y <= EventY + BoxTolerance)
					{
						return &Event;
					}
				}
			}
		}
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FGraphSeries::FormatValue(double Value) const
{
	return FString::Printf(TEXT("%g"), Value);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FGraphSeries::UpdateAutoZoomEx(const float InTopY, const float InBottomY, const double InMinEventValue, const double InMaxEventValue, const bool bIsAutoZoomAnimated)
{
	bool bViewportChanged = false;
	bIsAutoZoomDirty = false;

	const double LowValue = GetValueForY(InBottomY);
	const double HighValue = GetValueForY(InTopY);

	double MinValue = InMinEventValue;
	double MaxValue = InMaxEventValue;

	// If MinValue == MaxValue, we keep the previous baseline and scale, but only if the min/max value is already visible.
	if (MinValue == MaxValue && (MinValue < LowValue || MaxValue > HighValue))
	{
		MinValue = FMath::Min(MinValue, LowValue);
		MaxValue = FMath::Max(MaxValue, HighValue);
	}

	if (MinValue < MaxValue)
	{
		if (bIsAutoZoomAnimated)
		{
			// Interpolate the min-max interval (animating the vertical position and scale of the graph series).
			constexpr double InterpolationSpeed = 0.5;
			const double NewMinValue = InterpolationSpeed * MinValue + (1.0 - InterpolationSpeed) * LowValue;
			const double NewMaxValue = InterpolationSpeed * MaxValue + (1.0 - InterpolationSpeed) * HighValue;

			// Check if we reach the target min-max interval.
			const double ErrorTolerance = 0.5 / GetScaleY(); // delta value for dy ~= 0.5 pixels
			if (!FMath::IsNearlyEqual(NewMinValue, MinValue, ErrorTolerance) ||
				!FMath::IsNearlyEqual(NewMaxValue, MaxValue, ErrorTolerance))
			{
				MinValue = NewMinValue;
				MaxValue = NewMaxValue;

				// Request a new update so we can further interpolate the min-max interval.
				bIsAutoZoomDirty = true;
			}
		}

		double NewBaselineY;
		double NewScaleY;
		ComputeBaselineAndScale(MinValue, MaxValue, InTopY, InBottomY, NewBaselineY, NewScaleY);

		if (NewBaselineY != GetBaselineY() || NewScaleY != GetScaleY())
		{
			bViewportChanged = true;
		}

		SetBaselineY(NewBaselineY);
		SetScaleY(NewScaleY);
	}
	else
	{
		// If MinValue == MaxValue, we keep the previous baseline and scale.
	}

	return bViewportChanged;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphSeries::UpdateAutoZoom(const float InTopY, const float InBottomY, const double InMinEventValue, const double InMaxEventValue, const bool bIsAutoZoomAnimated)
{
	if (IsAutoZoomEnabled())
	{
		UpdateAutoZoomEx(InTopY, InBottomY, InMinEventValue, InMaxEventValue, bIsAutoZoomAnimated);

		if (bIsAutoZoomDirty)
		{
			SetDirtyFlag();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
