// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/ViewModels/GraphTrack.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Rendering/DrawElements.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBox.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/ViewModels/DrawHelpers.h"
#include "Insights/ViewModels/GraphSeries.h"
#include "Insights/ViewModels/GraphTrackBuilder.h"
#include "Insights/ViewModels/GraphTrackEvent.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingViewDrawHelper.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Insights/Widgets/SGraphSeriesList.h"

#define LOCTEXT_NAMESPACE "GraphTrack"

////////////////////////////////////////////////////////////////////////////////////////////////////
// FGraphTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FGraphTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

FGraphTrack::FGraphTrack()
	: FBaseTimingTrack()
	, WhiteBrush(FInsightsStyle::Get().GetBrush("WhiteBrush"))
	, PointBrush(FInsightsStyle::GetBrush("Graph.Point"))
	, BorderBrush(FInsightsStyle::Get().GetBrush("SingleBorder"))
	, Font(FAppStyle::Get().GetFontStyle("SmallFont"))
{
	SetValidLocations(ETimingTrackLocation::Scrollable | ETimingTrackLocation::TopDocked | ETimingTrackLocation::BottomDocked);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FGraphTrack::FGraphTrack(const FString& InName)
	: FBaseTimingTrack(InName)
	, WhiteBrush(FInsightsStyle::Get().GetBrush("WhiteBrush"))
	, PointBrush(FInsightsStyle::GetBrush("Graph.Point"))
	, BorderBrush(FInsightsStyle::Get().GetBrush("SingleBorder"))
	, Font(FAppStyle::Get().GetFontStyle("SmallFont"))
{
	SetValidLocations(ETimingTrackLocation::Scrollable | ETimingTrackLocation::TopDocked | ETimingTrackLocation::BottomDocked);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FGraphTrack::~FGraphTrack()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrack::Reset()
{
	AllSeries.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrack::PostUpdate(const ITimingTrackUpdateContext& Context)
{
	constexpr float HeaderWidth = 100.0f;
	constexpr float HeaderHeight = 14.0f;

	const float MouseX = static_cast<float>(Context.GetMousePosition().X);
	const float MouseY = static_cast<float>(Context.GetMousePosition().Y);

	if (MouseY >= GetPosY() && MouseY < GetPosY() + GetHeight())
	{
		SetHoveredState(true);
		SetHeaderHoveredState(MouseX < HeaderWidth && MouseY < GetPosY() + HeaderHeight);
	}
	else
	{
		SetHoveredState(false);
	}

	TimeScaleX = Context.GetViewport().GetScaleX();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrack::UpdateStats()
{
	NumDrawPoints = 0;
	NumDrawLines = 0;
	NumDrawBoxes = 0;

	for (const TSharedPtr<FGraphSeries>& Series : AllSeries)
	{
		if (Series->IsVisible())
		{
			NumDrawPoints += Series->Points.Num();
			for (int32 BatchIndex = 0; BatchIndex < Series->LinePoints.Num(); ++BatchIndex)
			{
				NumDrawLines += Series->LinePoints[BatchIndex].Num() / 2;
			}
			NumDrawBoxes += Series->Boxes.Num();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrack::PreDraw(const ITimingTrackDrawContext& Context) const
{
	UE::Insights::FDrawContext& DrawContext = Context.GetDrawContext();
	const FTimingTrackViewport& Viewport = Context.GetViewport();

	FDrawHelpers::DrawBackground(DrawContext, WhiteBrush, Viewport, GetPosY(), GetHeight());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	UE::Insights::FDrawContext& DrawContext = Context.GetDrawContext();
	const FTimingTrackViewport& Viewport = Context.GetViewport();

	const float AlignedPosY = FMath::RoundToFloat(GetPosY());
	const float AlignedHeight = FMath::RoundToFloat(GetHeight());

	// Set clipping area.
	{
		const FVector2f AbsPos = FVector2f(DrawContext.Geometry.GetAbsolutePosition());
		const float Scale = DrawContext.Geometry.GetAccumulatedLayoutTransform().GetScale();
		const float L = AbsPos.X;
		const float R = AbsPos.X + (Viewport.GetWidth() * Scale);
		const float T = AbsPos.Y + (GetPosY() * Scale);
		const float B = AbsPos.Y + ((GetPosY() + GetHeight()) * Scale);
		const FSlateClippingZone ClipZone(FSlateRect(L, T, R, B));
		DrawContext.ElementList.PushClip(ClipZone);
	}

	// Draw top line.
	//DrawContext.DrawBox(0.0f, GetPosY(), Viewport.Width, 1.0f, WhiteBrush, FLinearColor(0.05f, 0.05f, 0.05f, 1.0f));

	bool bDrawBaseline = IsAnyOptionEnabled(EGraphOptions::ShowBaseline);
	for (const TSharedPtr<FGraphSeries>& Series : AllSeries)
	{
		if (Series->IsVisible())
		{
			// Draw baseline (Value == 0)...
			if (bDrawBaseline)
			{
				bDrawBaseline = false; // only for the first visible series

				const float BaselineY = FMath::RoundToFloat(static_cast<float>(Series->GetBaselineY()));
				if (BaselineY >= 0.0f && BaselineY < AlignedHeight)
				{
					DrawContext.DrawBox(0.0f, AlignedPosY + BaselineY, Viewport.GetWidth(), 1.0f, WhiteBrush, FLinearColor(0.05f, 0.05f, 0.05f, 1.0f));
					DrawContext.LayerId++;
				}
			}

			DrawSeries(*Series, DrawContext, Viewport);
		}
	}

	if (IsAnyOptionEnabled(EGraphOptions::ShowThresholds))
	{
		const FLinearColor HighThresholdLineColor(1.0f, 0.0f, 0.0f, 0.75f);
		const FLinearColor LowThresholdLineColor(1.0f, 1.0f, 0.0f, 0.75f);

		for (const TSharedPtr<FGraphSeries>& Series : AllSeries)
		{
			if (Series->IsVisible() && (Series->HasHighThresholdValue() || Series->HasLowThresholdValue()))
			{
				if (Series->HasHighThresholdValue())
				{
					const float ThresholdLineY = Series->ValueViewport.GetRoundedYForValue(Series->GetHighThresholdValue());
					if (ThresholdLineY>= 0.0f && ThresholdLineY < AlignedHeight)
					{
						DrawContext.DrawBox(0.0f, AlignedPosY + ThresholdLineY, Viewport.GetWidth(), 1.0f, WhiteBrush, HighThresholdLineColor);
						DrawContext.LayerId++;
					}
				}

				if (Series->HasLowThresholdValue())
				{
					const float ThresholdLineY = Series->ValueViewport.GetRoundedYForValue(Series->GetLowThresholdValue());
					if (ThresholdLineY >= 0.0f && ThresholdLineY < AlignedHeight)
					{
						DrawContext.DrawBox(0.0f, AlignedPosY + ThresholdLineY, Viewport.GetWidth(), 1.0f, WhiteBrush, LowThresholdLineColor);
						DrawContext.LayerId++;
					}
				}

				break; // only for the first visible series with valid thresholds
			}
		}
	}

	if (IsAnyOptionEnabled(EGraphOptions::ShowVerticalAxisGrid))
	{
		DrawVerticalAxisGrid(Context);
	}

	if (IsAnyOptionEnabled(EGraphOptions::ShowHeader))
	{
		DrawHeader(Context);
	}

	// Restore clipping.
	DrawContext.ElementList.PopClip();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrack::DrawSeries(const FGraphSeries& Series, UE::Insights::FDrawContext& DrawContext, const FTimingTrackViewport& Viewport) const
{
	const float Scale = DrawContext.Geometry.GetAccumulatedLayoutTransform().GetScale();
	const float PixelUnit = 1.0f / Scale;

	// Set clipping area. This area tries to take into account the optional border area that allows
	// graph tracks to optionally act like 'event' tracks if required (with respect to layout, anyway).
	// The GetBorderY() - 1.0f calculation is designed to avoid clipping the line rasterization, as the custom verts
	// of the fill and the outer lines appear to get rasterized differently, the latter missing one pixel
	// on its upper side.
	{
		const FVector2f AbsPos = FVector2f(DrawContext.Geometry.GetAbsolutePosition());
		const float L = AbsPos.X;
		const float R = AbsPos.X + (Viewport.GetWidth() * Scale);
		const float T = AbsPos.Y + ((GetPosY() + (GetBorderY() - 1.0f)) * Scale);
		const float B = AbsPos.Y + ((GetPosY() + (GetHeight() - (GetBorderY() - 1.0f))) * Scale);
		const FSlateClippingZone ClipZone(FSlateRect(L, T, R, B));
		DrawContext.ElementList.PushClip(ClipZone);
	}

	if (IsAnyOptionEnabled(EGraphOptions::ShowBars))
	{
		int32 NumBoxes = Series.Boxes.Num();
		const float TrackPosY = GetPosY();
		const float BaselineY = FMath::RoundToFloat(static_cast<float>(Series.GetBaselineY()));
		for (int32 Index = 0; Index < NumBoxes; ++Index)
		{
			const FGraphSeries::FBox& Box = Series.Boxes[Index];
			if (BaselineY >= Box.Y)
			{
				DrawContext.DrawBox(Box.X, TrackPosY + Box.Y, Box.W, BaselineY - Box.Y + 1.0f, WhiteBrush, Series.Color);
			}
			else
			{
				DrawContext.DrawBox(Box.X, TrackPosY + BaselineY, Box.W, Box.Y - BaselineY + 1.0f, WhiteBrush, Series.Color);
			}
		}
		DrawContext.LayerId++;
	}

	const float LocalPosX = 0.0f;
	const float LocalPosY = FMath::RoundToFloat(GetPosY());

	FPaintGeometry Geo = DrawContext.Geometry.ToPaintGeometry();
	Geo.AppendTransform(FSlateLayoutTransform(FVector2D(LocalPosX * Scale, LocalPosY * Scale)));

	if (IsAnyOptionEnabled(EGraphOptions::ShowPolygon))
	{
		FSlateResourceHandle ResourceHandle = FSlateApplication::Get().GetRenderer()->GetResourceHandle(*WhiteBrush);
		const FSlateShaderResourceProxy* ResourceProxy = ResourceHandle.GetResourceProxy();

		FVector2D AtlasOffset = ResourceProxy ? FVector2D(ResourceProxy->StartUV) : FVector2D(0.0, 0.0);
		FVector2D AtlasUVSize = ResourceProxy ? FVector2D(ResourceProxy->SizeUV) : FVector2D(1.0, 1.0);
		FVector2f UV(AtlasOffset + FVector2D(0.0, 0.0) * AtlasUVSize);

		const FVector2D Size = DrawContext.Geometry.GetLocalSize();

		const FSlateRenderTransform& RenderTransform = Geo.GetAccumulatedRenderTransform();

		FColor FillColor = Series.FillColor.ToFColor(true);

		const double BaselineY = Series.GetBaselineY();

		for (int32 BatchIndex = 0; BatchIndex < Series.LinePoints.Num(); ++BatchIndex)
		{
			const TArray<FVector2D>& LinePoints = Series.LinePoints[BatchIndex];

			TArray<SlateIndex> Indices;
			TArray<FSlateVertex> Verts;

			Indices.Reserve(LinePoints.Num() * 6);
			Verts.Reserve(LinePoints.Num() * 2);

			int32 PrevSide = 0;

			for (int32 PointIndex = 0; PointIndex < LinePoints.Num(); ++PointIndex)
			{
				const FVector2D& LinePoint = LinePoints[PointIndex];

				// When crossing baseline the polygon needs to be intersected.
				int32 CrtSide = (LinePoint.Y > BaselineY) ? 1 : (LinePoint.Y < BaselineY) ? -1 : 0;
				if (PrevSide != 0 && CrtSide + PrevSide == 0) // alternating sides?
				{
					// Compute intersection point.
					const FVector2D& PrevLinePoint = LinePoints[PointIndex - 1];
					const double Delta = (PrevLinePoint.Y - BaselineY) / (PrevLinePoint.Y - LinePoint.Y);
					const double X = PrevLinePoint.X + (LinePoint.X - PrevLinePoint.X) * Delta;

					// Add an intersection point vertex.
					Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform,
						FVector2f((float)X, (float)BaselineY), UV, FillColor));

					// Add a value point vertex.
					Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform,
						FVector2f((float)LinePoint.X, (float)LinePoint.Y), UV, FillColor));

					// Add a baseline vertex.
					Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform,
						FVector2f((float)LinePoint.X, (float)BaselineY), UV, FillColor));

					SlateIndex NumVerts = (SlateIndex)Verts.Num();
					check(NumVerts >= 5);

					Indices.Add(NumVerts - 5);
					Indices.Add(NumVerts - 4);
					Indices.Add(NumVerts - 3);

					Indices.Add(NumVerts - 3);
					Indices.Add(NumVerts - 2);
					Indices.Add(NumVerts - 1);
				}
				else
				{
					// Add a value point vertex.
					Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform,
						FVector2f((float)LinePoint.X, (float)LinePoint.Y), UV, FillColor));

					// Add a baseline vertex.
					Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform,
						FVector2f((float)LinePoint.X, (float)BaselineY), UV, FillColor));

					SlateIndex NumVerts = (SlateIndex)Verts.Num();
					if (NumVerts >= 4)
					{
						Indices.Add(NumVerts - 4);
						Indices.Add(NumVerts - 3);
						Indices.Add(NumVerts - 2);

						Indices.Add(NumVerts - 2);
						Indices.Add(NumVerts - 3);
						Indices.Add(NumVerts - 1);
					}
				}
				PrevSide = CrtSide;
			}

			if (Indices.Num() > 0)
			{
				FSlateDrawElement::MakeCustomVerts(
					DrawContext.ElementList,
					DrawContext.LayerId,
					ResourceHandle,
					Verts,
					Indices,
					nullptr,
					0,
					0, ESlateDrawEffect::None);
				DrawContext.LayerId++;
			}
		}
	}

	if (IsAnyOptionEnabled(EGraphOptions::ShowLines))
	{
		FPaintGeometry LineGeo = Geo;
		LineGeo.AppendTransform(FSlateLayoutTransform(FVector2D(0.5f, 0.5f)));

		// Disable pixel snapping here so lines line up with boxes/polys correctly.
		const ESlateDrawEffect LineDrawEffects = DrawContext.DrawEffects | ESlateDrawEffect::NoPixelSnapping;

#if 0
		constexpr bool bAntialias = true;
		const float Thickness = FMath::Max(1.0f, Scale);
#else
		constexpr bool bAntialias = false;
		const float Thickness = 1.0f;
#endif

		for (int32 BatchIndex = 0; BatchIndex < Series.LinePoints.Num(); ++BatchIndex)
		{
			const TArray<FVector2D>& LinePoints = Series.LinePoints[BatchIndex];
			if (LinePoints.Num() > 0)
			{
				FSlateDrawElement::MakeLines(DrawContext.ElementList, DrawContext.LayerId, LineGeo, LinePoints, LineDrawEffects, Series.Color, bAntialias, Thickness);
			}
		}
		DrawContext.LayerId++;
	}

	// Restore clipping.
	DrawContext.ElementList.PopClip();

	if (IsAnyOptionEnabled(EGraphOptions::ShowPoints))
	{
		const int32 NumPoints = Series.Points.Num();

#define INSIGHTS_GRAPH_TRACK_DRAW_POINTS_AS_RECTANGLES 0
#if !INSIGHTS_GRAPH_TRACK_DRAW_POINTS_AS_RECTANGLES

		const float OffsetX = PixelUnit / 2.0f;
		const float OffsetY = PixelUnit / 2.0f;

		if (IsAnyOptionEnabled(EGraphOptions::ShowPointsWithBorder))
		{
			// Draw points (border).
			for (int32 Index = 0; Index < NumPoints; ++Index)
			{
				const FVector2D& Pt = Series.Points[Index];
				const float PtX = LocalPosX + static_cast<float>(Pt.X) - PointVisualSize / 2.0f - 1.0f + OffsetX;
				const float PtY = LocalPosY + static_cast<float>(Pt.Y) - PointVisualSize / 2.0f - 1.0f + OffsetY;
				DrawContext.DrawBox(PtX, PtY, PointVisualSize + 2.0f, PointVisualSize + 2.0f, PointBrush, Series.BorderColor);
			}
			DrawContext.LayerId++;
		}

		// Draw points (interior).
		for (int32 Index = 0; Index < NumPoints; ++Index)
		{
			const FVector2D& Pt = Series.Points[Index];
			const float PtX = LocalPosX + static_cast<float>(Pt.X) - PointVisualSize / 2.0f + OffsetX;
			const float PtY = LocalPosY + static_cast<float>(Pt.Y) - PointVisualSize / 2.0f + OffsetY;
			DrawContext.DrawBox(PtX, PtY, PointVisualSize, PointVisualSize, PointBrush, Series.Color);
		}
		DrawContext.LayerId++;

#else // Alternative way of drawing points; kept here for debugging purposes.

		//const float Angle = FMath::DegreesToRadians(45.0f);

		if (IsAnyOptionEnabled(EGraphOptions::ShowPointsWithBorder))
		{
			// Draw borders.
			const float BorderPtSize = PointVisualSize;
			//FVector2D BorderRotationPoint(BorderPtSize / 2.0f, BorderPtSize / 2.0f);
			for (int32 Index = 0; Index < NumPoints; ++Index)
			{
				const FVector2D& Pt = Series.Points[Index];
				const float PtX = LocalPosX + static_cast<float>(Pt.X) - BorderPtSize / 2.0f + 0.5f;
				const float PtY = LocalPosY + static_cast<float>(Pt.Y) - BorderPtSize / 2.0f + 0.5f;
				DrawContext.DrawBox(PtX, PtY, BorderPtSize, BorderPtSize, BorderBrush, Series.BorderColor);
				//DrawContext.DrawRotatedBox(PtX, PtY, BorderPtSize, BorderPtSize, BorderBrush, Series.BorderColor, Angle, BorderRotationPoint);
			}
			DrawContext.LayerId++;
		}

		// Draw points as rectangles.
		const float PtSize = PointVisualSize - 2.0f;
		//FVector2D RotationPoint(PtSize / 2.0f, PtSize / 2.0f);
		for (int32 Index = 0; Index < NumPoints; ++Index)
		{
			const FVector2D& Pt = Series.Points[Index];
			const float PtX = LocalPosX + static_cast<float>(Pt.X) - PtSize / 2.0f + 0.5f;
			const float PtY = LocalPosY + static_cast<float>(Pt.Y) - PtSize / 2.0f + 0.5f;
			DrawContext.DrawBox(PtX, PtY, PtSize, PtSize, WhiteBrush, Series.Color);
			//DrawContext.DrawRotatedBox(PtX, PtY, PtSize, PtSize, WhiteBrush, Series.Color, Angle, RotationPoint);
		}
		DrawContext.LayerId++;

#endif
	}

	if (IsAnyOptionEnabled(EGraphOptions::ShowDebugInfo)) // for debugging only
	{
		FPaintGeometry LineGeo = Geo;
		LineGeo.AppendTransform(FSlateLayoutTransform(FVector2D(0.5f, 0.5f)));

		// Disable pixel snapping here so lines line up with boxes/polys correctly.
		const ESlateDrawEffect LineDrawEffects = DrawContext.DrawEffects | ESlateDrawEffect::NoPixelSnapping;

		#if 0
		{
			// Draw white corner at (0, 0) using MakeLines.
			{
				TArray<FVector2f> HLine;
				HLine.Add(FVector2f(0.0f, 0.0f));
				HLine.Add(FVector2f(10.0f, 0.0f));
				FSlateDrawElement::MakeLines(DrawContext.ElementList, DrawContext.LayerId, LineGeo, HLine, LineDrawEffects, FLinearColor::White, false, 1.0f);

				TArray<FVector2f> VLine;
				VLine.Add(FVector2f(0.0f, 0.0f));
				VLine.Add(FVector2f(0.0f, 10.0f));
				FSlateDrawElement::MakeLines(DrawContext.ElementList, DrawContext.LayerId, LineGeo, VLine, LineDrawEffects, FLinearColor::White, false, 1.0f);
			}
			DrawContext.LayerId++;

			// Draw corner at (0, 0) using DrawBox.
			DrawContext.DrawBox(LocalPosX + 2.0f, LocalPosY, 6.0f, PixelUnit, WhiteBrush, Series.Color);
			DrawContext.DrawBox(LocalPosX, LocalPosY + 2.0f, PixelUnit, 6.0f, WhiteBrush, Series.Color);
			DrawContext.LayerId++;
		}
		#endif

		const int32 NumDbgPoints = Series.Points.Num();

		float Thickness = 1.0f;

		// Draw white cross lines (17x17 screen pixels) at each point.
		for (int32 Index = 0; Index < NumDbgPoints; ++Index)
		{
			const FVector2D& Pt = Series.Points[Index];
			const float PtX = static_cast<float>(Pt.X);
			const float PtY = static_cast<float>(Pt.Y);

			TArray<FVector2f> HLine;
			HLine.Add(FVector2f(PtX - 8.0f * PixelUnit, PtY));
			HLine.Add(FVector2f(PtX + 9.0f * PixelUnit, PtY));
			FSlateDrawElement::MakeLines(DrawContext.ElementList, DrawContext.LayerId, LineGeo, HLine, LineDrawEffects, FLinearColor::White, false, Thickness);

			TArray<FVector2f> VLine;
			VLine.Add(FVector2f(PtX, PtY - 8.0f * PixelUnit));
			VLine.Add(FVector2f(PtX, PtY + 9.0f * PixelUnit));
			FSlateDrawElement::MakeLines(DrawContext.ElementList, DrawContext.LayerId, LineGeo, VLine, LineDrawEffects, FLinearColor::White, false, Thickness);
		}
		DrawContext.LayerId++;

		// Draw black cross lines (11x11 screen pixels) at each point.
		for (int32 Index = 0; Index < NumDbgPoints; ++Index)
		{
			const FVector2D& Pt = Series.Points[Index];
			const float PtX = LocalPosX + static_cast<float>(Pt.X);
			const float PtY = LocalPosY + static_cast<float>(Pt.Y);
			DrawContext.DrawBox(PtX - 5.0f * PixelUnit, PtY, 11.0f * PixelUnit, PixelUnit, WhiteBrush, FLinearColor::Black);
			DrawContext.DrawBox(PtX, PtY - 5.0f * PixelUnit, PixelUnit, 11.0f * PixelUnit, WhiteBrush, FLinearColor::Black);
		}
		DrawContext.LayerId++;

		// Draw a red screen pixel at each point.
		for (int32 Index = 0; Index < NumDbgPoints; ++Index)
		{
			const FVector2D& Pt = Series.Points[Index];
			const float PtX = LocalPosX + static_cast<float>(Pt.X);
			const float PtY = LocalPosY + static_cast<float>(Pt.Y);
			DrawContext.DrawBox(PtX, PtY, PixelUnit, PixelUnit, WhiteBrush, FLinearColor::Red);
		}
		DrawContext.LayerId++;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrack::DrawEvent(const ITimingTrackDrawContext& Context, const ITimingEvent& InTimingEvent, EDrawEventMode InDrawMode) const
{
	ensure(InTimingEvent.CheckTrack(this));
	ensure(InTimingEvent.Is<FGraphTrackEvent>());

	const FGraphTrackEvent& GraphEvent = InTimingEvent.As<FGraphTrackEvent>();
	const TSharedPtr<const FGraphSeries> Series = GraphEvent.GetSeries();

	const FTimingTrackViewport& Viewport = Context.GetViewport();
	UE::Insights::FDrawContext& DrawContext = Context.GetDrawContext();

	const float EventX1 = Viewport.TimeToSlateUnitsRounded(GraphEvent.GetStartTime());
	const float EventX2 = Viewport.TimeToSlateUnitsRounded(Viewport.RestrictEndTime(GraphEvent.GetEndTime()));

	const float EventY1 = GetPosY() + Series->GetRoundedYForValue(GraphEvent.GetValue());

	const FLinearColor HighlightColor(1.0f, 1.0f, 0.0f, 1.0f);

	// Draw highlighted box.
	if (IsAnyOptionEnabled(EGraphOptions::ShowBars))
	{
		float W = EventX2 - EventX1;
		ensure(W >= 0); // we expect events to be sorted

		// Timing events are displayed with minimum 1px (including empty ones).
		if (W == 0)
		{
			W = 1.0f;
		}

		const float EventY2 = GetPosY() + static_cast<float>(Series->GetBaselineY());

		const float Y1 = FMath::Min(EventY1, EventY2);
		const float DY = FMath::Abs(EventY2 - EventY1);

		DrawContext.DrawBox(EventX1 - 1.0f, Y1 - 1.0f, W + 2.0f, DY + 3.0f, WhiteBrush, HighlightColor);
		DrawContext.LayerId++;
		DrawContext.DrawBox(EventX1, Y1, W, DY + 1.0f, WhiteBrush, Series->Color);
		DrawContext.LayerId++;
	}

	const float PX = EventX1;
	const float PY = EventY1 - 0.5f;

	// Draw highlighted line.
	if ((EventX2 > EventX1) &&
		(AreAllOptionsEnabled(EGraphOptions::UseEventDuration | EGraphOptions::ShowLines) ||
		 AreAllOptionsEnabled(EGraphOptions::UseEventDuration | EGraphOptions::ShowPolygon) ||
		 IsAnyOptionEnabled(EGraphOptions::ShowBars)))
	{
		float W = EventX2 - EventX1;
		ensure(W >= 0); // we expect events to be sorted

		// Timing events are displayed with minimum 1px (including empty ones).
		if (W == 0)
		{
			W = 1.0f;
		}

		DrawContext.DrawBox(PX - 1.0f, PY - 1.0f, W + 2.0f, 3.0f, WhiteBrush, HighlightColor);
		DrawContext.LayerId++;
		DrawContext.DrawBox(PX, PY, W, 1.0f, WhiteBrush, Series->Color);
		DrawContext.LayerId++;
	}

	// Draw highlighted point.
	DrawContext.DrawBox(PX - PointVisualSize / 2.0f - 1.5f, PY - PointVisualSize / 2.0f - 1.5f, PointVisualSize + 4.0f, PointVisualSize + 4.0f, PointBrush, HighlightColor);
	DrawContext.LayerId++;
	DrawContext.DrawBox(PX - PointVisualSize / 2.0f + 0.5f, PY - PointVisualSize / 2.0f + 0.5f, PointVisualSize, PointVisualSize, PointBrush, Series->Color);
	DrawContext.LayerId++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrack::DrawVerticalAxisGrid(const ITimingTrackDrawContext& Context) const
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrack::DrawHeader(const ITimingTrackDrawContext& Context) const
{
	const FTimingViewDrawHelper& Helper = *static_cast<const FTimingViewDrawHelper*>(&Context.GetHelper());
	UE::Insights::FDrawContext& DrawContext = Context.GetDrawContext();
	Helper.DrawTrackHeader(*this, DrawContext.LayerId, DrawContext.LayerId + 1);
	DrawContext.LayerId += 2;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
{
	if (InTooltipEvent.CheckTrack(this) && InTooltipEvent.Is<FGraphTrackEvent>())
	{
		const FGraphTrackEvent& TooltipEvent = InTooltipEvent.As<FGraphTrackEvent>();
		const TSharedRef<const FGraphSeries> Series = TooltipEvent.GetSeries();

		InOutTooltip.ResetContent();
		InOutTooltip.AddTitle(Series->GetName().ToString(), Series->GetColor());

		using namespace UE::Insights;
		const double Precision = FMath::Max(1.0 / TimeScaleX, FTimeValue::Nanosecond);
		InOutTooltip.AddNameValueTextLine(TEXT("Time:"), FormatTime(TooltipEvent.GetStartTime(), Precision));
		if (Series->HasEventDuration())
		{
			InOutTooltip.AddNameValueTextLine(TEXT("Duration:"), FormatTimeAuto(TooltipEvent.GetDuration()));
		}
		InOutTooltip.AddNameValueTextLine(TEXT("Value:"), Series->FormatValue(TooltipEvent.GetValue()));
		InOutTooltip.UpdateLayout();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FGraphTrack::GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const
{
	const float LocalPosX = InPosX;
	const float LocalPosY = InPosY - GetPosY();

	const bool bCheckLine = AreAllOptionsEnabled(EGraphOptions::UseEventDuration | EGraphOptions::ShowLines) ||
							AreAllOptionsEnabled(EGraphOptions::UseEventDuration | EGraphOptions::ShowPolygon);

	const bool bCheckBox = IsAnyOptionEnabled(EGraphOptions::ShowBars);

	// Search series in reverse order.
	for (int32 SeriesIndex = AllSeries.Num() - 1; SeriesIndex >= 0; --SeriesIndex)
	{
		const TSharedPtr<FGraphSeries>& Series = AllSeries[SeriesIndex];
		if (Series->IsVisible())
		{
			const FGraphSeriesEvent* Event = Series->GetEvent(LocalPosX, LocalPosY, Viewport, bCheckLine, bCheckBox);
			if (Event != nullptr)
			{
				return MakeShared<FGraphTrackEvent>(SharedThis(this), Series.ToSharedRef(), *Event);
			}
		}
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Options", LOCTEXT("ContextMenu_Section_Options", "Options"));
	{
		if (EnumHasAnyFlags(VisibleOptions, EGraphOptions::ShowDebugInfo)) // debug functionality
		{
			FUIAction Action_ShowDebugInfo
			(
				FExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_Execute, EGraphOptions::ShowDebugInfo),
				FCanExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_CanExecute, EGraphOptions::ShowDebugInfo),
				FIsActionChecked::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_IsChecked, EGraphOptions::ShowDebugInfo)
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_ShowDebugInfo", "Show Debug Info"),
				LOCTEXT("ContextMenu_ShowDebugInfo_Desc", "Shows debug info."),
				FSlateIcon(),
				Action_ShowDebugInfo,
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}

		if (EnumHasAnyFlags(VisibleOptions, EGraphOptions::ShowPoints))
		{
			FUIAction Action_ShowPoints
			(
				FExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_Execute, EGraphOptions::ShowPoints),
				FCanExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_CanExecute, EGraphOptions::ShowPoints),
				FIsActionChecked::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_IsChecked, EGraphOptions::ShowPoints)
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_ShowPoints", "Show Points"),
				LOCTEXT("ContextMenu_ShowPoints_Desc", "Shows points."),
				FSlateIcon(),
				Action_ShowPoints,
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}

		if (EnumHasAnyFlags(VisibleOptions, EGraphOptions::ShowPointsWithBorder))
		{
			FUIAction Action_ShowPointsWithBorder
			(
				FExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_Execute, EGraphOptions::ShowPointsWithBorder),
				FCanExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ShowPointsWithBorder_CanExecute),
				FIsActionChecked::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_IsChecked, EGraphOptions::ShowPointsWithBorder)
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_ShowPointsWithBorder", "Show Points with Border"),
				LOCTEXT("ContextMenu_ShowPointsWithBorder_Desc", "Shows border around points."),
				FSlateIcon(),
				Action_ShowPointsWithBorder,
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}

		if (EnumHasAnyFlags(VisibleOptions, EGraphOptions::ShowLines))
		{
			FUIAction Action_ShowLines
			(
				FExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_Execute, EGraphOptions::ShowLines),
				FCanExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_CanExecute, EGraphOptions::ShowLines),
				FIsActionChecked::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_IsChecked, EGraphOptions::ShowLines)
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_ShowLines", "Show Connected Lines"),
				LOCTEXT("ContextMenu_ShowLines_Desc", "Shows connected lines. Each event is a single point in time."),
				FSlateIcon(),
				Action_ShowLines,
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}

		if (EnumHasAnyFlags(VisibleOptions, EGraphOptions::ShowPolygon))
		{
			FUIAction Action_ShowPolygon
			(
				FExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_Execute, EGraphOptions::ShowPolygon),
				FCanExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_CanExecute, EGraphOptions::ShowPolygon),
				FIsActionChecked::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_IsChecked, EGraphOptions::ShowPolygon)
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_ShowPolygon", "Show Polygon"),
				LOCTEXT("ContextMenu_ShowPolygon_Desc", "Shows filled polygon under the graph series."),
				FSlateIcon(),
				Action_ShowPolygon,
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}

		if (EnumHasAnyFlags(VisibleOptions, EGraphOptions::UseEventDuration))
		{
			FUIAction Action_UseEventDuration
			(
				FExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_Execute, EGraphOptions::UseEventDuration),
				FCanExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_UseEventDuration_CanExecute),
				FIsActionChecked::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_IsChecked, EGraphOptions::UseEventDuration)
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_UseEventDuration", "Use Event Duration"),
				LOCTEXT("ContextMenu_UseEventDuration_Desc", "Uses duration of timing events (for Connected Lines and Polygon)."),
				FSlateIcon(),
				Action_UseEventDuration,
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}

		if (EnumHasAnyFlags(VisibleOptions, EGraphOptions::ShowBars))
		{
			FUIAction Action_ShowBars
			(
				FExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_Execute, EGraphOptions::ShowBars),
				FCanExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_CanExecute, EGraphOptions::ShowBars),
				FIsActionChecked::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_IsChecked, EGraphOptions::ShowBars)
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_ShowBars", "Show Bars"),
				LOCTEXT("ContextMenu_ShowBars_Desc", "Shows bars. Width of bars corresponds to duration of timing events."),
				FSlateIcon(),
				Action_ShowBars,
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}

		if (EnumHasAnyFlags(VisibleOptions, EGraphOptions::AutoZoomIncludesBaseline))
		{
			FUIAction Action_AutoZoomIncludesBaseline
			(
				FExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_Execute, EGraphOptions::AutoZoomIncludesBaseline),
				FCanExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_CanExecute, EGraphOptions::AutoZoomIncludesBaseline),
				FIsActionChecked::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_IsChecked, EGraphOptions::AutoZoomIncludesBaseline)
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_AutoZoomIncludesBaseline", "Auto Zoom Includes Baseline"),
				LOCTEXT("ContextMenu_AutoZoomIncludesBaseline_Desc", "If enabled, the auto zoom will also include the baseline (value == 0)."),
				FSlateIcon(),
				Action_AutoZoomIncludesBaseline,
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}

		if (EnumHasAnyFlags(VisibleOptions, EGraphOptions::AutoZoomIncludesThresholds))
		{
			FUIAction Action_AutoZoomIncludesThresholds
			(
				FExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_Execute, EGraphOptions::AutoZoomIncludesThresholds),
				FCanExecuteAction::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_CanExecute, EGraphOptions::AutoZoomIncludesThresholds),
				FIsActionChecked::CreateSP(this, &FGraphTrack::ContextMenu_ToggleOption_IsChecked, EGraphOptions::AutoZoomIncludesThresholds)
			);
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ContextMenu_AutoZoomIncludesThresholds", "Auto Zoom Includes Thresholds"),
				LOCTEXT("ContextMenu_AutoZoomIncludesThresholds_Desc", "If enabled, the auto zoom will also include the high and low thresholds, if available."),
				FSlateIcon(),
				Action_AutoZoomIncludesThresholds,
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Series", LOCTEXT("ContextMenu_Section_Series", "Series"));
	{
		MenuBuilder.AddWidget(
			SNew(SBox)
			.MinDesiredWidth(250.0f)
			.MaxDesiredHeight(135.0f)
			[
				SNew(SGraphSeriesList, SharedThis(this))
			],
			FText::GetEmpty(), true);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FGraphTrack::ContextMenu_ToggleOption_CanExecute(EGraphOptions Option)
{
	return EnumHasAnyFlags(EditableOptions, Option);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrack::ContextMenu_ToggleOption_Execute(EGraphOptions Option)
{
	ToggleOptions(Option);
	SetDirtyFlag();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FGraphTrack::ContextMenu_ToggleOption_IsChecked(EGraphOptions Option)
{
	return IsAnyOptionEnabled(Option);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FGraphTrack::ContextMenu_ShowPointsWithBorder_CanExecute()
{
	return EnumHasAnyFlags(EditableOptions, EGraphOptions::ShowPointsWithBorder)
		&& IsAnyOptionEnabled(EGraphOptions::ShowPoints);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FGraphTrack::ContextMenu_UseEventDuration_CanExecute()
{
	return EnumHasAnyFlags(EditableOptions, EGraphOptions::UseEventDuration)
		&& IsAnyOptionEnabled(EGraphOptions::ShowLines | EGraphOptions::ShowPolygon);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FRandomGraphTrack
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FRandomGraphTrack)

////////////////////////////////////////////////////////////////////////////////////////////////////

FRandomGraphTrack::FRandomGraphTrack()
	: FGraphTrack()
{
	EnabledOptions = //EGraphOptions::ShowDebugInfo |
					 EGraphOptions::ShowPoints |
					 EGraphOptions::ShowPointsWithBorder |
					 EGraphOptions::ShowLines |
					 //EGraphOptions::ShowPolygon |
					 //EGraphOptions::UseEventDuration |
					 //EGraphOptions::ShowBars |
					 //EGraphOptions::ShowBaseline |
					 //EGraphOptions::ShowThresholds |
					 //EGraphOptions::ShowVerticalAxisGrid |
					 EGraphOptions::ShowHeader |
					 EGraphOptions::None;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FRandomGraphTrack::AddDefaultSeries()
{
	TSharedRef<FGraphSeries> Series0 = MakeShared<FGraphSeries>();
	Series0->SetName(TEXT("Random Blue"));
	Series0->SetDescription(TEXT("Random series; for debugging purposes"));
	Series0->SetColor(FLinearColor(0.1f, 0.5f, 1.0f, 1.0f), FLinearColor(0.4f, 0.8f, 1.0f, 1.0f));
	Series0->SetVisibility(true);
	AllSeries.Add(Series0);

	TSharedRef<FGraphSeries> Series1 = MakeShared<FGraphSeries>();
	Series1->SetName(TEXT("Random Yellow"));
	Series1->SetDescription(TEXT("Random series; for debugging purposes"));
	Series1->SetColor(FLinearColor(0.9f, 0.9f, 0.1f, 1.0f), FLinearColor(1.0f, 1.0f, 0.4f, 1.0f));
	Series1->SetVisibility(false);
	AllSeries.Add(Series1);

	TSharedRef<FGraphSeries> Series2 = MakeShared<FGraphSeries>();
	Series2->SetName(TEXT("Random Red"));
	Series2->SetDescription(TEXT("Random series; for debugging purposes"));
	Series2->SetColor(FLinearColor(1.0f, 0.1f, 0.2f, 1.0f), FLinearColor(1.0f, 0.4f, 0.5f, 1.0f));
	Series2->SetVisibility(true);
	AllSeries.Add(Series2);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FRandomGraphTrack::~FRandomGraphTrack()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FRandomGraphTrack::Update(const ITimingTrackUpdateContext& Context)
{
	FGraphTrack::Update(Context);

	if (IsDirty() || Context.GetViewport().IsHorizontalViewportDirty())
	{
		ClearDirtyFlag();

		NumAddedEvents = 0;

		const FTimingTrackViewport& Viewport = Context.GetViewport();

		int32 Seed = 0;
		for (TSharedPtr<FGraphSeries>& Series : AllSeries)
		{
			Series->SetBaselineY(GetHeight() / 2.0);
			Series->SetScaleY(GetHeight());
			GenerateSeries(*Series, Viewport, 1000000, Seed++);
		}

		UpdateStats();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FRandomGraphTrack::GenerateSeries(FGraphSeries& Series, const FTimingTrackViewport& Viewport, const int32 EventCount, int32 Seed)
{
	//////////////////////////////////////////////////
	// Generate random events.

	constexpr double MinDeltaTime = 0.0000001; // 100ns
	constexpr double MaxDeltaTime = 0.01; // 100ms

	float MinValue = (Seed == 0) ? 0.0f : (Seed == 1) ? -0.25f : -0.5f;
	float MaxValue = (Seed == 0) ? 0.5f : (Seed == 1) ? +0.25f :  0.0f;

	struct FGraphEvent
	{
		double Time;
		double Duration;
		double Value;
	};

	TArray<FGraphEvent> Events;
	Events.Reserve(EventCount);

	FRandomStream RandomStream(Seed);
	double NextT = 0.0;
	for (int32 Index = 0; Index < EventCount; ++Index)
	{
		FGraphEvent Ev;
		Ev.Time = NextT;
		const double TimeAdvance = RandomStream.GetFraction() * (MaxDeltaTime - MinDeltaTime);
		NextT += MinDeltaTime + TimeAdvance;
		Ev.Duration = MinDeltaTime + RandomStream.GetFraction() * TimeAdvance;
		Ev.Value = MinValue + RandomStream.GetFraction() * (MaxValue - MinValue);
		Events.Add(Ev);
	}

	//////////////////////////////////////////////////
	// Optimize and build draw lists.
	{
		FGraphTrackBuilder Builder(*this, Series, Viewport);

		int32 Index = 0;
		while (Index < EventCount && Events[Index].Time < Viewport.GetStartTime())
		{
			++Index;
		}
		if (Index > 0)
		{
			Index--; // one point outside screen (left side)
		}
		while (Index < EventCount)
		{
			const FGraphEvent& Ev = Events[Index];
			Builder.AddEvent(Ev.Time, Ev.Duration, Ev.Value);

			if (Ev.Time > Viewport.GetEndTime())
			{
				// one point outside screen (right side)
				break;
			}

			++Index;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
