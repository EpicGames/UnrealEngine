// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/ViewModels/GraphTrackBuilder.h"

// TraceInsights
#include "Insights/ViewModels/GraphSeries.h"
#include "Insights/ViewModels/GraphTrack.h"
#include "Insights/ViewModels/GraphTrackEvent.h"
#include "Insights/ViewModels/TimingTrackViewport.h"

#define LOCTEXT_NAMESPACE "GraphTrack"

////////////////////////////////////////////////////////////////////////////////////////////////////

FGraphTrackBuilder::FGraphTrackBuilder(FGraphTrack& InTrack, FGraphSeries& InSeries, const FTimingTrackViewport& InViewport)
	: Track(InTrack)
	, Series(InSeries)
	, Viewport(InViewport)
{
	Series.Events.Reset();
	BeginPoints();
	BeginConnectedLines();
	BeginBoxes();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FGraphTrackBuilder::~FGraphTrackBuilder()
{
	EndPoints();
	EndConnectedLines();
	EndBoxes();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrackBuilder::AddEvent(double Time, double Duration, double Value, bool bConnected)
{
	Track.NumAddedEvents++;

	bool bKeepEvent = (Viewport.GetViewportDXForDuration(Duration) > 1.0f); // always keep the events wider than 1px

	//if (Track.IsAnyOptionEnabled(EGraphOptions::ShowPoints))
	{
		bKeepEvent |= AddPoint(Time, Value);
	}

	if (Track.IsAnyOptionEnabled(EGraphOptions::ShowLines | EGraphOptions::ShowPolygon))
	{
		AddConnectedLine(Time, Value, !bConnected);

		if (Track.IsAnyOptionEnabled(EGraphOptions::UseEventDuration) && Duration != 0.0)
		{
			AddConnectedLine(Time + Duration, Value, false);
		}
	}

	if (Track.IsAnyOptionEnabled(EGraphOptions::ShowBars))
	{
		AddBox(Time, Duration, Value);
	}

	if (bKeepEvent)
	{
		Series.Events.Add({ Time, Duration, Value });
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FGraphTrackBuilder - Points
////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrackBuilder::BeginPoints()
{
	Series.Points.Reset();

	PointsCurrentX = -DBL_MAX;

	PointsAtCurrentX.Reset();
	int32 MaxPointsPerLineScan = FMath::CeilToInt(Track.GetHeight() / FGraphTrack::PointSizeY) + 1;
	if (MaxPointsPerLineScan > 0)
	{
		PointsAtCurrentX.AddDefaulted(MaxPointsPerLineScan);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FGraphTrackBuilder::AddPoint(double Time, double Value)
{
	const float X = Viewport.TimeToSlateUnitsRounded(Time);
	if (X < -FGraphTrack::PointVisualSize / 2.0f || X >= Viewport.GetWidth() + FGraphTrack::PointVisualSize / 2.0f)
	{
		return false;
	}

	// Align the X with a grid of GraphTrackPointDX pixels in size, in the global space (i.e. scroll independent).
	const double AlignedX = FMath::RoundToDouble(Time * Viewport.GetScaleX() / FGraphTrack::PointSizeX) * FGraphTrack::PointSizeX;

	if (AlignedX > PointsCurrentX + FGraphTrack::PointSizeX - 0.5)
	{
		FlushPoints();
		PointsCurrentX = AlignedX;
	}

	const float Y = Series.GetRoundedYForValue(Value);

	bool bIsNewVisiblePoint = false;

	int32 Index = FMath::RoundToInt(Y / FGraphTrack::PointSizeY);
	if (Index >= 0 && Index < PointsAtCurrentX.Num())
	{
		FPointInfo& Pt = PointsAtCurrentX[Index];
		bIsNewVisiblePoint = !Pt.bValid;
		Pt.bValid = true;
		Pt.X = X;
		Pt.Y = Y;
	}

	return bIsNewVisiblePoint;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrackBuilder::FlushPoints()
{
	for (int32 Index = 0; Index < PointsAtCurrentX.Num(); ++Index)
	{
		FPointInfo& Pt = PointsAtCurrentX[Index];
		if (Pt.bValid)
		{
			Pt.bValid = false;
			Series.Points.Add(FVector2D(Pt.X, Pt.Y));
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrackBuilder::EndPoints()
{
	FlushPoints();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FGraphTrackBuilder - Connected Lines
////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrackBuilder::BeginConnectedLines()
{
	Series.LinePoints.Reset();
	Series.LinePoints.AddDefaulted();

	LinesCurrentX = -FLT_MAX;
	LinesMinY = FLT_MAX;
	LinesMaxY = -FLT_MAX;
	LinesFirstY = FLT_MAX;
	LinesLastY = FLT_MAX;
	bIsLastLineAdded = false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrackBuilder::AddConnectedLine(double Time, double Value, bool bNewBatch)
{
	if (bIsLastLineAdded)
	{
		return;
	}

	const float X = FMath::Max(LinesCurrentX, Viewport.TimeToSlateUnitsRounded(Viewport.RestrictEndTime(Time)));
	const float Y = Series.GetRoundedYForValue(Value);

	//ensure(X >= LinesCurrentX); // we are assuming events are already sorted by Time

	// Start a new batch
	if (bNewBatch)
	{
		if (Series.LinePoints.Num() > 0 && Series.LinePoints.Last().Num() > 0)
		{
			Series.LinePoints.AddDefaulted();
		}

		// Reset the reduction
		LinesCurrentX = -FLT_MAX;
		LinesMinY = FLT_MAX;
		LinesMaxY = -FLT_MAX;
		LinesFirstY = FLT_MAX;
		LinesLastY = FLT_MAX;
	}

	if (X < 0)
	{
		LinesCurrentX = X;
		LinesLastY = Y;
		return;
	}

	if (X >= Viewport.GetWidth())
	{
		if (!bIsLastLineAdded)
		{
			bIsLastLineAdded = true;

			if (LinesLastY != FLT_MAX)
			{
				FlushConnectedLine();

				AddConnectedLinePoint(LinesCurrentX, LinesLastY);
				AddConnectedLinePoint(X, Y);
			}

			// Reset the "reduction line" so last FlushConnectedLine() call will do nothing.
			LinesMinY = Y;
			LinesMaxY = Y;
		}
		return;
	}

	if (X > LinesCurrentX)
	{
		if (LinesLastY != FLT_MAX)
		{
			FlushConnectedLine();

			AddConnectedLinePoint(LinesCurrentX, LinesLastY);
			AddConnectedLinePoint(X, Y);
		}

		// Advance the "reduction line".
		LinesCurrentX = X;
		LinesMinY = Y;
		LinesMaxY = Y;
		LinesFirstY = Y;
		LinesLastY = Y;
	}
	else
	{
		// Merge current line with the "reduction line".
		if (Y < LinesMinY)
		{
			LinesMinY = Y;
		}
		if (Y > LinesMaxY)
		{
			LinesMaxY = Y;
		}
		LinesLastY = Y;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrackBuilder::FlushConnectedLine()
{
	if (LinesCurrentX >= 0.0f && LinesMinY != LinesMaxY)
	{
		AddConnectedLinePoint(LinesCurrentX, LinesMaxY);
		AddConnectedLinePoint(LinesCurrentX, LinesMinY);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrackBuilder::AddConnectedLinePoint(float X, float Y)
{
	TArray<FVector2D>& LinePoints = Series.LinePoints.Last();

	if (LinePoints.Num() > 0)
	{
		const FVector2D& LastPoint = LinePoints.Last();
		if (X == LastPoint.X && Y == LastPoint.Y)
		{
			return;
		}
	}

	LinePoints.Add(FVector2D(X, Y));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrackBuilder::EndConnectedLines()
{
	FlushConnectedLine();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FGraphTrackBuilder - Boxes
////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrackBuilder::BeginBoxes()
{
	Series.Boxes.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrackBuilder::AddBox(double Time, double Duration, double Value)
{
	float X1 = Viewport.TimeToSlateUnitsRounded(Time);
	if (X1 > Viewport.GetWidth())
	{
		return;
	}

	double EndTime = Viewport.RestrictEndTime(Time + Duration);
	float X2 = Viewport.TimeToSlateUnitsRounded(EndTime);
	if (X2 < 0)
	{
		return;
	}

	float W = X2 - X1;
	ensure(W >= 0); // we expect events to be sorted

	// Timing events are displayed with minimum 1px (including empty ones).
	if (W == 0)
	{
		W = 1.0f;
	}

	const float Y = Series.GetRoundedYForValue(Value);

	// simple reduction
	if (W == 1.0f && Series.Boxes.Num() > 0)
	{
		FGraphSeries::FBox& LastBox = Series.Boxes.Last();
		if (LastBox.W == 1.0f && LastBox.X == X1)
		{
			const float RoundedBaselineY = static_cast<float>(FMath::RoundToFloat(Series.GetBaselineY()));

			if (LastBox.Y < RoundedBaselineY)
			{
				if (Y < RoundedBaselineY)
				{
					// Merge current box with previous one.
					if (Y < LastBox.Y)
					{
						LastBox.Y = Y;
					}
					return;
				}
			}
			else
			{
				if (Y >= RoundedBaselineY)
				{
					// Merge current box with previous one.
					if (Y > LastBox.Y)
					{
						LastBox.Y = Y;
					}
					return;
				}
			}
		}
	}

	FGraphSeries::FBox Box;
	Box.X = X1;
	Box.W = W;
	Box.Y = Y;
	Series.Boxes.Add(Box);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrackBuilder::FlushBox()
{
	// TODO: reduction algorithm
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FGraphTrackBuilder::EndBoxes()
{
	FlushBox();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
