// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/ViewModels/GraphTrackEvent.h"

#include "Insights/ViewModels/GraphTrack.h"

#define LOCTEXT_NAMESPACE "GraphTrack"

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FGraphTrackEvent)

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedRef<const FBaseTimingTrack> FGraphTrackEvent::GetTrack() const
{
	return StaticCastSharedRef<const FBaseTimingTrack, const FGraphTrack>(Track);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
