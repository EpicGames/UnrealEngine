// Copyright Epic Games, Inc. All Rights Reserved.

#include "LogFilter.h"

#include "Async/AsyncWork.h"

// TraceInsights
#include "Insights/Log.h"
#include "Insights/Widgets/SLogView.h"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Expression context to test the given messages against the current text filter.
 */
class FLogFilter_TextFilterExpressionContext : public ITextFilterExpressionContext
{
public:
	explicit FLogFilter_TextFilterExpressionContext(const FLogMessageRecord& InMessage) : Message(&InMessage) {}

	/** Test the given value against the strings extracted from the current item */
	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		return TextFilterUtils::TestBasicStringExpression(Message->GetMessageAsString(), InValue, InTextComparisonMode);
	}

	/**
	* Perform a complex expression test for the current item
	* No complex expressions in this case - always returns false
	*/
	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		return false;
	}

private:
	/** Message that is being filtered */
	const FLogMessageRecord* Message;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FLogFilter
////////////////////////////////////////////////////////////////////////////////////////////////////

FLogFilter::FLogFilter()
	: ChangeNumber(0)
	, VerbosityThreshold(ELogVerbosity::All)
	, bIsFilterSetByVerbosity(false)
	, bIsFilterSetByCategory(false)
	, bIsFilterSetByText(false)
	, bShowAllCategories(true)
	, AvailableLogCategories()
	, EnabledLogCategories()
	, TextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::BasicString)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLogFilter::FLogFilter(const FLogFilter& Other)
	: ChangeNumber(Other.ChangeNumber)
	, VerbosityThreshold(Other.VerbosityThreshold)
	, bIsFilterSetByVerbosity(Other.bIsFilterSetByVerbosity)
	, bIsFilterSetByCategory(Other.bIsFilterSetByCategory)
	, bIsFilterSetByText(Other.bIsFilterSetByText)
	, bShowAllCategories(Other.bShowAllCategories)
	//, AvailableLogCategories(Other.AvailableLogCategories) -- not needed in FLogFilter copies
	, EnabledLogCategories(Other.EnabledLogCategories)
	, TextFilterExpressionEvaluator(Other.TextFilterExpressionEvaluator)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLogFilter& FLogFilter::operator=(const FLogFilter& Other)
{
	ChangeNumber = Other.ChangeNumber;
	VerbosityThreshold = Other.VerbosityThreshold;
	bIsFilterSetByVerbosity = Other.bIsFilterSetByVerbosity;
	bIsFilterSetByCategory = Other.bIsFilterSetByCategory;
	bIsFilterSetByText = Other.bIsFilterSetByText;
	bShowAllCategories = Other.bShowAllCategories;
	//AvailableLogCategories = Other.AvailableLogCategories; -- not needed in FLogFilter copies
	EnabledLogCategories = Other.EnabledLogCategories;
	TextFilterExpressionEvaluator = Other.TextFilterExpressionEvaluator;
	return *this;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLogFilter::Reset()
{
	ChangeNumber = 0;
	VerbosityThreshold = ELogVerbosity::All;
	bIsFilterSetByVerbosity = false;
	bIsFilterSetByCategory = false;
	bIsFilterSetByText = false;
	bShowAllCategories = true;
	AvailableLogCategories.Reset();
	EnabledLogCategories.Reset();
	TextFilterExpressionEvaluator.SetFilterText(FText::GetEmpty());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FLogFilter::IsMessageAllowed(const FLogMessageRecord& Message)
{
	// Filter by Verbosity
	if (bIsFilterSetByVerbosity)
	{
		if (Message.GetVerbosity() > VerbosityThreshold)
		{
			return false;
		}
	}

	// Filter by Category
	if (bIsFilterSetByCategory)
	{
		if (!EnabledLogCategories.Contains(FName(Message.GetCategory())))
		{
			return false;
		}
	}

	// Filter by Message text
	if (bIsFilterSetByText)
	{
		//FString Msg = Message.Message.ToString();
		//if (!Msg.Contains(FilterText, bIgnoreCase ? ESearchCase::IgnoreCase : ESearchCase::CaseSensitive))
		//{
		//	return true;
		//}

		if (!TextFilterExpressionEvaluator.TestTextFilter(FLogFilter_TextFilterExpressionContext(Message)))
		{
			return false;
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLogFilter::SyncAvailableCategories(const TSet<FName> Categories)
{
	// Remove obsolete categories.
	int32 OldNumAvailableLogCategories = AvailableLogCategories.Num();
	for (int32 Index = OldNumAvailableLogCategories - 1; Index >= 0; --Index)
	{
		if (!Categories.Contains(AvailableLogCategories[Index]))
		{
			EnabledLogCategories.Remove(AvailableLogCategories[Index]);
			AvailableLogCategories.RemoveAt(Index);
		}
	}
	if (AvailableLogCategories.Num() != OldNumAvailableLogCategories)
	{
		OnFilterByCategoryChanged();
	}

	// Add new categories (if any; duplicates will be ignored).
	for (const FName& Category : Categories)
	{
		AddAvailableLogCategory(Category);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLogFilter::AddAvailableLogCategory(const FName& LogCategory)
{
	// Use an insert-sort to keep AvailableLogCategories alphabetically sorted.
	int32 InsertIndex = 0;
	for (InsertIndex = AvailableLogCategories.Num() - 1; InsertIndex >= 0; --InsertIndex)
	{
		FName CheckCategory = AvailableLogCategories[InsertIndex];
		// No duplicates
		if (CheckCategory == LogCategory)
		{
			return;
		}
		else if (CheckCategory.Compare(LogCategory) < 0)
		{
			break;
		}
	}
	AvailableLogCategories.Insert(LogCategory, InsertIndex + 1);

	if (bShowAllCategories)
	{
		EnabledLogCategories.Add(LogCategory);
		OnFilterByCategoryChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLogFilter::ToggleShowAllCategories()
{
	bShowAllCategories = !bShowAllCategories;

	EnabledLogCategories.Reset();

	if (bShowAllCategories)
	{
		for (const auto& LogCategory : AvailableLogCategories)
		{
			EnabledLogCategories.Add(LogCategory);
		}
	}

	OnFilterByCategoryChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLogFilter::EnableLogCategory(const FName& LogCategory)
{
	bool bAlreadyEnabled = false;
	EnabledLogCategories.Add(LogCategory, &bAlreadyEnabled);
	if (!bAlreadyEnabled)
	{
		OnFilterByCategoryChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLogFilter::DisableLogCategory(const FName& LogCategory)
{
	if (EnabledLogCategories.Remove(LogCategory) > 0)
	{
		OnFilterByCategoryChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLogFilter::ToggleLogCategory(const FName& LogCategory)
{
	if (EnabledLogCategories.Remove(LogCategory) == 0)
	{
		EnabledLogCategories.Add(LogCategory);
	}
	OnFilterByCategoryChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLogFilter::EnableOnlyCategory(const FName& LogCategory)
{
	EnabledLogCategories.Reset();
	EnabledLogCategories.Add(LogCategory);
	OnFilterByCategoryChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FLogFilter::IsLogCategoryEnabled(const FName& LogCategory) const
{
	return EnabledLogCategories.Contains(LogCategory);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FLogFilteringAsyncTask
////////////////////////////////////////////////////////////////////////////////////////////////////

void FLogFilteringAsyncTask::DoWork()
{
	bool bCanceled = false;
	FilteredMessages.Reset();

#if !WITH_EDITOR
	if (Filter.IsFilterSetByText())
	{
		UE_LOGF(TraceInsights, Log, "[LogView] FLogFilteringAsyncTask::DoWork [%d to %d] by Text (\"%ls\")", StartIndex, EndIndex, *Filter.GetFilterText().ToString());
	}
	else
	{
		UE_LOGF(TraceInsights, Log, "[LogView] FLogFilteringAsyncTask::DoWork [%d to %d]", StartIndex, EndIndex);
	}
#endif // !WITH_EDITOR

	for (int32 Index = StartIndex; Index < EndIndex && !bCanceled; ++Index)
	{
		TSharedPtr<FLogMessageRecord> RecordPtr = LogView->GetCache().GetUncached(Index);

		if (RecordPtr.IsValid() && Filter.IsMessageAllowed(*RecordPtr))
		{
			FilteredMessages.Add(Index);
		}

		bCanceled = LogView->IsFilteringAsyncTaskCancelRequested();
	}

	if (bCanceled)
	{
		FilteredMessages.Reset();
#if !WITH_EDITOR
		UE_LOGF(TraceInsights, Log, "[LogView] FLogFilteringAsyncTask::DoWork CANCELED");
#endif // !WITH_EDITOR
	}
	else
	{
#if !WITH_EDITOR
		UE_LOGF(TraceInsights, Log, "[LogView] FLogFilteringAsyncTask::DoWork DONE (%d filtered messages)", FilteredMessages.Num());
#endif // !WITH_EDITOR
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
