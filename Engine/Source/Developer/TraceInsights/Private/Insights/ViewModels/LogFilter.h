// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Async/AsyncWork.h"
#include "Logging/LogVerbosity.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Stats/Stats.h"

// TraceInsights
#include "Insights/ViewModels/LogMessage.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename TTask> class FAsyncTask;

namespace UE::Insights
{

class SLogView;

////////////////////////////////////////////////////////////////////////////////////////////////////

/**
  * Holds information about Log filters.
  */
class FLogFilter
{
public:
	FLogFilter();
	FLogFilter(const FLogFilter& Other);
	FLogFilter& operator=(const FLogFilter& Other);

	void Reset();

	uint64 GetChangeNumber() const { return ChangeNumber; }

	/** Returns true if any messages should be filtered out. */
	bool IsFilterSet() const { return bIsFilterSetByVerbosity || bIsFilterSetByCategory || bIsFilterSetByText; }

	/**
	  * Checks the given message against set filters.
	  * @return true if the log message passes filter test and false if log message is filtered out
	  */
	bool IsMessageAllowed(const FLogMessageRecord& LogMessage);

	//////////////////////////////////////////////////
	// Filtering by Verbosity

	ELogVerbosity::Type GetVerbosityThreshold() const { return VerbosityThreshold; }
	void SetVerbosityThreshold(ELogVerbosity::Type InVerbosityThreshold)
	{
		VerbosityThreshold = InVerbosityThreshold;
		bIsFilterSetByVerbosity = VerbosityThreshold != ELogVerbosity::All;
		++ChangeNumber;
	}

public:
	bool IsFilterSetByVerbosity() const { return bIsFilterSetByVerbosity; }

	//////////////////////////////////////////////////
	// Available Log Categories

	const TArray<FName>& GetAvailableLogCategories() const { return AvailableLogCategories; }

	/** Sync list of available categories with a new one. */
	void SyncAvailableCategories(const TSet<FName> Categories);

	/** Adds a Log Category to the list of available categories, if it isn't already present. */
	void AddAvailableLogCategory(const FName& LogCategory);

	//////////////////////////////////////////////////
	// Filtering by Log Categories

	bool IsShowAllCategoriesEnabled() const { return bShowAllCategories; }

	/** Toggles the ShowAllCategories switch. */
	void ToggleShowAllCategories();

	/** Enables a Log Category in the filter. */
	void EnableLogCategory(const FName& LogCategory);

	/** Disables a Log Category in the filter. */
	void DisableLogCategory(const FName& LogCategory);

	/** Enables or disables a Log Category in the filter. */
	void ToggleLogCategory(const FName& LogCategory);

	/** Enables only the specified Log Category in the filter (disables all other). */
	void EnableOnlyCategory(const FName& LogCategory);

	/** Returns true if the specified log category is enabled. */
	bool IsLogCategoryEnabled(const FName& LogCategory) const;

	/** Returns the set of filter-enabled categories. */
	const TSet<FName>& GetEnabledLogCategories() const { return EnabledLogCategories; }

private:
	void OnFilterByCategoryChanged()
	{
		bIsFilterSetByCategory = (EnabledLogCategories.Num() < AvailableLogCategories.Num());
		++ChangeNumber;
	}

public:
	bool IsFilterSetByCategory() const { return bIsFilterSetByCategory; }

	//////////////////////////////////////////////////
	// Filtering by Message Text

	/** Set the Text to be used to filter by message text. */
	void SetFilterText(const FText& InFilterText)
	{
		//FilterText = InFilterText;
		//bIsFilterSetByText = (FilterText.Len() > 0);

		TextFilterExpressionEvaluator.SetFilterText(InFilterText);
		bIsFilterSetByText = TextFilterExpressionEvaluator.GetFilterType() != ETextFilterExpressionType::Empty || !TextFilterExpressionEvaluator.GetFilterText().IsEmpty();

		++ChangeNumber;
	}

	/** Get the Text currently being used to filter by message text. */
	const FText GetFilterText() const { return TextFilterExpressionEvaluator.GetFilterText(); }

	/** Returns Evaluator syntax errors (if any). */
	FText GetSyntaxErrors() { return TextFilterExpressionEvaluator.GetFilterErrorText(); }

	bool IsFilterSetByText() const { return bIsFilterSetByText; }

	//////////////////////////////////////////////////

private:
	/** A number incremented each time the filter changes. */
	uint64 ChangeNumber;

	ELogVerbosity::Type VerbosityThreshold;

	bool bIsFilterSetByVerbosity;
	bool bIsFilterSetByCategory;
	bool bIsFilterSetByText;

	/** true to allow all Log categories */
	bool bShowAllCategories;

	/** Array of Log Categories which are available for filter -- i.e. have been used in a log this session */
	TArray<FName> AvailableLogCategories;

	/** Array of Log Categories which are being used in the filter */
	TSet<FName> EnabledLogCategories;

	//FText FilterText;
	//bool bIgnoreCase;
	//bool bMatchWholeWord;
	//bool bUseRegEx;
	//bool bComplex;

	/** Expression evaluator that can be used to perform complex text filter queries */
	FTextFilterExpressionEvaluator TextFilterExpressionEvaluator;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FLogFilteringAsyncTask : public FNonAbandonableTask
{
public:
	FLogFilteringAsyncTask(int32 InStartIndex, int32 InEndIndex, const FLogFilter& InFilter, TSharedPtr<SLogView> InLogView)
		: StartIndex(InStartIndex)
		, EndIndex(InEndIndex)
		, Filter(InFilter)
		, LogView(InLogView)
	{
	}

	void DoWork();

	int32 GetStartIndex() const { return StartIndex; }
	int32 GetEndIndex() const { return EndIndex; }
	const FLogFilter& GetFilter() const { return Filter; }

	const TArray<uint32>& GetFilteredMessages() const { return FilteredMessages; }

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FLogFilteringAsyncTask, STATGROUP_ThreadPoolAsyncTasks);
	}

private:
	// [StartIndex, EndIndex) is the range of log messages to filter; accessible from worker thread.
	int32 StartIndex;
	int32 EndIndex;

	/** A copy of the filter settings. */
	FLogFilter Filter;

	/** Shared pointer to parent LogView widget. Used for accessing the cache and to check if cancel is requested. */
	TSharedPtr<SLogView> LogView;

	/** The output filtered messages. */
	TArray<uint32> FilteredMessages;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
