// Copyright Epic Games, Inc. All Rights Reserved.

#include "LogMessage.h"

#include "Misc/OutputDeviceHelper.h"
#include "Misc/ScopeLock.h"

// TraceServices
#include "TraceServices/Model/Log.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/Log.h"

#define LOCTEXT_NAMESPACE "UE::Insights::SLogView"

////////////////////////////////////////////////////////////////////////////////////////////////////
// FLogMessageRecord
////////////////////////////////////////////////////////////////////////////////////////////////////

FLogMessageRecord::FLogMessageRecord(const TraceServices::FLogMessageInfo& TraceLogMessage)
	: Time(TraceLogMessage.Time)
	//, Category(FText::FromString(TraceLogMessage.Category))
	//, Message(FText::FromString(TraceLogMessage.Message))
	, Line(TraceLogMessage.Line)
	, Index(static_cast<uint32>(TraceLogMessage.Index))
	, Verbosity(TraceLogMessage.Verbosity)
{
	// Strip the "Log" prefix.
	FString CategoryStr(TraceLogMessage.Category->Name);
	if (CategoryStr.StartsWith(TEXT("Log")))
	{
		CategoryStr.RightChopInline(3, EAllowShrinking::No);
	}
	Category = FText::FromString(MoveTemp(CategoryStr));

	FString UsableFile;
	UE::Insights::FInsightsManager::Get()->GetSourceFilePathHelper().GetUsableFilePath(TraceLogMessage.File, UsableFile);
	File = FText::FromString(UsableFile);

	// Strip the trailing whitespaces (ex. some messages ends with "\n" and we do not want the LogView rows to have an unnecessary increased height).
	FString MessageStr(TraceLogMessage.Message);
	MessageStr.TrimEndInline();
	Message = FText::FromString(MoveTemp(MessageStr));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FLogMessageRecord::GetIndexAsText() const
{
	return FText::AsNumber(Index);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FLogMessageRecord::GetTimeAsText() const
{
	return FText::FromString(UE::Insights::FormatTimeHMS(Time, UE::Insights::FTimeValue::Microsecond));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FLogMessageRecord::GetAbsoluteDateTime(double BaseDateTime) const
{
	if (BaseDateTime == 0)
	{
		return LOCTEXT("UnknownDateTime", "Unknown Date-Time");
	}
	
	FDateTime DT = FDateTime::FromUnixTimestampDecimal(BaseDateTime + Time);
	return FText::FromString(FString::Printf(TEXT("%04d.%02d.%02d-%02d.%02d.%02d:%03d"),
		DT.GetYear(),
		DT.GetMonth(),
		DT.GetDay(),
		DT.GetHour(),
		DT.GetMinute(),
		DT.GetSecond(),
		DT.GetMillisecond()));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FLogMessageRecord::GetVerbosityAsText() const
{
	return FText::FromString(FString(ToString(Verbosity)));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FLogMessageRecord::GetLineAsText() const
{
	return FText::AsNumber(Line);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FLogMessageRecord::ToDisplayString() const
{
	FTextBuilder TextBuilder;
	TextBuilder.AppendLine(GetTimeAsText());
	TextBuilder.AppendLineFormat(NSLOCTEXT("SLogView", "CategoryLine", "Category: {0}"), Category);
	TextBuilder.AppendLineFormat(NSLOCTEXT("SLogView", "MessageLine", "Message: {0}"), Message);
	return TextBuilder.ToText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FLogMessageCache
////////////////////////////////////////////////////////////////////////////////////////////////////

FLogMessageCache::FLogMessageCache()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLogMessageCache::SetSession(TSharedPtr<const TraceServices::IAnalysisSession> InSession)
{
	if (Session != InSession)
	{
		Reset();
		Session = InSession;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FLogMessageCache::Reset()
{
	FScopeLock Lock(&CriticalSection);
	Map.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLogMessageRecord& FLogMessageCache::Get(uint64 Index)
{
	{
		FScopeLock Lock(&CriticalSection);
		if (Map.Contains(Index))
		{
			return Map[Index];
		}

		// Just an arbitrary limit. Will purge the cache after this limit, to avoid using too much memory.
		if (Map.Num() > 10000)
		{
			Map.Reset();
		}
	}

	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());
		LogProvider.ReadMessage(Index, [this, Index](const TraceServices::FLogMessageInfo& Message)
		{
			LLM_SCOPE_BYTAG(Insights);
			FScopeLock Lock(&CriticalSection);
			FLogMessageRecord Entry(Message);
			Map.Add(Index, MoveTemp(Entry));
		});
	}

	{
		FScopeLock Lock(&CriticalSection);
		if (Map.Contains(Index))
		{
			return Map[Index];
		}
	}

	return InvalidEntry;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FLogMessageRecord> FLogMessageCache::GetUncached(uint64 Index) const
{
	TSharedPtr<FLogMessageRecord> EntryPtr;

	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());
		LogProvider.ReadMessage(Index, [&EntryPtr](const TraceServices::FLogMessageInfo& Message)
		{
			EntryPtr = MakeShared<FLogMessageRecord>(Message);
		});
	}

	return EntryPtr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
