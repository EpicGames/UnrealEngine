// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace TraceServices
{
	struct FLogMessageInfo;
	class IAnalysisSession;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

class FLogMessageRecord
{
public:
	FLogMessageRecord() = default;
	explicit FLogMessageRecord(const TraceServices::FLogMessageInfo& Message);

	uint32 GetIndex() const { return Index; }
	double GetTime() const { return Time; }
	ELogVerbosity::Type GetVerbosity() const { return Verbosity; }
	const TCHAR* GetCategory() const { return *Category.ToString(); }
	const TCHAR* GetMessage() const { return *Message.ToString(); }
	const TCHAR* GetFile() const { return *File.ToString(); }
	uint32 GetLine() const { return Line; }

	FString GetCategoryAsString() const { return Category.ToString(); }
	FString GetMessageAsString() const { return Message.ToString(); }

	FText GetIndexAsText() const;
	FText GetTimeAsText() const;
	FText GetAbsoluteDateTime(double BaseDateTime) const;
	FText GetVerbosityAsText() const;
	FText GetCategoryAsText() const { return Category; }
	FText GetMessageAsText() const { return Message; }
	FText GetFileAsText() const { return File; }
	FText GetLineAsText() const;

	FText ToDisplayString() const;

private:
	double Time = 0.0;
	FText Category;
	FText Message;
	FText File;
	uint32 Line = 0;
	uint32 Index = 0;
	ELogVerbosity::Type Verbosity = ELogVerbosity::Type::NoLogging;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FLogMessageCache
{
public:
	FLogMessageCache();

	void SetSession(TSharedPtr<const TraceServices::IAnalysisSession> InSession);
	void Reset();

	FLogMessageRecord& Get(uint64 Index);
	TSharedPtr<FLogMessageRecord> GetUncached(uint64 Index) const;

private:
	FCriticalSection CriticalSection;
	TSharedPtr<const TraceServices::IAnalysisSession> Session;
	TMap<uint64, FLogMessageRecord> Map;
	FLogMessageRecord InvalidEntry;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FLogMessageCategory
{
public:
	FName Name;
	bool bIsVisible;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FLogMessage
{
public:
	FLogMessage(const int32 InIndex) : Index(InIndex) {}

	int32 GetIndex() const { return Index; }

private:
	int32 Index;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
