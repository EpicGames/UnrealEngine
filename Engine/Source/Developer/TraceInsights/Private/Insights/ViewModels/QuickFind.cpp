// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuickFind.h"

namespace Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FQuickFind::FQuickFind(TSharedPtr<UE::Insights::FFilterConfigurator> InFilterConfiguratorViewModel)
{
	FilterConfigurator = InFilterConfiguratorViewModel;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FQuickFind::~FQuickFind()
{
	OnDestroyedEvent.Broadcast();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace Insights