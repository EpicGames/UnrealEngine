// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"

namespace UE::Insights { class FFilterConfigurator; }

namespace Insights
{
 
////////////////////////////////////////////////////////////////////////////////////////////////////

class FQuickFind
{
public:
	FQuickFind(TSharedPtr<UE::Insights::FFilterConfigurator> InFilterConfiguratorViewModel);

	~FQuickFind();

	TSharedPtr<UE::Insights::FFilterConfigurator> GetFilterConfigurator() { return FilterConfigurator; }

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// OnDestroyedEvent
public:
	/** The event to execute when an instance is destroyed. */
	DECLARE_MULTICAST_DELEGATE(FOnDestroyedEvent);
	FOnDestroyedEvent& GetOnDestroyedEvent() { return OnDestroyedEvent; }

private:
	FOnDestroyedEvent OnDestroyedEvent;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// OnFindFirstEvent
public:
	/** The event to execute when the user selects the "Find First" option. */
	DECLARE_MULTICAST_DELEGATE(FOnFindFirstEvent);
	FOnFindFirstEvent& GetOnFindFirstEvent() { return OnFindFirstEvent; }

private:
	FOnFindFirstEvent OnFindFirstEvent;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// OnFindPreviousEvent
public:
	/** The event to execute when the user selects the "Find Previous" option. */
	DECLARE_MULTICAST_DELEGATE(FOnFindPreviousEvent);
	FOnFindPreviousEvent& GetOnFindPreviousEvent() { return OnFindPreviousEvent; }

private:
	FOnFindPreviousEvent OnFindPreviousEvent;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// OnFindNextEvent
public:
	/** The event to execute when the user selects the "Find Next" option. */
	DECLARE_MULTICAST_DELEGATE(FOnFindNextEvent);
	FOnFindNextEvent& GetOnFindNextEvent() { return OnFindNextEvent; }

private:
	FOnFindNextEvent OnFindNextEvent;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// OnFindLastEvent
public:
	/** The event to execute when the user selects the "Find Last" option. */
	DECLARE_MULTICAST_DELEGATE(FOnFindLastEvent);
	FOnFindLastEvent& GetOnFindLastEvent() { return OnFindLastEvent; }

private:
	FOnFindLastEvent OnFindLastEvent;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// OnFindMaxEvent
public:
	/** The event to execute when the user selects the "Find Max" option. */
	DECLARE_MULTICAST_DELEGATE(FOnFindMaxEvent);
	FOnFindMaxEvent& GetOnFindMaxEvent() { return OnFindMaxEvent; }

private:
	FOnFindMaxEvent OnFindMaxEvent;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// OnFindMinEvent
public:
	/** The event to execute when the user selects the "Find Min" option. */
	DECLARE_MULTICAST_DELEGATE(FOnFindMinEvent);
	FOnFindMinEvent& GetOnFindMinEvent() { return OnFindMinEvent; }

private:
	FOnFindMinEvent OnFindMinEvent;	

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// OnFilterAllEvent
public:
	/** The event to execute when the user selects the "Filter All" option. */
	DECLARE_MULTICAST_DELEGATE(FOnFilterAllEvent);
	FOnFilterAllEvent& GetOnFilterAllEvent() { return OnFilterAllEvent; }

private:
	FOnFilterAllEvent OnFilterAllEvent;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// OnClearAllFilters
public:
	/** The event to execute when the user selects the "Clear Filters" option. */
	DECLARE_MULTICAST_DELEGATE(FOnClearFiltersEvent);
	FOnClearFiltersEvent& GetOnClearFiltersEvent() { return OnClearFiltersEvent; }

private:
	FOnClearFiltersEvent OnClearFiltersEvent;

	////////////////////////////////////////////////////////////////////////////////////////////////////

private:
	TSharedPtr<UE::Insights::FFilterConfigurator> FilterConfigurator;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace Insights