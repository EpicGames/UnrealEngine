// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/ViewModels/ThreadTrackEvent.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FThreadTrackEvent)

////////////////////////////////////////////////////////////////////////////////////////////////////

FThreadTrackEvent::FThreadTrackEvent(const TSharedRef<const FBaseTimingTrack> InTrack, double InStartTime, double InEndTime, uint32 InDepth)
	: FTimingEvent(InTrack, InStartTime, InEndTime, InDepth)
{}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FThreadTrackEvent::GetTimerIndex() const
{
	return TimerIndex;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTrackEvent::SetTimerIndex(uint32 InTimerIndex)
{
	TimerIndex = InTimerIndex;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FThreadTrackEvent::GetTimerId() const
{
	return (uint32)GetType();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FThreadTrackEvent::SetTimerId(uint32 InTimerId)
{
	SetType(InTimerId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////
