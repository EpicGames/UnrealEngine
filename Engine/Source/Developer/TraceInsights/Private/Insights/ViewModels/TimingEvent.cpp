// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/ViewModels/TimingEvent.h"

#include "Insights/ViewModels/BaseTimingTrack.h"

#define LOCTEXT_NAMESPACE "TimingEvent"

////////////////////////////////////////////////////////////////////////////////////////////////////
// ITimingEvent, FTimingEvent, ITimingEventRelation
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(ITimingEvent)
INSIGHTS_IMPLEMENT_RTTI(FTimingEvent)
INSIGHTS_IMPLEMENT_RTTI(ITimingEventRelation)

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FTimingEvent::ComputeEventColor(uint32 Id)
{
	return (Id * 0x2c2c57ed) | 0xFF000000;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FTimingEvent::ComputeEventColor(const TCHAR* Str)
{
	uint32 Color = 0;
	// Let the special event names (ex. "<unknown>", "<noname>", "<invalid>", etc.) to be colored in black.
	if (Str != nullptr && *Str != TEXT('<'))
	{
		for (const TCHAR* c = Str; *c; ++c)
		{
			Color = (Color + *c) * 0x2c2c57ed;
		}
	}
	return Color | 0xFF000000;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FTimingEvent::ComputeEventColor(const TCHAR* Str, TConstArrayView<uint8> Metadata)
{
	uint32 Color = 0;
	// Let the special event names (ex. "<unknown>", "<noname>", "<invalid>", etc.) to be colored in black.
	if (Str != nullptr && *Str != TEXT('<'))
	{
		for (const TCHAR* c = Str; *c; ++c)
		{
			Color = (Color + *c) * 0x2c2c57ed;
		}
	}

	for (int Index = 0; Index < Metadata.Num(); ++Index)
	{
		Color = (Color + Metadata[Index]) * 0x2c2c57ed;
	}

	return Color | 0xFF000000;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// ITimingEventFilter, FTimingEventFilter, ...
////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(ITimingEventFilter)
INSIGHTS_IMPLEMENT_RTTI(FTimingEventFilter)
INSIGHTS_IMPLEMENT_RTTI(FAcceptNoneTimingEventFilter)
INSIGHTS_IMPLEMENT_RTTI(FAcceptAllTimingEventFilter)
INSIGHTS_IMPLEMENT_RTTI(FAggregatedTimingEventFilter)
INSIGHTS_IMPLEMENT_RTTI(FAllAggregatedTimingEventFilter)
INSIGHTS_IMPLEMENT_RTTI(FAnyAggregatedTimingEventFilter)
INSIGHTS_IMPLEMENT_RTTI(FTimingEventFilterByMinDuration)
INSIGHTS_IMPLEMENT_RTTI(FTimingEventFilterByMaxDuration)
INSIGHTS_IMPLEMENT_RTTI(FTimingEventFilterByEventType)
INSIGHTS_IMPLEMENT_RTTI(FTimingEventFilterByFrameIndex)

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingEventFilter::FilterTrack(const FBaseTimingTrack& InTrack) const
{
	return (!bFilterByTrackTypeName || InTrack.IsKindOf(TrackTypeName)) &&
	       (!bFilterByTrackInstance || &InTrack == TrackInstance.Get());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
