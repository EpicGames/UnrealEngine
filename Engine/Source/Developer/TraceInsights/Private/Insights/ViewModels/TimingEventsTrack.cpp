// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/ViewModels/TimingEventsTrack.h"

#include "Fonts/FontMeasure.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/Stopwatch.h"

// TraceInsights
#include "Insights/TimingProfilerCommon.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingViewDrawHelper.h"

#define LOCTEXT_NAMESPACE "TimingEventsTrack"

////////////////////////////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FTimingEventsTrack)

bool FTimingEventsTrack::bUseDownSampling = true;

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingEventsTrack::FTimingEventsTrack()
	: FBaseTimingTrack()
	, NumLanes(0)
	, DrawState(MakeShared<FTimingEventsTrackDrawState>())
	, FilteredDrawState(MakeShared<FTimingEventsTrackDrawState>())
{
	SetValidLocations(ETimingTrackLocation::Scrollable | ETimingTrackLocation::TopDocked | ETimingTrackLocation::BottomDocked);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingEventsTrack::FTimingEventsTrack(const FString& InName)
	: FBaseTimingTrack(InName)
	, NumLanes(0)
	, DrawState(MakeShared<FTimingEventsTrackDrawState>())
	, FilteredDrawState(MakeShared<FTimingEventsTrackDrawState>())
{
	SetValidLocations(ETimingTrackLocation::Scrollable | ETimingTrackLocation::TopDocked | ETimingTrackLocation::BottomDocked);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingEventsTrack::~FTimingEventsTrack()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrack::Reset()
{
	FBaseTimingTrack::Reset();

	NumLanes = 0;
	DrawState->Reset();
	FilteredDrawState->Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrack::PreUpdate(const ITimingTrackUpdateContext& Context)
{
	for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
	{
		Track->PreUpdate(Context);
	}

	if (IsDirty() || Context.GetViewport().IsHorizontalViewportDirty())
	{
		ClearDirtyFlag();

		int32 MaxDepth = -1;

		{
			FTimingEventsTrackDrawStateBuilder Builder(*DrawState, Context.GetViewport(), Context.GetGeometry().Scale);

			BuildDrawState(Builder, Context);

			Builder.Flush();

			MaxDepth = FMath::Max(MaxDepth, Builder.GetMaxDepth());
			MaxDepth = FMath::Max(MaxDepth, GetMaxDepth());
		}

		const TSharedPtr<ITimingEventFilter> EventFilter = Context.GetEventFilter();
		if ((EventFilter.IsValid() && EventFilter->FilterTrack(*this)) || HasCustomFilter())
		{
			const FTimingTrackViewport& Viewport = Context.GetViewport();

			const bool bFastLastBuild = FilteredDrawStateInfo.LastBuildDuration < 0.005; // LastBuildDuration < 5ms
			const bool bFilterPointerChanged = !FilteredDrawStateInfo.LastEventFilter.HasSameObject(EventFilter.Get());
			const bool bFilterContentChanged = EventFilter.IsValid() ? FilteredDrawStateInfo.LastFilterChangeNumber != EventFilter->GetChangeNumber() : false;

			if (bFastLastBuild || bFilterPointerChanged || bFilterContentChanged)
			{
				FilteredDrawStateInfo.LastEventFilter = EventFilter;
				FilteredDrawStateInfo.LastFilterChangeNumber = EventFilter.IsValid() ? EventFilter->GetChangeNumber() : 0;
				FilteredDrawStateInfo.ViewportStartTime = Context.GetViewport().GetStartTime();
				FilteredDrawStateInfo.ViewportScaleX = Context.GetViewport().GetScaleX();
				FilteredDrawStateInfo.Counter = 0;
			}
			else
			{
				if (FilteredDrawStateInfo.ViewportStartTime == Viewport.GetStartTime() &&
					FilteredDrawStateInfo.ViewportScaleX == Viewport.GetScaleX())
				{
					if (FilteredDrawStateInfo.Counter > 0)
					{
						FilteredDrawStateInfo.Counter--;
					}
				}
				else
				{
					FilteredDrawStateInfo.ViewportStartTime = Context.GetViewport().GetStartTime();
					FilteredDrawStateInfo.ViewportScaleX = Context.GetViewport().GetScaleX();
					FilteredDrawStateInfo.Counter = 1; // wait
				}
			}

			if (FilteredDrawStateInfo.Counter == 0)
			{
				UE::Insights::FStopwatch Stopwatch;
				Stopwatch.Start();
				{
					FTimingEventsTrackDrawStateBuilder Builder(*FilteredDrawState, Context.GetViewport(), Context.GetGeometry().Scale);
					BuildFilteredDrawState(Builder, Context);
					Builder.Flush();
				}
				Stopwatch.Stop();
				FilteredDrawStateInfo.LastBuildDuration = Stopwatch.GetAccumulatedTime();
			}
			else
			{
				FilteredDrawState->Reset();
				FilteredDrawStateInfo.Opacity = 0.0f;
				SetDirtyFlag();
			}
		}
		else
		{
			FilteredDrawStateInfo.LastBuildDuration = 0.0;

			if (FilteredDrawStateInfo.LastEventFilter.IsValid())
			{
				FilteredDrawStateInfo.LastEventFilter.Reset();
				FilteredDrawStateInfo.LastFilterChangeNumber = 0;
				FilteredDrawStateInfo.Counter = 0;
				FilteredDrawState->Reset();
			}
		}

		SetNumLanes(MaxDepth + 1);
	}

	UpdateTrackHeight(Context);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrack::UpdateTrackHeight(const ITimingTrackUpdateContext& Context)
{
	const FTimingTrackViewport& Viewport = Context.GetViewport();

	const float CurrentTrackHeight = GetHeight();
	float DesiredTrackHeight = 0.0f;
	if (IsChildTrack())
	{
		DesiredTrackHeight = Viewport.GetLayout().ComputeChildTrackHeight(NumLanes);
	}
	else
	{
		DesiredTrackHeight = Viewport.GetLayout().ComputeTrackHeight(NumLanes);
	}

	DesiredTrackHeight += GetChildTracksTopHeight(Viewport.GetLayout());

	if (CurrentTrackHeight < DesiredTrackHeight)
	{
		float NewTrackHeight;
		if (Viewport.IsDirty(ETimingTrackViewportDirtyFlags::VLayoutChanged))
		{
			NewTrackHeight = DesiredTrackHeight;
		}
		else
		{
			NewTrackHeight = FMath::CeilToFloat(CurrentTrackHeight * 0.9f + DesiredTrackHeight * 0.1f);
		}
		SetHeight(NewTrackHeight);
	}
	else if (CurrentTrackHeight > DesiredTrackHeight)
	{
		float NewTrackHeight;
		if (Viewport.IsDirty(ETimingTrackViewportDirtyFlags::VLayoutChanged))
		{
			NewTrackHeight = DesiredTrackHeight;
		}
		else
		{
			NewTrackHeight = FMath::FloorToFloat(CurrentTrackHeight * 0.9f + DesiredTrackHeight * 0.1f);
		}
		SetHeight(NewTrackHeight);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrack::PostUpdate(const ITimingTrackUpdateContext& Context)
{
	for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
	{
		Track->PostUpdate(Context);
	}

	FBaseTimingTrack::PostUpdate(Context);

	constexpr float HeaderWidth = 100.0f;
	constexpr float HeaderHeight = 14.0f;

	const float MouseY = static_cast<float>(Context.GetMousePosition().Y);
	if (MouseY >= GetPosY() && MouseY < GetPosY() + GetHeight())
	{
		SetHoveredState(true);

		const float MouseX = static_cast<float>(Context.GetMousePosition().X);
		SetHeaderHoveredState(MouseX < HeaderWidth && MouseY < GetPosY() + HeaderHeight);
	}
	else
	{
		SetHoveredState(false);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
	{
		Track->Draw(Context);
	}

	DrawEvents(Context, 1.0f);

	if (!IsChildTrack())
	{
		DrawHeader(Context);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrack::PostDraw(const ITimingTrackDrawContext& Context) const
{
	for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
	{
		Track->PostDraw(Context);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrack::DrawEvents(const ITimingTrackDrawContext& Context, const float OffsetY) const
{
	const FTimingViewDrawHelper& Helper = *static_cast<const FTimingViewDrawHelper*>(&Context.GetHelper());

	if ((Context.GetEventFilter().IsValid() && Context.GetEventFilter()->FilterTrack(*this)) || HasCustomFilter())
	{
		Helper.DrawFadedEvents(GetDrawState(), *this, OffsetY, 0.1f);

		if (UpdateFilteredDrawStateOpacity())
		{
			Helper.DrawEvents(GetFilteredDrawState(), *this, OffsetY);
		}
		else
		{
			Helper.DrawFadedEvents(GetFilteredDrawState(), *this, OffsetY, GetFilteredDrawStateOpacity());
		}
	}
	else
	{
		Helper.DrawEvents(GetDrawState(), *this, OffsetY);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrack::DrawMarkers(const ITimingTrackDrawContext& Context, float LineY, float LineH) const
{
	const FTimingViewDrawHelper& Helper = *static_cast<const FTimingViewDrawHelper*>(&Context.GetHelper());

	if (Context.GetEventFilter().IsValid())
	{
		Helper.DrawMarkers(GetDrawState(), LineY, LineH, 0.2f);
		Helper.DrawMarkers(GetFilteredDrawState(), LineY, LineH, 0.75f * GetFilteredDrawStateOpacity());
	}
	else
	{
		Helper.DrawMarkers(GetDrawState(), LineY, LineH, 0.2f);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTimingEventsTrack::GetHeaderBackgroundLayerId(const ITimingTrackDrawContext& Context) const
{
	const FTimingViewDrawHelper& Helper = *static_cast<const FTimingViewDrawHelper*>(&Context.GetHelper());
	return Helper.GetHeaderBackgroundLayerId();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FTimingEventsTrack::GetHeaderTextLayerId(const ITimingTrackDrawContext& Context) const
{
	const FTimingViewDrawHelper& Helper = *static_cast<const FTimingViewDrawHelper*>(&Context.GetHelper());
	return Helper.GetHeaderTextLayerId();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrack::DrawHeader(const ITimingTrackDrawContext& Context) const
{
	const FTimingViewDrawHelper& Helper = *static_cast<const FTimingViewDrawHelper*>(&Context.GetHelper());
	Helper.DrawTrackHeader(*this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrack::DrawEvent(const ITimingTrackDrawContext& Context, const ITimingEvent& InTimingEvent, EDrawEventMode InDrawMode) const
{
	for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
	{
		if (InTimingEvent.CheckTrack(&Track.Get()))
		{
			Track->DrawEvent(Context, InTimingEvent, InDrawMode);
			return;
		}
	}

	if (InTimingEvent.CheckTrack(this) && InTimingEvent.Is<FTimingEvent>())
	{
		const FTimingEvent& TrackEvent = InTimingEvent.As<FTimingEvent>();
		const FTimingViewLayout& Layout = Context.GetViewport().GetLayout();
		float Y = TrackEvent.GetTrack()->GetPosY();
		Y += GetChildTracksTopHeight(Layout);
		Y += Layout.GetLaneY(TrackEvent.GetDepth());

		const FTimingViewDrawHelper& Helper = *static_cast<const FTimingViewDrawHelper*>(&Context.GetHelper());
		Helper.DrawTimingEventHighlight(TrackEvent.GetStartTime(), TrackEvent.GetEndTime(), Y, InDrawMode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FTimingEventsTrack::GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const
{
	const FTimingViewLayout& Layout = Viewport.GetLayout();

	float TopLaneY = GetPosY() + Layout.TimelineDY;
	float TrackLanesHeight = GetHeight();

	for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
	{
		float HeaderDY = InPosY - Track->GetPosY();
		float TrackHeightWithPadding = Track->GetHeight() + Layout.ChildTimelineDY;
		if (HeaderDY >= 0.0f && HeaderDY < TrackHeightWithPadding)
		{
			return Track->GetEvent(InPosX, InPosY, Viewport);
		}

		TopLaneY += TrackHeightWithPadding;
		TrackLanesHeight -= TrackHeightWithPadding;
	}

	const float DY = InPosY - TopLaneY;

	// If mouse is not above first sub-track or below last sub-track...
	if (DY >= 0 && DY < TrackLanesHeight)
	{
		const int32 Depth = static_cast<int32>(DY / (Layout.EventH + Layout.EventDY));
		const double SecondsPerPixel = 1.0 / Viewport.GetScaleX();
		const double TimeAtPosX = Viewport.SlateUnitsToTime(InPosX);

		return GetEvent(TimeAtPosX, SecondsPerPixel, Depth);
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<const ITimingEvent> FTimingEventsTrack::GetEvent(double InTime, double SecondsPerPixel, int32 Depth) const
{
	const double StartTime0 = InTime;
	const double EndTime0 = InTime;

	auto EventFilter = [Depth](double, double, uint32 EventDepth)
	{
		return EventDepth == Depth;
	};

	TSharedPtr<const ITimingEvent> FoundEvent = SearchEvent(FTimingEventSearchParameters(InTime, InTime, ETimingEventSearchFlags::StopAtFirstMatch, EventFilter));

	if (!FoundEvent.IsValid())
	{
		const double StartTime = InTime;
		const double EndTime = InTime + SecondsPerPixel; // +1px
		FoundEvent = SearchEvent(FTimingEventSearchParameters(StartTime, EndTime, ETimingEventSearchFlags::StopAtFirstMatch, EventFilter));
	}

	if (!FoundEvent.IsValid())
	{
		const double StartTime = InTime - SecondsPerPixel; // -1px
		const double EndTime = InTime + 2.0 * SecondsPerPixel; // +2px
		FoundEvent = SearchEvent(FTimingEventSearchParameters(StartTime, EndTime, ETimingEventSearchFlags::StopAtFirstMatch, EventFilter));
	}

	return FoundEvent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<ITimingEventFilter> FTimingEventsTrack::GetFilterByEvent(const TSharedPtr<const ITimingEvent> InTimingEvent) const
{
	if (InTimingEvent.IsValid() && InTimingEvent->Is<FTimingEvent>())
	{
		const FTimingEvent& Event = InTimingEvent->As<FTimingEvent>();
		TSharedRef<FTimingEventFilter> EventFilterRef = MakeShared<FTimingEventFilterByEventType>(Event.GetType());
		EventFilterRef->SetFilterByTrackTypeName(true);
		EventFilterRef->SetTrackTypeName(GetTypeName());
		return EventFilterRef;
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrack::DrawSelectedEventInfo(const FString& InText, const FTimingTrackViewport& Viewport, const UE::Insights::FDrawContext& DrawContext, const FSlateBrush* WhiteBrush, const FSlateFontInfo& Font) const
{
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const float FontScale = DrawContext.Geometry.Scale;
	const FVector2D Size = FontMeasureService->Measure(InText, Font, FontScale) / FontScale;
	const float W = static_cast<float>(Size.X);
	const float H = static_cast<float>(Size.Y);
	const float X = Viewport.GetWidth() - W - 23.0f;
	const float Y = Viewport.GetPosY() + Viewport.GetHeight() - H - 18.0f;

	const FLinearColor BackgroundColor(0.05f, 0.05f, 0.05f, 1.0f);
	const FLinearColor TextColor(0.7f, 0.7f, 0.7f, 1.0f);

	DrawContext.DrawBox(X - 8.0f, Y - 2.0f, W + 16.0f, H + 4.0f, WhiteBrush, BackgroundColor);
	DrawContext.LayerId++;

	DrawContext.DrawText(X, Y, InText, Font, TextColor);
	DrawContext.LayerId++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrack::DrawSelectedEventInfoEx(const FString& InText, const FString& InLeftText, const FString& InTopText, const FTimingTrackViewport& Viewport, const UE::Insights::FDrawContext& DrawContext, const FSlateBrush* WhiteBrush, const FSlateFontInfo& Font) const
{
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const float FontScale = DrawContext.Geometry.Scale;

	const FLinearColor BackgroundColor(0.05f, 0.05f, 0.05f, 1.0f);
	const FLinearColor TextColor(0.7f, 0.7f, 0.7f, 1.0f);
	const FLinearColor LeftTextColor(0.9f, 0.9f, 0.5f, 1.0f);
	const FLinearColor TopTextColor(0.3f, 0.3f, 0.3f, 1.0f);

	const FVector2D Size = FontMeasureService->Measure(InText, Font, FontScale) / FontScale;
	const float W = static_cast<float>(Size.X);
	const float H = static_cast<float>(Size.Y);
	const float X = Viewport.GetWidth() - W - 23.0f;
	const float Y = Viewport.GetPosY() + Viewport.GetHeight() - H - 18.0f;
	DrawContext.DrawBox(DrawContext.LayerId, X - 8.0f, Y - 2.0f, W + 16.0f, H + 4.0f, WhiteBrush, BackgroundColor);
	DrawContext.DrawText(DrawContext.LayerId + 1, X, Y, InText, Font, TextColor);

	if (InLeftText.Len() > 0)
	{
		const FVector2D Size2 = FontMeasureService->Measure(InLeftText, Font, FontScale) / FontScale;
		const float W2 = static_cast<float>(Size2.X);
		const float H2 = static_cast<float>(Size2.Y);
		const float X2 = X - W2 - 4.0f;
		const float Y2 = Y;
		DrawContext.DrawBox(DrawContext.LayerId, X2 - 8.0f, Y2 - 2.0f, W2 + 16.0f, H2 + 4.0f, WhiteBrush, BackgroundColor);
		DrawContext.DrawText(DrawContext.LayerId + 1, X2, Y2, InLeftText, Font, LeftTextColor);
	}
	if (InTopText.Len() > 0)
	{
		const FVector2D Size2 = FontMeasureService->Measure(InTopText, Font, FontScale) / FontScale;
		const float W2 = static_cast<float>(Size2.X);
		const float H2 = static_cast<float>(Size2.Y);
		const float X2 = Viewport.GetWidth() - W2 - 23.0f;
		const float Y2 = Y - H2 - 4.0f;
		DrawContext.DrawBox(DrawContext.LayerId, X2 - 8.0f, Y2 - 2.0f, W2 + 16.0f, H2 + 4.0f, WhiteBrush, BackgroundColor);
		DrawContext.DrawText(DrawContext.LayerId + 1, X2, Y2, InTopText, Font, TopTextColor);
	}

	DrawContext.LayerId += 2;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
