// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/ViewModels/TimingTrackViewport.h"

#include "Widgets/Layout/SScrollBar.h"

#include <limits>

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingTrackViewport::UpdateSize(const float InWidth, const float InHeight)
{
	bool bSizeChanged = false;
	if (Width != InWidth)
	{
		Width = InWidth;
		EndTime = SlateUnitsToTime(Width);
		AddDirtyFlags(ETimingTrackViewportDirtyFlags::HSizeChanged);
		bSizeChanged = true;
	}
	if (Height != InHeight)
	{
		Height = InHeight;
		AddDirtyFlags(ETimingTrackViewportDirtyFlags::VSizeChanged);
		bSizeChanged = true;
	}
	return bSizeChanged;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingTrackViewport::ScrollAtTime(const double Time)
{
	const double NewStartTime = AlignTimeToPixel(Time);
	if (NewStartTime != StartTime)
	{
		StartTime = NewStartTime;
		EndTime = SlateUnitsToTime(Width);
		AddDirtyFlags(ETimingTrackViewportDirtyFlags::HPositionChanged);
		return true;
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingTrackViewport::CenterOnTimeInterval(const double Time, const double Duration)
{
	double NewStartTime = Time;
	const double ViewportDuration = static_cast<double>(Width) / ScaleX;
	if (Duration < ViewportDuration)
	{
		NewStartTime -= (ViewportDuration - Duration) / 2.0;
	}
	return ScrollAtTime(NewStartTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingTrackViewport::ZoomOnTimeInterval(const double Time, const double Duration)
{
	const double NewScaleX = FMath::Clamp(static_cast<double>(Width) / Duration, MinScaleX, MaxScaleX);
	double NewStartTime = Time;
	const double NewViewportDuration = static_cast<double>(Width) / NewScaleX;
	if (Duration < NewViewportDuration)
	{
		NewStartTime -= (NewViewportDuration - Duration) / 2;
	}
	NewStartTime = AlignTimeToPixel(NewStartTime, NewScaleX);
	if (NewStartTime != StartTime || NewScaleX != ScaleX)
	{
		StartTime = NewStartTime;
		ScaleX = NewScaleX;
		EndTime = SlateUnitsToTime(Width);
		AddDirtyFlags(ETimingTrackViewportDirtyFlags::HPositionChanged | ETimingTrackViewportDirtyFlags::HScaleChanged);
		return true;
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingTrackViewport::RelativeZoomWithFixedX(const float Delta, const float X)
{
	constexpr double ZoomStep = 0.25; // as percent
	double NewScaleX;

	if (Delta > 0)
	{
		NewScaleX = ScaleX * FMath::Pow(1.0 + ZoomStep, static_cast<double>(Delta));
	}
	else
	{
		NewScaleX = ScaleX * FMath::Pow(1.0 / (1.0 + ZoomStep), static_cast<double>(-Delta));
	}

	return ZoomWithFixedX(NewScaleX, X);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingTrackViewport::ZoomWithFixedX(const double NewScaleX, const float X)
{
	const double LocalNewScaleX = FMath::Clamp(NewScaleX, MinScaleX, MaxScaleX);
	if (LocalNewScaleX != ScaleX)
	{
		// Time at local position X should remain the same. So we resolve equation:
		//   StartTime + X / ScaleX == NewStartTime + X / NewScaleX
		//   ==> NewStartTime = StartTime + X / ScaleX - X / NewScaleX
		//   ==> NewStartTime = StartTime + X * (1 / ScaleX - 1 / NewScaleX)
		StartTime += static_cast<double>(X) * (1.0 / ScaleX - 1.0 / LocalNewScaleX);
		ScaleX = LocalNewScaleX;
		EndTime = SlateUnitsToTime(Width);
		AddDirtyFlags(ETimingTrackViewportDirtyFlags::HPositionChanged | ETimingTrackViewportDirtyFlags::HScaleChanged);
		return true;
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingTrackViewport::SetScaleX(const double NewScaleX)
{
	const double LocalNewScaleX = FMath::Clamp(NewScaleX, MinScaleX, MaxScaleX);
	if (LocalNewScaleX != ScaleX)
	{
		ScaleX = LocalNewScaleX;
		AddDirtyFlags(ETimingTrackViewportDirtyFlags::HScaleChanged);
		return true;
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double FTimingTrackViewport::RestrictEndTime(const double InEndTime) const
{
	if (InEndTime == DBL_MAX || InEndTime == std::numeric_limits<double>::infinity())
	{
		return MaxValidTime;
	}
	else
	{
		return InEndTime;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double FTimingTrackViewport::RestrictDuration(const double InStartTime, const double InEndTime) const
{
	if (InEndTime == DBL_MAX || InEndTime == std::numeric_limits<double>::infinity())
	{
		return MaxValidTime - InStartTime;
	}
	else
	{
		return InEndTime - InStartTime;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingTrackViewport::GetHorizontalScrollLimits(double& OutMinT, double& OutMaxT)
{
	const double ViewportDuration = static_cast<double>(Width) / ScaleX;
	if (MaxValidTime < ViewportDuration)
	{
		OutMinT = MaxValidTime - ViewportDuration;
		OutMaxT = MinValidTime;
	}
	else
	{
		OutMinT = MinValidTime - 0.25 * ViewportDuration;
		OutMaxT = MaxValidTime - 0.75 * ViewportDuration;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingTrackViewport::EnforceHorizontalScrollLimits(const double U)
{
	double MinT, MaxT;
	GetHorizontalScrollLimits(MinT, MaxT);

	double NewStartTime = StartTime;
	if (NewStartTime < MinT)
	{
		NewStartTime = (1.0 - U) * NewStartTime + U * MinT;
		if (FMath::IsNearlyEqual(NewStartTime, MinT, 1.0 / ScaleX))
		{
			NewStartTime = MinT;
		}
	}
	else if (NewStartTime > MaxT)
	{
		NewStartTime = (1.0 - U) * NewStartTime + U * MaxT;
		if (FMath::IsNearlyEqual(NewStartTime, MaxT, 1.0 / ScaleX))
		{
			NewStartTime = MaxT;
		}
		if (NewStartTime < MinT)
		{
			NewStartTime = MinT;
		}
	}

	return ScrollAtTime(NewStartTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingTrackViewport::UpdateLayout()
{
	if (Layout.Update())
	{
		AddDirtyFlags(ETimingTrackViewportDirtyFlags::VLayoutChanged);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingTrackViewport::OnUserScrolled(TSharedPtr<SScrollBar> ScrollBar, float ScrollOffset)
{
	const double S = 1.0 / (MaxValidTime - MinValidTime);
	const double Page = EndTime - StartTime;
	const float ThumbSizeFraction = FMath::Clamp<float>(static_cast<float>(Page * S), 0.0f, 1.0f);
	const float OffsetFraction = FMath::Clamp<float>(ScrollOffset, 0.0f, 1.0f - ThumbSizeFraction);

	ScrollBar->SetState(OffsetFraction, ThumbSizeFraction);

	const double NewStartTime = MinValidTime + static_cast<double>(OffsetFraction) * (MaxValidTime - MinValidTime);
	if (NewStartTime != StartTime)
	{
		ScrollAtTime(NewStartTime);
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingTrackViewport::UpdateScrollBar(TSharedPtr<SScrollBar> ScrollBar) const
{
	const double S = 1.0 / (MaxValidTime - MinValidTime);
	const double Page = EndTime - StartTime;
	const float ThumbSizeFraction = FMath::Clamp<float>(static_cast<float>(Page * S), 0.0f, 1.0f);
	const float ScrollOffset = static_cast<float>((StartTime - MinValidTime) * S);
	const float OffsetFraction = FMath::Clamp<float>(ScrollOffset, 0.0f, 1.0f - ThumbSizeFraction);

	ScrollBar->SetState(OffsetFraction, ThumbSizeFraction);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingTrackViewport::OnUserScrolledY(TSharedPtr<SScrollBar> ScrollBar, float ScrollOffset)
{
	const float S = 1.0f / ScrollHeight;
	const float Page = Height - TopOffset - BottomOffset;
	const float ThumbSizeFraction = FMath::Clamp<float>(static_cast<float>(Page * S), 0.0f, 1.0f);
	const float OffsetFraction = FMath::Clamp<float>(ScrollOffset, 0.0f, 1.0f - ThumbSizeFraction);

	ScrollBar->SetState(OffsetFraction, ThumbSizeFraction);

	const float NewScrollPosY = OffsetFraction * ScrollHeight;
	if (NewScrollPosY != ScrollPosY)
	{
		ScrollPosY = NewScrollPosY;
		AddDirtyFlags(ETimingTrackViewportDirtyFlags::VPositionChanged);
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingTrackViewport::UpdateScrollBarY(TSharedPtr<SScrollBar> ScrollBar) const
{
	const float S = 1.0f / ScrollHeight;
	const float Page = Height - TopOffset - BottomOffset;
	const float ThumbSizeFraction = FMath::Clamp<float>(Page * S, 0.0f, 1.0f);
	const float ScrollOffset = ScrollPosY * S;
	const float OffsetFraction = FMath::Clamp<float>(ScrollOffset, 0.0f, 1.0f - ThumbSizeFraction);

	ScrollBar->SetState(OffsetFraction, ThumbSizeFraction);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
