// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimingViewDrawHelper.h"

#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/DrawElements.h"
#include "Styling/AppStyle.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/TimingProfiler/Tracks/MarkersTimingTrack.h"
#include "Insights/ViewModels/DrawHelpers.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingEventsTrack.h"
#include "Insights/ViewModels/TimingTrackViewport.h"

#define INSIGHTS_USE_LEGACY_BORDER 0

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimingEventsTrackDrawStateBuilder
////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingEventsTrackDrawStateBuilder::FTimingEventsTrackDrawStateBuilder(FTimingEventsTrackDrawState& InState, const FTimingTrackViewport& InViewport, float InFontScale)
	: DrawState(InState)
	, Viewport(InViewport)
	, MaxDepth(-1)
	, LastEventX2()
	, LastBox()
	, EventFont(FAppStyle::Get().GetFontStyle("SmallFont"))
	, FontScale(InFontScale)
{
	DrawState.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrackDrawStateBuilder::AppendDurationToEventName(FString& InOutEventName, const double InDuration)
{
	InOutEventName += TEXT(" (");
	InOutEventName += UE::Insights::FormatTimeAuto(InDuration);
	InOutEventName += TEXT(")");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrackDrawStateBuilder::AddEvent(double EventStartTime, double EventEndTime, uint32 EventDepth, const TCHAR* EventName, uint64 EventType, uint32 EventColor)
{
	if (EventColor == 0)
	{
		//EventColor = FTimingEvent::ComputeEventColor(static_cast<uint32>(EventType));
		EventColor = FTimingEvent::ComputeEventColor(EventName);
	}

	AddEvent(EventStartTime, EventEndTime, EventDepth, EventColor,
		[EventName, EventType, EventStartTime, EventEndTime](float Width)
		{
			//FString Name = FString::Printf(TEXT("%s [%llu]"), EventName, EventType);
			FString Name = EventName;

			const float MinWidth = static_cast<float>(Name.Len()) * 4.0f + 32.0f;
			if (Width > MinWidth)
			{
				AppendDurationToEventName(Name, EventEndTime - EventStartTime);
			}

			return Name;
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrackDrawStateBuilder::AddEvent(double EventStartTime, double EventEndTime, uint32 EventDepth, uint32 EventColor, GetEventNameCallback GetEventNameCallback)
{
	DrawState.NumEvents++;

	float EventX1 = Viewport.TimeToSlateUnitsRounded(EventStartTime);
	if (EventX1 > Viewport.GetWidth())
	{
		return;
	}

	const double RestrictedEndTime = Viewport.RestrictEndTime(EventEndTime);
	float EventX2 = Viewport.TimeToSlateUnitsRounded(RestrictedEndTime);
	if (EventX2 < 0)
	{
		return;
	}

	// Timing events are displayed with minimum 1px (including empty ones).
	if (EventX1 == EventX2)
	{
		EventX2 = EventX1 + 1.0f;
	}

	const int32 Depth = static_cast<int32>(EventDepth);
	if (Depth > MaxDepth)
	{
		MaxDepth = Depth;
	}

	// Ensure we have enough slots in array. See LastBox[Depth] usage.
	if (LastBox.Num() <= Depth)
	{
		LastBox.AddDefaulted(Depth + 1 - LastBox.Num());
	}

	// Ensure we have enough slots in array. See LastEventX2[Depth] usage.
	while (LastEventX2.Num() <= Depth)
	{
		LastEventX2.Add(-1.0f);
	}

	// Limit event width on the viewport's left side.
	// This also makes the text to be displayed in viewport, for very long events.
	const float MinX = -2.0f; // -2 allows event border to remain outside screen
	if (EventX1 < MinX)
	{
		EventX1 = MinX;
	}

	// Limit event width on the viewport's right side.
	const float MaxX = Viewport.GetWidth() + 2.0f; // +2 allows event border to remain outside screen
	if (EventX2 > MaxX)
	{
		EventX2 = MaxX;
	}

	const float EventW = EventX2 - EventX1;

	// Optimization...
	if (EventW == 1.0f && EventX1 == LastEventX2[Depth] - 1.0f)
	{
		// Do no draw 1 pixel event if the last event was ended on that pixel.
		return;
	}

	//////////////////////////////////////////////////
	// Coloring

	FLinearColor EventColorFill;
	EventColorFill.R = ((EventColor >> 16) & 0xFF) / 255.0f;
	EventColorFill.G = ((EventColor >>  8) & 0xFF) / 255.0f;
	EventColorFill.B = ((EventColor      ) & 0xFF) / 255.0f;
	EventColorFill.A = ((EventColor >> 24) & 0xFF) / 255.0f;

	//constexpr float BorderColorFactor = 0.75f; // darker border
	constexpr float BorderColorFactor = 1.25f; // brighter border

	//////////////////////////////////////////////////

	// Save X2, for current depth.
	LastEventX2[Depth] = EventX2;

	if (EventW > 2.0f)
	{
		FBoxData& Box = LastBox[Depth];
		if (Box.X1 < Box.X2)
		{
			FlushBox(Box, Depth);
			Box.Reset();
		}

		// Fill inside of the timing event box.
		DrawState.InsideBoxes.AddUninitialized();
		FTimingEventsTrackDrawState::FBoxPrimitive& InsideBox = DrawState.InsideBoxes.Last();
		InsideBox.Depth = Depth;
		InsideBox.X = EventX1 + 1.0f;
		InsideBox.W = EventW - 2.0f;
		InsideBox.Color = EventColorFill;

		// Add border around the timing event box.
		DrawState.Borders.AddUninitialized();
		FTimingEventsTrackDrawState::FBoxPrimitive& BorderBox = DrawState.Borders.Last();
		BorderBox.Depth = Depth;
		BorderBox.X = EventX1;
		BorderBox.W = EventW;
		BorderBox.Color = FLinearColor(EventColorFill.R * BorderColorFactor, EventColorFill.G * BorderColorFactor, EventColorFill.B * BorderColorFactor, EventColorFill.A);
	}
	else // 1px or 2px boxes
	{
		FBoxData& Box = LastBox[Depth];

		// Check if we can merge this box with previous one, if any.
		// Note: We are assuming events are processed in sorted order by X1.
		if (EventColor == Box.Color && // same color
			EventX1 <= Box.X2) // overlapping or adjacent
		{
			// Merge it with previous box.
			Box.X2 = EventX2;
			DrawState.NumMergedBoxes++;
		}
		else
		{
			// Flush previous box, if any.
			if (Box.X1 < Box.X2)
			{
				FlushBox(Box, Depth);
			}

			// Start new "merge box".
			Box.X1 = EventX1;
			Box.X2 = EventX2;
			Box.Color = EventColor;
			Box.LinearColor = FLinearColor(EventColorFill.R * BorderColorFactor, EventColorFill.G * BorderColorFactor, EventColorFill.B * BorderColorFactor, EventColorFill.A);
		}
	}

	// Draw the name of the timing event.
	if (EventW > 8.0f)
	{
		const FString Name = GetEventNameCallback(EventW);

		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const int32 HorizontalOffset = FMath::RoundToInt((EventW - 2.0f) * FontScale);
		const int32 LastWholeCharacterIndex = FontMeasureService->FindLastWholeCharacterIndexBeforeOffset(Name, EventFont, HorizontalOffset, FontScale);

		if (LastWholeCharacterIndex >= 0)
		{
			// Grey threshold is shifted toward black (0.4 instead of 0.5 in test below) due to "area rule":
			// a large gray surface (background of a timing event in this case) is perceived lighter than a smaller area (text pixels).
			// Ref: https://books.google.ro/books?id=0pVr7dhmdWYC
			const bool bIsDarkColor = (EventColorFill.GetLuminance() < 0.4f);

			DrawState.Texts.AddDefaulted();
			FTimingEventsTrackDrawState::FTextPrimitive& DrawText = DrawState.Texts.Last();
			DrawText.Depth = Depth;
			DrawText.X = EventX1 + 2.0f;
			DrawText.Text = Name.Left(LastWholeCharacterIndex + 1);
			DrawText.bWhite = bIsDarkColor;
			DrawText.Color = FLinearColor(EventColorFill.R * BorderColorFactor, EventColorFill.G * BorderColorFactor, EventColorFill.B * BorderColorFactor, EventColorFill.A);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrackDrawStateBuilder::FlushBox(const FBoxData& Box, const int32 Depth)
{
	DrawState.Boxes.AddUninitialized();
	FTimingEventsTrackDrawState::FBoxPrimitive& DrawBox = DrawState.Boxes.Last();
	DrawBox.Depth = Depth;
	DrawBox.X = Box.X1;
	DrawBox.W = Box.X2 - Box.X1;
	DrawBox.Color = Box.LinearColor;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsTrackDrawStateBuilder::Flush()
{
	// Flush merged boxes.
	for (int32 Depth = 0; Depth <= MaxDepth; ++Depth)
	{
		const FBoxData& Box = LastBox[Depth];
		if (Box.X1 < Box.X2)
		{
			FlushBox(Box, Depth);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimingViewDrawHelper
////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingViewDrawHelper::FTimingViewDrawHelper(const UE::Insights::FDrawContext& InDrawContext, const FTimingTrackViewport& InViewport)
	: DrawContext(InDrawContext)
	, Viewport(InViewport)
	, WhiteBrush(FInsightsStyle::Get().GetBrush("WhiteBrush"))
	, EventBorderBrush(FInsightsStyle::Get().GetBrush("EventBorder"))
	, HoveredEventBorderBrush(FInsightsStyle::Get().GetBrush("HoveredEventBorder"))
	, SelectedEventBorderBrush(FInsightsStyle::Get().GetBrush("SelectedEventBorder"))
	, BackgroundAreaBrush(WhiteBrush)
	, ValidAreaColor(0.07f, 0.07f, 0.07f, 1.0f)
	, InvalidAreaColor(0.1f, 0.07f, 0.07f, 1.0f)
	, EdgeColor(0.05f, 0.05f, 0.05f, 1.0f)
	, EventFont(FAppStyle::Get().GetFontStyle("SmallFont"))
	, ValidAreaX(0.0f)
	, ValidAreaW(0.0f)
	, NumEvents(0)
	, NumMergedBoxes(0)
	, NumDrawBoxes(0)
	, NumDrawBorders(0)
	, NumDrawTexts(0)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FTimingViewDrawHelper::FTimingViewDrawHelper(const ::FDrawContext& InDrawContext, const FTimingTrackViewport& InViewport)
	: FTimingViewDrawHelper((const UE::Insights::FDrawContext&)InDrawContext, InViewport)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

////////////////////////////////////////////////////////////////////////////////////////////////////

FTimingViewDrawHelper::~FTimingViewDrawHelper()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewDrawHelper::DrawBackground() const
{
	const float Y = Viewport.GetPosY();
	const float H = FMath::CeilToFloat(Viewport.GetHeight());
	FDrawHelpers::DrawBackground(DrawContext, BackgroundAreaBrush, Viewport, Y, H, ValidAreaX, ValidAreaW); // also computes ValidAreaX and ValidAreaW
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewDrawHelper::DrawEvents(const FTimingEventsTrackDrawState& DrawState, const FTimingEventsTrack& Track, const float OffsetY) const
{
	const float TrackH = Track.GetHeight();
	if (TrackH > 0.0f)
	{
		const FTimingViewLayout& Layout = Viewport.GetLayout();

		NumEvents += DrawState.GetNumEvents();
		NumMergedBoxes += DrawState.GetNumMergedBoxes();

		float TopLaneY = Track.GetPosY();
		TopLaneY += OffsetY + Layout.TimelineDY;
		TopLaneY += Track.GetChildTracksTopHeight(Layout);

		// Draw filled boxes (merged borders).
		//if (Layout.EventH > 0.0f)
		{
			const int32 EventFillLayerId = ReservedLayerId + ToInt32(EDrawLayer::EventFill);
			const float EventFillH = Layout.EventH;
			for (const FTimingEventsTrackDrawState::FBoxPrimitive& Box : DrawState.Boxes)
			{
				const float Y = TopLaneY + (Layout.EventH + Layout.EventDY) * static_cast<float>(Box.Depth);
				DrawContext.DrawBox(EventFillLayerId, Box.X, Y, Box.W, EventFillH, WhiteBrush, Box.Color);
			}
			NumDrawBoxes += DrawState.Boxes.Num();
		}

		// Draw filled boxes (event inside area).
		if (Layout.EventH > 2.0f)
		{
			const int32 EventFillLayerId = ReservedLayerId + ToInt32(EDrawLayer::EventFill);
			const float EventFillH = Layout.EventH - 2.0f;
			for (const FTimingEventsTrackDrawState::FBoxPrimitive& Box : DrawState.InsideBoxes)
			{
				const float Y = TopLaneY + (Layout.EventH + Layout.EventDY) * static_cast<float>(Box.Depth) + 1.0f;
				DrawContext.DrawBox(EventFillLayerId, Box.X, Y, Box.W, EventFillH, WhiteBrush, Box.Color);
			}
			NumDrawBoxes += DrawState.InsideBoxes.Num();
		}

		// Draw borders.
		//if (Layout.EventH > 0.0f)
		{
			const int32 EventBorderLayerId = ReservedLayerId + ToInt32(EDrawLayer::EventBorder);
			const float EventBorderH = Layout.EventH;
			for (const FTimingEventsTrackDrawState::FBoxPrimitive& Box : DrawState.Borders)
			{
				const float Y = TopLaneY + (Layout.EventH + Layout.EventDY) * static_cast<float>(Box.Depth);
#if INSIGHTS_USE_LEGACY_BORDER
				DrawContext.DrawBox(EventBorderLayerId, Box.X, Y, Box.W, EventBorderH, EventBorderBrush, Box.Color);
#else
				DrawContext.DrawBox(EventBorderLayerId, Box.X, Y, Box.W, EventBorderH, WhiteBrush, Box.Color);
#endif
			}
			NumDrawBorders += DrawState.Borders.Num();
		}

		// Draw texts.
		constexpr float TextMinEventH = 7.0f;
		if (Layout.EventH > TextMinEventH)
		{
			float TextOpacity = 1.0f;
			if (Layout.EventH < FTimingViewLayout::NormalLayoutEventH)
			{
				TextOpacity = (Layout.EventH - TextMinEventH + 1.0f) / (FTimingViewLayout::NormalLayoutEventH - TextMinEventH + 1.0f);
			}
			const FLinearColor WhiteColor(1.0f, 1.0f, 1.0f, TextOpacity);
			const FLinearColor BlackColor(0.0f, 0.0f, 0.0f, TextOpacity);

			const float TextY0 = TopLaneY + 1.0f - (FTimingViewLayout::NormalLayoutEventH - Layout.EventH) / 2.0f;
			const float TextDY = Layout.EventH + Layout.EventDY;

			const int32 EventTextLayerId = ReservedLayerId + ToInt32(EDrawLayer::EventText);
			for (const FTimingEventsTrackDrawState::FTextPrimitive& Text : DrawState.Texts)
			{
				const float Y = TextY0 + TextDY * static_cast<float>(Text.Depth);
				DrawContext.DrawText(EventTextLayerId, Text.X, Y, Text.Text, EventFont, Text.bWhite ? WhiteColor : BlackColor);
			}
			NumDrawTexts += DrawState.Texts.Num();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewDrawHelper::DrawFadedEvents(const FTimingEventsTrackDrawState& DrawState, const FTimingEventsTrack& Track, const float OffsetY, const float Opacity) const
{
	const float TrackH = Track.GetHeight();
	if (TrackH > 0.0f)
	{
		const FTimingViewLayout& Layout = Viewport.GetLayout();

		NumEvents += DrawState.GetNumEvents();
		NumMergedBoxes += DrawState.GetNumMergedBoxes();

		float TopLaneY = Track.GetPosY();
		TopLaneY += OffsetY + Layout.TimelineDY;
		TopLaneY += Track.GetChildTracksTopHeight(Layout);

		// Draw filled boxes (merged borders).
		//if (Layout.EventH > 0.0f)
		{
			const int32 EventFillLayerId = ReservedLayerId + ToInt32(EDrawLayer::EventFill);
			const float EventFillH = Layout.EventH;
			for (const FTimingEventsTrackDrawState::FBoxPrimitive& Box : DrawState.Boxes)
			{
				const float Y = TopLaneY + (Layout.EventH + Layout.EventDY) * static_cast<float>(Box.Depth);
				DrawContext.DrawBox(EventFillLayerId, Box.X, Y, Box.W, EventFillH, WhiteBrush, Box.Color.CopyWithNewOpacity(Opacity));
			}
			NumDrawBoxes += DrawState.Boxes.Num();
		}

		// Draw filled boxes (event inside area).
		if (Layout.EventH > 2.0f)
		{
			const int32 EventFillLayerId = ReservedLayerId + ToInt32(EDrawLayer::EventFill);
			const float EventFillH = Layout.EventH - 2.0f;
			for (const FTimingEventsTrackDrawState::FBoxPrimitive& Box : DrawState.InsideBoxes)
			{
				const float Y = TopLaneY + (Layout.EventH + Layout.EventDY) * static_cast<float>(Box.Depth) + 1.0f;
				DrawContext.DrawBox(EventFillLayerId, Box.X, Y, Box.W, EventFillH, WhiteBrush, Box.Color.CopyWithNewOpacity(Opacity));
			}
			NumDrawBoxes += DrawState.InsideBoxes.Num();
		}

		// Draw borders.
		//if (Layout.EventH > 0.0f)
		{
			const int32 EventBorderLayerId = ReservedLayerId + ToInt32(EDrawLayer::EventBorder);
			const float EventBorderH = Layout.EventH;
			for (const FTimingEventsTrackDrawState::FBoxPrimitive& Box : DrawState.Borders)
			{
				const float Y = TopLaneY + (Layout.EventH + Layout.EventDY) * static_cast<float>(Box.Depth);
#if INSIGHTS_USE_LEGACY_BORDER
				DrawContext.DrawBox(EventBorderLayerId, Box.X, Y, Box.W, EventBorderH, EventBorderBrush, Box.Color.CopyWithNewOpacity(Opacity));
#else
				FSlateRoundedBoxBrush Brush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), 0.0f, Box.Color.CopyWithNewOpacity(Opacity), 1.0f);
				DrawContext.DrawBox(EventBorderLayerId, Box.X, Y, Box.W, EventBorderH, &Brush, FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
#endif
			}
			NumDrawBorders += DrawState.Borders.Num();
		}

		// Draw texts.
		constexpr float TextMinEventH = 7.0f;
		if (Layout.EventH > TextMinEventH)
		{
			float TextOpacity = Opacity;
			if (Layout.EventH < FTimingViewLayout::NormalLayoutEventH)
			{
				TextOpacity *= (Layout.EventH - TextMinEventH + 1.0f) / (FTimingViewLayout::NormalLayoutEventH - TextMinEventH + 1.0f);
			}
			const FLinearColor WhiteColor(1.0f, 1.0f, 1.0f, TextOpacity);
			const FLinearColor BlackColor(0.0f, 0.0f, 0.0f, TextOpacity);

			const float TextY0 = TopLaneY + 1.0f - (FTimingViewLayout::NormalLayoutEventH - Layout.EventH) / 2.0f;
			const float TextDY = Layout.EventH + Layout.EventDY;

			const int32 EventTextLayerId = ReservedLayerId + ToInt32(EDrawLayer::EventText);
			for (const FTimingEventsTrackDrawState::FTextPrimitive& Text : DrawState.Texts)
			{
				const float Y = TextY0 + (TextDY) * static_cast<float>(Text.Depth);
				DrawContext.DrawText(EventTextLayerId, Text.X, Y, Text.Text, EventFont, Text.bWhite ? WhiteColor : BlackColor);
			}
			NumDrawTexts += DrawState.Texts.Num();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewDrawHelper::DrawLineEvents(const FTimingEventsTrackDrawState& DrawState, const FTimingEventsTrack& Track, const float OffsetY) const
{
	const float TrackH = Track.GetHeight();

	if (TrackH > 0.0f)
	{
		const FTimingViewLayout& Layout = Viewport.GetLayout();

		NumEvents += DrawState.GetNumEvents();
		NumMergedBoxes += DrawState.GetNumMergedBoxes();

		float TopLaneY = Track.GetPosY();
		TopLaneY += OffsetY + Layout.TimelineDY;
		TopLaneY += Track.GetChildTracksTopHeight(Layout);

		constexpr float LineH = 3.0f;
		constexpr float LineEndsDH = 1.0f;
		constexpr float LineEndsH = LineH + 2 * LineEndsDH;

		// Draw filled boxes (merged borders).
		//if (Layout.EventH > 0.0f)
		{
			const int32 EventFillLayerId = ReservedLayerId + ToInt32(EDrawLayer::EventFill);
			const float Y = TopLaneY + Layout.EventH - LineEndsH + LineEndsDH;
			for (const FTimingEventsTrackDrawState::FBoxPrimitive& Box : DrawState.Boxes)
			{
				DrawContext.DrawBox(EventFillLayerId, Box.X, Y, Box.W, LineEndsH, WhiteBrush, Box.Color);
			}
			NumDrawBoxes += DrawState.Boxes.Num();
		}

		// Draw filled boxes (event inside area).
		if (Layout.EventH > 2.0f)
		{
			const int32 EventFillLayerId = ReservedLayerId + ToInt32(EDrawLayer::EventFill);
			const float Y = TopLaneY + Layout.EventH - LineEndsH + LineEndsDH + 1.0f;
			for (const FTimingEventsTrackDrawState::FBoxPrimitive& Box : DrawState.InsideBoxes)
			{
				DrawContext.DrawBox(EventFillLayerId, Box.X, Y, Box.W, LineH, WhiteBrush, Box.Color);
			}
			NumDrawBoxes += DrawState.InsideBoxes.Num();
		}

		// Draw borders (just two vertical lines at ends).
		//if (Layout.EventH > 0.0f)
		{
			const int32 EventBorderLayerId = ReservedLayerId + ToInt32(EDrawLayer::EventBorder);
			const float Y = TopLaneY + Layout.EventH - LineEndsH + LineEndsDH;
			for (const FTimingEventsTrackDrawState::FBoxPrimitive& Box : DrawState.Borders)
			{
				DrawContext.DrawBox(EventBorderLayerId, Box.X, Y, 1.0f, LineEndsH, WhiteBrush, Box.Color);
				DrawContext.DrawBox(EventBorderLayerId, Box.X + Box.W - 1.0f, Y, 1.0f, LineEndsH, WhiteBrush, Box.Color);
			}
			NumDrawBorders += DrawState.Borders.Num();
		}

		// Draw texts.
		constexpr float TextMinEventH = 7.0f;
		if (Layout.EventH > TextMinEventH)
		{
			float TextOpacity = 1.0f;
			if (Layout.EventH < FTimingViewLayout::NormalLayoutEventH)
			{
				TextOpacity *= (Layout.EventH - TextMinEventH + 1.0f) / (FTimingViewLayout::NormalLayoutEventH - TextMinEventH + 1.0f);
			}

			const float TextY = TopLaneY - (FTimingViewLayout::NormalLayoutEventH - Layout.EventH) / 2.0f - 1.0f;

			const int32 EventTextLayerId = ReservedLayerId + ToInt32(EDrawLayer::EventText);
			for (const FTimingEventsTrackDrawState::FTextPrimitive& Text : DrawState.Texts)
			{
				DrawContext.DrawText(EventTextLayerId, Text.X, TextY, Text.Text, EventFont, Text.Color.CopyWithNewOpacity(Text.Color.A * TextOpacity));
			}
			NumDrawTexts += DrawState.Texts.Num();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewDrawHelper::DrawFadedLineEvents(const FTimingEventsTrackDrawState& DrawState, const FTimingEventsTrack& Track, const float OffsetY, const float Opacity) const
{
	// TODO
	DrawLineEvents(DrawState, Track, OffsetY);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewDrawHelper::DrawContextSwitchMarkers(const FTimingEventsTrackDrawState& DrawState, float LineY, float LineH, float Opacity, bool bDrawOverlays, bool bDrawVerticalLines) const
{
	if (LineH > 0.0f)
	{
		const int32 LayerId = ReservedLayerId + ToInt32(EDrawLayer::EventHighlight);

		if (bDrawVerticalLines)
		{
			// Draw vertical lines (merged into large boxes).
			for (const FTimingEventsTrackDrawState::FBoxPrimitive& Box : DrawState.Boxes)
			{
				const FLinearColor Color = Box.Color.CopyWithNewOpacity(Opacity);
				DrawContext.DrawBox(LayerId, Box.X, LineY, Box.W, LineH, WhiteBrush, Color);
			}

			// Draw vertical lines (edges of larger events).
			for (const FTimingEventsTrackDrawState::FBoxPrimitive& Box : DrawState.Borders)
			{
				const FLinearColor Color = Box.Color.CopyWithNewOpacity(Opacity);
				DrawContext.DrawBox(LayerId, Box.X, LineY, 1.0f, LineH, WhiteBrush, Color);
				DrawContext.DrawBox(LayerId, Box.X + Box.W - 1.0f, LineY, 1.0f, LineH, WhiteBrush, Color);
			}
		}

		if (bDrawOverlays)
		{
			// Draw overlay for idle areas.
			const FSlateBrush* IdleBrush = WhiteBrush;
			const FLinearColor IdleColor(0.0f, 0.0f, 0.0f, 0.3f);
			const int32 Count1 = DrawState.Boxes.Num();
			const int32 Count2 = DrawState.Borders.Num();
			int32 Index1 = 0;
			int32 Index2 = 0;
			const float MinX = Viewport.TimeToSlateUnitsRounded(Viewport.GetMinValidTime());
			float CurrentX = FMath::Max(MinX, 0.0f);
			while (Index1 < Count1 || Index2 < Count2)
			{
				const float X1 = (Index1 < Count1) ? DrawState.Boxes[Index1].X : FLT_MAX;
				const float X2 = (Index2 < Count2) ? DrawState.Borders[Index2].X : FLT_MAX;
				if (X1 < X2)
				{
					if (X1 - CurrentX > 0.0f)
					{
						DrawContext.DrawBox(LayerId, CurrentX, LineY, X1 - CurrentX, LineH, IdleBrush, IdleColor);
					}
					CurrentX = FMath::Max(CurrentX, X1 + DrawState.Boxes[Index1].W);
					++Index1;
				}
				else
				{
					if (X2 - CurrentX > 0.0f)
					{
						DrawContext.DrawBox(LayerId, CurrentX, LineY, X2 - CurrentX, LineH, IdleBrush, IdleColor);
					}
					CurrentX = FMath::Max(CurrentX, X2 + DrawState.Borders[Index2].W);
					++Index2;
				}
			}
			const float MaxX = Viewport.TimeToSlateUnitsRounded(Viewport.GetMaxValidTime());
			const float LastAreaW = FMath::Min(MaxX, Viewport.GetWidth()) - CurrentX;
			if (LastAreaW > 0.0f)
			{
				DrawContext.DrawBox(LayerId, CurrentX, LineY, LastAreaW, LineH, IdleBrush, IdleColor);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewDrawHelper::DrawMarkers(const FTimingEventsTrackDrawState& DrawState, float LineY, float LineH, float Opacity) const
{
	if (LineH > 0.0f)
	{
		const FTimingViewLayout& Layout = Viewport.GetLayout();

		// Draw markers from filled boxes (merged borders).
		for (const FTimingEventsTrackDrawState::FBoxPrimitive& Box : DrawState.Boxes)
		{
			DrawContext.DrawBox(Box.X, LineY, Box.W, LineH, WhiteBrush, Box.Color.CopyWithNewOpacity(Opacity));
		}

		// Draw markers from borders.
		for (const FTimingEventsTrackDrawState::FBoxPrimitive& Box : DrawState.Borders)
		{
			DrawContext.DrawBox(Box.X, LineY, 1.0f, LineH, WhiteBrush, Box.Color.CopyWithNewOpacity(Opacity));
			if (Box.W > 1.0f)
			{
				DrawContext.DrawBox(Box.X + Box.W - 1.0f, LineY, 1.0f, LineH, WhiteBrush, Box.Color.CopyWithNewOpacity(Opacity));
			}
		}

		DrawContext.LayerId++;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewDrawHelper::DrawTrackHeader(const FBaseTimingTrack& Track) const
{
	const int32 HeaderLayerId = ReservedLayerId + ToInt32(EDrawLayer::HeaderBackground);
	const int32 HeaderTextLayerId = ReservedLayerId + ToInt32(EDrawLayer::HeaderText);

	DrawTrackHeader(Track, HeaderLayerId, HeaderTextLayerId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewDrawHelper::DrawTrackHeader(const FBaseTimingTrack& Track, const int32 HeaderLayerId, const int32 HeaderTextLayerId) const
{
	const float TrackY = Track.GetPosY();
	const float TrackH = Track.GetHeight();

	if (TrackH > 0.0f)
	{
		// Draw a horizontal line between tracks (top line of a track).
		const int32 TrackBackgroundLayerId = HeaderLayerId - ToInt32(EDrawLayer::HeaderBackground) + ToInt32(EDrawLayer::TrackBackground);
		DrawContext.DrawBox(TrackBackgroundLayerId, 0.0f, TrackY, Viewport.GetWidth(), 1.0f, WhiteBrush, EdgeColor);

		// Draw header with name of tracks.
		if (TrackH > 4.0f)
		{
			const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			const float FontScale = DrawContext.Geometry.Scale;
			float TextWidth = static_cast<float>(FontMeasureService->Measure(Track.GetName(), EventFont, FontScale).X / FontScale);

			const float HeaderX = 0.0f;
			const float HeaderW = TextWidth + 4.0f;

			const float HeaderY = TrackY + 1.0f;
			const float HeaderH = FMath::Min(12.0f, Track.GetHeight() - 1.0f);

			if (HeaderH > 0)
			{
				const FLinearColor HeaderBackgroundColor = GetTrackNameBackgroundColor(Track);
				DrawContext.DrawBox(HeaderLayerId, HeaderX, HeaderY, HeaderW, HeaderH, WhiteBrush, HeaderBackgroundColor);

				const float TextX = HeaderX + 2.0f;
				const float TextY = HeaderY + HeaderH / 2.0f - 7.0f;
				const FLinearColor HeaderTextColor = GetTrackNameTextColor(Track);
				DrawContext.DrawText(HeaderTextLayerId, TextX, TextY, Track.GetName(), EventFont, HeaderTextColor);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewDrawHelper::BeginDrawTracks() const
{
	// Reserve layers.
	ReservedLayerId = DrawContext.LayerId;
	DrawContext.LayerId += ToInt32(EDrawLayer::Count);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewDrawHelper::EndDrawTracks() const
{
	if (Viewport.GetWidth() > 0.0f)
	{
		const int32 TrackBackgroundLayerId = ReservedLayerId + ToInt32(EDrawLayer::TrackBackground);

		const float TopY = Viewport.GetPosY() + Viewport.GetTopOffset();
		const float BottomY = Viewport.GetPosY() + Viewport.GetHeight() - Viewport.GetBottomOffset();

		if (TopY < BottomY)
		{
			// Y position of the first pixel below the last track.
			const float Y = TopY + Viewport.GetScrollHeight() - Viewport.GetScrollPosY() - 1.0f;

			if (Y >= TopY && Y < BottomY)
			{
				// Draw a last horizontal line.
				DrawContext.DrawBox(TrackBackgroundLayerId, 0.0f, Y, Viewport.GetWidth(), 1.0f, WhiteBrush, EdgeColor);
			}

			// Note: ValidAreaX and ValidAreaW are computed in DrawBackground.
			if (ValidAreaW > 0.0f)
			{
				const float TopInvalidAreaH = FMath::Min(0.0f - Viewport.GetScrollPosY(), Viewport.GetScrollableAreaHeight());
				if (TopInvalidAreaH > 0.0f)
				{
					// Draw invalid area (top).
					DrawContext.DrawBox(TrackBackgroundLayerId, ValidAreaX, TopY, ValidAreaW, TopInvalidAreaH, BackgroundAreaBrush, InvalidAreaColor);
				}

				const float BottomInvalidAreaH = FMath::Min(BottomY - Y - 1.0f, Viewport.GetScrollableAreaHeight());
				if (BottomInvalidAreaH > 0.0f)
				{
					// Draw invalid area (bottom).
					DrawContext.DrawBox(TrackBackgroundLayerId, ValidAreaX, BottomY - BottomInvalidAreaH, ValidAreaW, BottomInvalidAreaH, BackgroundAreaBrush, InvalidAreaColor);
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewDrawHelper::DrawTimingEventHighlight(double StartTime, double EndTime, float Y, EDrawEventMode Mode) const
{
	float EventX1 = Viewport.TimeToSlateUnitsRounded(StartTime);
	if (EventX1 > Viewport.GetWidth())
	{
		return;
	}

	EndTime = Viewport.RestrictEndTime(EndTime);
	float EventX2 = Viewport.TimeToSlateUnitsRounded(EndTime);
	if (EventX2 < 0)
	{
		return;
	}

	// Timing events are displayed with minimum 1px (including empty ones).
	if (EventX1 == EventX2)
	{
		EventX2 = EventX1 + 1.0f;
	}

	// Limit event width on the viewport's left side.
	const float MinX = -2.0f; // -2 allows event border to remain outside screen
	if (EventX1 < MinX)
	{
		EventX1 = MinX;
	}

	// Limit event width on the viewport's right side.
	const float MaxX = Viewport.GetWidth() + 2.0f; // +2 allows event border to remain outside screen
	if (EventX2 > MaxX)
	{
		EventX2 = MaxX;
	}

	const float EventW = EventX2 - EventX1;

	const FTimingViewLayout& Layout = Viewport.GetLayout();

	const int32 LayerId = ReservedLayerId + ToInt32(EDrawLayer::EventHighlight);

#if INSIGHTS_USE_LEGACY_BORDER
	constexpr float BorderOffset = 1.0f;
#else
	constexpr float BorderOffset = 2.0f;
#endif

	if (Mode == EDrawEventMode::Hovered)
	{
		const FLinearColor Color(1.0f, 1.0f, 0.0f, 1.0f); // yellow

		// Draw border around the timing event box.
#if INSIGHTS_USE_LEGACY_BORDER
		DrawContext.DrawBox(LayerId, EventX1 - BorderOffset, Y - BorderOffset, EventW + 2 * BorderOffset, Layout.EventH + 2 * BorderOffset, HoveredEventBorderBrush, Color);
#else
		FSlateRoundedBoxBrush Brush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), 2.0f, Color, 2.0f);
		DrawContext.DrawBox(LayerId, EventX1 - BorderOffset, Y - BorderOffset, EventW + 2 * BorderOffset, Layout.EventH + 2 * BorderOffset, &Brush, FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
#endif
	}
	else // EDrawEventMode::Selected or EDrawEventMode::SelectedAndHovered
	{
		// Animate color from white (if selected and hovered) or yellow (if only selected) to black, using a squared sine function.
		const double Time = static_cast<double>(FPlatformTime::Cycles64()) * FPlatformTime::GetSecondsPerCycle64();
		float Hue = static_cast<float>(FMath::Sin(2.0 * Time));
		Hue = Hue * Hue; // squared, to ensure only positive [0 - 1] values
		const float Blue = (Mode == EDrawEventMode::SelectedAndHovered) ? 0.0f : Hue;
		const FLinearColor Color(Hue, Hue, Blue, 1.0f);

		// Draw border around the timing event box.
#if INSIGHTS_USE_LEGACY_BORDER
		DrawContext.DrawBox(LayerId, EventX1 - BorderOffset, Y - BorderOffset, EventW + 2 * BorderOffset, Layout.EventH + 2 * BorderOffset, SelectedEventBorderBrush, Color);
#else
		FSlateRoundedBoxBrush Brush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), 2.0f, Color, 2.0f);
		DrawContext.DrawBox(LayerId, EventX1 - BorderOffset, Y - BorderOffset, EventW + 2 * BorderOffset, Layout.EventH + 2 * BorderOffset, &Brush, FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
#endif
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FTimingViewDrawHelper::GetTrackNameTextColor(const FBaseTimingTrack& Track) const
{
	return Track.IsHovered() ?
		FLinearColor(1.0f, 1.0f, 0.5f, 1.0f) :
		FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FTimingViewDrawHelper::GetTrackNameBackgroundColor(const FBaseTimingTrack& Track) const
{
	if (Track.IsSelected())
	{
		return Track.IsHovered() ?
			FLinearColor(0.00f, 0.1f, 0.4f, 1.0f) :
			FLinearColor(0.00f, 0.2f, 0.5f, 1.0f);
	}
	else
	{
		return Track.IsHovered() ?
			FLinearColor(0.04f, 0.04f, 0.04f, 1.0f) :
			FLinearColor(0.05f, 0.05f, 0.05f, 1.0f);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewDrawHelper::DrawRelations(const TArray<TUniquePtr<ITimingEventRelation>>& Relations, ITimingEventRelation::EDrawFilter Filter) const
{
	for (const TUniquePtr<ITimingEventRelation> &Relation : Relations)
	{
		Relation->Draw(DrawContext, Viewport, *this, Filter);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef INSIGHTS_USE_LEGACY_BORDER
