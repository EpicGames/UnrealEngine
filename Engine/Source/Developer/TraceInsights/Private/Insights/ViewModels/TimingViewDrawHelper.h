// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Fonts/SlateFontInfo.h"
#include "Math/Color.h"

// TraceInsights
#include "Insights/ViewModels/ITimingEvent.h"
#include "Insights/ViewModels/ITimingViewDrawHelper.h"
#include "Insights/ViewModels/TimingEventsTrack.h" // for ITimingEventsTrackDrawStateBuilder

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FSlateBrush;

struct FDrawContext; // for backward compatibility
namespace UE::Insights { class FDrawContext; }

class ITimingTrackDrawContext;
class FTimingEventsTrack;
class FTimingTrackViewport;

enum class EDrawEventMode : uint32;

struct FTimingViewLayout;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FTimingEventsTrackDrawState
{
	struct FBoxPrimitive
	{
		int32 Depth;
		float X;
		float W;
		FLinearColor Color;
	};

	struct FTextPrimitive
	{
		int32 Depth;
		float X;
		FString Text;
		bool bWhite;
		FLinearColor Color;
	};

	FTimingEventsTrackDrawState()
		: Boxes()
		, InsideBoxes()
		, Borders()
		, Texts()
		, NumLanes(0)
		, NumEvents(0)
		, NumMergedBoxes(0)
	{
	}

	void Reset()
	{
		Boxes.Reset();
		InsideBoxes.Reset();
		Borders.Reset();
		Texts.Reset();
		NumLanes = 0;
		NumEvents = 0;
		NumMergedBoxes = 0;
	}

	int32 GetNumLanes() const { return NumLanes; }

	int32 GetNumEvents() const { return NumEvents; }
	int32 GetNumMergedBoxes() const { return NumMergedBoxes; }
	int32 GetTotalNumBoxes() const { return Boxes.Num() + InsideBoxes.Num(); }

	TArray<FBoxPrimitive> Boxes;
	TArray<FBoxPrimitive> InsideBoxes;
	TArray<FBoxPrimitive> Borders;
	TArray<FTextPrimitive> Texts;

	int32 NumLanes;

	// Debug stats.
	int32 NumEvents;
	int32 NumMergedBoxes;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingEventsTrackDrawStateBuilder : public ITimingEventsTrackDrawStateBuilder
{
private:
	struct FBoxData
	{
		float X1;
		float X2;
		uint32 Color;
		FLinearColor LinearColor;

		FBoxData() : X1(0.0f), X2(0.0f), Color(0) {}
		void Reset() { X1 = 0.0f; X2 = 0.0f; Color = 0; }
	};

public:
	explicit FTimingEventsTrackDrawStateBuilder(FTimingEventsTrackDrawState& InState, const FTimingTrackViewport& InViewport, float InFontScale);
	virtual ~FTimingEventsTrackDrawStateBuilder() = default;

	/**
	 * Non-copyable
	 */
	FTimingEventsTrackDrawStateBuilder(const FTimingEventsTrackDrawStateBuilder&) = delete;
	FTimingEventsTrackDrawStateBuilder& operator=(const FTimingEventsTrackDrawStateBuilder&) = delete;

	const FTimingTrackViewport& GetViewport() const { return Viewport; }

	virtual void AddEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName, uint64 InEventType = 0, uint32 InEventColor = 0) override;
	virtual void AddEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, uint32  InEventColor, GetEventNameCallback InGetEventNameCallback) override;
	void Flush();

	int32 GetMaxDepth() const { return MaxDepth; }

	static void AppendDurationToEventName(FString& InOutEventName, const double InDuration);

private:
	void FlushBox(const FBoxData& Box, const int32 Depth);

private:
	FTimingEventsTrackDrawState& DrawState; // cached draw state to build
	const FTimingTrackViewport& Viewport;

	int32 MaxDepth;

	TArray<float> LastEventX2; // X2 value for last event on each depth
	TArray<FBoxData> LastBox;

	const FSlateFontInfo EventFont;
	float FontScale;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingViewDrawHelper final : public ITimingViewDrawHelper
{
private:
	enum class EDrawLayer : int32
	{
		TrackBackground,
		EventBorder,
		EventFill,
		EventText,
		EventHighlight,
		RelationBackground,
		Relation,
		HeaderBackground,
		HeaderText,

		Count,
	};
	static int32 ToInt32(EDrawLayer Layer) { return static_cast<int32>(Layer); }

public:
	explicit FTimingViewDrawHelper(const UE::Insights::FDrawContext& InDrawContext, const FTimingTrackViewport& InViewport);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	explicit FTimingViewDrawHelper(const ::FDrawContext& InDrawContext, const FTimingTrackViewport& InViewport);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	~FTimingViewDrawHelper();

	/**
	 * Non-copyable
	 */
	FTimingViewDrawHelper(const FTimingViewDrawHelper&) = delete;
	FTimingViewDrawHelper& operator=(const FTimingViewDrawHelper&) = delete;

	// ITimingViewDrawHelper interface
	virtual const FSlateBrush* GetWhiteBrush() const override { return WhiteBrush; }
	virtual const FSlateBrush* GetEventBorderBrush() const override { return EventBorderBrush; }
	virtual const FSlateBrush* GetHoveredEventBorderBrush() const override { return HoveredEventBorderBrush; }
	virtual const FSlateBrush* GetSelectedEventBorderBrush() const override { return SelectedEventBorderBrush; }
	virtual const FSlateFontInfo& GetEventFont() const override { return EventFont; }
	virtual FLinearColor GetEdgeColor() const override { return EdgeColor; }
	virtual FLinearColor GetValidAreaColor() const override { return ValidAreaColor; }
	virtual FLinearColor GetInvalidAreaColor() const override { return InvalidAreaColor; }
	virtual FLinearColor GetTrackNameTextColor(const FBaseTimingTrack& Track) const override;
	virtual FLinearColor GetTrackNameBackgroundColor(const FBaseTimingTrack& Track) const override;
	virtual int32 GetHeaderBackgroundLayerId() const override { return ReservedLayerId + ToInt32(EDrawLayer::HeaderBackground); }
	virtual int32 GetHeaderTextLayerId() const override { return ReservedLayerId + ToInt32(EDrawLayer::HeaderText); }
	virtual int32 GetRelationLayerId() const override {return ReservedLayerId + ToInt32(EDrawLayer::Relation); };
	virtual int32 GetTextLayerId() const override {return ReservedLayerId + ToInt32(EDrawLayer::EventText); };
	virtual int32 GetFirstLayerId() const override { return ReservedLayerId; }
	virtual int32 GetNumLayerIds() const override { return ToInt32(EDrawLayer::Count); }

	const UE::Insights::FDrawContext& GetDrawContext() const { return DrawContext; }
	const FTimingTrackViewport& GetViewport() const { return Viewport; }

	void DrawBackground() const;

	//////////////////////////////////////////////////

	void BeginDrawTracks() const;

	// OffsetY = 1.0f is for the top horizontal line (which separates the timelines) added by DrawTrackHeader.
	void DrawEvents(const FTimingEventsTrackDrawState& DrawState, const FTimingEventsTrack& Track, const float OffsetY = 1.0f) const;
	void DrawFadedEvents(const FTimingEventsTrackDrawState& DrawState, const FTimingEventsTrack& Track, const float OffsetY = 1.0f, const float Opacity = 0.1f) const;

	void DrawLineEvents(const FTimingEventsTrackDrawState& DrawState, const FTimingEventsTrack& Track, const float OffsetY = 1.0f) const;
	void DrawFadedLineEvents(const FTimingEventsTrackDrawState& DrawState, const FTimingEventsTrack& Track, const float OffsetY = 1.0f, const float Opacity = 0.1f) const;
	void DrawContextSwitchMarkers(const FTimingEventsTrackDrawState& DrawState, float LineY, float LineH, float Opacity, bool bDrawOverlays, bool bDrawVerticalLines) const;

	void DrawMarkers(const FTimingEventsTrackDrawState& DrawState, float LineY, float LineH, float Opacity) const;

	void DrawTrackHeader(const FBaseTimingTrack& Track) const;
	void DrawTrackHeader(const FBaseTimingTrack& Track, const int32 HeaderLayerId, const int32 HeaderTextLayerId) const;

	void EndDrawTracks() const;

	void DrawRelations(const TArray<TUniquePtr<ITimingEventRelation>>& Relations, ITimingEventRelation::EDrawFilter Filter) const;

	//////////////////////////////////////////////////

	void DrawTimingEventHighlight(double StartTime, double EndTime, float Y, EDrawEventMode Mode) const;

	int32 GetNumEvents() const { return NumEvents; }
	int32 GetNumMergedBoxes() const { return NumMergedBoxes; }
	int32 GetNumDrawBoxes() const { return NumDrawBoxes; }
	int32 GetNumDrawBorders() const { return NumDrawBorders; }
	int32 GetNumDrawTexts() const { return NumDrawTexts; }

private:
	const UE::Insights::FDrawContext& DrawContext;
	const FTimingTrackViewport& Viewport;

	const FSlateBrush* WhiteBrush;
	const FSlateBrush* EventBorderBrush;
	const FSlateBrush* HoveredEventBorderBrush;
	const FSlateBrush* SelectedEventBorderBrush;
	const FSlateBrush* BackgroundAreaBrush;
	const FLinearColor ValidAreaColor;
	const FLinearColor InvalidAreaColor;
	const FLinearColor EdgeColor;
	const FSlateFontInfo EventFont;

	mutable int32 ReservedLayerId;

	mutable float ValidAreaX;
	mutable float ValidAreaW;

	mutable int32 NumEvents;
	mutable int32 NumMergedBoxes;
	mutable int32 NumDrawBoxes;
	mutable int32 NumDrawBorders;
	mutable int32 NumDrawTexts;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
