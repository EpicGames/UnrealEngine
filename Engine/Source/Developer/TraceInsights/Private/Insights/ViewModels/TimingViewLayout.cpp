// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/ViewModels/TimingViewLayout.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewLayout::ForceNormalMode()
{
	bIsCompactMode = false;
	EventH = NormalLayoutEventH;
	EventDY = NormalLayoutEventDY;
	TimelineDY = NormalLayoutTimelineDY;
	ChildTimelineDY = NormalLayoutChildTimelineDY;
	MinTimelineH = NormalLayoutMinTimelineH;
	TargetMinTimelineH = NormalLayoutMinTimelineH;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingViewLayout::ForceCompactMode()
{
	bIsCompactMode = true;
	EventH = CompactLayoutEventH;
	EventDY = CompactLayoutEventDY;
	TimelineDY = CompactLayoutTimelineDY;
	ChildTimelineDY = CompactLayoutChildTimelineDY;
	MinTimelineH = CompactLayoutMinTimelineH;
	TargetMinTimelineH = CompactLayoutMinTimelineH;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTimingViewLayout::Update()
{
	constexpr float LayoutTransitionSpeed = 0.25f;
	bool bHasChanged = false;

	if (bIsCompactMode)
	{
		if (EventH > CompactLayoutEventH)
		{
			EventH -= LayoutTransitionSpeed;
			bHasChanged = true;
		}
		if (EventDY > CompactLayoutEventDY)
		{
			EventDY -= LayoutTransitionSpeed;
			bHasChanged = true;
		}
		if (TimelineDY > CompactLayoutTimelineDY)
		{
			TimelineDY -= LayoutTransitionSpeed;
			bHasChanged = true;
		}		
		if (ChildTimelineDY > CompactLayoutChildTimelineDY)
		{
			ChildTimelineDY -= LayoutTransitionSpeed / 2;
			bHasChanged = true;
		}
	}
	else
	{
		if (EventH < NormalLayoutEventH)
		{
			EventH += LayoutTransitionSpeed;
			bHasChanged = true;
		}
		if (EventDY < NormalLayoutEventDY)
		{
			EventDY += LayoutTransitionSpeed;
			bHasChanged = true;
		}
		if (TimelineDY < NormalLayoutTimelineDY)
		{
			TimelineDY += LayoutTransitionSpeed;
			bHasChanged = true;
		}		
		if (ChildTimelineDY < NormalLayoutChildTimelineDY)
		{
			ChildTimelineDY += LayoutTransitionSpeed / 2;
			bHasChanged = true;
		}
	}

	if (MinTimelineH > TargetMinTimelineH)
	{
		MinTimelineH -= LayoutTransitionSpeed;
		bHasChanged = true;
	}
	else if (MinTimelineH < TargetMinTimelineH)
	{
		MinTimelineH += LayoutTransitionSpeed;
		bHasChanged = true;
	}

	return bHasChanged;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
