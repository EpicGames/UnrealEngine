// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/ViewModels/TooltipDrawState.h"

#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/AppStyle.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"

// TraceInsights
#include "Insights/InsightsStyle.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FTooltipDrawState::DefaultTitleColor(0.9f, 0.9f, 0.5f, 1.0f);
FLinearColor FTooltipDrawState::DefaultNameColor(0.6f, 0.6f, 0.6f, 1.0f);
FLinearColor FTooltipDrawState::DefaultValueColor(1.0f, 1.0f, 1.0f, 1.0f);

////////////////////////////////////////////////////////////////////////////////////////////////////

FTooltipDrawState::FTooltipDrawState()
	: WhiteBrush(FInsightsStyle::Get().GetBrush("WhiteBrush"))
	, Font(FAppStyle::Get().GetFontStyle("SmallFont"))
	, BackgroundColor(0.05f, 0.05f, 0.05f, 1.0f)
	, Size(0.0f, 0.0f)
	, DesiredSize(0.0f, 0.0f)
	, Position(0.0f, 0.0f)
	, ValueOffsetX(0.0f)
	, NewLineY(0.0f)
	, Opacity(0.0f)
	, DesiredOpacity(0.0f)
	, FontScale(1.0f)
	, Texts()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTooltipDrawState::~FTooltipDrawState()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTooltipDrawState::Reset()
{
	Opacity = 0.0f;
	DesiredOpacity = 0.0f;

	ResetContent();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTooltipDrawState::ResetContent()
{
	Texts.Reset();
	ImageBrush.Reset();

	ValueOffsetX = 0.0f;
	NewLineY = BorderY;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTooltipDrawState::AddTitle(FStringView Title)
{
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const FVector2D TextSize = FontMeasureService->Measure(Title, Font, FontScale) / FontScale;
	Texts.Add({ BorderX, NewLineY, TextSize, FString(Title), DefaultTitleColor, FDrawTextType::Title });

	NewLineY += DefaultTitleHeight;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTooltipDrawState::AddTitle(FStringView Title, const FLinearColor& Color)
{
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const FVector2D TextSize = FontMeasureService->Measure(Title, Font, FontScale) / FontScale;
	Texts.Add({ BorderX, NewLineY, TextSize, FString(Title), Color, FDrawTextType::Misc });

	NewLineY += DefaultTitleHeight;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTooltipDrawState::AddNameValueTextLine(FStringView Name, FStringView Value)
{
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const FVector2D NameTextSize = FontMeasureService->Measure(Name, Font, FontScale) / FontScale;
	Texts.Add({ 0.0f, NewLineY, NameTextSize, FString(Name), DefaultNameColor, FDrawTextType::Name });

	const FVector2D ValueTextSize = FontMeasureService->Measure(Value, Font, FontScale) / FontScale;
	Texts.Add({ 0.0f, NewLineY, ValueTextSize, FString(Value), DefaultValueColor, FDrawTextType::Value });

	NewLineY += DefaultLineHeight;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTooltipDrawState::AddTextLine(FStringView Text, const FLinearColor& Color)
{
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const FVector2D TextSize = FontMeasureService->Measure(Text, Font, FontScale) / FontScale;
	Texts.Add({ BorderX, NewLineY, TextSize, FString(Text), Color, FDrawTextType::Misc });

	NewLineY += DefaultLineHeight;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTooltipDrawState::AddTextLine(const float X, const float Y, FStringView Text, const FLinearColor& Color)
{
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const FVector2D TextSize = FontMeasureService->Measure(Text, Font, FontScale) / FontScale;
	Texts.Add({ X, Y, TextSize, FString(Text), Color, FDrawTextType::Misc });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTooltipDrawState::UpdateLayout()
{
	ValueOffsetX = 0.0f;
	for (const FDrawTextInfo& TextInfo : Texts)
	{
		if (TextInfo.Type == FDrawTextType::Name)
		{
			const float TextW = static_cast<float>(TextInfo.TextSize.X);
			if (ValueOffsetX < TextW)
			{
				ValueOffsetX = TextW;
			}
		}
	}
	ValueOffsetX += BorderX;

	DesiredSize.Set(BorderX, NewLineY + BorderY);
	for (const FDrawTextInfo& TextInfo : Texts)
	{
		float RightX;
		switch (TextInfo.Type)
		{
		case FDrawTextType::Name:
			RightX = ValueOffsetX;
			break;

		case FDrawTextType::Value:
			RightX = ValueOffsetX + NameValueDX + static_cast<float>(TextInfo.TextSize.X);
			break;

		default:
			RightX = TextInfo.X + static_cast<float>(TextInfo.TextSize.X);
		}
		if (DesiredSize.X < RightX)
		{
			DesiredSize.X = RightX;
		}
	}
	DesiredSize.X += BorderX;

	if (ImageBrush.IsValid())
	{
		FVector2D ImageSize = ImageBrush->GetImageSize();
		if (DesiredSize.X < ImageSize.X)
		{
			DesiredSize.X = ImageSize.X;
		}
		if (DesiredSize.Y < ImageSize.Y)
		{
			DesiredSize.Y = ImageSize.Y;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTooltipDrawState::Update()
{
	if (Size.X != DesiredSize.X)
	{
		Size.X = Size.X * 0.75 + DesiredSize.X * 0.25;

		if (FMath::IsNearlyEqual(Size.X, DesiredSize.X))
		{
			Size.X = DesiredSize.X;
		}
	}

	if (Size.Y != DesiredSize.Y)
	{
		Size.Y = Size.Y * 0.5 + DesiredSize.Y * 0.5;

		if (FMath::IsNearlyEqual(Size.Y, DesiredSize.Y))
		{
			Size.Y = DesiredSize.Y;
		}
	}

	float RealDesiredOpacity;
	if (DesiredSize.X > 1.0)
	{
		const float DesiredOpacityByTooltipWidth = static_cast<float>(1.0 - FMath::Abs(Size.X - DesiredSize.X) / DesiredSize.X);

		if (FMath::IsNearlyEqual(DesiredOpacity, DesiredOpacityByTooltipWidth, 0.001f))
		{
			RealDesiredOpacity = DesiredOpacity;
		}
		else
		{
			RealDesiredOpacity = FMath::Min(DesiredOpacity, DesiredOpacityByTooltipWidth);
		}
	}
	else
	{
		RealDesiredOpacity = 0.0f;
	}

	if (Opacity != RealDesiredOpacity)
	{
		if (Opacity < RealDesiredOpacity)
		{
			// slow fade in
			Opacity = Opacity * 0.9f + RealDesiredOpacity * 0.1f;
		}
		else
		{
			// fast fade out
			Opacity = Opacity * 0.75f + RealDesiredOpacity * 0.25f;
		}

		if (FMath::IsNearlyEqual(Opacity, RealDesiredOpacity, 0.001f))
		{
			Opacity = RealDesiredOpacity;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTooltipDrawState::SetPosition(const FVector2D& MousePosition, const float MinX, const float MaxX, const float MinY, const float MaxY)
{
	Position.X = FMath::Max(MinX, FMath::Min(MousePosition.X + 12.0, MaxX - Size.X));
	Position.Y = FMath::Max(MinY, FMath::Min(MousePosition.Y + 15.0, MaxY - Size.Y));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTooltipDrawState::Draw(const UE::Insights::FDrawContext& DrawContext) const
{
	if (Opacity > 0.0f && Size.X > 0.0 && Size.Y > 0.0)
	{
		// Draw background.
		DrawContext.DrawBox(static_cast<float>(Position.X), static_cast<float>(Position.Y), static_cast<float>(Size.X), static_cast<float>(Size.Y), WhiteBrush, BackgroundColor.CopyWithNewOpacity(Opacity));
		if (Size.X < DesiredSize.X)
		{
			DrawContext.DrawBox(static_cast<float>(Position.X + Size.X), static_cast<float>(Position.Y), static_cast<float>(DesiredSize.X - Size.X), static_cast<float>(Size.Y), WhiteBrush, BackgroundColor.CopyWithNewOpacity(Opacity * 0.5f));
		}
		DrawContext.LayerId++;

		// Draw border.
		//DrawContext.DrawBox(static_cast<float>(Position.X), static_cast<float>(Position.Y), static_cast<float>(Size.X), static_cast<float>(Size.Y), BorderBrush, BorderColor.CopyWithNewOpacity(Opacity));
		//DrawContext.LayerId++;

		// Draw optional background image.
		if (ImageBrush.IsValid())
		{
			DrawContext.DrawBox(static_cast<float>(Position.X), static_cast<float>(Position.Y), static_cast<float>(DesiredSize.X), static_cast<float>(DesiredSize.Y), ImageBrush.Get(), FLinearColor::White);
			DrawContext.LayerId++;
		}

		// Draw cached texts.
		for (const FDrawTextInfo& TextInfo : Texts)
		{
			float X = static_cast<float>(Position.X);
			switch (TextInfo.Type)
			{
				case FDrawTextType::Name:
					X += ValueOffsetX - static_cast<float>(TextInfo.TextSize.X);
					break;

				case FDrawTextType::Value:
					X += ValueOffsetX + NameValueDX;
					break;

				default:
					X += TextInfo.X;
			}
			DrawContext.DrawText(X, static_cast<float>(Position.Y) + TextInfo.Y, TextInfo.Text, Font, TextInfo.Color.CopyWithNewOpacity(Opacity));
		}
		DrawContext.LayerId++;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
