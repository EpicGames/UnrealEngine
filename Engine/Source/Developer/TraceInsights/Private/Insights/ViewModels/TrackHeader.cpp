// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackHeader.h"

#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/ViewModels/BaseTimingTrack.h"
#include "Insights/ViewModels/TimingTrackViewport.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "TrackHeader"

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTrackHeader
////////////////////////////////////////////////////////////////////////////////////////////////////

FTrackHeader::FTrackHeader(FBaseTimingTrack& InParentTrack)
	: ParentTrack(InParentTrack)
	, WhiteBrush(FInsightsStyle::Get().GetBrush("WhiteBrush"))
	, Font(FAppStyle::Get().GetFontStyle("SmallFont"))
	, FontScale(1.0f)
{
	Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrackHeader::Reset()
{
	Width = 80.0f;
	Height = 14.0f;
	bIsInBackground = false;
	bCanBeCollapsed = false;
	bIsCollapsed = false;
	TargetHoveredAnimPercent = 0.0f;
	CurrentHoveredAnimPercent = 0.0f;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrackHeader::UpdateSize()
{
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const float TextWidth = static_cast<float>(FontMeasureService->Measure(ParentTrack.GetName(), Font, FontScale).X / FontScale);

	Width = TextWidth + 4.0f;
	if (bCanBeCollapsed)
	{
		Width += 9.0f;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrackHeader::Update(const ITimingTrackUpdateContext& InContext)
{
	if (CurrentHoveredAnimPercent != TargetHoveredAnimPercent)
	{
		if (CurrentHoveredAnimPercent < TargetHoveredAnimPercent)
		{
			const float ShowAnimSpeed = 2 * InContext.GetDeltaTime();
			CurrentHoveredAnimPercent += ShowAnimSpeed;
			if (CurrentHoveredAnimPercent > TargetHoveredAnimPercent)
			{
				CurrentHoveredAnimPercent = TargetHoveredAnimPercent;
			}
		}
		else
		{
			const float HideAnimSpeed = 3 * InContext.GetDeltaTime();
			CurrentHoveredAnimPercent -= HideAnimSpeed;
			if (CurrentHoveredAnimPercent < TargetHoveredAnimPercent)
			{
				CurrentHoveredAnimPercent = TargetHoveredAnimPercent;
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrackHeader::PostUpdate(const ITimingTrackUpdateContext& Context)
{
	const float MouseX = static_cast<float>(Context.GetMousePosition().X);
	const float MouseY = static_cast<float>(Context.GetMousePosition().Y);

	ParentTrack.SetHeaderHoveredState(MouseX < Width && MouseY < ParentTrack.GetPosY() + Height);

	if (ParentTrack.IsHovered())
	{
		TargetHoveredAnimPercent = 1.0f;
	}
	else
	{
		TargetHoveredAnimPercent = 0.0f;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrackHeader::Draw(const ITimingTrackDrawContext& Context) const
{
	if (bIsInBackground)
	{
		// Draw the track's header, in background.
		if (!ParentTrack.IsHovered() || CurrentHoveredAnimPercent < 1.0f)
		{
			DrawInternal(Context, true);
		}
	}
	else
	{
		UE::Insights::FDrawContext& DrawContext = Context.GetDrawContext();

		// Draw a horizontal line between tracks (top line of a track).
		DrawContext.DrawBox(0.0f, ParentTrack.GetPosY(), Context.GetViewport().GetWidth(), 1.0f, WhiteBrush, FLinearColor(0.05f, 0.05f, 0.05f, 1.0f));

		DrawInternal(Context, false);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrackHeader::PostDraw(const ITimingTrackDrawContext& Context) const
{
	if (bIsInBackground)
	{
		// When track is hovered, the track's header is draw on top.
		if (ParentTrack.IsHovered() || CurrentHoveredAnimPercent > 0.0f)
		{
			DrawInternal(Context, false);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTrackHeader::DrawInternal(const ITimingTrackDrawContext& Context, bool bDrawBackgroundLayer) const
{
	UE::Insights::FDrawContext& DrawContext = Context.GetDrawContext();

	const float Opacity = bIsInBackground ? CurrentHoveredAnimPercent : 1.0f;

	if (!bDrawBackgroundLayer)
	{
		DrawContext.DrawBox(0.0f, ParentTrack.GetPosY(), Width, Height, WhiteBrush, FLinearColor(0.04f, 0.04f, 0.04f, Opacity));
		DrawContext.LayerId++;
	}

	FLinearColor Color;
	if (bIsInBackground && (bDrawBackgroundLayer || CurrentHoveredAnimPercent == 0.0))
	{
		Color = FLinearColor(0.07f, 0.07f, 0.07f, 1.0f);
	}
	else if (ParentTrack.IsHeaderHovered())
	{
		Color = FLinearColor(1.0f, 1.0f, 0.0f, Opacity);
	}
	else
	{
		Color = FLinearColor(1.0f, 1.0f, 1.0f, Opacity);
	}

	// Draw text.
	DrawContext.DrawText(2.0f, ParentTrack.GetPosY() + 1.0f, ParentTrack.GetName(), Font, Color);

	if (bCanBeCollapsed)
	{
		const float ArrowSizeX = 4.0f;
		const float ArrowSizeY = 8.0f;
		const float ArrowX = Width - ArrowSizeX - 4.0f;
		const float ArrowY = ParentTrack.GetPosY() + 3.0f;

		// Draw arrow indicating "collapsed" state.
		if (bIsCollapsed)
		{
			// Draw "right empty arrow".
			//TODO: use a brush/image instead
			TArray<FVector2D> Points =
			{
				FVector2D(ArrowX, ArrowY),
				FVector2D(ArrowX, ArrowY + ArrowSizeY),
				FVector2D(ArrowX + ArrowSizeX, ArrowY + ArrowSizeY / 2.0f),
				FVector2D(ArrowX, ArrowY)
			};
			FSlateDrawElement::MakeLines(DrawContext.ElementList, DrawContext.LayerId, DrawContext.Geometry.ToPaintGeometry(), Points, DrawContext.DrawEffects, Color, false, 1.0f);
		}
		else
		{
			// Draw "down-right filled arrow".
			//TODO: use a brush/image instead
			for (float A = 1.0f; A < ArrowSizeY; A += 1.0f)
			{
				DrawContext.DrawBox(ArrowX - 3.0f + ArrowSizeY - A, ArrowY + A - 1.0f, A, 1.0f, WhiteBrush, Color);
			}
		}
	}

	DrawContext.LayerId++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
