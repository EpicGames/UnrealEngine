// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/SlateFontInfo.h"

class ITimingTrackUpdateContext;
class ITimingTrackDrawContext;
class FBaseTimingTrack;
struct FSlateBrush;

class FTrackHeader
{
public:
	FTrackHeader(FBaseTimingTrack& InParentTrack);
	~FTrackHeader() = default;

	void Reset();

	bool IsInBackground() const { return bIsInBackground; }
	void SetIsInBackground(bool bOnOff) { bIsInBackground = bOnOff; }

	bool CanBeCollapsed() const { return bCanBeCollapsed; }
	void SetCanBeCollapsed(bool bOnOff) { bCanBeCollapsed = bOnOff; }

	bool IsCollapsed() const { return bIsCollapsed; }
	void SetIsCollapsed(bool bOnOff) { bIsCollapsed = bOnOff; }
	void ToggleCollapsed() { bIsCollapsed = !bIsCollapsed; }

	void UpdateSize();

	void Update(const ITimingTrackUpdateContext& Context);
	void PostUpdate(const ITimingTrackUpdateContext& Context);

	void Draw(const ITimingTrackDrawContext& Context) const;
	void PostDraw(const ITimingTrackDrawContext& Context) const;

	float GetFontScale() const { return FontScale; }
	void SetFontScale(float InFontScale) { FontScale = InFontScale; }

private:
	void DrawInternal(const ITimingTrackDrawContext& Context, bool bDrawBackgroundLayer) const;

private:
	FBaseTimingTrack& ParentTrack;

	float Width;
	float Height;

	bool bIsInBackground;

	bool bCanBeCollapsed;
	bool bIsCollapsed;

	float TargetHoveredAnimPercent; // [0.0 .. 1.0], 0.0 = hidden, 1.0 = visible
	float CurrentHoveredAnimPercent;

	// Slate resources
	const FSlateBrush* WhiteBrush;
	const FSlateFontInfo Font;
	float FontScale;
};
