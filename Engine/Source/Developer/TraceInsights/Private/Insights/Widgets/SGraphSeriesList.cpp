// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphSeriesList.h"

#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SHeaderRow.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/ViewModels/GraphSeries.h"
#include "Insights/ViewModels/GraphTrack.h"

#define LOCTEXT_NAMESPACE "SGraphSeriesList"

////////////////////////////////////////////////////////////////////////////////////////////////////
// SGraphSeriesListEntry
////////////////////////////////////////////////////////////////////////////////////////////////////

// A list entry widget for a graph series
class SGraphSeriesListEntry : public STableRow<TSharedPtr<FGraphSeries>>
{
public:
	SLATE_BEGIN_ARGS(SGraphSeriesListEntry) {}

	SLATE_ARGUMENT(TSharedPtr<FGraphSeries>, GraphSeries)

	SLATE_ARGUMENT(TSharedPtr<FGraphTrack>, GraphTrack)

	SLATE_ATTRIBUTE(FText, SearchText)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		GraphTrack = InArgs._GraphTrack;
		GraphSeries = InArgs._GraphSeries;
		SearchText = InArgs._SearchText;

		STableRow<TSharedPtr<FGraphSeries>>::Construct(STableRow<TSharedPtr<FGraphSeries>>::FArguments(), InOwnerTable);
	}

	virtual void ConstructChildren( ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent ) override
	{
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this](){ return GraphSeries.Pin()->IsVisible() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
				{
					GraphSeries.Pin()->SetVisibility(InCheckBoxState == ECheckBoxState::Checked);
					GraphSeries.Pin()->SetDirtyFlag();
					GraphTrack.Pin()->SetDirtyFlag();
				})
				.Content()
				[
					SNew(STextBlock)
					.Text(GraphSeries.Pin()->GetName())
					.HighlightText(SearchText)
				]
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(2.0f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.WidthOverride(12.0f)
				.HeightOverride(12.0f)
				[
					SNew(SImage)
					.ColorAndOpacity_Lambda([this]() { return FSlateColor(GraphSeries.Pin()->GetColor()); })
					.Image(FAppStyle::Get().GetBrush("Icons.FilledCircle"))
				]
			]
		];
	}

	// The track containing the series we represent
	TWeakPtr<FGraphTrack> GraphTrack;

	// The series we represent
	TWeakPtr<FGraphSeries> GraphSeries;

	// The search text to highlight
	TAttribute<FText> SearchText;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// SGraphSeriesList
////////////////////////////////////////////////////////////////////////////////////////////////////

void SGraphSeriesList::Construct(const FArguments& InArgs, const TSharedRef<FGraphTrack>& InGraphTrack)
{
	GraphTrack = InGraphTrack;

	FilteredSeries = InGraphTrack->GetSeries();

	ListView = SNew(SListView<TSharedPtr<FGraphSeries>>)
		.IsFocusable(true)
		.SelectionMode(ESelectionMode::None)
		.ListItemsSource(&FilteredSeries)
		.OnGenerateRow(this, &SGraphSeriesList::OnGenerateRow);

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f)
			[
				// Checkbox for bulk operations
				SNew(SCheckBox)
				.IsChecked_Lambda([this]()
				{
					int32 NumVisible = 0;
					for (TSharedPtr<FGraphSeries> Series : FilteredSeries)
					{
						if (Series->IsVisible())
						{
							NumVisible++;
						}
					}

					if (NumVisible == 0)
					{
						return ECheckBoxState::Unchecked;
					}
					else if (NumVisible == FilteredSeries.Num())
					{
						return ECheckBoxState::Checked;
					}

					return ECheckBoxState::Undetermined;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
				{
					const bool bVisible = InCheckBoxState != ECheckBoxState::Unchecked;
					for (TSharedPtr<FGraphSeries> Series : FilteredSeries)
					{
						Series->SetVisibility(bVisible);
						Series->SetDirtyFlag();
					}

					GraphTrack.Pin()->SetDirtyFlag();
				})
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				// Search box allows for filtering
				SAssignNew(SearchBox, SSearchBox)
				.OnTextChanged_Lambda([this](const FText& InText){ SearchText = InText; RefreshFilter(); })
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SScrollBorder, ListView.ToSharedRef())
			[
				ListView.ToSharedRef()
			]
		]
	];

	// Set focus to the search box on creation
	FSlateApplication::Get().SetKeyboardFocus(SearchBox);
	FSlateApplication::Get().SetUserFocus(0, SearchBox);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableRow> SGraphSeriesList::OnGenerateRow(TSharedPtr<FGraphSeries> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SGraphSeriesListEntry, OwnerTable)
		.GraphTrack(GraphTrack.Pin())
		.GraphSeries(Item)
		.SearchText_Lambda([this](){ return SearchText; });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SGraphSeriesList::RefreshFilter()
{
	FilteredSeries.Reset();

	for (TSharedPtr<FGraphSeries> Series : GraphTrack.Pin()->GetSeries())
	{
		if (SearchText.IsEmpty() || Series->GetName().ToString().Contains(SearchText.ToString()))
		{
			FilteredSeries.Add(Series);
		}
	}

	ListView->RequestListRefresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
