// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class SSearchBox;
class FGraphTrack;
class FGraphSeries;

// A widget representing all the series in a graph track, allowing management of their visibility
class SGraphSeriesList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGraphSeriesList) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FGraphTrack>& InGraphTrack);

private:
	// Generate a row for the list
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FGraphSeries> Item, const TSharedRef<STableViewBase>& OwnerTable);

	// Refresh the list as the filter has changed
	void RefreshFilter();

private:
	// The graph track we are operating on
	TWeakPtr<FGraphTrack> GraphTrack;

	// The search widget
	TSharedPtr<SSearchBox> SearchBox;

	// The list view widget
	TSharedPtr<SListView<TSharedPtr<FGraphSeries>>> ListView;

	// Text we are searching for
	FText SearchText;

	// Filtered list of series
	TArray<TSharedPtr<FGraphSeries>> FilteredSeries;
};
