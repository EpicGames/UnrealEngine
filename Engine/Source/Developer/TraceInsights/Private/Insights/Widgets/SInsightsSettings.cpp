// Copyright Epic Games, Inc. All Rights Reserved.

#include "SInsightsSettings.h"

#include "Fonts/SlateFontInfo.h"
#include "Misc/Paths.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

// TraceInsights
#include "Insights/InsightsManager.h"
#include "Insights/InsightsSettings.h"
#include "Insights/InsightsStyle.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "SInsightsSettings"

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SInsightsSettings::Construct(const FArguments& InArgs)
{
	OnClose = InArgs._OnClose;
	SettingPtr = InArgs._SettingPtr;

	const TSharedRef<SGridPanel> SettingsGrid = SNew(SGridPanel);
	int32 CurrentRowPos = 0;

	AddTitle(LOCTEXT("SettingsTitle","Unreal Insights - Settings"), SettingsGrid, CurrentRowPos);
	AddSeparator(SettingsGrid, CurrentRowPos);
	AddHeader(LOCTEXT("TimingProfilerTitle","Timing Insights - defaults (applies when a new analysis session starts)"), SettingsGrid, CurrentRowPos);
	AddOption
	(
		LOCTEXT("bAutoHideEmptyTracks_T","Auto hide empty CPU/GPU tracks in Timing View"),
		LOCTEXT("bAutoHideEmptyTracks_TT","If enabled, the empty CPU/GPU tracks will be hidden in the Timing View."),
		SettingPtr->bAutoHideEmptyTracks,
		SettingPtr->GetDefaults().bAutoHideEmptyTracks,
		SettingsGrid,
		CurrentRowPos
	);
	AddOption
	(
		LOCTEXT("bAutoZoomOnFrameSelection_T", "Auto zoom on frame selection"),
		LOCTEXT("bAutoZoomOnFrameSelection_TT", "If enabled, the Timing View will also be zoomed when a new frame is selected in the Frames track."),
		SettingPtr->bAutoZoomOnFrameSelection,
		SettingPtr->GetDefaults().bAutoZoomOnFrameSelection,
		SettingsGrid,
		CurrentRowPos
	);
	AddSeparator(SettingsGrid, CurrentRowPos);
	AddFooter(SettingsGrid, CurrentRowPos);

	ChildSlot
	[
		SettingsGrid
	];

	SettingPtr->EnterEditMode();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SInsightsSettings::AddTitle(const FText& TitleText, const TSharedRef<SGridPanel>& Grid, int32& RowPos)
{
	Grid->AddSlot(0, RowPos++)
	.Padding(2.0f)
	[
		SNew(STextBlock)
		.Font(FAppStyle::Get().GetFontStyle("HeadingExtraSmall"))
		.Text(TitleText)
	];
	RowPos++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SInsightsSettings::AddSeparator(const TSharedRef<SGridPanel>& Grid, int32& RowPos)
{
	Grid->AddSlot(0, RowPos++)
	.Padding(2.0f)
	.ColumnSpan(2)
	[
		SNew(SSeparator)
		.Orientation(Orient_Horizontal)
	];
	RowPos++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SInsightsSettings::AddHeader(const FText& HeaderText, const TSharedRef<SGridPanel>& Grid, int32& RowPos)
{
	Grid->AddSlot(0, RowPos++)
	.Padding(2.0f)
	[
		SNew(STextBlock)
		.Font(FAppStyle::Get().GetFontStyle("Font.Large.Bold"))
		.Text(HeaderText)
	];
	RowPos++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SInsightsSettings::AddOption(const FText& OptionName, const FText& OptionDesc, bool& Value, const bool& Default, const TSharedRef<SGridPanel>& Grid, int32& RowPos)
{
	Grid->AddSlot(0, RowPos)
	.Padding(2.0f)
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.Text(OptionName)
		.ToolTipText(OptionDesc)
	];

	Grid->AddSlot(1, RowPos)
	.Padding(2.0f)
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Fill)
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.IsChecked(this, &SInsightsSettings::OptionValue_IsChecked, (const bool*)&Value)
			.OnCheckStateChanged(this, &SInsightsSettings::OptionValue_OnCheckStateChanged, (bool*)&Value)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("ResetToDefaultToolTip", "Reset to default"))
			.ButtonStyle(FAppStyle::Get(), TEXT("NoBorder"))
			.ContentPadding(0.0f)
			.Visibility(this, &SInsightsSettings::OptionDefault_GetDiffersFromDefaultAsVisibility, (const bool*)&Value, (const bool*)&Default)
			.OnClicked(this, &SInsightsSettings::OptionDefault_OnClicked, (bool*)&Value, (const bool*)&Default)
			.Content()
			[
				SNew(SImage)
				.Image(FInsightsStyle::GetBrush("Icons.DiffersFromDefault"))
			]
		]
	];

	RowPos++;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SInsightsSettings::AddFooter(const TSharedRef<SGridPanel>& Grid, int32& RowPos)
{
	Grid->AddSlot(0, RowPos)
	.Padding(2.0f)
	.ColumnSpan(2)
	.HAlign(HAlign_Right)
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(132)
		.HAlign(HAlign_Center)
		[
			SNew(SButton)
			.OnClicked(this, &SInsightsSettings::SaveAndClose_OnClicked)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Save"))
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SaveAndCloseTitle","Save and close"))
				]
			]
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(132)
		.HAlign(HAlign_Center)
		[
			SNew(SButton)
			.OnClicked(this, &SInsightsSettings::ResetToDefaults_OnClicked)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FInsightsStyle::GetBrush("Icons.ResetToDefault"))
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ResetToDefaultsTitle","Reset to defaults"))
				]
			]
		]
	];

	RowPos++;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SInsightsSettings::SaveAndClose_OnClicked()
{
	OnClose.ExecuteIfBound();
	SettingPtr->ExitEditMode();
	SettingPtr->SaveToConfig();

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SInsightsSettings::ResetToDefaults_OnClicked()
{
	SettingPtr->ResetToDefaults();

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SInsightsSettings::OptionValue_OnCheckStateChanged(ECheckBoxState CheckBoxState, bool* ValuePtr)
{
	*ValuePtr = CheckBoxState == ECheckBoxState::Checked ? true : false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState SInsightsSettings::OptionValue_IsChecked(const bool* ValuePtr) const
{
	return *ValuePtr ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EVisibility SInsightsSettings::OptionDefault_GetDiffersFromDefaultAsVisibility(const bool* ValuePtr, const bool* DefaultPtr) const
{
	return *ValuePtr != *DefaultPtr ? EVisibility::Visible : EVisibility::Hidden;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SInsightsSettings::OptionDefault_OnClicked(bool* ValuePtr, const bool* DefaultPtr)
{
	*ValuePtr = *DefaultPtr;

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
