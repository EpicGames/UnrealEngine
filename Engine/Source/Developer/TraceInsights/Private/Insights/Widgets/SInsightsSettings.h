// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FInsightsSettings;
class SGridPanel;
enum class ECheckBoxState : uint8;

/** Widget used to modify settings for the profiler, created on demand and destroyed on close. */
class SInsightsSettings : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SInsightsSettings) {}
		SLATE_EVENT(FSimpleDelegate, OnClose)
		SLATE_ARGUMENT(FInsightsSettings*, SettingPtr)
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	void Construct(const FArguments& InArgs);

private:
	void AddTitle(const FText& TitleText, const TSharedRef<SGridPanel>& Grid, int32& RowPos);
	void AddSeparator(const TSharedRef<SGridPanel>& Grid, int32& RowPos);
	void AddHeader(const FText& HeaderText, const TSharedRef<SGridPanel>& Grid, int32& RowPos);
	void AddOption(const FText& OptionName, const FText& OptionDesc, bool& Value, const bool& Default, const TSharedRef<SGridPanel>& Grid, int32& RowPos);
	void AddFooter(const TSharedRef<SGridPanel>& Grid, int32& RowPos);

	FReply SaveAndClose_OnClicked();
	FReply ResetToDefaults_OnClicked();

	EVisibility OptionDefault_GetDiffersFromDefaultAsVisibility(const bool* ValuePtr, const bool* DefaultPtr) const;
	FReply OptionDefault_OnClicked(bool* ValuePtr, const bool* DefaultPtr);

	void OptionValue_OnCheckStateChanged(ECheckBoxState CheckBoxState, bool* ValuePtr);
	ECheckBoxState OptionValue_IsChecked(const bool* ValuePtr) const;

private:
	/** Delegate to call when this profiler settings is closed. */
	FSimpleDelegate OnClose;

	/** Pointer to the profiler settings. */
	FInsightsSettings* SettingPtr;
};
