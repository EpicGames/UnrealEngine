// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLogView.h"

#include "Algo/BinarySearch.h"
#include "Async/AsyncWork.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformFileManager.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"
#include "Logging/MessageLog.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"

// TraceServices
#include "TraceServices/Model/Log.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/Common/Utf8FileWriter.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/Log.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Tracks/MarkersTimingTrack.h" // for FTimeMarkerTrackBuilder::GetColorBy*
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/ViewModels/TimingViewDrawHelper.h"
#include "Insights/Widgets/STimingView.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::SLogView"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FLogViewCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

class FLogViewCommands : public TCommands<FLogViewCommands>
{
public:
	FLogViewCommands()
	: TCommands<FLogViewCommands>(
		TEXT("LogViewCommands"),
		NSLOCTEXT("Contexts", "LogViewCommands", "Insights - Log View"),
		NAME_None,
		FInsightsStyle::GetStyleSetName())
	{
	}

	virtual ~FLogViewCommands()
	{
	}

	// UI_COMMAND takes long for the compiler to optimize
	UE_DISABLE_OPTIMIZATION_SHIP
	virtual void RegisterCommands() override
	{
		UI_COMMAND(Command_HideSelectedCategory,
			"Hide Category",
			"Hides the selected log category.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_ShowOnlySelectedCategory,
			"Show Only Selected Category",
			"Shows only the selected log category (hides all other log categories).",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_ShowAllCategories,
			"Show All Categories",
			"Resets the category filter (shows all log categories).",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_ToggleTimeFormat,
			"Toggle Time Format",
			"Toggles between Relative and Real Time Formats.",
			EUserInterfaceActionType::Button,
			FInputChord());

		UI_COMMAND(Command_CopySelected,
			"Copy",
			"Copies the selected log (with all its properties) to clipboard.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Control, EKeys::C));

		UI_COMMAND(Command_CopyMessage,
			"Copy Message",
			"Copies the message text of the selected log to clipboard.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Shift, EKeys::C));

		UI_COMMAND(Command_CopyRange,
			"Copy Range",
			"Copies all the logs in the selected time range (highlighted in blue) to clipboard.",
			EUserInterfaceActionType::Button,
			FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::C));

		UI_COMMAND(Command_CopyAll,
			"Copy All",
			"Copies all the (filtered) logs to clipboard.",
			EUserInterfaceActionType::Button, FInputChord());

		UI_COMMAND(Command_SaveRange,
			"Save Range As...",
			"Saves all the logs in the selected time range (highlighted in blue) to a text file (tab-separated values or comma-separated values).",
			EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::S));

		UI_COMMAND(Command_SaveAll,
			"Save All As...",
			"Saves all the (filtered) logs to a text file (tab-separated values or comma-separated values).",
			EUserInterfaceActionType::Button, FInputChord());

		UI_COMMAND(Command_OpenSource,
			"Open Source",
			"Opens the source file of the selected message in the registered IDE.",
			EUserInterfaceActionType::Button, FInputChord());
	}
	UE_ENABLE_OPTIMIZATION_SHIP

	TSharedPtr<FUICommandInfo> Command_HideSelectedCategory;
	TSharedPtr<FUICommandInfo> Command_ShowOnlySelectedCategory;
	TSharedPtr<FUICommandInfo> Command_ShowAllCategories;
	TSharedPtr<FUICommandInfo> Command_ToggleTimeFormat;
	TSharedPtr<FUICommandInfo> Command_CopySelected;
	TSharedPtr<FUICommandInfo> Command_CopyMessage;
	TSharedPtr<FUICommandInfo> Command_CopyRange;
	TSharedPtr<FUICommandInfo> Command_CopyAll;
	TSharedPtr<FUICommandInfo> Command_SaveRange;
	TSharedPtr<FUICommandInfo> Command_SaveAll;
	TSharedPtr<FUICommandInfo> Command_OpenSource;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// SLogMessageRow
////////////////////////////////////////////////////////////////////////////////////////////////////

class SLogMessageRow : public SMultiColumnTableRow<TSharedPtr<FLogMessage>>
{
	SLATE_BEGIN_ARGS(SLogMessageRow) {}
	SLATE_END_ARGS()

public:
	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InLogMessage The log message displayed by this row.
	 * @param InOwnerTableView The table to which the row must be added.
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<FLogMessage> InLogMessage, TSharedRef<SLogView> InParentWidget, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		WeakLogMessage = MoveTemp(InLogMessage);
		WeakParentWidget = InParentWidget;

		SMultiColumnTableRow<TSharedPtr<FLogMessage>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);

		TSharedRef<SWidget> Row = ChildSlot.GetChildAt(0);

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FInsightsStyle::Get().GetBrush("WhiteBrush"))
			.BorderBackgroundColor(this, &SLogMessageRow::GetBackgroundColor)
			[
				Row
			]
		];
	}

public:
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == LogViewColumns::IdColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.Text(this, &SLogMessageRow::GetIndex)
				];
		}
		else if (ColumnName == LogViewColumns::SessionTimeColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.Text(this, &SLogMessageRow::GetTime)
					.ColorAndOpacity(this, &SLogMessageRow::GetColorAndOpacity)
				];
		}
		else if (ColumnName == LogViewColumns::RealTimeColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.Text(this, &SLogMessageRow::GetTime)
					.ColorAndOpacity(this, &SLogMessageRow::GetColorAndOpacity)
				];
		}
		else if (ColumnName == LogViewColumns::VerbosityColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.Text(this, &SLogMessageRow::GetVerbosity)
					.ColorAndOpacity(this, &SLogMessageRow::GetColorByVerbosity)
				];
		}
		else if (ColumnName == LogViewColumns::CategoryColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.Text(this, &SLogMessageRow::GetCategory)
					.ColorAndOpacity(this, &SLogMessageRow::GetColorByCategory)
				];
		}
		else if (ColumnName == LogViewColumns::MessageColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.Text(this, &SLogMessageRow::GetMessage)
					.HighlightText(this, &SLogMessageRow::GetMessageHighlightText)
				];
		}
		else if (ColumnName == LogViewColumns::FileColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.Text(this, &SLogMessageRow::GetFile)
				];
		}
		else if (ColumnName == LogViewColumns::LineColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.Text(this, &SLogMessageRow::GetLine)
				];
		}
		else
		{
			return SNew(STextBlock).Text(LOCTEXT("UnknownColumn", "Unknown Column"));
		}
	}

	FSlateColor GetBackgroundColor() const
	{
		TSharedPtr<SLogView> ParentWidgetPin = WeakParentWidget.Pin();
		TSharedPtr<FLogMessage> LogMessagePin = WeakLogMessage.Pin();
		if (ParentWidgetPin.IsValid() && LogMessagePin.IsValid())
		{
			TSharedPtr<FLogMessage> SelectedLogMessage = ParentWidgetPin->GetSelectedLogMessage();
			if (!SelectedLogMessage || SelectedLogMessage->GetIndex() != LogMessagePin->GetIndex()) // if row is not selected
			{
				FLogMessageRecord& CacheEntry = ParentWidgetPin->GetCache().Get(LogMessagePin->GetIndex());
				const double Time = CacheEntry.GetTime();

				TSharedPtr<TimingProfiler::STimingView> TimingView = ParentWidgetPin->GetTimingView();
				if (TimingView && TimingView->IsTimeSelectedInclusive(Time))
				{
					return FSlateColor(FLinearColor(0.25f, 0.5f, 1.0f, 0.25f));
				}
			}
		}

		return FSlateColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
	}

	FSlateColor GetColorAndOpacity() const
	{
		bool IsSelected = false;

		TSharedPtr<SLogView> ParentWidgetPin = WeakParentWidget.Pin();
		TSharedPtr<FLogMessage> LogMessagePin = WeakLogMessage.Pin();
		if (ParentWidgetPin.IsValid() && LogMessagePin.IsValid())
		{
			TSharedPtr<FLogMessage> SelectedLogMessage = ParentWidgetPin->GetSelectedLogMessage();
			if (SelectedLogMessage && SelectedLogMessage->GetIndex() == LogMessagePin->GetIndex())
			{
				IsSelected = true;
			}

			FLogMessageRecord& CacheEntry = ParentWidgetPin->GetCache().Get(LogMessagePin->GetIndex());
			const double Time = CacheEntry.GetTime();

			// Verify if time is monotonically increasing.
			if (LogMessagePin->GetIndex() > 0)
			{
				FLogMessageRecord& PrevCacheEntry = ParentWidgetPin->GetCache().Get(LogMessagePin->GetIndex() - 1);
				if (Time < PrevCacheEntry.GetTime())
				{
					return FSlateColor(FLinearColor(1.0f, 0.3f, 0.3f, 1.0f));
				}
			}

			TSharedPtr<TimingProfiler::STimingView> TimingView = ParentWidgetPin->GetTimingView();
			if (TimingView && TimingView->IsTimeSelectedInclusive(Time))
			{
				if (IsSelected)
				{
					return FSlateColor(FLinearColor(0.0f, 0.05f, 0.2f, 1.0f));
				}
				else
				{
					return FSlateColor(FLinearColor(0.4f, 0.8f, 1.6f, 1.0f));
				}
			}
		}

		if (IsSelected)
		{
			return FSlateColor(FLinearColor::Black);
		}
		else
		{
			return FSlateColor(FLinearColor::White);
		}
	}

	FText GetIndex() const
	{
		TSharedPtr<SLogView> ParentWidgetPin = WeakParentWidget.Pin();
		TSharedPtr<FLogMessage> LogMessagePin = WeakLogMessage.Pin();
		if (ParentWidgetPin.IsValid() && LogMessagePin.IsValid())
		{
			FLogMessageRecord& CacheEntry = ParentWidgetPin->GetCache().Get(LogMessagePin->GetIndex());
			return CacheEntry.GetIndexAsText();
		}
		else
		{
			return FText();
		}
	}

	FText GetTime() const
	{
		TSharedPtr<SLogView> ParentWidgetPin = WeakParentWidget.Pin();
		TSharedPtr<FLogMessage> LogMessagePin = WeakLogMessage.Pin();
		if (ParentWidgetPin.IsValid() && LogMessagePin.IsValid())
		{
			FLogMessageRecord& CacheEntry = ParentWidgetPin->GetCache().Get(LogMessagePin->GetIndex());
			TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
			return ParentWidgetPin->IsUsingRelativeTime() ? CacheEntry.GetTimeAsText() : CacheEntry.GetAbsoluteDateTime(Session->GetBaseDateTime());
		}
		else
		{
			return FText();
		}
	}

	FText GetVerbosity() const
	{
		TSharedPtr<SLogView> ParentWidgetPin = WeakParentWidget.Pin();
		TSharedPtr<FLogMessage> LogMessagePin = WeakLogMessage.Pin();
		if (ParentWidgetPin.IsValid() && LogMessagePin.IsValid())
		{
			FLogMessageRecord& CacheEntry = ParentWidgetPin->GetCache().Get(LogMessagePin->GetIndex());
			return CacheEntry.GetVerbosityAsText();
		}
		else
		{
			return FText();
		}
	}

	FSlateColor GetColorByVerbosity() const
	{
		if (IsSelected())
			return FSlateColor(FLinearColor::Black);

		TSharedPtr<SLogView> ParentWidgetPin = WeakParentWidget.Pin();
		TSharedPtr<FLogMessage> LogMessagePin = WeakLogMessage.Pin();
		if (ParentWidgetPin.IsValid() && LogMessagePin.IsValid())
		{
			FLogMessageRecord& CacheEntry = ParentWidgetPin->GetCache().Get(LogMessagePin->GetIndex());
			return FSlateColor(TimingProfiler::FTimeMarkerTrackBuilder::GetColorByVerbosity(CacheEntry.GetVerbosity()));
		}
		else
		{
			return FSlateColor(FLinearColor::White);
		}
	}

	FText GetCategory() const
	{
		TSharedPtr<SLogView> ParentWidgetPin = WeakParentWidget.Pin();
		TSharedPtr<FLogMessage> LogMessagePin = WeakLogMessage.Pin();
		if (ParentWidgetPin.IsValid() && LogMessagePin.IsValid())
		{
			FLogMessageRecord& CacheEntry = ParentWidgetPin->GetCache().Get(LogMessagePin->GetIndex());
			return CacheEntry.GetCategoryAsText();
		}
		else
		{
			return FText();
		}
	}

	FSlateColor GetColorByCategory() const
	{
		if (IsSelected())
			return FSlateColor(FLinearColor::Black);

		TSharedPtr<SLogView> ParentWidgetPin = WeakParentWidget.Pin();
		TSharedPtr<FLogMessage> LogMessagePin = WeakLogMessage.Pin();
		if (ParentWidgetPin.IsValid() && LogMessagePin.IsValid())
		{
			FLogMessageRecord& CacheEntry = ParentWidgetPin->GetCache().Get(LogMessagePin->GetIndex());
			return FSlateColor(TimingProfiler::FTimeMarkerTrackBuilder::GetColorByCategory(CacheEntry.GetCategory()));
		}
		else
		{
			return FSlateColor(FLinearColor::White);
		}
	}

	FText GetMessage() const
	{
		TSharedPtr<SLogView> ParentWidgetPin = WeakParentWidget.Pin();
		TSharedPtr<FLogMessage> LogMessagePin = WeakLogMessage.Pin();
		if (ParentWidgetPin.IsValid() && LogMessagePin.IsValid())
		{
			FLogMessageRecord& CacheEntry = ParentWidgetPin->GetCache().Get(LogMessagePin->GetIndex());
			return CacheEntry.GetMessageAsText();
		}
		else
		{
			return FText();
		}
	}

	FText GetMessageHighlightText() const
	{
		TSharedPtr<SLogView> ParentWidgetPin = WeakParentWidget.Pin();
		if (ParentWidgetPin.IsValid())
		{
			return ParentWidgetPin->GetFilterText();
		}
		else
		{
			return FText();
		}
	}

	FText GetFile() const
	{
		TSharedPtr<SLogView> ParentWidgetPin = WeakParentWidget.Pin();
		TSharedPtr<FLogMessage> LogMessagePin = WeakLogMessage.Pin();
		if (ParentWidgetPin.IsValid() && LogMessagePin.IsValid())
		{
			FLogMessageRecord& CacheEntry = ParentWidgetPin->GetCache().Get(LogMessagePin->GetIndex());
			return CacheEntry.GetFileAsText();
		}
		else
		{
			return FText();
		}
	}

	FText GetLine() const
	{
		TSharedPtr<SLogView> ParentWidgetPin = WeakParentWidget.Pin();
		TSharedPtr<FLogMessage> LogMessagePin = WeakLogMessage.Pin();
		if (ParentWidgetPin.IsValid() && LogMessagePin.IsValid())
		{
			FLogMessageRecord& CacheEntry = ParentWidgetPin->GetCache().Get(LogMessagePin->GetIndex());
			return CacheEntry.GetLineAsText();
		}
		else
		{
			return FText();
		}
	}

	FText GetRowToolTip() const
	{
		TSharedPtr<SLogView> ParentWidgetPin = WeakParentWidget.Pin();
		TSharedPtr<FLogMessage> LogMessagePin = WeakLogMessage.Pin();
		if (ParentWidgetPin.IsValid() && LogMessagePin.IsValid())
		{
			FLogMessageRecord& CacheEntry = ParentWidgetPin->GetCache().Get(LogMessagePin->GetIndex());
			return CacheEntry.ToDisplayString();
		}
		else
		{
			return FText();
		}
	}

private:
	TWeakPtr<FLogMessage> WeakLogMessage;
	TWeakPtr<SLogView> WeakParentWidget;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// SLogView
////////////////////////////////////////////////////////////////////////////////////////////////////

SLogView::SLogView()
	: Filter(*TimingProfiler::FTimingProfilerManager::Get()->GetLogFilter())
	, FilteringStartIndex(0)
	, FilteringEndIndex(0)
	, FilteringChangeNumber(0)
	, bIsFilteringAsyncTaskCancelRequested(false)
	, TotalNumCategories(0)
	, TotalNumMessages(0)
	, TotalNumInserts(0)
	, bIsDirty(false)
	, bUseRelativeTime(true)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SLogView::~SLogView()
{
	Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::Reset()
{
	//ListView
	//ExternalScrollbar
	FilterTextBox->SetText(FText::GetEmpty());

	Filter.Reset();
	FilterChangeNumber = 0;

	FilteringStartIndex = 0;
	FilteringEndIndex = 0;
	FilteringChangeNumber = 0;

	// Clean up our async task if we're deleted before it is completed.
	if (FilteringAsyncTask.IsValid())
	{
		if (!FilteringAsyncTask->Cancel())
		{
			bIsFilteringAsyncTaskCancelRequested = true;
			FilteringAsyncTask->EnsureCompletion();
		}
		FilteringAsyncTask.Reset();
	}

	//bIsFilteringAsyncTaskCancelRequested = false;
	//FilteringStopwatch.Stop();

	TotalNumCategories = 0;
	TotalNumMessages = 0;
	TotalNumInserts = 0;

	bIsDirty = false;
	DirtyStopwatch.Stop();

	StatsText = FText::GetEmpty();

	Cache.Reset();

	FilteredMessages.Reset();

	bUseRelativeTime = true;

	ListView->RebuildList();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SLogView::Construct(const FArguments& InArgs)
{
	SAssignNew(ExternalScrollbar, SScrollBar)
	.AlwaysShowScrollbar(true);

	FSlimHorizontalToolBarBuilder ToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);
	ToolbarBuilder.SetStyle(&FInsightsStyle::Get(), "SecondaryToolbar");

	ToolbarBuilder.BeginSection("Filters");
	{
		// Verbosity Threshold
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &SLogView::MakeVerbosityThresholdMenu),
			LOCTEXT("VerbosityThresholdText", "Verbosity Threshold"),
			LOCTEXT("VerbosityThresholdToolTip", "Filter log messages by verbosity threshold."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.Filter.ToolBar"),
			false
		);

		// Category Filter
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &SLogView::MakeCategoryFilterMenu),
			LOCTEXT("CategoryFilterText", "Category Filter"),
			LOCTEXT("CategoryFilterToolTip", "Filter log messages by category."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.Filter.ToolBar"),
			false
		);

		// Text Filter (Search Box)
		ToolbarBuilder.AddWidget(
			SAssignNew(FilterTextBox, SSearchBox)
			.HintText(LOCTEXT("FilterTextBoxHint", "Search log messages"))
			.ToolTipText(LOCTEXT("FilterTextBoxToolTip", "Type here to filter the list of log messages."))
			.OnTextChanged(this, &SLogView::FilterTextBox_OnTextChanged)
		);

		// Stats Text (number of logs)
		ToolbarBuilder.AddWidget(
			SNew(SBox)
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(this, &SLogView::GetStatsText)
				.ColorAndOpacity(this, &SLogView::GetStatsTextColor)
			]
		);
	}
	ToolbarBuilder.EndSection();

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(0.0f))
		[
			ToolbarBuilder.MakeWidget()
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(0.0f)
			.VAlign(VAlign_Fill)
			[
				SNew(SScrollBox)
				.Orientation(Orient_Horizontal)

				+ SScrollBox::Slot()
				.VAlign(VAlign_Fill)
				[
					SAssignNew(ListView, SListView<TSharedPtr<FLogMessage>>)
					.ExternalScrollbar(ExternalScrollbar)
					.FixedLineScrollOffset(0.0)
					.SelectionMode(ESelectionMode::Single)
					.OnMouseButtonClick(this, &SLogView::OnMouseButtonClick)
					.OnSelectionChanged(this, &SLogView::OnSelectionChanged)
					.ListItemsSource(&FilteredMessages)
					.OnGenerateRow(this, &SLogView::OnGenerateRow)
					.ConsumeMouseWheel(EConsumeMouseWheel::Always)
					.OnContextMenuOpening(FOnContextMenuOpening::CreateSP(this, &SLogView::ListView_GetContextMenu))
					.HeaderRow
					(
						SNew(SHeaderRow)

						+ SHeaderRow::Column(LogViewColumns::IdColumnName)
						.ManualWidth(60.0f)
						.DefaultLabel(LOCTEXT("IdColumn", "Index"))

						+ SHeaderRow::Column(LogViewColumns::SessionTimeColumnName)
						.ManualWidth(100.0f)
						.DefaultLabel(GetTimeFormatColumnName())

						+ SHeaderRow::Column(LogViewColumns::VerbosityColumnName)
						.ManualWidth(80.0f)
						.DefaultLabel(LOCTEXT("VerbosityColumn", "Verbosity"))

						+ SHeaderRow::Column(LogViewColumns::CategoryColumnName)
						.ManualWidth(120.0f)
						.DefaultLabel(LOCTEXT("CategoryColumn", "Category"))

						+ SHeaderRow::Column(LogViewColumns::MessageColumnName)
						.ManualWidth(880.0f)
						.DefaultLabel(LOCTEXT("MessageColumn", "Message"))

						+ SHeaderRow::Column(LogViewColumns::FileColumnName)
						.ManualWidth(600.0f)
						.DefaultLabel(LOCTEXT("FileColumn", "File"))

						+ SHeaderRow::Column(LogViewColumns::LineColumnName)
						.ManualWidth(60.0f)
						.DefaultLabel(LOCTEXT("LineColumn", "Line"))
					)
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f)
			[
				ExternalScrollbar.ToSharedRef()
			]
		]
	];

	InitCommandList();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableRow> SLogView::OnGenerateRow(TSharedPtr<FLogMessage> InLogMessage, const TSharedRef<STableViewBase>& OwnerTable)
{
	// Generate a row for the log message corresponding to InLogMessage.
	return SNew(SLogMessageRow, InLogMessage, SharedThis(this), OwnerTable);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::InitCommandList()
{
	FLogViewCommands::Register();
	CommandList = MakeShared<FUICommandList>();
	CommandList->MapAction(FLogViewCommands::Get().Command_HideSelectedCategory, FExecuteAction::CreateSP(this, &SLogView::HideSelectedCategory), FCanExecuteAction::CreateSP(this, &SLogView::CanHideSelectedCategory));
	CommandList->MapAction(FLogViewCommands::Get().Command_ShowOnlySelectedCategory, FExecuteAction::CreateSP(this, &SLogView::ShowOnlySelectedCategory), FCanExecuteAction::CreateSP(this, &SLogView::CanShowOnlySelectedCategory));
	CommandList->MapAction(FLogViewCommands::Get().Command_ShowAllCategories, FExecuteAction::CreateSP(this, &SLogView::ShowAllCategories), FCanExecuteAction::CreateSP(this, &SLogView::CanShowAllCategories));
	CommandList->MapAction(FLogViewCommands::Get().Command_ToggleTimeFormat, FExecuteAction::CreateSP(this, &SLogView::ToggleTimeFormat), FCanExecuteAction::CreateSP(this, &SLogView::CanToggleTimeFormat));
	CommandList->MapAction(FLogViewCommands::Get().Command_CopySelected, FExecuteAction::CreateSP(this, &SLogView::CopySelected), FCanExecuteAction::CreateSP(this, &SLogView::CanCopySelected));
	CommandList->MapAction(FLogViewCommands::Get().Command_CopyMessage, FExecuteAction::CreateSP(this, &SLogView::CopyMessage), FCanExecuteAction::CreateSP(this, &SLogView::CanCopyMessage));
	CommandList->MapAction(FLogViewCommands::Get().Command_CopyRange, FExecuteAction::CreateSP(this, &SLogView::CopyRange), FCanExecuteAction::CreateSP(this, &SLogView::CanCopyRange));
	CommandList->MapAction(FLogViewCommands::Get().Command_CopyAll, FExecuteAction::CreateSP(this, &SLogView::CopyAll), FCanExecuteAction::CreateSP(this, &SLogView::CanCopyAll));
	CommandList->MapAction(FLogViewCommands::Get().Command_SaveRange, FExecuteAction::CreateSP(this, &SLogView::SaveRange), FCanExecuteAction::CreateSP(this, &SLogView::CanSaveRange));
	CommandList->MapAction(FLogViewCommands::Get().Command_SaveAll, FExecuteAction::CreateSP(this, &SLogView::SaveAll), FCanExecuteAction::CreateSP(this, &SLogView::CanSaveAll));
	CommandList->MapAction(FLogViewCommands::Get().Command_OpenSource, FExecuteAction::CreateSP(this, &SLogView::OpenSource), FCanExecuteAction::CreateSP(this, &SLogView::CanOpenSource));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	LLM_SCOPE_BYTAG(Insights);

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();

	Cache.SetSession(Session);

	int32 LogProviderNumCategories = 0;
	int32 LogProviderNumMessages = 0;
	int32 LogProviderNumInserts = 0;

	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());

		LogProviderNumCategories = static_cast<int32>(LogProvider.GetCategoryCount());
		LogProviderNumMessages = static_cast<int32>(LogProvider.GetMessageCount());
		LogProviderNumInserts = static_cast<int32>(LogProvider.GetInsertCount());
	}

	if (LogProviderNumCategories != TotalNumCategories)
	{
		TSet<FName> Categories;
		TMap<FName, int32> DuplicatedCategories;

		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());

			// Re-read the number of categories (as it might have changed between the read locks).
			LogProviderNumCategories = static_cast<int32>(LogProvider.GetCategoryCount());

			LogProvider.EnumerateCategories([&Categories, &DuplicatedCategories](const TraceServices::FLogCategoryInfo& Category)
			{
				FStringView CategoryStr(Category.Name);
				if (CategoryStr.StartsWith(TEXT("Log")))
				{
					CategoryStr.RightChopInline(3);
				}
				FName CategoryName(CategoryStr);
				if (Categories.Contains(CategoryName))
				{
					int32* CountPtr = DuplicatedCategories.Find(CategoryName);
					if (CountPtr)
					{
						++(*CountPtr);
					}
					else
					{
						DuplicatedCategories.Add(CategoryName, 1);
					}
				}
				else
				{
					Categories.Add(CategoryName);
				}
			});
		}

		TotalNumCategories = LogProviderNumCategories;
		UE_LOGF(TraceInsights, Log, "[LogView] Total Log Categories: %d", TotalNumCategories);

		Filter.SyncAvailableCategories(Categories);
		const int32 NumAvailableLogCategories = Filter.GetAvailableLogCategories().Num();
		if (DuplicatedCategories.Num() > 0)
		{
			UE_LOGF(TraceInsights, Warning, "[LogView] Duplicated Log Categories: %d (+%dx)", DuplicatedCategories.Num(), TotalNumCategories - NumAvailableLogCategories);
			for (const auto& KV : DuplicatedCategories)
			{
				UE_LOGF(TraceInsights, Log, "[LogView]    \"%ls\" (+%dx)", *KV.Key.GetPlainNameString(), KV.Value);
			}
		}
		UE_LOGF(TraceInsights, Log, "[LogView] Unique Log Categories: %d", NumAvailableLogCategories);

		FilteredMessages.Reset();
		TotalNumMessages = 0;
		bIsDirty = true;
		DirtyStopwatch.Start();
	}

	if (LogProviderNumInserts != TotalNumInserts)
	{
		TotalNumInserts = LogProviderNumInserts;
		Cache.Reset();
	}

	if (LogProviderNumMessages != TotalNumMessages)
	{
		bIsDirty = true;
		DirtyStopwatch.Start();

		if (LogProviderNumMessages > TotalNumMessages)
		{
			if (Filter.IsFilterSet())
			{
				// Filter messages async.
				if (!FilteringAsyncTask.IsValid())
				{
					FilteringStopwatch.Restart();

					bIsFilteringAsyncTaskCancelRequested = false;
					FilteringStartIndex = TotalNumMessages;
					FilteringEndIndex = LogProviderNumMessages;
					FilteringChangeNumber = Filter.GetChangeNumber();
					FilteringAsyncTask = MakeUnique<FAsyncTask<FLogFilteringAsyncTask>>(FilteringStartIndex, FilteringEndIndex, Filter, SharedThis(this));
#if !WITH_EDITOR
					UE_LOGF(TraceInsights, Log, "[LogView] Start async task for filtering by%ls%ls%ls (\"%ls\") (%d to %d)",
						Filter.IsFilterSetByVerbosity() ? TEXT(" Verbosity,") : TEXT(""),
						Filter.IsFilterSetByCategory() ? TEXT(" Category,") : TEXT(""),
						Filter.IsFilterSetByText() ? TEXT(" Text") : TEXT(""),
						*Filter.GetFilterText().ToString(),
						FilteringStartIndex, FilteringEndIndex);
#endif // !WITH_EDITOR
					FilteringAsyncTask->StartBackgroundTask();
				}
				else
				{
					// A task is already in progress.
					if (FilteringStartIndex == TotalNumMessages &&
						FilteringEndIndex <= LogProviderNumMessages &&
						FilteringChangeNumber == Filter.GetChangeNumber())
					{
						// The filter is still valid. Just wait.
					}
					else
					{
						// The filter used by running task is obsolete. Cancel the task.
						bIsFilteringAsyncTaskCancelRequested = true;
					}
				}
			}
			else // no filtering
			{
				FilteringStopwatch.Restart();

				for (int32 Index = TotalNumMessages; Index < LogProviderNumMessages; Index++)
				{
					FilteredMessages.Add(MakeShared<FLogMessage>(Index));
				}

				const int32 NumAddedMessages = LogProviderNumMessages - TotalNumMessages;
				TotalNumMessages = LogProviderNumMessages;
				TSharedPtr<FLogMessage> SelectedLogMessage = GetSelectedLogMessage();
				ListView->RebuildList();
				if (SelectedLogMessage.IsValid())
				{
					// Restore selection.
					SelectLogMessageByLogIndex(SelectedLogMessage->GetIndex());
				}
				bIsDirty = false;
				DirtyStopwatch.Reset();
				UpdateStatsText();

				FilteringStopwatch.Stop();
#if !WITH_EDITOR
				uint64 DurationMs = FilteringStopwatch.GetAccumulatedTimeMs();
				if (DurationMs > 10) // avoids spams
				{
					UE_LOGF(TraceInsights, Log, "[LogView] Updated (no filter; %d added / %d total messages) in %llu ms.",
						NumAddedMessages, LogProviderNumMessages, DurationMs);
				}
#endif // !WITH_EDITOR
			}
		}
		else // if (LogProviderNumMessages < TotalNumMessages)
		{
			// Just reset. On next Tick() the list will grow if needed.
#if !WITH_EDITOR
			UE_LOGF(TraceInsights, Log, "[LogView] RESET");
#endif // !WITH_EDITOR
			Cache.Reset();
			FilteredMessages.Reset();
			TotalNumMessages = 0;
			ListView->RebuildList();
			bIsDirty = (LogProviderNumMessages != 0);
			if (bIsDirty)
			{
				DirtyStopwatch.Start();
			}
			else
			{
				DirtyStopwatch.Reset();
			}
			UpdateStatsText();
		}
	}
	else if (bIsDirty && !FilteringAsyncTask.IsValid())
	{
		bIsDirty = false;
		DirtyStopwatch.Reset();
		UpdateStatsText();
	}

	if (FilteringAsyncTask.IsValid() &&
		FilteringAsyncTask->IsDone())
	{
		// A filtering async task has completed. Check if filter used is still valid.
		if (!bIsFilteringAsyncTaskCancelRequested &&
			FilteringStartIndex == TotalNumMessages &&
			FilteringEndIndex <= LogProviderNumMessages &&
			FilteringChangeNumber == Filter.GetChangeNumber())
		{
			FLogFilteringAsyncTask& Task = FilteringAsyncTask->GetTask();
			const TArray<uint32>& FilteredMessageIndices = Task.GetFilteredMessages();

			// Add filtered messages to current FilteredMessages array.
			const int32 NumFilteredMessages = FilteredMessageIndices.Num();
			for (int32 Index = 0; Index < NumFilteredMessages; Index++)
			{
				FilteredMessages.Add(MakeShared<FLogMessage>(FilteredMessageIndices[Index]));
			}

			TotalNumMessages = Task.GetEndIndex();
			TSharedPtr<FLogMessage> SelectedLogMessage = GetSelectedLogMessage();
			ListView->RebuildList();
			if (SelectedLogMessage.IsValid())
			{
				// Restore selection.
				SelectLogMessageByLogIndex(SelectedLogMessage->GetIndex());
			}
			bIsDirty = false;
			DirtyStopwatch.Reset();
			UpdateStatsText();

			FilteringStopwatch.Stop();
#if !WITH_EDITOR
			uint64 DurationMs = FilteringStopwatch.GetAccumulatedTimeMs();
			if (DurationMs > 10) // avoids spams
			{
				int32 NumAsyncFilteredMessages = Task.GetEndIndex() - Task.GetStartIndex();
				double Speed = static_cast<double>(NumAsyncFilteredMessages) / FilteringStopwatch.GetAccumulatedTime();
				UE_LOGF(TraceInsights, Log, "[LogView] Updated (%d added / %d async filtered / %d total messages) in %llu ms (%.2f messages/second).",
					NumFilteredMessages, NumAsyncFilteredMessages, TotalNumMessages, DurationMs, Speed);
			}
#endif // !WITH_EDITOR
		}

		FilteringAsyncTask.Reset();
	}

	//SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FLogMessage> SLogView::GetSelectedLogMessage() const
{
	TArray<TSharedPtr<FLogMessage>> SelectedItems = ListView->GetSelectedItems();
	return (SelectedItems.Num() == 1) ? SelectedItems[0] : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::SelectLogMessage(TSharedPtr<FLogMessage> LogMessage)
{
	ListView->SetSelection(LogMessage, ESelectInfo::Direct);
	if (LogMessage.IsValid())
	{
		ListView->RequestScrollIntoView(LogMessage);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::SelectLogMessageByLogIndex(int32 LogIndex)
{
	if (FilteredMessages.Num() == 0)
	{
		return;
	}

	// We are assuming the FilteredMessages list is sorted by log index.
	// Find the exact match of the log index in the filtered messages list.
	int32 MessageIndex = Algo::BinarySearchBy(FilteredMessages, LogIndex, &FLogMessage::GetIndex);
	if (MessageIndex == INDEX_NONE)
	{
		return;
	}

	SelectLogMessage(FilteredMessages[MessageIndex]);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::SelectLogMessageByClosestTime(double Time)
{
	if (FilteredMessages.Num() == 0)
	{
		return;
	}

	// Find the message with the closest time, in the unfiltered list of messages.
	int32 LogIndex = 0;
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());
		LogIndex = (int32)LogProvider.BinarySearchClosestByTime(Time);
	}
	else
	{
		return;
	}

	// We are assuming the FilteredMessages list is sorted by log index.
	// Find first filtered message with log index >= index.
	int32 MessageIndex = Algo::LowerBoundBy(FilteredMessages, LogIndex, &FLogMessage::GetIndex);
	if (MessageIndex >= FilteredMessages.Num())
	{
		MessageIndex = FilteredMessages.Num() - 1;
	}

	SelectLogMessage(FilteredMessages[MessageIndex]);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::OnSelectedLogMessageChanged(TSharedPtr<FLogMessage> LogMessage)
{
	if (LogMessage.IsValid())
	{
		TSharedPtr<UE::Insights::TimingProfiler::STimingView> TimingView = GetTimingView();
		if (TimingView)
		{
			const double Time = Cache.Get(LogMessage->GetIndex()).GetTime();

			if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
			{
				TimingView->SelectToTimeMarker(Time);
			}
			else
			{
				TimingView->SetAutoScroll(false);
				TimingView->SetAndCenterOnTimeMarker(Time);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::OnMouseButtonClick(TSharedPtr<FLogMessage> LogMessage)
{
	OnSelectedLogMessageChanged(LogMessage);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::OnSelectionChanged(TSharedPtr<FLogMessage> LogMessage, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct &&
		SelectInfo != ESelectInfo::OnMouseClick)
	{
		OnSelectedLogMessageChanged(LogMessage);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::FilterTextBox_OnTextChanged(const FText& InFilterText)
{
	if (Filter.GetFilterText().ToString().Equals(InFilterText.ToString(), ESearchCase::CaseSensitive))
	{
		// nothing to do
		return;
	}

	// Set filter phrases.
	Filter.SetFilterText(InFilterText);

	// Report possible syntax errors back to the user.
	FilterTextBox->SetError(Filter.GetSyntaxErrors());

	OnFilterChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::OnFilterChanged()
{
	const FString FilterText = FilterTextBox->GetText().ToString();
#if !WITH_EDITOR
	UE_LOGF(TraceInsights, Log, "[LogView] OnFilterChanged: \"%ls\"", *FilterText);
#endif // !WITH_EDITOR
	Cache.Reset();
	FilteredMessages.Reset();
	TotalNumMessages = 0;
	bIsDirty = true;
	DirtyStopwatch.Start();
	// The next Tick() will update the filtered list of messages.
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::UpdateStatsText()
{
	if (FilteredMessages.Num() == TotalNumMessages)
	{
		StatsText = FText::Format(LOCTEXT("StatsText1", "{0} logs"), FText::AsNumber(TotalNumMessages));
	}
	else
	{
		StatsText = FText::Format(LOCTEXT("StatsText2", "{0} / {1} logs"), FText::AsNumber(FilteredMessages.Num()), FText::AsNumber(TotalNumMessages));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SLogView::GetStatsText() const
{
	if (bIsDirty)
	{
		DirtyStopwatch.Update();
		double DT = DirtyStopwatch.GetAccumulatedTime();
		if (DT > 1.0)
		{
			return FText::Format(LOCTEXT("StatsTextEx", "{0} (filtering... please wait... {1}s)"), StatsText, FText::AsNumber(FMath::RoundToInt(DT)));
		}
	}

	return StatsText;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SLogView::GetStatsTextColor() const
{
	if (bIsDirty)
	{
		return FSlateColor(FLinearColor(1.0f, 0.5f, 0.5f, 1.0f));
	}
	else
	{
		return FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> SLogView::ListView_GetContextMenu()
{
	TSharedPtr<FLogMessage> SelectedLogMessage = GetSelectedLogMessage();

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection("LogViewContextMenu");
	{
		if (SelectedLogMessage.IsValid())
		{
			FLogMessageRecord& Record = Cache.Get(SelectedLogMessage->GetIndex());
			FName CategoryName(Record.GetCategory());

			MenuBuilder.AddMenuEntry
			(
				FLogViewCommands::Get().Command_HideSelectedCategory,
				NAME_None,
				FText::Format(LOCTEXT("HideCategory", "Hide \"{0}\" Category"), Record.GetCategoryAsText()),
				FText::Format(LOCTEXT("HideCategory_Tooltip", "Hide the \"{0}\" log category."), Record.GetCategoryAsText()),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Hidden")
			);

			MenuBuilder.AddMenuEntry
			(
				FLogViewCommands::Get().Command_ShowOnlySelectedCategory,
				NAME_None,
				FText::Format(LOCTEXT("ShowOnlyCategory", "Show Only \"{0}\" Category"), Record.GetCategoryAsText()),
				FText::Format(LOCTEXT("ShowOnlyCategory_Tooltip", "Show only the \"{0}\" log category (hide all other log categories)."), Record.GetCategoryAsText()),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Visible")
			);
		}

		MenuBuilder.AddMenuEntry
		(
			FLogViewCommands::Get().Command_ShowAllCategories,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Visible")
		);

		MenuBuilder.AddMenuEntry
		(
			FLogViewCommands::Get().Command_ToggleTimeFormat,
			NAME_None,
			bUseRelativeTime ?
			LOCTEXT("SwitchToRealTime", "Switch to Real Time") :
			LOCTEXT("SwitchToSessionTime", "Switch To Session Time"),
			bUseRelativeTime ?
			LOCTEXT("ToggleTimeFormatRealTime_Tooltip", "Changes the Time column to reflect real (UTC) timestamps.") :
			LOCTEXT("ToggleTimeFormatSessionTime_Tooltip", "Changes the Time column to reflect session relative timestamps."),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TimersView")
		);

		MenuBuilder.AddSeparator();

		MenuBuilder.AddMenuEntry
		(
			FLogViewCommands::Get().Command_CopySelected,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);

		MenuBuilder.AddMenuEntry
		(
			FLogViewCommands::Get().Command_CopyMessage,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);

		MenuBuilder.AddMenuEntry
		(
			FLogViewCommands::Get().Command_CopyRange,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);

		MenuBuilder.AddMenuEntry
		(
			FLogViewCommands::Get().Command_CopyAll,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);

		MenuBuilder.AddSeparator();

		MenuBuilder.AddMenuEntry
		(
			FLogViewCommands::Get().Command_SaveRange,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);

		MenuBuilder.AddMenuEntry
		(
			FLogViewCommands::Get().Command_SaveAll,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save")
		);

		MenuBuilder.AddSeparator();

		// Open Source in IDE
		{
			FString File;
			uint32 Line = 0;
			if (SelectedLogMessage.IsValid())
			{
				FLogMessageRecord& Record = Cache.Get(SelectedLogMessage->GetIndex());
				File = Record.GetFile();
				Line = Record.GetLine();
			}

			FString UsablePath;
			const bool bPathExists = FInsightsManager::Get()->GetSourceFilePathHelper().GetUsableFilePath(File, UsablePath);

			ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
			ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();
			if (SourceCodeAccessor.CanAccessSourceCode())
			{
				const FText ItemLabel = FText::Format(LOCTEXT("ContextMenu_OpenSource", "Open Source in {0}"),
					SourceCodeAccessor.GetNameText());

				FText ItemToolTip;
				if (!UsablePath.IsEmpty())
				{
					ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSource_Desc1", "Opens the source file of the selected message in {0}.\n{1} ({2})"),
						SourceCodeAccessor.GetNameText(),
						FText::FromString(UsablePath),
						FText::AsNumber(Line, &FNumberFormattingOptions::DefaultNoGrouping()));
				}
				else
				{
					ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSource_Desc2", "Opens the source file of the selected message in {0}."),
						SourceCodeAccessor.GetNameText());
				}

				MenuBuilder.AddMenuEntry
				(
					FLogViewCommands::Get().Command_OpenSource,
					NAME_None,
					ItemLabel,
					ItemToolTip,
					FSlateIcon(SourceCodeAccessor.GetStyleSet(), SourceCodeAccessor.GetOpenIconName())
				);
			}
			else
			{
				const FText ItemLabel = LOCTEXT("ContextMenu_OpenSource_NoAccessor", "Open Source");

				FText ItemToolTip;
				if (!UsablePath.IsEmpty())
				{
					ItemToolTip = FText::Format(LOCTEXT("ContextMenu_OpenSource_NoAccessor_Desc1", "{0} ({1})\nSource Code Accessor is not available."),
						FText::FromString(UsablePath),
						FText::AsNumber(Line, &FNumberFormattingOptions::DefaultNoGrouping()));
				}
				else
				{
					ItemToolTip = LOCTEXT("ContextMenu_OpenSource_NoAccessor_Desc2", "Source Code Accessor is not available.");
				}

				MenuBuilder.AddMenuEntry
				(
					ItemLabel,
					ItemToolTip,
					FSlateIcon(SourceCodeAccessor.GetStyleSet(), SourceCodeAccessor.GetOpenIconName()),
					FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([]() { return false; })),
					NAME_None,
					EUserInterfaceActionType::None
				);
			}
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SLogView::MakeVerbosityThresholdMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("VerbosityThreshold"/*, LOCTEXT("LogVerbosityThresholdMenu_Section_VerbosityThreshold", "Verbosity Threshold")*/);
	CreateVerbosityThresholdMenuSection(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::CreateVerbosityThresholdMenuSection(FMenuBuilder& MenuBuilder)
{
	struct FVerbosityThresholdInfo
	{
		ELogVerbosity::Type Verbosity;
		FText Label;
		FText ToolTip;
	};

	FVerbosityThresholdInfo VerbosityThresholds[] =
	{
		{ ELogVerbosity::VeryVerbose, LOCTEXT("VerbosityThresholdVeryVerbose", "VeryVerbose"), LOCTEXT("VerbosityThresholdVeryVerbose_Tooltip", "Show all log messages (any verbosity level)."), },
		{ ELogVerbosity::Verbose,     LOCTEXT("VerbosityThresholdVerbose",     "Verbose"),     LOCTEXT("VerbosityThresholdVerbose_Tooltip",     "Show Verbose, Log, Display, Warning, Error and Fatal log messages (i.e. hide VeryVerbose log messages)."), },
		{ ELogVerbosity::Log,         LOCTEXT("VerbosityThresholdLog",         "Log"),         LOCTEXT("VerbosityThresholdLog_Tooltip",         "Show Log, Display, Warning, Error and Fatal log messages (i.e. hide Verbose and VeryVerbose log messages)."), },
		{ ELogVerbosity::Display,     LOCTEXT("VerbosityThresholdDisplay",     "Display"),     LOCTEXT("VerbosityThresholdDisplay_Tooltip",     "Show Display, Warning, Error and Fatal log messages (i.e. hide Log, Verbose and VeryVerbose log messages)."), },
		{ ELogVerbosity::Warning,     LOCTEXT("VerbosityThresholdWarning",     "Warning"),     LOCTEXT("VerbosityThresholdWarning_Tooltip",     "Show only Warning, Error and Fatal log messages."), },
		{ ELogVerbosity::Error,       LOCTEXT("VerbosityThresholdError",       "Error"),       LOCTEXT("VerbosityThresholdError_Tooltip",       "Show only Error and Fatal log messages."), },
		{ ELogVerbosity::Fatal,       LOCTEXT("VerbosityThresholdFatal",       "Fatal"),       LOCTEXT("VerbosityThresholdFatal_Tooltip",       "Show only Fatal log messages."), },
	};

	for (int32 Index = 0; Index < sizeof(VerbosityThresholds) / sizeof(FVerbosityThresholdInfo); ++Index)
	{
		const FVerbosityThresholdInfo& Threshold = VerbosityThresholds[Index];

		const TSharedRef<SWidget> TextBlock = SNew(SHorizontalBox)
#if 0 // shadow
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(Threshold.Label)
				.ColorAndOpacity(FSlateColor(UE::Insights::TimingProfiler::FTimeMarkerTrackBuilder::GetColorByVerbosity(Threshold.Verbosity)))
				.ShadowColorAndOpacity(FLinearColor(0.02f, 0.02f, 0.02f, 1.0f))
				.ShadowOffset(FVector2D(1.0f, 1.0f))
			]
#else // rounded background
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.0f, -2.0f, 0.0f, -2.0f))
			[
				SNew(SBorder)
				.BorderImage(FInsightsStyle::Get().GetBrush("RoundedBackground"))
				.BorderBackgroundColor(FLinearColor(0.03f, 0.03f, 0.03f, 1.0f))
				.Padding(FMargin(12.0f, 2.0f, 12.0f, 2.0f))
				[
					SNew(STextBlock)
					.Text(Threshold.Label)
					.ColorAndOpacity(FSlateColor(UE::Insights::TimingProfiler::FTimeMarkerTrackBuilder::GetColorByVerbosity(Threshold.Verbosity)))
					.ShadowColorAndOpacity(FLinearColor(0.02f, 0.02f, 0.02f, 1.0f))
					.ShadowOffset(FVector2D(1.0f, 1.0f))
				]
			]
#endif
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.WidthOverride(12.0f)
				.HeightOverride(12.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(this, &SLogView::VerbosityThreshold_GetSuffixColor, Threshold.Verbosity)
					.Image(this, &SLogView::VerbosityThreshold_GetSuffixGlyph, Threshold.Verbosity)
				]
			];

		MenuBuilder.AddMenuEntry
		(
			FUIAction(
				FExecuteAction::CreateSP(this, &SLogView::VerbosityThreshold_Execute, Threshold.Verbosity),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SLogView::VerbosityThreshold_IsChecked, Threshold.Verbosity)),
			TextBlock,
			NAME_None,
			Threshold.ToolTip,
			EUserInterfaceActionType::RadioButton
		);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SLogView::MakeCategoryFilterMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("QuickFilter", LOCTEXT("CategoryFilterMenu_Section_QuickFilter", "Quick Filter"));
	{
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ShowAllCategories", "Show/Hide All"),
			LOCTEXT("ShowAllCategories_Tooltip", "Change filtering to show/hide all categories"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SLogView::ShowHideAllCategories_Execute),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SLogView::ShowHideAllCategories_IsChecked)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Categories", LOCTEXT("CategoryFilterMenu_Section_Categories", "Categories"));
	CreateCategoriesFilterMenuSection(MenuBuilder);
	MenuBuilder.EndSection();

	const float MaxMenuHeight = FMath::Clamp(static_cast<float>(this->GetCachedGeometry().GetLocalSize().Y) - 40.0f, 100.0f, 500.0f);
	return MenuBuilder.MakeWidget(nullptr, FMath::RoundToInt(MaxMenuHeight));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::CreateCategoriesFilterMenuSection(FMenuBuilder& MenuBuilder)
{
	for (const FName& CategoryName : Filter.GetAvailableLogCategories())
	{
		const FString CategoryString = CategoryName.ToString();
		const FText CategoryText(FText::AsCultureInvariant(CategoryString));

		const TSharedRef<SWidget> TextBlock = SNew(STextBlock)
			.Text(FText::AsCultureInvariant(CategoryString))
			.ShadowColorAndOpacity(FLinearColor(0.05f, 0.05f, 0.05f, 1.0f))
			.ShadowOffset(FVector2D(1.0f, 1.0f))
			.ColorAndOpacity(FSlateColor(UE::Insights::TimingProfiler::FTimeMarkerTrackBuilder::GetColorByCategory(*CategoryString)));

		MenuBuilder.AddMenuEntry
		(
			FUIAction(
				FExecuteAction::CreateSP(this, &SLogView::ToggleCategory, CategoryName),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SLogView::IsLogCategoryEnabled, CategoryName)),
			TextBlock,
			NAME_None,
			FText::Format(LOCTEXT("Category_Tooltip", "Filter the Log View to show/hide category: {0}"), CategoryText),
			EUserInterfaceActionType::ToggleButton
		);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SLogView::VerbosityThreshold_IsChecked(ELogVerbosity::Type Verbosity) const
{
	return Verbosity == Filter.GetVerbosityThreshold();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::VerbosityThreshold_Execute(ELogVerbosity::Type Verbosity)
{
	Filter.SetVerbosityThreshold(Verbosity);
	OnFilterChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* SLogView::VerbosityThreshold_GetSuffixGlyph(ELogVerbosity::Type Verbosity) const
{
	return Verbosity <= Filter.GetVerbosityThreshold() ?
		FAppStyle::Get().GetBrush("Icons.Check") :
		FAppStyle::Get().GetBrush("Icons.X");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SLogView::VerbosityThreshold_GetSuffixColor(ELogVerbosity::Type Verbosity) const
{
	return Verbosity <= Filter.GetVerbosityThreshold() ?
		FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f)) :
		FSlateColor(FLinearColor(1.0f, 0.0f, 0.0f, 1.0f));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SLogView::ShowHideAllCategories_IsChecked() const
{
	return Filter.IsShowAllCategoriesEnabled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::ShowHideAllCategories_Execute()
{
	Filter.ToggleShowAllCategories();
	OnFilterChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SLogView::IsLogCategoryEnabled(FName InName) const
{
	return Filter.IsLogCategoryEnabled(InName);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::ToggleCategory(FName InName)
{
	Filter.ToggleLogCategory(InName);
	OnFilterChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SLogView::CanHideSelectedCategory() const
{
	return ListView->GetSelectedItems().Num() == 1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::HideSelectedCategory()
{
	TSharedPtr<FLogMessage> SelectedLogMessage = GetSelectedLogMessage();
	if (SelectedLogMessage.IsValid())
	{
		FLogMessageRecord& Record = Cache.Get(SelectedLogMessage->GetIndex());
		FName CategoryName(Record.GetCategory());
		Filter.DisableLogCategory(CategoryName);
		OnFilterChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SLogView::CanShowOnlySelectedCategory() const
{
	return ListView->GetSelectedItems().Num() == 1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::ShowOnlySelectedCategory()
{
	TSharedPtr<FLogMessage> SelectedLogMessage = GetSelectedLogMessage();
	if (SelectedLogMessage.IsValid())
	{
		FLogMessageRecord& Record = Cache.Get(SelectedLogMessage->GetIndex());
		FName CategoryName(Record.GetCategory());
		Filter.EnableOnlyCategory(CategoryName);
		OnFilterChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SLogView::CanShowAllCategories() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::ShowAllCategories()
{
	if (Filter.IsShowAllCategoriesEnabled())
	{
		Filter.ToggleShowAllCategories(); // hide
		Filter.ToggleShowAllCategories(); // show
	}
	else
	{
		Filter.ToggleShowAllCategories(); // show
	}
	OnFilterChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SLogView::CanToggleTimeFormat() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::ToggleTimeFormat()
{
	bUseRelativeTime = !bUseRelativeTime;
	TSharedPtr<SHeaderRow> Row = ListView->GetHeaderRow();

	Row->RemoveColumn(Row->GetColumns()[1].ColumnId);
	Row->InsertColumn(
		SHeaderRow::Column(bUseRelativeTime ? LogViewColumns::SessionTimeColumnName : LogViewColumns::RealTimeColumnName)
		.ManualWidth(bUseRelativeTime ? 100.0f : 155.0f)
		.DefaultLabel(GetTimeFormatColumnName())
		, 1);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SLogView::GetTimeFormatColumnName() const
{
	return bUseRelativeTime ? LOCTEXT("SessionTimeColumn", "Session Time") : LOCTEXT("RealTimeColumn", "Real Time");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SLogView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return CommandList->ProcessCommandBindings(InKeyEvent) == true ? FReply::Handled() : FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::AppendFormatMessageDetailed(const FLogMessageRecord& Log, TStringBuilderBase<TCHAR>& InOutStringBuilder) const
{
	using namespace UE::Insights;
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	InOutStringBuilder.Appendf(TEXT("Index=%d"), Log.GetIndex());
	InOutStringBuilder.Append(TEXT("\nTime="));
	InOutStringBuilder.Append(bUseRelativeTime ? FormatTimeHMS(Log.GetTime(), FTimeValue::Microsecond) : Log.GetAbsoluteDateTime(Session->GetBaseDateTime()).ToString());
	InOutStringBuilder.Append(TEXT("\nVerbosity="));
	InOutStringBuilder.Append(::ToString(Log.GetVerbosity()));
	InOutStringBuilder.Append(TEXT("\nCategory="));
	InOutStringBuilder.Append(Log.GetCategoryAsString());
	InOutStringBuilder.Append(TEXT("\nMessage="));
	InOutStringBuilder.Append(Log.GetMessageAsString());
	InOutStringBuilder.Append(TEXT("\nFile="));
	InOutStringBuilder.Append(Log.GetFile());
	InOutStringBuilder.Appendf(TEXT("\nLine=%d\n"), Log.GetLine());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::AppendFormatMessageDelimitedHeader(TStringBuilderBase<TCHAR>& InOutStringBuilder, TCHAR Separator) const
{
	InOutStringBuilder.Append(TEXT("Index"));
	InOutStringBuilder.AppendChar(Separator);
	InOutStringBuilder.Append(TEXT("Time"));
	InOutStringBuilder.AppendChar(Separator);
	InOutStringBuilder.Append(TEXT("Verbosity"));
	InOutStringBuilder.AppendChar(Separator);
	InOutStringBuilder.Append(TEXT("Category"));
	InOutStringBuilder.AppendChar(Separator);
	InOutStringBuilder.Append(TEXT("Message"));
	InOutStringBuilder.AppendChar(Separator);
	InOutStringBuilder.Append(TEXT("File"));
	InOutStringBuilder.AppendChar(Separator);
	InOutStringBuilder.Append(TEXT("Line"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::AppendFormatMessageDelimited(const FLogMessageRecord& Log, TStringBuilderBase<TCHAR>& InOutStringBuilder, TCHAR Separator) const
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	InOutStringBuilder.Appendf(TEXT("%d"), Log.GetIndex());
	InOutStringBuilder.AppendChar(Separator);
	InOutStringBuilder.Append(bUseRelativeTime ? FormatTimeHMS(Log.GetTime(), FTimeValue::Microsecond) : Log.GetAbsoluteDateTime(Session->GetBaseDateTime()).ToString());
	InOutStringBuilder.AppendChar(Separator);
	InOutStringBuilder.Append(::ToString(Log.GetVerbosity()));
	InOutStringBuilder.AppendChar(Separator);
	InOutStringBuilder.Append(Log.GetCategoryAsString());
	InOutStringBuilder.AppendChar(Separator);
	FString Message = Log.GetMessageAsString();
	if (Separator == TEXT('\t'))
	{
		Message.ReplaceCharInline(TEXT('\t'), TEXT(' '), ESearchCase::CaseSensitive);
		InOutStringBuilder.Append(Message);
	}
	else if (Separator == TEXT(','))
	{
		InOutStringBuilder.AppendChar(TEXT('\"'));
		FString EscapedMessage = Message.Replace(TEXT("\""), TEXT("\"\""), ESearchCase::CaseSensitive);
		InOutStringBuilder.Append(EscapedMessage);
		InOutStringBuilder.AppendChar(TEXT('\"'));
	}
	else
	{
		InOutStringBuilder.Append(Message);
	}
	InOutStringBuilder.AppendChar(Separator);
	InOutStringBuilder.Append(Log.GetFile());
	InOutStringBuilder.AppendChar(Separator);
	InOutStringBuilder.Appendf(TEXT("%d"), Log.GetLine());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SLogView::CanCopySelected() const
{
	return ListView->GetSelectedItems().Num() == 1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::CopySelected() const
{
	TSharedPtr<FLogMessage> SelectedLogMessage = GetSelectedLogMessage();
	if (SelectedLogMessage)
	{
		const FLogMessageRecord& Log = Cache.Get(SelectedLogMessage->GetIndex());
		TStringBuilder<1024> StringBuilder;
		AppendFormatMessageDetailed(Log, StringBuilder);
		FPlatformApplicationMisc::ClipboardCopy(StringBuilder.ToString());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SLogView::CanCopyMessage() const
{
	return ListView->GetSelectedItems().Num() == 1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::CopyMessage() const
{
	TSharedPtr<FLogMessage> SelectedLogMessage = GetSelectedLogMessage();
	if (SelectedLogMessage)
	{
		const FLogMessageRecord& Log = Cache.Get(SelectedLogMessage->GetIndex());
		FPlatformApplicationMisc::ClipboardCopy(*Log.GetMessageAsString());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SLogView::CanCopyRange() const
{
	if (FilteredMessages.Num() == 0)
	{
		return false;
	}

	TSharedPtr<UE::Insights::TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView)
	{
		return false;
	}

	const double SelectionStartTime = TimingView->GetSelectionStartTime();
	const double SelectionEndTime = TimingView->GetSelectionEndTime();
	return SelectionStartTime < SelectionEndTime;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::CopyRange() const
{
	if (FilteredMessages.Num() == 0)
	{
		return;
	}

	TSharedPtr<UE::Insights::TimingProfiler::STimingView> TimingView = GetTimingView();
	if (!TimingView)
	{
		return;
	}

	const double SelectionStartTime = TimingView->GetSelectionStartTime();
	const double SelectionEndTime = TimingView->GetSelectionEndTime();
	if (SelectionStartTime >= SelectionEndTime)
	{
		return;
	}

	TStringBuilder<1024> StringBuilder;
	AppendFormatMessageDelimitedHeader(StringBuilder, TEXT('\t'));
	StringBuilder.AppendChar(TEXT('\n'));

	int32 NumMessagesInSelectedRange = 0;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());

		for (const TSharedPtr<FLogMessage>& Message : FilteredMessages)
		{
			LogProvider.ReadMessage(Message->GetIndex(),
				[this, SelectionStartTime, SelectionEndTime, &StringBuilder, &NumMessagesInSelectedRange]
				(const TraceServices::FLogMessageInfo& MessageInfo)
				{
					if (MessageInfo.Time >= SelectionStartTime && MessageInfo.Time <= SelectionEndTime)
					{
						FLogMessageRecord Log(MessageInfo);
						AppendFormatMessageDelimited(Log, StringBuilder, TEXT('\t'));
						StringBuilder.AppendChar(TEXT('\n'));
						++NumMessagesInSelectedRange;
					}
				});
		}
	}

	if (NumMessagesInSelectedRange > 0)
	{
		FPlatformApplicationMisc::ClipboardCopy(StringBuilder.ToString());
		UE_LOGF(TraceInsights, Log, "Copied %d logs to clipboard.", NumMessagesInSelectedRange);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SLogView::CanCopyAll() const
{
	return FilteredMessages.Num() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::CopyAll() const
{
	if (FilteredMessages.Num() == 0)
	{
		return;
	}

	TStringBuilder<1024> StringBuilder;
	AppendFormatMessageDelimitedHeader(StringBuilder, TEXT('\t'));
	StringBuilder.AppendChar(TEXT('\n'));

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());

		for (const TSharedPtr<FLogMessage>& Message : FilteredMessages)
		{
			LogProvider.ReadMessage(Message->GetIndex(),
				[this, &StringBuilder]
				(const TraceServices::FLogMessageInfo& MessageInfo)
				{
					FLogMessageRecord Log(MessageInfo);
					AppendFormatMessageDelimited(Log, StringBuilder, TEXT('\t'));
					StringBuilder.AppendChar(TEXT('\n'));
				});
		}
	}

	FPlatformApplicationMisc::ClipboardCopy(StringBuilder.ToString());
	UE_LOGF(TraceInsights, Log, "Copied %d logs to clipboard.", FilteredMessages.Num());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SLogView::CanSaveRange() const
{
	return CanCopyRange();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::SaveRange() const
{
	constexpr bool bSaveLogsInSelectedRangeOnly = true;
	SaveLogsToFile(bSaveLogsInSelectedRangeOnly);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SLogView::CanSaveAll() const
{
	return FilteredMessages.Num() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::SaveAll() const
{
	constexpr bool bSaveLogsInSelectedRangeOnly = false;
	SaveLogsToFile(bSaveLogsInSelectedRangeOnly);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::SaveLogsToFile(bool bSaveLogsInSelectedRangeOnly) const
{
	if (FilteredMessages.Num() == 0)
	{
		return;
	}

	double SelectionStartTime = 0.0;
	double SelectionEndTime = 0.0;
	if (bSaveLogsInSelectedRangeOnly)
	{
		TSharedPtr<UE::Insights::TimingProfiler::STimingView> TimingView = GetTimingView();
		if (!TimingView)
		{
			return;
		}

		SelectionStartTime = TimingView->GetSelectionStartTime();
		SelectionEndTime = TimingView->GetSelectionEndTime();
		if (SelectionStartTime >= SelectionEndTime)
		{
			return;
		}
	}

	TArray<FString> SaveFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bDialogResult = false;
	if (DesktopPlatform)
	{
		const FString DefaultBrowsePath = FPaths::ProjectLogDir();
		bDialogResult = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("SaveLogsToFileTitle", "Save Logs").ToString(),
			DefaultBrowsePath,
			TEXT(""),
			TEXT("Comma-Separated Values (*.csv)|*.csv|Tab-Separated Values (*.tsv)|*.tsv|Text Files (*.txt)|*.txt|All Files (*.*)|*.*"),
			EFileDialogFlags::None,
			SaveFilenames
		);
	}

	if (!bDialogResult || SaveFilenames.Num() == 0)
	{
		return;
	}

	FString& Filename = SaveFilenames[0];

	TUniquePtr<IFileHandle> ExportFileHandle{ FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*Filename) };
	if (!ExportFileHandle)
	{
		FMessageLog ReportMessageLog(FInsightsManager::Get()->GetLogListingName());
		ReportMessageLog.Error(LOCTEXT("FailedToOpenFile", "Save logs failed. Failed to open file for write."));
		ReportMessageLog.Notify();
		return;
	}

	UE::Insights::FStopwatch Stopwatch;
	Stopwatch.Start();

	const bool bIsCSV = Filename.EndsWith(TEXT(".csv"));
	const TCHAR Separator = bIsCSV ? TEXT(',') : TEXT('\t');
	FUtf8FileWriter Writer(ExportFileHandle.Get(), bIsCSV);

	// Write header.
	{
		TStringBuilder<1024> Line;
		AppendFormatMessageDelimitedHeader(Line, Separator);
		Writer.GetStringBuilder().Append(Line.ToView());
		Writer.AppendLineEnd();
	}

	int32 NumMessagesInSelectedRange = 0;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());

		for (const TSharedPtr<FLogMessage>& Message : FilteredMessages)
		{
			LogProvider.ReadMessage(Message->GetIndex(),
				[this, bSaveLogsInSelectedRangeOnly, SelectionStartTime, SelectionEndTime, Separator, &Writer, &NumMessagesInSelectedRange]
				(const TraceServices::FLogMessageInfo& MessageInfo)
				{
					if (!bSaveLogsInSelectedRangeOnly || (MessageInfo.Time >= SelectionStartTime && MessageInfo.Time <= SelectionEndTime))
					{
						FLogMessageRecord Log(MessageInfo);
						TStringBuilder<1024> Line;
						AppendFormatMessageDelimited(Log, Line, Separator);
						Writer.GetStringBuilder().Append(Line.ToView());
						Writer.AppendLineEnd();
						++NumMessagesInSelectedRange;
					}
				});
		}
	}

	Writer.Flush();
	ExportFileHandle->Flush();
	ExportFileHandle.Reset();

	Stopwatch.Stop();
	const double TotalTime = Stopwatch.GetAccumulatedTime();
	UE_LOGF(TraceInsights, Log, "Saved %d logs to file in %.3fs.", NumMessagesInSelectedRange, TotalTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SLogView::CanOpenSource() const
{
	return GetSelectedLogMessage().IsValid();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SLogView::OpenSource() const
{
	TSharedPtr<FLogMessage> SelectedLogMessage = GetSelectedLogMessage();
	if (SelectedLogMessage.IsValid())
	{
		FLogMessageRecord& Record = Cache.Get(SelectedLogMessage->GetIndex());
		const FString File = Record.GetFile();
		const uint32 Line = Record.GetLine();

		ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
		if (FPaths::FileExists(File))
		{
			ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();
			SourceCodeAccessor.OpenFileAtLine(File, Line);
		}
		else
		{
			SourceCodeAccessModule.OnOpenFileFailed().Broadcast(File);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<UE::Insights::TimingProfiler::STimingView> SLogView::GetTimingView() const
{
	using namespace UE::Insights::TimingProfiler;
	TSharedPtr<STimingProfilerWindow> ProfilerWindow = FTimingProfilerManager::Get()->GetProfilerWindow();
	return ProfilerWindow.IsValid() ? ProfilerWindow->GetTimingView() : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
