// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogVerbosity.h"
#include "Misc/ScopeLock.h"
#include "SlateFwd.h"
#include "Templates/UniquePtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"

// TraceInsights
#include "Insights/ViewModels/LogFilter.h"
#include "Insights/ViewModels/LogMessage.h"

class FMenuBuilder;

namespace UE::Insights::TimingProfiler { class STimingView; }

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace LogViewColumns
{
	static const FName IdColumnName(TEXT("Id"));
	static const FName SessionTimeColumnName(TEXT("SessionTime"));
	static const FName RealTimeColumnName(TEXT("RealTime"));
	static const FName VerbosityColumnName(TEXT("Verbosity"));
	static const FName CategoryColumnName(TEXT("Category"));
	static const FName MessageColumnName(TEXT("Message"));
	static const FName FileColumnName(TEXT("File"));
	static const FName LineColumnName(TEXT("Line"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Trace log window.
 */
class SLogView : public SCompoundWidget
{
public:
	/** Default constructor. */
	SLogView();

	/** Virtual destructor. */
	virtual ~SLogView();

	void Reset();

	SLATE_BEGIN_ARGS(SLogView) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget.
	 * @param InArgs - The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs);

	/**
	 * Ticks this widget. Override in derived classes, but always call the parent implementation.
	 *
	 * @param AllottedGeometry - The space allotted for this widget
	 * @param InCurrentTime - Current absolute real time
	 * @param InDeltaTime - Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	FLogMessageCache& GetCache() { return Cache; }

	TSharedPtr<FLogMessage> GetSelectedLogMessage() const;
	void SelectLogMessage(TSharedPtr<FLogMessage> LogMessage);
	void SelectLogMessageByLogIndex(int32 LogIndex);
	void SelectLogMessageByClosestTime(double Time);

	FText GetFilterText() const { return FilterTextBox->GetText(); }

	bool IsFilteringAsyncTaskCancelRequested() const { return bIsFilteringAsyncTaskCancelRequested; }

	TSharedPtr<TimingProfiler::STimingView> GetTimingView() const;

	bool IsUsingRelativeTime() const { return bUseRelativeTime; }

protected:
	/** Generate a new list view row. */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FLogMessage> InLogMessage, const TSharedRef<STableViewBase>& OwnerTable);

	void InitCommandList();

	void OnMouseButtonClick(TSharedPtr<FLogMessage> LogMessage);
	void OnSelectionChanged(TSharedPtr<FLogMessage> LogMessage, ESelectInfo::Type SelectInfo);
	void OnSelectedLogMessageChanged(TSharedPtr<FLogMessage> LogMessage);

	void FilterTextBox_OnTextChanged(const FText& InFilterText);
	void OnFilterChanged();

	void UpdateStatsText();
	FText GetStatsText() const;
	FSlateColor GetStatsTextColor() const;

	TSharedPtr<SWidget> ListView_GetContextMenu();
	TSharedRef<SWidget> MakeVerbosityThresholdMenu();
	void CreateVerbosityThresholdMenuSection(FMenuBuilder& MenuBuilder);
	TSharedRef<SWidget> MakeCategoryFilterMenu();
	void CreateCategoriesFilterMenuSection(FMenuBuilder& MenuBuilder);

	bool VerbosityThreshold_IsChecked(ELogVerbosity::Type Verbosity) const;
	void VerbosityThreshold_Execute(ELogVerbosity::Type Verbosity);
	const FSlateBrush* VerbosityThreshold_GetSuffixGlyph(ELogVerbosity::Type Verbosity) const;
	FSlateColor VerbosityThreshold_GetSuffixColor(ELogVerbosity::Type Verbosity) const;

	bool ShowHideAllCategories_IsChecked() const;
	void ShowHideAllCategories_Execute();

	bool IsLogCategoryEnabled(FName InName) const;
	void ToggleCategory(FName InName);

	bool CanHideSelectedCategory() const;
	void HideSelectedCategory();

	bool CanShowOnlySelectedCategory() const;
	void ShowOnlySelectedCategory();

	bool CanShowAllCategories() const;
	void ShowAllCategories();
	
	bool CanToggleTimeFormat() const;
	void ToggleTimeFormat();
	FText GetTimeFormatColumnName() const;

	void AppendFormatMessageDetailed(const FLogMessageRecord& Log, TStringBuilderBase<TCHAR>& InOutStringBuilder) const;
	void AppendFormatMessageDelimitedHeader(TStringBuilderBase<TCHAR>& InOutStringBuilder, TCHAR Separator = TEXT('\t')) const;
	void AppendFormatMessageDelimited(const FLogMessageRecord& Log, TStringBuilderBase<TCHAR>& InOutStringBuilder, TCHAR Separator = TEXT('\t')) const;

	bool CanCopySelected() const;
	void CopySelected() const;

	bool CanCopyMessage() const;
	void CopyMessage() const;

	bool CanCopyRange() const;
	void CopyRange() const;

	bool CanCopyAll() const;
	void CopyAll() const;

	bool CanSaveRange() const;
	void SaveRange() const;

	bool CanSaveAll() const;
	void SaveAll() const;

	void SaveLogsToFile(bool bSaveLogsInSelectedRangeOnly) const;

	bool CanOpenSource() const;
	void OpenSource() const;

protected:
	TSharedPtr<FUICommandList> CommandList;

	/** The list view widget. */
	TSharedPtr<SListView<TSharedPtr<FLogMessage>>> ListView;

	/** External scrollbar used to synchronize tree view position. */
	TSharedPtr<SScrollBar> ExternalScrollbar;

	/** The search box widget used to filter logs by message text. */
	TSharedPtr<SSearchBox> FilterTextBox;

	FLogFilter& Filter;
	uint64 FilterChangeNumber;

	int32 FilteringStartIndex; // Start index (of the range of log messages to filter) currently used by the async task
	int32 FilteringEndIndex; // End index (of the range of log messages to filter) currently used by the async task
	uint64 FilteringChangeNumber; // Change number of the filter currently used by the async task
	TUniquePtr<FAsyncTask<FLogFilteringAsyncTask>> FilteringAsyncTask; // The async task to filter log messages on a worker thread
	mutable volatile bool bIsFilteringAsyncTaskCancelRequested; // true if we want the async task to finish asap

	/** Stopwatch used to measure how long it takes to filter the message list. */
	mutable FStopwatch FilteringStopwatch;

	/**
	 * Total number of log categories (including duplicates; last value read from LogProvider).
	 * Used to detect when more log categories are added in the LogProvider, so we can update UI.
	 */
	int32 TotalNumCategories;

	/**
	 * Total number of log messages (last value read from LogProvider).
	 * Used to detect when new log messages are added in the LogProvider, so we can update UI.
	 */
	int32 TotalNumMessages;

	/**
	 * Total number of inserts in the source table of log messages (last value read from LogProvider).
	 * Used to detect when the order of log messages changes in source table, so we can invalidate the cache.
	 */
	int32 TotalNumInserts;

	/** true if the list of messages is not yet updated (the filter has changed and/or the source trace messages have changed) */
	bool bIsDirty;

	/** Stopwatch used to measure the time since the list of messages has become dirty. */
	mutable FStopwatch DirtyStopwatch;

	/** Stats */
	FText StatsText;

	/** Cached log messages. */
	mutable FLogMessageCache Cache;

	/** List of trace log messages to show in list view. */
	TArray<TSharedPtr<FLogMessage>> FilteredMessages;

	/** Whether to use Session Relative or Real Time (Absolute) Timestamp for the Log "Time" Column. */
	bool bUseRelativeTime;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
