// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/Widgets/SMajorTabWindow.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#if WITH_EDITOR
	#include "EngineAnalytics.h"
	#include "AnalyticsEventAttribute.h"
	#include "Interfaces/IAnalyticsProvider.h"
#endif // WITH_EDITOR

// TraceInsights
#include "Insights/Common/InsightsMenuBuilder.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/Log.h"
#include "Insights/TraceInsightsModule.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "SMajorTabWindow"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// SMajorTabWindow
////////////////////////////////////////////////////////////////////////////////////////////////////

SMajorTabWindow::SMajorTabWindow(const FName& InMajorTabId)
	: MajorTabId(InMajorTabId)
	, DurationActive(0.0f)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SMajorTabWindow::~SMajorTabWindow()
{
	CloseAllOpenTabs();

#if WITH_EDITOR
	if (DurationActive > 0.0f && FEngineAnalytics::IsAvailable())
	{
		FString EventName = GetAnalyticsEventName();
		FEngineAnalytics::GetProvider().RecordEvent(EventName, FAnalyticsEventAttribute(TEXT("Duration"), DurationActive));
	}
#endif // WITH_EDITOR
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* SMajorTabWindow::GetAnalyticsEventName() const
{
	return TEXT("Insights.Usage.MajorTabWindow");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMajorTabWindow::Reset()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMajorTabWindow::ShowTab(const FName& TabId)
{
	if (TabManager->HasTabSpawner(TabId))
	{
		TabManager->TryInvokeTab(TabId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMajorTabWindow::HideTab(const FName& TabId)
{
	TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(TabId);
	if (Tab.IsValid())
	{
		Tab->RequestCloseTab();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMajorTabWindow::AddOpenTab(const TSharedRef<SDockTab>& DockTab)
{
	UE_LOGF(TraceInsights, Log, "[Tab] SHOW %ls", *DockTab->GetContent()->GetTypeAsString());
	OpenTabs.Add(DockTab);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMajorTabWindow::RemoveOpenTab(const TSharedRef<SDockTab>& DockTab)
{
	UE_LOGF(TraceInsights, Log, "[Tab] HIDE %ls", *DockTab->GetContent()->GetTypeAsString());
	OpenTabs.Remove(DockTab);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMajorTabWindow::CloseAllOpenTabs()
{
	WindowClosedEvent.Broadcast();

	// Close all tabs.
	TArray<TSharedPtr<SDockTab>> LocalOpenTabs;
	for (TSharedPtr<SDockTab>& Tab : OpenTabs)
	{
		LocalOpenTabs.Add(Tab);
	}
	for (TSharedPtr<SDockTab>& Tab : LocalOpenTabs)
	{
		Tab->RequestCloseTab();
	}
	LocalOpenTabs.Reset();
	check(OpenTabs.IsEmpty());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FWorkspaceItem> SMajorTabWindow::CreateWorkspaceMenuGroup()
{
	check(TabManager.IsValid());
	return TabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("MajorTabMenuGroupName", "Insights"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMajorTabWindow::RegisterTabSpawners()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMajorTabWindow::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	using namespace UE::Insights;

	//////////////////////////////////////////////////
	// Create & initialize tab manager.

	TabManager = FGlobalTabmanager::Get()->NewTabManager(ConstructUnderMajorTab);

	const auto& PersistLayout = [](const TSharedRef<FTabManager::FLayout>& LayoutToSave)
	{
		FLayoutSaveRestore::SaveToConfig(FTraceInsightsModule::GetUnrealInsightsLayoutIni(), LayoutToSave);
	};
	TabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateLambda(PersistLayout));

	//////////////////////////////////////////////////

	TSharedRef<FWorkspaceItem> MenuGroup = CreateWorkspaceMenuGroup();
	WorkspaceMenuGroup = MenuGroup;

	Extension = MakeShared<FInsightsMajorTabExtender>(TabManager, MenuGroup);

	RegisterTabSpawners();

	//////////////////////////////////////////////////

	FTraceInsightsModule& TraceInsightsModule = FModuleManager::GetModuleChecked<FTraceInsightsModule>("TraceInsights");

	const FOnRegisterMajorTabExtensions* ExtensionDelegate = TraceInsightsModule.FindMajorTabLayoutExtension(GetMajorTabId());
	if (ExtensionDelegate)
	{
		ExtensionDelegate->Broadcast(*Extension);
	}

	// Register any new minor tabs.
	for (const FMinorTabConfig& MinorTabConfig : Extension->GetMinorTabs())
	{
		FTabSpawnerEntry& TabSpawnerEntry = TabManager->RegisterTabSpawner(MinorTabConfig.TabId, MinorTabConfig.OnSpawnTab, MinorTabConfig.CanSpawnTab);

		TabSpawnerEntry
		.SetDisplayName(MinorTabConfig.TabLabel)
		.SetTooltipText(MinorTabConfig.TabTooltip)
		.SetIcon(MinorTabConfig.TabIcon)
		.SetReuseTabMethod(MinorTabConfig.OnFindTabToReuse);

		if (MinorTabConfig.WorkspaceGroup.IsValid())
		{
			TabSpawnerEntry.SetGroup(MinorTabConfig.WorkspaceGroup.ToSharedRef());
		}
	}

	// Check for layout overrides.
	FInsightsMajorTabConfig TabConfig = TraceInsightsModule.FindMajorTabConfig(GetMajorTabId());

	// Create tab layout.
	TSharedRef<FTabManager::FLayout> Layout = [&TabConfig, this]() -> TSharedRef<FTabManager::FLayout>
	{
		if (TabConfig.Layout.IsValid())
		{
			return TabConfig.Layout.ToSharedRef();
		}
		else
		{
			return CreateDefaultTabLayout();
		}
	}();

	Layout->ProcessExtensions(Extension->GetLayoutExtender());
	Layout = FLayoutSaveRestore::LoadFromConfig(FTraceInsightsModule::GetUnrealInsightsLayoutIni(), Layout);

	// Create & initialize main menu.
	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(TSharedPtr<FUICommandList>(), Extension->GetMenuExtender());
	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("MenuLabel", "Menu"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateStatic(&SMajorTabWindow::FillMenu, TabManager),
		FName(TEXT("Menu"))
	);

#if !WITH_EDITOR
	TSharedRef<SWidget> MenuWidget = MenuBarBuilder.MakeWidget();
	MenuWidget->SetClipping(EWidgetClipping::ClipToBoundsWithoutIntersecting);
#endif

	ChildSlot
	[
		SNew(SOverlay)

#if !WITH_EDITOR
		// Menu
		+ SOverlay::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		.Padding(34.0f, -60.0f, 0.0f, 0.0f)
		[
			MenuWidget
		]
#endif

		// Overlay slot for the main window area
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 5.0f))
			[
				CreateToolbar(Extension->GetMenuExtender())
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				TabManager->RestoreFrom(Layout, ConstructUnderWindow, false, EOutputCanBeNullptr::IfNoTabValid).ToSharedRef()
			]
		]

		// Session hint overlay
		+ SOverlay::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.Visibility_Lambda([this]() { return IsValidSession() ? EVisibility::Collapsed : EVisibility::Visible; })
			.BorderImage(FAppStyle::Get().GetBrush("PopupText.Background"))
			.Padding(8.0f)
			[
				SNew(STextBlock)
#if WITH_EDITOR
				.Text(LOCTEXT("NoTraceOverlayText", "No trace data available."))
#else
				.Text(LOCTEXT("SelectTraceOverlayText", "Please select a trace."))
#endif
			]
		]
	];

#if !WITH_EDITOR
	// Tell tab-manager about the global menu bar.
	TabManager->SetMenuMultiBox(MenuBarBuilder.GetMultiBox(), MenuWidget);
#endif

	// Tell clients about creation.
	TraceInsightsModule.OnMajorTabCreated().Broadcast(GetMajorTabId(), TabManager.ToSharedRef());
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<FTabManager::FLayout> SMajorTabWindow::CreateDefaultTabLayout() const
{
	return FTabManager::NewLayout("InsightsMajorTabLayout_v1.0")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(1.0f)
				->SetHideTabWell(true)
			)
		);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SMajorTabWindow::CreateToolbar(TSharedPtr<FExtender> Extender)
{
	return SNew(SBox);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMajorTabWindow::FillMenu(FMenuBuilder& MenuBuilder, const TSharedPtr<FTabManager> TabManager)
{
	if (!TabManager.IsValid())
	{
		return;
	}

	UE::Insights::FInsightsManager::Get()->GetInsightsMenuBuilder()->PopulateMenu(MenuBuilder);

	TabManager->PopulateLocalTabSpawnerMenu(MenuBuilder);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SMajorTabWindow::IsValidSession() const
{
	return UE::Insights::FInsightsManager::Get()->GetSession().IsValid();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EActiveTimerReturnType SMajorTabWindow::UpdateActiveDuration(double InCurrentTime, float InDeltaTime)
{
	DurationActive += InDeltaTime;

	// The window will explicitly unregister this active timer when the mouse leaves.
	return EActiveTimerReturnType::Continue;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMajorTabWindow::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);

	if (!ActiveTimerHandle.IsValid())
	{
		ActiveTimerHandle = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SMajorTabWindow::UpdateActiveDuration));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SMajorTabWindow::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseLeave(MouseEvent);

	auto PinnedActiveTimerHandle = ActiveTimerHandle.Pin();
	if (PinnedActiveTimerHandle.IsValid())
	{
		UnRegisterActiveTimer(PinnedActiveTimerHandle.ToSharedRef());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SMajorTabWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid())
	{
		return CommandList->ProcessCommandBindings(InKeyEvent) ? FReply::Handled() : FReply::Unhandled();
	}
	else
	{
		return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SMajorTabWindow::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (UE::Insights::FInsightsManager::Get()->OnDragOver(DragDropEvent))
	{
		return FReply::Handled();
	}

	return SCompoundWidget::OnDragOver(MyGeometry, DragDropEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SMajorTabWindow::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (UE::Insights::FInsightsManager::Get()->OnDrop(DragDropEvent))
	{
		return FReply::Handled();
	}

	return SCompoundWidget::OnDrop(MyGeometry, DragDropEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace Insights

#undef LOCTEXT_NAMESPACE
