// Copyright Epic Games, Inc. All Rights Reserved.

#include "SModulesView.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

// TraceServices
#include "TraceServices/Model/Modules.h"

// TraceInsights
#include "DesktopPlatformModule.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::SModulesView"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FModule
//
// ViewModel for modules
////////////////////////////////////////////////////////////////////////////////////////////////////

class FModule
{
public:
	FModule(const TraceServices::FModule* InModule)
		: Module(InModule)
	{
		check(InModule);
	}

	const TraceServices::FModule* GetModule() const
	{
		return Module;
	}

	uint64 GetBaseAddress() const
	{
		return Module->Base;
	}

	FText GetAddressRangeText() const
	{
		TStringBuilder<64> Sb;
		Sb.Appendf(TEXT("0x%012llX-0x%012llX"), Module->Base, Module->Base + Module->Size);
		return FText::FromStringView(Sb.ToView());
	}

	FText GetModuleNameText() const
	{
		return FText::FromStringView(Module->Name);
	}

	FText GetModuleFullNameText() const
	{
		return FText::FromStringView(Module->FullName);
	}

	inline uint32 GetDiscoveredSymbolCount() const
	{
		// Workaround for the fact that Stats.Discovered does not include the Stats.Cached value.
		return Module->Stats.Discovered + Module->Stats.Cached;
	}

	FText GetDiscoveredAsText() const
	{
		// Discovered != Cached + Resolved + Failed ?
		const uint32 DiscoveredSymbolCount = GetDiscoveredSymbolCount();
		if (DiscoveredSymbolCount != Module->Stats.Cached + Module->Stats.Resolved + Module->Stats.Failed)
		{
			return FText::Format(LOCTEXT("DiscoveredStarredFmt", "{0} (!)"), FText::AsNumber(DiscoveredSymbolCount));
		}
		return FText::AsNumber(DiscoveredSymbolCount);
	}

	inline uint32 GetCachedSymbolCount() const
	{
		return Module->Stats.Cached;
	}

	FText GetCachedAsText() const
	{
		return FText::AsNumber(GetCachedSymbolCount());
	}

	inline uint32 GetResolvedSymbolCount() const
	{
		return Module->Stats.Resolved;
	}

	FText GetResolvedAsText() const
	{
		return FText::AsNumber(GetResolvedSymbolCount());
	}

	inline uint32 GetFailedSymbolCount() const
	{
		return Module->Stats.Failed;
	}

	FText GetFailedAsText() const
	{
		return FText::AsNumber(GetFailedSymbolCount());
	}

	inline uint32 GetAvailableSymbolCount() const
	{
		return Module->Stats.Available;
	}

	FText GetAvailableAsText() const
	{
		return FText::AsNumber(GetAvailableSymbolCount());
	}

	FText GetStatusText() const
	{
		return FText::FromStringView(ModuleStatusToString(Module->Status.load()));
	}

	FText GetSymbolsFileText() const
	{
		return Module->StatusMessage ? FText::FromStringView(Module->StatusMessage) : FText();
	}

	FText GetToolTipText() const { return FText(); }

	inline uint32 GetSize() const
	{
		return Module->Size;
	}

	FText GetSizeAsText() const
	{
		return FText::AsNumber(GetSize());
	}

	FText GetSizeAsToolTipText() const
	{
		TStringBuilder<64> Sb;
		Sb.Appendf(TEXT("0x%X"), Module->Size);
		return FText::FromStringView(Sb.ToView());
	}

private:
	const TraceServices::FModule* Module;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// SModuleRow
////////////////////////////////////////////////////////////////////////////////////////////////////

class SModuleRow : public SMultiColumnTableRow<TSharedPtr<FModule>>
{
	SLATE_BEGIN_ARGS(SModuleRow) {}
	SLATE_END_ARGS()

public:
	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InModule The log message displayed by this row.
	 * @param InOwnerTableView The table to which the row must be added.
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<FModule> InModule, TSharedRef<SModulesView> InParentWidget, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		WeakModule = MoveTemp(InModule);
		WeakParentWidget = InParentWidget;

		SMultiColumnTableRow<TSharedPtr<FModule>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);

		TSharedRef<SWidget> Row = ChildSlot.GetChildAt(0);

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FInsightsStyle::Get().GetBrush("WhiteBrush"))
			.BorderBackgroundColor(this, &SModuleRow::GetBackgroundColor)
			[
				Row
			]
		];
	}

public:
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == ModulesViewColumns::AddressRangeColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.Font(FInsightsStyle::Get().GetFontStyle("Mono.10"))
					.ColorAndOpacity(this, &SModuleRow::GetAddressRangeColor)
					.Text(this, &SModuleRow::GetAddressRange)
				];
		}
		else if (ColumnName == ModulesViewColumns::ModuleNameColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.Text(this, &SModuleRow::GetModuleName)
					.ToolTipText(this, &SModuleRow::GetModuleFullName)
				];
		}
		else if (ColumnName == ModulesViewColumns::DiscoveredColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.ColorAndOpacity(this, &SModuleRow::GetDiscoveredColor)
					.Text(this, &SModuleRow::GetDiscoveredAsText)
				];
		}
		else if (ColumnName == ModulesViewColumns::CachedColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.ColorAndOpacity(this, &SModuleRow::GetCachedColor)
					.Text(this, &SModuleRow::GetCachedAsText)
				];
		}
		else if (ColumnName == ModulesViewColumns::ResolvedColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.ColorAndOpacity(this, &SModuleRow::GetResolvedColor)
					.Text(this, &SModuleRow::GetResolvedAsText)
				];
		}
		else if (ColumnName == ModulesViewColumns::FailedColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.ColorAndOpacity(this, &SModuleRow::GetFailedColor)
					.Text(this, &SModuleRow::GetFailedAsText)
				];
		}
		else if (ColumnName == ModulesViewColumns::AvailableColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.ColorAndOpacity(this, &SModuleRow::GetStatusColor)
					.Text(this, &SModuleRow::GetAvailableAsText)
				];
		}
		else if (ColumnName == ModulesViewColumns::StatusColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.ColorAndOpacity(this, &SModuleRow::GetStatusColor)
					.Text(this, &SModuleRow::GetStatus)
				];
		}
		else if (ColumnName == ModulesViewColumns::SymbolsFileColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.ColorAndOpacity(this, &SModuleRow::GetStatusColor)
					.Text(this, &SModuleRow::GetSymbolsFile)
				];
		}
		else if (ColumnName == ModulesViewColumns::SizeColumnName)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				[
					SNew(STextBlock)
					.ColorAndOpacity(this, &SModuleRow::GetSizeColor)
					.Text(this, &SModuleRow::GetSizeAsText)
					.ToolTipText(this, &SModuleRow::GetSizeAsToolTipText)
				];
		}
		else
		{
			return SNew(STextBlock).Text(LOCTEXT("UnknownColumn", "Unknown Column"));
		}
	}

	FText GetAddressRange() const
	{
		TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			return ModulePin->GetAddressRangeText();
		}
		else
		{
			return FText();
		}
	}

	FText GetModuleName() const
	{
		TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			return ModulePin->GetModuleNameText();
		}
		else
		{
			return FText();
		}
	}

	FText GetModuleFullName() const
	{
		TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			return ModulePin->GetModuleFullNameText();
		}
		else
		{
			return FText();
		}
	}

	FText GetDiscoveredAsText() const
	{
		const TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			return ModulePin->GetDiscoveredAsText();
		}
		else
		{
			return FText();
		}
	}

	FText GetCachedAsText() const
	{
		const TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			return ModulePin->GetCachedAsText();
		}
		else
		{
			return FText();
		}
	}

	FText GetResolvedAsText() const
	{
		const TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			return ModulePin->GetResolvedAsText();
		}
		else
		{
			return FText();
		}
	}

	FText GetFailedAsText() const
	{
		const TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			return ModulePin->GetFailedAsText();
		}
		else
		{
			return FText();
		}
	}

	FText GetAvailableAsText() const
	{
		const TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			return ModulePin->GetAvailableAsText();
		}
		else
		{
			return FText();
		}
	}

	FText GetStatus() const
	{
		TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			return ModulePin->GetStatusText();
		}
		else
		{
			return FText();
		}
	}

	FText GetSymbolsFile() const
	{
		TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			return ModulePin->GetSymbolsFileText();
		}
		else
		{
			return FText();
		}
	}

	FText GetRowToolTip() const
	{
		TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			return ModulePin->GetToolTipText();
		}
		else
		{
			return FText();
		}
	}

	FText GetSizeAsText() const
	{
		const TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			return ModulePin->GetSizeAsText();
		}
		else
		{
			return FText();
		}
	}

	FText GetSizeAsToolTipText() const
	{
		const TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			return ModulePin->GetSizeAsToolTipText();
		}
		else
		{
			return FText();
		}
	}

	inline const FLinearColor GetRedColor() const       { return FLinearColor(1.0f, 0.7f, 0.7f, 1.0f); }
	inline const FLinearColor GetYellowColor() const    { return FLinearColor(1.0f, 0.9f, 0.6f, 1.0f); }
	inline const FLinearColor GetGreenColor() const     { return FLinearColor(0.7f, 1.0f, 0.7f, 1.0f); }
	inline const FLinearColor GetWhileColor() const     { return FLinearColor(1.0f, 1.0f, 1.0f, 1.0f); }
	inline const FLinearColor GetGrayColor() const      { return FLinearColor(0.5f, 0.5f, 0.5f, 1.0f); }
	inline const FLinearColor GetDarkGrayColor() const  { return FLinearColor(0.2f, 0.2f, 0.2f, 1.0f); }
	inline const FLinearColor GetBlackColor() const     { return FLinearColor(0.0f, 0.0f, 0.0f, 1.0f); }

	FSlateColor GetBackgroundColor() const
	{
		return FSlateColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
	}

	FSlateColor GetAddressRangeColor() const
	{
		return IsSelected() ? GetGrayColor() : GetDarkGrayColor();
	}

	FSlateColor GetDiscoveredColor() const
	{
		TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			const TraceServices::FModule* Module = ModulePin->GetModule();
			check(Module != nullptr);

			if (Module->Stats.Failed > 0)
			{
				if (Module->Stats.Cached + Module->Stats.Resolved > Module->Stats.Failed)
				{
					return GetYellowColor();
				}
				else
				{
					return GetRedColor();
				}
			}
			else if (Module->Stats.Cached > 0 || Module->Stats.Resolved > 0)
			{
				return GetGreenColor();
			}
			else if (Module->Stats.Discovered > 0)
			{
				return GetWhileColor();
			}
			else
			{
				return IsSelected() ? GetGrayColor() : GetDarkGrayColor();
			}
		}
		else
		{
			return GetBlackColor();
		}
	}

	FSlateColor GetCachedColor() const
	{
		TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			const TraceServices::FModule* Module = ModulePin->GetModule();
			check(Module != nullptr);

			if (Module->Stats.Cached > 0)
			{
				return GetGreenColor();
			}
			else
			{
				return IsSelected() ? GetGrayColor() : GetDarkGrayColor();
			}
		}
		else
		{
			return GetBlackColor();
		}
	}

	FSlateColor GetResolvedColor() const
	{
		TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			const TraceServices::FModule* Module = ModulePin->GetModule();
			check(Module != nullptr);

			if (Module->Stats.Resolved > 0)
			{
				return GetGreenColor();
			}
			else
			{
				return IsSelected() ? GetGrayColor() : GetDarkGrayColor();
			}
		}
		else
		{
			return GetBlackColor();
		}
	}

	FSlateColor GetFailedColor() const
	{
		TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			const TraceServices::FModule* Module = ModulePin->GetModule();
			check(Module != nullptr);

			if (Module->Stats.Failed > 0)
			{
				return GetRedColor();
			}
			else
			{
				return IsSelected() ? GetGrayColor() : GetDarkGrayColor();
			}
		}
		else
		{
			return GetBlackColor();
		}
	}

	FSlateColor GetStatusColor() const
	{
		TSharedPtr<FModule> ModulePin = WeakModule.Pin();
		if (ModulePin.IsValid())
		{
			const TraceServices::FModule* Module = ModulePin->GetModule();
			check(Module != nullptr);

			const TraceServices::EModuleStatus Status = Module->Status;
			switch (Status)
			{
				case TraceServices::EModuleStatus::Pending:
					return GetYellowColor();

				case TraceServices::EModuleStatus::Loaded:
					return GetGreenColor();

				case TraceServices::EModuleStatus::VersionMismatch:
				case TraceServices::EModuleStatus::NotFound:
				case TraceServices::EModuleStatus::Failed:
					return GetRedColor();

				default:
					return IsSelected() ? GetGrayColor() : GetDarkGrayColor();
			}
		}
		else
		{
			return GetBlackColor();
		}
	}

	FSlateColor GetSizeColor() const
	{
		return GetWhileColor();
	}

private:
	TWeakPtr<FModule> WeakModule;
	TWeakPtr<SModulesView> WeakParentWidget;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// SModulesView
////////////////////////////////////////////////////////////////////////////////////////////////////

SModulesView::SModulesView()
	: LastUpdateTime(0.0)
	, SortColumn(ModulesViewColumns::DiscoveredColumnName)
	, SortMode(EColumnSortMode::Descending)
	, TotalSize(0)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SModulesView::~SModulesView()
{
	Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SModulesView::Reset()
{
	//ListView
	//ExternalScrollbar
	Modules.Reset();
	TotalSize = 0;

	ListView->RebuildList();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SModulesView::Construct(const FArguments& InArgs)
{
	SAssignNew(ExternalScrollbar, SScrollBar)
	.AlwaysShowScrollbar(true);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SBox)
			.VAlign(VAlign_Fill)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(0.0f)
				.VAlign(VAlign_Fill)
				[
					SNew(SScrollBox)
					.Orientation(Orient_Horizontal)

					+ SScrollBox::Slot()
					.VAlign(VAlign_Fill)
					[
						SAssignNew(ListView, SListView<TSharedPtr<FModule>>)
						.ExternalScrollbar(ExternalScrollbar)
						.SelectionMode(ESelectionMode::Single)
						.OnMouseButtonClick(this, &SModulesView::OnMouseButtonClick)
						.OnSelectionChanged(this, &SModulesView::OnSelectionChanged)
						.ListItemsSource(&Modules)
						.OnGenerateRow(this, &SModulesView::OnGenerateRow)
						.ConsumeMouseWheel(EConsumeMouseWheel::Always)
						.OnContextMenuOpening(FOnContextMenuOpening::CreateSP(this, &SModulesView::ListView_GetContextMenu))
						.HeaderRow
						(
							SNew(SHeaderRow)

							+ SHeaderRow::Column(ModulesViewColumns::AddressRangeColumnName)
							.ManualWidth(320.0f)
							.DefaultLabel(LOCTEXT("AddressRangeColumn", "Address Range"))
							.InitialSortMode(EColumnSortMode::Ascending)
							.SortMode(this, &SModulesView::GetSortModeForColumn, ModulesViewColumns::AddressRangeColumnName)
							.OnSort(this, &SModulesView::OnSortModeChanged)

							+ SHeaderRow::Column(ModulesViewColumns::SizeColumnName)
							.ManualWidth(80.0f)
							.HAlignHeader(HAlign_Right)
							.HAlignCell(HAlign_Right)
							.DefaultLabel(LOCTEXT("SizeColumn", "Size"))
							.InitialSortMode(EColumnSortMode::Descending)
							.SortMode(this, &SModulesView::GetSortModeForColumn, ModulesViewColumns::SizeColumnName)
							.OnSort(this, &SModulesView::OnSortModeChanged)

							+ SHeaderRow::Column(ModulesViewColumns::ModuleNameColumnName)
							.ManualWidth(300.0f)
							.DefaultLabel(LOCTEXT("ModuleNameColumn", "Name"))
							.InitialSortMode(EColumnSortMode::Ascending)
							.SortMode(this, &SModulesView::GetSortModeForColumn, ModulesViewColumns::ModuleNameColumnName)
							.OnSort(this, &SModulesView::OnSortModeChanged)

							+ SHeaderRow::Column(ModulesViewColumns::DiscoveredColumnName)
							.ManualWidth(76.0f)
							.HAlignHeader(HAlign_Right)
							.HAlignCell(HAlign_Right)
							.DefaultLabel(LOCTEXT("DiscoveredColumn", "Discovered"))
							.DefaultTooltip(LOCTEXT("DiscoveredColumnTooltip", "The total number of symbols discovered (used) in the trace.\nDiscovered = Cached + Resolved + Failed"))
							.InitialSortMode(EColumnSortMode::Descending)
							.SortMode(this, &SModulesView::GetSortModeForColumn, ModulesViewColumns::DiscoveredColumnName)
							.OnSort(this, &SModulesView::OnSortModeChanged)

							+ SHeaderRow::Column(ModulesViewColumns::CachedColumnName)
							.ManualWidth(76.0f)
							.HAlignHeader(HAlign_Right)
							.HAlignCell(HAlign_Right)
							.DefaultLabel(LOCTEXT("CachedColumn", "Cached"))
							.DefaultTooltip(LOCTEXT("CachedColumnTooltip", "The number of discovered symbols loaded from the cache (*.ucache file)."))
							.InitialSortMode(EColumnSortMode::Descending)
							.SortMode(this, &SModulesView::GetSortModeForColumn, ModulesViewColumns::CachedColumnName)
							.OnSort(this, &SModulesView::OnSortModeChanged)

							+ SHeaderRow::Column(ModulesViewColumns::ResolvedColumnName)
							.ManualWidth(76.0f)
							.HAlignHeader(HAlign_Right)
							.HAlignCell(HAlign_Right)
							.DefaultLabel(LOCTEXT("ResolvedColumn", "Resolved"))
							.DefaultTooltip(LOCTEXT("ResolvedColumnTooltip", "The number of discovered symbols successfully resolved in the current analysis session."))
							.InitialSortMode(EColumnSortMode::Descending)
							.SortMode(this, &SModulesView::GetSortModeForColumn, ModulesViewColumns::ResolvedColumnName)
							.OnSort(this, &SModulesView::OnSortModeChanged)

							+ SHeaderRow::Column(ModulesViewColumns::FailedColumnName)
							.ManualWidth(76.0f)
							.HAlignHeader(HAlign_Right)
							.HAlignCell(HAlign_Right)
							.DefaultLabel(LOCTEXT("FailedColumn", "Failed"))
							.DefaultTooltip(LOCTEXT("FailedColumnTooltip", "The number of discovered symbols that failed to resolve."))
							.InitialSortMode(EColumnSortMode::Descending)
							.SortMode(this, &SModulesView::GetSortModeForColumn, ModulesViewColumns::FailedColumnName)
							.OnSort(this, &SModulesView::OnSortModeChanged)

							+ SHeaderRow::Column(ModulesViewColumns::AvailableColumnName)
							.ManualWidth(76.0f)
							.HAlignHeader(HAlign_Right)
							.HAlignCell(HAlign_Right)
							.DefaultLabel(LOCTEXT("AvailableColumn", "Available"))
							.DefaultTooltip(LOCTEXT("AvailableColumnTooltip", "The total number of symbols available in the loaded symbols file."))
							.InitialSortMode(EColumnSortMode::Descending)
							.SortMode(this, &SModulesView::GetSortModeForColumn, ModulesViewColumns::AvailableColumnName)
							.OnSort(this, &SModulesView::OnSortModeChanged)

							+ SHeaderRow::Column(ModulesViewColumns::StatusColumnName)
							.ManualWidth(100.0f)
							.DefaultLabel(LOCTEXT("StatusColumn", "Status"))
							.InitialSortMode(EColumnSortMode::Ascending)
							.SortMode(this, &SModulesView::GetSortModeForColumn, ModulesViewColumns::StatusColumnName)
							.OnSort(this, &SModulesView::OnSortModeChanged)

							+ SHeaderRow::Column(ModulesViewColumns::SymbolsFileColumnName)
							.ManualWidth(800.0f)
							.DefaultLabel(LOCTEXT("SymbolsFileColumn", "Symbols File"))
							.InitialSortMode(EColumnSortMode::Ascending)
							.SortMode(this, &SModulesView::GetSortModeForColumn, ModulesViewColumns::SymbolsFileColumnName)
							.OnSort(this, &SModulesView::OnSortModeChanged)
						)
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f)
				[
					SNew(SBox)
					.WidthOverride(FOptionalSize(13.0f))
					[
						ExternalScrollbar.ToSharedRef()
					]
				]
			]
		]

		// Status bar
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			// Module stats
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SModulesView::GetStatsText)
			]

			// Search Path Information
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.Padding(2.0f, 0.0f)
			.FillWidth(1)
			[
				SNew(SHorizontalBox)
				.ToolTip(
					SNew(SToolTip)
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(4.0f, 4.0f)
						[
							SNew(STextBlock)
							.Text(this, &SModulesView::GetSymbolSearchPaths)
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(4.0f, 4.0f)
						[
							SNew(SSeparator)
							.Orientation(Orient_Horizontal)
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(4.0f, 4.0f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("TT_SymbolDirectoryWarning", ".EXE/.DLL and .PDB files must all be located in the same folder."))
						]
					]
				)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(2.0f, 0.0f)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FInsightsStyle::GetBrush("Icons.Hint.TreeItem"))
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(2.0f, 0.0f)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SearchPaths", "Search Paths"))
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableRow> SModulesView::OnGenerateRow(TSharedPtr<FModule> InModule, const TSharedRef<STableViewBase>& OwnerTable)
{
	// Generate a row for the log message corresponding to InModule.
	return SNew(SModuleRow, InModule, SharedThis(this), OwnerTable);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SModulesView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	using namespace TraceServices;

	if ((InCurrentTime - LastUpdateTime) < 1.0)
	{
		return;
	}
	LastUpdateTime = InCurrentTime;

	const TSharedPtr<const IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (const IModuleProvider* ModuleProvider = Session->ReadProvider<IModuleProvider>("ModuleProvider"))
	{
		const int32 ModuleCount = Modules.Num();
		ModuleProvider->EnumerateModules(Modules.Num(), [this](const TraceServices::FModule& InModule)
		{
			Modules.Emplace(MakeShared<FModule>(&InModule));
			TotalSize += InModule.Size;
		});
		if (Modules.Num() != ModuleCount)
		{
			UpdateSorting();
			ListView->RequestListRefresh();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FModule> SModulesView::GetSelectedModule() const
{
	TArray<TSharedPtr<FModule>> SelectedItems = ListView->GetSelectedItems();
	return (SelectedItems.Num() == 1) ? SelectedItems[0] : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SModulesView::SelectModule(TSharedPtr<FModule> Module)
{
	if (Module.IsValid())
	{
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SModulesView::OnMouseButtonClick(TSharedPtr<FModule> Module)
{
	SelectModule(Module);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SModulesView::OnSelectionChanged(TSharedPtr<FModule> Module, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct &&
		SelectInfo != ESelectInfo::OnMouseClick)
	{
		SelectModule(Module);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SModulesView::GetStatsText() const
{
	return FText::Format(LOCTEXT("StatusBarTextFmt", "{0} modules ({1} bytes)"), FText::AsNumber(Modules.Num()), FText::AsNumber(TotalSize));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor SModulesView::GetStatsTextColor() const
{
	return FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SModulesView::GetSymbolSearchPaths() const
{
	const TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		if (const TraceServices::IModuleProvider* ModuleProvider = Session->ReadProvider<TraceServices::IModuleProvider>("ModuleProvider"))
		{
			TStringBuilder<2048> PathsListStr;
			ModuleProvider->EnumerateSymbolSearchPaths([&](FStringView Path) { PathsListStr << TEXT("\n \u2022 ") << Path; });
			if (PathsListStr.Len())
			{
				return FText::Format(LOCTEXT("SymbolResolutionPaths", "Symbol search paths (in order of precedence): {0}"),
					FText::FromStringView(PathsListStr));
			}
			else
			{
				return LOCTEXT("SymbolResolutionNoPaths", "No symbol paths has been setup.\nUse configuration file, UE_INSIGHTS_SYMBOL_PATH environment variable or manually select a file or directory from the Modules view.");
			}
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> SModulesView::ListView_GetContextMenu()
{
	TSharedPtr<FModule> SelectedModule = GetSelectedModule();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("ModulesViewContextMenu");
	{
		if (SelectedModule.IsValid())
		{
			MenuBuilder.AddMenuEntry(
				TAttribute<FText>(this, &SModulesView::LoadSymbolsFile_Label),
				LOCTEXT("LoadSymbolsFile_TooltipReady", "Load symbols for a module by specifying a file.\nIf successful, tries to load other failed modules from the same directory."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SModulesView::LoadSymbols_Execute, SelectedModule, true),
				          FCanExecuteAction::CreateSP(this, &SModulesView::LoadSymbols_CanExecute),
				          EUIActionRepeatMode::RepeatDisabled),
				NAME_None,
				EUserInterfaceActionType::Button
			);
			MenuBuilder.AddMenuEntry(
				TAttribute<FText>(this, &SModulesView::LoadSymbolsDirectory_Label),
				LOCTEXT("LoadSymbolsDirectory_TooltipReady", "Load symbols for a module by specifying a directory.\nIf successful, tries to load other failed modules from the same directory."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SModulesView::LoadSymbols_Execute, SelectedModule, false),
				          FCanExecuteAction::CreateSP(this, &SModulesView::LoadSymbols_CanExecute),
				          EUIActionRepeatMode::RepeatDisabled),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SModulesView::LoadSymbolsFile_Label() const
{
	if (LoadSymbols_CanExecute())
	{
		return LOCTEXT("LoadSymbolsFile_Label", "Load symbols from file...");
	}
	return LOCTEXT("LoadSymbolsFile_LabelBusy", "Load symbols from file... (Operation in progress)");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SModulesView::LoadSymbolsDirectory_Label() const
{
	if (LoadSymbols_CanExecute())
	{
		return LOCTEXT("LoadSymbolsDirectory_Label", "Load symbols from directory...");
	}
	return LOCTEXT("LoadSymbolsDirectory_LabelBusy", "Load symbols from directory... (Operation in progress)");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SModulesView::LoadSymbols_CanExecute() const
{
	return !LoadSymbolsTask.IsValid() || LoadSymbolsTask->IsComplete();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SModulesView::LoadSymbols_Execute(TSharedPtr<FModule> Module, bool bOpenFile)
{
	static FString DefaultPath;
	bool bHasSelected = false;
	FString SelectedDirectory;
	TArray<FString> SelectedFiles;
	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		const FString Title = LOCTEXT("LoadSymbolsBrowseFile", "Try loading symbols from...").ToString();
		if (bOpenFile)
		{
			bHasSelected = DesktopPlatform->OpenFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
				Title,
				DefaultPath,
				Module->GetModuleNameText().ToString(),
				TEXT(""),
				0,
				SelectedFiles
			);
		}
		else
		{
			bHasSelected = DesktopPlatform->OpenDirectoryDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
				Title,
				DefaultPath,
				SelectedDirectory
				);
		}
	}

	if (!bHasSelected)
	{
		return;
	}
	const FString& SelectedFile = bOpenFile ? SelectedFiles.Top() : SelectedDirectory;

	// Make sure the last selected folder is default next time
	DefaultPath = SelectedFile;

	using namespace TraceServices;
	const TSharedPtr<const IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (IModuleProvider* ModuleProvider = const_cast<IAnalysisSession*>(Session.Get())->EditProvider<IModuleProvider>("ModuleProvider"))
	{
		LoadSymbolsTask = ModuleProvider->LoadSymbolsForModuleUsingPath(Module->GetBaseAddress(), *SelectedFile);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EColumnSortMode::Type SModulesView::GetSortModeForColumn(const FName ColumnId) const
{
	return ColumnId == SortColumn ? SortMode : EColumnSortMode::None;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SModulesView::OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortColumn = ColumnId;
	SortMode = InSortMode;
	UpdateSorting();

	if (ListView.IsValid())
	{
		ListView->RebuildList();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SModulesView::UpdateSorting()
{
	if (SortColumn == ModulesViewColumns::AddressRangeColumnName)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetBaseAddress() < B->GetBaseAddress(); });
		}
		else
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetBaseAddress() > B->GetBaseAddress(); });
		}
	}
	else if (SortColumn == ModulesViewColumns::ModuleNameColumnName)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{
					if (A->GetModule()->Name == nullptr)
					{
						return true;
					}
					if (B->GetModule()->Name == nullptr)
					{
						return false;
					}
					return FCString::Stricmp(A->GetModule()->Name, B->GetModule()->Name) < 0;
				});
		}
		else
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{
					if (A->GetModule()->Name == nullptr)
					{
						return false;
					}
					if (B->GetModule()->Name == nullptr)
					{
						return true;
					}
					return FCString::Stricmp(A->GetModule()->Name, B->GetModule()->Name) > 0;
				});
		}
	}
	else if (SortColumn == ModulesViewColumns::DiscoveredColumnName)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetDiscoveredSymbolCount() < B->GetDiscoveredSymbolCount(); });
		}
		else
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetDiscoveredSymbolCount() > B->GetDiscoveredSymbolCount(); });
		}
	}
	else if (SortColumn == ModulesViewColumns::CachedColumnName)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetCachedSymbolCount() < B->GetCachedSymbolCount(); });
		}
		else
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetCachedSymbolCount() > B->GetCachedSymbolCount(); });
		}
	}
	else if (SortColumn == ModulesViewColumns::ResolvedColumnName)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetResolvedSymbolCount() < B->GetResolvedSymbolCount(); });
		}
		else
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetResolvedSymbolCount() > B->GetResolvedSymbolCount(); });
		}
	}
	else if (SortColumn == ModulesViewColumns::FailedColumnName)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetFailedSymbolCount() < B->GetFailedSymbolCount(); });
		}
		else
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetFailedSymbolCount() > B->GetFailedSymbolCount(); });
		}
	}
	else if (SortColumn == ModulesViewColumns::AvailableColumnName)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetAvailableSymbolCount() < B->GetAvailableSymbolCount(); });
		}
		else
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetAvailableSymbolCount() > B->GetAvailableSymbolCount(); });
		}
	}
	else if (SortColumn == ModulesViewColumns::StatusColumnName)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetModule()->Status < B->GetModule()->Status; });
		}
		else
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetModule()->Status > B->GetModule()->Status; });
		}
	}
	else if (SortColumn == ModulesViewColumns::SymbolsFileColumnName)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{
					if (A->GetModule()->StatusMessage == nullptr)
					{
						return true;
					}
					if (B->GetModule()->StatusMessage == nullptr)
					{
						return false;
					}
					return FCString::Stricmp(A->GetModule()->StatusMessage, B->GetModule()->StatusMessage) < 0;
				});
		}
		else
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{
					if (A->GetModule()->StatusMessage == nullptr)
					{
						return false;
					}
					if (B->GetModule()->StatusMessage == nullptr)
					{
						return true;
					}
					return FCString::Stricmp(A->GetModule()->StatusMessage, B->GetModule()->StatusMessage) > 0;
				});
		}
	}
	else if (SortColumn == ModulesViewColumns::SizeColumnName)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetSize() < B->GetSize(); });
		}
		else
		{
			Modules.Sort([](const TSharedPtr<FModule>& A, const TSharedPtr<FModule>& B)
				{ return A->GetSize() > B->GetSize(); });
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} //namespace UE::Insights

#undef LOCTEXT_NAMESPACE
