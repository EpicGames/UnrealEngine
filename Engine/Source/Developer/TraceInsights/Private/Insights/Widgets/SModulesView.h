// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "SlateFwd.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class FMenuBuilder;

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

class FModule;

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace ModulesViewColumns
{
	static const FName AddressRangeColumnName(TEXT("BaseAddress"));
	static const FName ModuleNameColumnName(TEXT("ModuleName"));
	static const FName DiscoveredColumnName(TEXT("Discovered"));
	static const FName CachedColumnName(TEXT("Cached"));
	static const FName ResolvedColumnName(TEXT("Resolved"));
	static const FName FailedColumnName(TEXT("Failed"));
	static const FName AvailableColumnName(TEXT("Available"));
	static const FName StatusColumnName(TEXT("Status"));
	static const FName SymbolsFileColumnName(TEXT("SymbolsFile"));
	static const FName SizeColumnName(TEXT("Size"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Modules window.
 */
class SModulesView : public SCompoundWidget
{
public:
	/** Default constructor. */
	SModulesView();

	/** Virtual destructor. */
	virtual ~SModulesView();

	void Reset();

	SLATE_BEGIN_ARGS(SModulesView) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget.
	 * @param InArgs - The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs);

	/**
	 * Ticks this widget. Override in derived classes, but always call the parent implementation.
	 *
	 * @param AllottedGeometry - The space allotted for this widget
	 * @param InCurrentTime - Current absolute real time
	 * @param InDeltaTime - Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	TSharedPtr<FModule> GetSelectedModule() const;

protected:
	/** Generate a new list view row. */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FModule> InModule, const TSharedRef<STableViewBase>& OwnerTable);

	void SelectModule(TSharedPtr<FModule> Module);
	void OnMouseButtonClick(TSharedPtr<FModule> Module);
	void OnSelectionChanged(TSharedPtr<FModule> Module, ESelectInfo::Type SelectInfo);

	FText GetStatsText() const;
	FSlateColor GetStatsTextColor() const;

	FText GetSymbolSearchPaths() const;

	TSharedPtr<SWidget> ListView_GetContextMenu();
	FText LoadSymbolsFile_Label() const;
	FText LoadSymbolsDirectory_Label() const;
	bool LoadSymbols_CanExecute() const;

	void LoadSymbols_Execute(TSharedPtr<FModule> Module, bool bOpenFile);

	void UpdateSorting();

private:
	EColumnSortMode::Type GetSortModeForColumn(const FName ColumnId) const;
	void OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode);

protected:
	/** The list view widget. */
	TSharedPtr<SListView<TSharedPtr<FModule>>> ListView;

	/** External scrollbar used to synchronize tree view position. */
	TSharedPtr<SScrollBar> ExternalScrollbar;

	/** List of modules (view model) to show in list view. */
	TArray<TSharedPtr<FModule>> Modules;

	/** Reference to ongoing load symbols task. */
	FGraphEventRef LoadSymbolsTask;

	/** Last time we queried about discovered modules. */
	double LastUpdateTime;

	/** The current sorting column. */
	FName SortColumn;

	/** Sorting mode (ascending or descending) for current sorting column. */
	EColumnSortMode::Type SortMode;

	/** The total size of modules. */
	uint64 TotalSize;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
