// Copyright Epic Games, Inc. All Rights Reserved.

#include "SQuickFind.h"

#include "SlateOptMacros.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableViewBase.h"

// TraceInsightsCore
#include "InsightsCore/Filter/ViewModels/FilterConfigurator.h"
#include "InsightsCore/Filter/Widgets/SFilterConfigurator.h"

// TraceInsights
#include "Insights/InsightsStyle.h"
#include "Insights/ViewModels/QuickFind.h"

#define LOCTEXT_NAMESPACE "SQuickFind"

namespace Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// SQuickFind
////////////////////////////////////////////////////////////////////////////////////////////////////

SQuickFind::SQuickFind()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SQuickFind::~SQuickFind()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SQuickFind::InitCommandList()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SQuickFind::Construct(const FArguments& InArgs, TSharedPtr<FQuickFind> InQuickFindViewModel)
{
	ChildSlot
	[
		SNew(SVerticalBox)

		// Tree view
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.0f, 0.0f, 0.0f, 0.0f)
		[
			SAssignNew(FilterConfigurator, UE::Insights::SFilterConfigurator, InQuickFindViewModel->GetFilterConfigurator())
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 4.0f, 4.0f, 4.0f)
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.VAlign(EVerticalAlignment::VAlign_Bottom)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(1.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("FindFirst", "Find First"))
				.ToolTipText(LOCTEXT("FindFirstDesc", "Find First\nSelects the first timing event that matches the search criteria."))
				.OnClicked(this, &SQuickFind::FindFirst_OnClicked)
				.Content()
				[
					SNew(SImage)
					.Image(FInsightsStyle::Get().GetBrush("Icons.FindFirst.ToolBar"))
				]
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(1.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("FindPrev", "Find Previous"))
				.ToolTipText(LOCTEXT("FindPrevDesc", "Find Previous\nSelects the previous timing event that matches the search criteria."))
				.OnClicked(this, &SQuickFind::FindPrevious_OnClicked)
				.Content()
				[
					SNew(SImage)
					.Image(FInsightsStyle::Get().GetBrush("Icons.FindPrevious.ToolBar"))
				]
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(1.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("FindNext", "Find Next"))
				.ToolTipText(LOCTEXT("FindNextDesc", "Find Next\nSelects the next timing event that matches the search criteria."))
				.OnClicked(this, &SQuickFind::FindNext_OnClicked)
				.Content()
				[
					SNew(SImage)
					.Image(FInsightsStyle::Get().GetBrush("Icons.FindNext.ToolBar"))
				]
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(1.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("FindLast", "Find Last"))
				.ToolTipText(LOCTEXT("FindLastDesc", "Find Last\nSelects the last timing event that matches the search criteria."))
				.OnClicked(this, &SQuickFind::FindLast_OnClicked)
				.Content()
				[
					SNew(SImage)
					.Image(FInsightsStyle::Get().GetBrush("Icons.FindLast.ToolBar"))
				]
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(1.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("FindMax", "FindMax"))
				.ToolTipText(LOCTEXT("FindMaxDesc", "Find Max\nSelects the timing event with the largest duration that matches the search criteria."))
				.OnClicked(this, &SQuickFind::FindMax_OnClicked)
				.Content()
				[
					SNew(SImage)
					.Image(FInsightsStyle::Get().GetBrush("Icons.FindMaxInstance.ToolBar"))
				]
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(1.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("FindMin", "Find Min"))
				.ToolTipText(LOCTEXT("FindMinDesc", "Find Min\nSelects the timing event with the shortest duration that matches the search criteria."))
				.OnClicked(this, &SQuickFind::FindMin_OnClicked)
				.Content()
				[
					SNew(SImage)
					.Image(FInsightsStyle::Get().GetBrush("Icons.FindMinInstance.ToolBar"))
				]
			]

		+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(1.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("FilterAll", "Apply Filter"))
				.ToolTipText(LOCTEXT("FilterAllDesc", "Apply Filter\nApplies the current filter to all tracks.\nIt highlights timing events that matches the filter."))
				.OnClicked(this, &SQuickFind::FilterAll_OnClicked)
				.Content()
				[
					SNew(SImage)
					.Image(FInsightsStyle::Get().GetBrush("Icons.HighlightEvents.ToolBar"))
				]
			]

		+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(1.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("ClearFilters", "Clear Filters"))
				.ToolTipText(LOCTEXT("ClearFiltersDesc", "Clear Filters\nClears all filters (used to highlight timing events) applied to the tracks."))
				.OnClicked(this, &SQuickFind::ClearFilters_OnClicked)
				.Content()
				[
					SNew(SImage)
					.Image(FInsightsStyle::Get().GetBrush("Icons.ResetHighlight.ToolBar"))
				]
			]
		]
	];

	SHeaderRow::FColumn::FArguments ColumnArgs;
	ColumnArgs
		.ColumnId(TEXT("Filter"))
		.DefaultLabel(LOCTEXT("FilterColumnHeader", "Filter"))
		.HAlignHeader(HAlign_Fill)
		.VAlignHeader(VAlign_Fill)
		.HeaderContentPadding(FMargin(2.0f))
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Fill)
		.HeaderContent()
		[
			SNew(SBox)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FilterColumnHeader", "Filter"))
			]
		];

	QuickFindViewModel = InQuickFindViewModel;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SQuickFind::FindFirst_OnClicked()
{
	QuickFindViewModel->GetFilterConfigurator()->GetRootNode()->ProcessFilter();
	QuickFindViewModel->GetOnFindFirstEvent().Broadcast();

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SQuickFind::FindPrevious_OnClicked()
{
	QuickFindViewModel->GetFilterConfigurator()->GetRootNode()->ProcessFilter();
	QuickFindViewModel->GetOnFindPreviousEvent().Broadcast();

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SQuickFind::FindNext_OnClicked()
{
	QuickFindViewModel->GetFilterConfigurator()->GetRootNode()->ProcessFilter();
	QuickFindViewModel->GetOnFindNextEvent().Broadcast();

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SQuickFind::FindLast_OnClicked()
{
	QuickFindViewModel->GetFilterConfigurator()->GetRootNode()->ProcessFilter();
	QuickFindViewModel->GetOnFindLastEvent().Broadcast();

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SQuickFind::FindMax_OnClicked()
{
	QuickFindViewModel->GetFilterConfigurator()->GetRootNode()->ProcessFilter();
	QuickFindViewModel->GetOnFindMaxEvent().Broadcast();

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SQuickFind::FindMin_OnClicked()
{
	QuickFindViewModel->GetFilterConfigurator()->GetRootNode()->ProcessFilter();
	QuickFindViewModel->GetOnFindMinEvent().Broadcast();

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SQuickFind::FilterAll_OnClicked()
{
	QuickFindViewModel->GetOnFilterAllEvent().Broadcast();

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SQuickFind::ClearFilters_OnClicked()
{
	QuickFindViewModel->GetOnClearFiltersEvent().Broadcast();

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace Insights

#undef LOCTEXT_NAMESPACE
