// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STreeView.h"

// TraceInsightsCore
#include "InsightsCore/Filter/ViewModels/FilterConfiguratorNode.h"
#include "InsightsCore/Filter/ViewModels/Filters.h"

class SDockTab;

namespace UE::Insights { class SFilterConfigurator; }

namespace Insights
{

class FQuickFind;

////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * A custom widget used to configure custom filters.
 */
class SQuickFind: public SCompoundWidget
{
public:
	SQuickFind();

	virtual ~SQuickFind();

	SLATE_BEGIN_ARGS(SQuickFind) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<FQuickFind> InFilterConfiguratorViewModel);

	void Reset();

	void SetParentTab(const TSharedPtr<SDockTab> InTab) { ParentTab = InTab; }
	const TWeakPtr<SDockTab> GetParentTab() { return ParentTab; };

private:
	void InitCommandList();

	FReply FindFirst_OnClicked();
	FReply FindPrevious_OnClicked();
	FReply FindNext_OnClicked();
	FReply FindLast_OnClicked();
	FReply FindMax_OnClicked();
	FReply FindMin_OnClicked();


	FReply FilterAll_OnClicked();

	FReply ClearFilters_OnClicked();

private:
	TSharedPtr<UE::Insights::SFilterConfigurator> FilterConfigurator;

	TSharedPtr<FQuickFind> QuickFindViewModel;

	TWeakPtr<SDockTab> ParentTab;

	FDelegateHandle OnViewModelDestroyedHandle;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace Insights
