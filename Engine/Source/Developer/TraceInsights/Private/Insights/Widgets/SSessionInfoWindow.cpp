// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSessionInfoWindow.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/Paths.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#if WITH_EDITOR
	#include "EngineAnalytics.h"
	#include "AnalyticsEventAttribute.h"
	#include "Interfaces/IAnalyticsProvider.h"
#endif // WITH_EDITOR

// TraceServices
#include "TraceServices/Model/Diagnostics.h"
#include "TraceServices/ModuleService.h"

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/Common/InsightsMenuBuilder.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::SSessionInfoWindow"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

const FName FSessionInfoTabs::SessionInfoID(TEXT("SessionInfo"));

////////////////////////////////////////////////////////////////////////////////////////////////////

SSessionInfoWindow::SSessionInfoWindow()
	: DurationActive(0.0f)
	, AnalysisSession(nullptr)
	, bIsSessionInfoSet(false)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SSessionInfoWindow::~SSessionInfoWindow()
{
#if WITH_EDITOR
	if (DurationActive > 0.0f && FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Insights.Usage.SessionInfo"), FAnalyticsEventAttribute(TEXT("Duration"), DurationActive));
	}
#endif // WITH_EDITOR
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SSessionInfoWindow::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	// Create & initialize tab manager.
	TabManager = FGlobalTabmanager::Get()->NewTabManager(ConstructUnderMajorTab);

	TSharedRef<FWorkspaceItem> AppMenuGroup = TabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("SessionMenuGroupName", "Session Info"));

	TabManager->RegisterTabSpawner(FSessionInfoTabs::SessionInfoID, FOnSpawnTab::CreateRaw(this, &SSessionInfoWindow::SpawnTab_SessionInfo))
		.SetDisplayName(LOCTEXT("SessionInfo", "Session Info"))
		.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.SessionInfo"))
		.SetGroup(AppMenuGroup);

	TSharedRef<FTabManager::FLayout> Layout = []() -> TSharedRef<FTabManager::FLayout>
	{
		// Create tab layout.
		return FTabManager::NewLayout("SessionInfoLayout_v1.0")
			->AddArea
			(
				FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(FSessionInfoTabs::SessionInfoID, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
			);
	}();

	// Create & initialize main menu.
	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(TSharedPtr<FUICommandList>());
	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("MenuLabel", "Menu"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateStatic(&SSessionInfoWindow::FillMenu, TabManager),
		FName(TEXT("Menu"))
	);

#if !WITH_EDITOR
	TSharedRef<SWidget> MenuWidget = MenuBarBuilder.MakeWidget();
	MenuWidget->SetClipping(EWidgetClipping::ClipToBoundsWithoutIntersecting);
#endif

	ChildSlot
	[
		SNew(SOverlay)

#if !WITH_EDITOR
		// Menu
		+ SOverlay::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		.Padding(34.0f, -60.0f, 0.0f, 0.0f)
		[
			MenuWidget
		]
#endif

		// Overlay slot for the main window area
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			TabManager->RestoreFrom(Layout, ConstructUnderWindow).ToSharedRef()
		]
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSessionInfoWindow::BeginSection(TSharedPtr<SVerticalBox> InVerticalBox, const FText& InSectionName) const
{
	InVerticalBox->AddSlot()
		.AutoHeight()
		.Padding(0.0f, 0.0f, 0.0f, 4.0f)
		[
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor(0.03f, 0.03f, 0.03f, 1.0f))
			.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
			.Padding(FMargin(0.0f, 4.0f, 0.0f, 4.0f))
			[
				SNew(SBox)
				.Padding(FMargin(16.0f, 0.0f, 16.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text(InSectionName)
					.ColorAndOpacity(FLinearColor(0.7f, 0.7f, 0.7f, 1.0f))
				]
			]
		];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSessionInfoWindow::EndSection(TSharedPtr<SVerticalBox> InVerticalBox) const
{
	InVerticalBox->AddSlot()
		.AutoHeight()
		.Padding(0.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SBox)
			.Padding(FMargin(0.0f))
			.HeightOverride(4.0f)
		];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> SSessionInfoWindow::CreateTextBox(const TAttribute<FText>& InText, bool bMultiLine) const
{
	TSharedPtr<SWidget> TextBox;
	if (bMultiLine)
	{
		TextBox = SNew(SMultiLineEditableTextBox)
			.Text(InText)
			.AutoWrapText(true)
			.BackgroundColor(FLinearColor(0.1f, 0.1f, 0.1f, 1.0f))
			.IsReadOnly(true);
	}
	else
	{
		TextBox = SNew(SEditableTextBox)
			.Text(InText)
			.BackgroundColor(FLinearColor(0.1f, 0.1f, 0.1f, 1.0f))
			.IsReadOnly(true);
	}
	return TextBox.ToSharedRef();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSessionInfoWindow::AddInfoLine(TSharedPtr<SVerticalBox> InVerticalBox,
									 const FText& InHeader,
									 FText(SSessionInfoWindow::* InGetTextMethodPtr)() const,
									 EVisibility(SSessionInfoWindow::* InVisibilityMethodPtr)() const,
									 bool bMultiLine) const
{
	InVerticalBox->AddSlot()
		.AutoHeight()
		.Padding(16.0f, 4.0f, 16.0f, 4.0f)
		[
			SNew(SHorizontalBox)
			.Visibility(this, InVisibilityMethodPtr)

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Top)
			.AutoWidth()
			.Padding(0.0f)
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f, 16.0f, 0.0f))
				.HAlign(HAlign_Right)
				.MinDesiredWidth(160.0f)
				[
					SNew(STextBlock)
					.Text(InHeader)
				]
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.FillWidth(1.0f)
			.Padding(0.0f)
			[
				CreateTextBox(TAttribute<FText>(this, InGetTextMethodPtr), bMultiLine)
			]
		];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSessionInfoWindow::AddSimpleInfoLine(TSharedPtr<SVerticalBox> InVerticalBox, const TAttribute<FText>& InValue, bool bMultiLine) const
{
	InVerticalBox->AddSlot()
		.AutoHeight()
		.Padding(16.0f, 4.0f, 16.0f, 4.0f)
		[
			CreateTextBox(InValue, bMultiLine)
		];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> SSessionInfoWindow::SpawnTab_SessionInfo(const FSpawnTabArgs& Args)
{
	TSharedPtr<SScrollBar> VScrollbar;
	SAssignNew(VScrollbar, SScrollBar)
	.Orientation(Orient_Vertical);

	TSharedPtr<SVerticalBox> VerticalBox;

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			SNew(SOverlay)

			// Overlay slot for the ScrollBox containing the data
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				SNew(SScrollBox)
				.Orientation(Orient_Vertical)
				.ExternalScrollbar(VScrollbar)

				+ SScrollBox::Slot()
				[
					SAssignNew(VerticalBox, SVerticalBox)
				]
			]

			// Overlay slot for the vertical scrollbar
			+ SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Fill)
			[
				SNew(SBox)
				.WidthOverride(FOptionalSize(13.0f))
				[
					VScrollbar.ToSharedRef()
				]
			]
		];

	BeginSection(VerticalBox, LOCTEXT("SessionInfo_SectionText", "Session Info"));

	AddInfoLine(VerticalBox, LOCTEXT("SessionName_HeaderText",		"Session Name"),		&SSessionInfoWindow::GetSessionNameText,	&SSessionInfoWindow::IsAlwaysVisible);
	AddInfoLine(VerticalBox, LOCTEXT("Uri_HeaderText",				"URI"),					&SSessionInfoWindow::GetUriText,			&SSessionInfoWindow::IsVisibleUriText);
	AddInfoLine(VerticalBox, LOCTEXT("Platform_HeaderText",			"Platform"),			&SSessionInfoWindow::GetPlatformText,		&SSessionInfoWindow::IsVisiblePlatformText);
	AddInfoLine(VerticalBox, LOCTEXT("AppName_HeaderText",			"Application Name"),	&SSessionInfoWindow::GetAppNameText,		&SSessionInfoWindow::IsVisibleAppNameText);
	AddInfoLine(VerticalBox, LOCTEXT("ProjectName_HeaderText",		"Project Name"),		&SSessionInfoWindow::GetProjectNameText,	&SSessionInfoWindow::IsVisibleProjectNameText);
	AddInfoLine(VerticalBox, LOCTEXT("BuildConfig_HeaderText",		"Build Config"),		&SSessionInfoWindow::GetBuildConfigText,	&SSessionInfoWindow::IsVisibleBuildConfigText);
	AddInfoLine(VerticalBox, LOCTEXT("BuildTarget_HeaderText",		"Build Target"),		&SSessionInfoWindow::GetBuildTargetText,	&SSessionInfoWindow::IsVisibleBuildTargetText);
	AddInfoLine(VerticalBox, LOCTEXT("BuildVersion_HeaderText",		"Build Version"),		&SSessionInfoWindow::GetBuildVersionText,	&SSessionInfoWindow::IsVisibleBuildVersionText);
	AddInfoLine(VerticalBox, LOCTEXT("Branch_HeaderText",			"Branch"),				&SSessionInfoWindow::GetBranchText,			&SSessionInfoWindow::IsVisibleBranchText);
	AddInfoLine(VerticalBox, LOCTEXT("Changelist_HeaderText",		"Changelist"),			&SSessionInfoWindow::GetChangelistText,		&SSessionInfoWindow::IsVisibleChangelistText);
	AddInfoLine(VerticalBox, LOCTEXT("EngineVersion_HeaderText",	"Engine Version"),		&SSessionInfoWindow::GetEngineVersionText,	&SSessionInfoWindow::IsVisibleEngineVersionText);
	AddInfoLine(VerticalBox, LOCTEXT("CommandLine_HeaderText",		"Command Line"),		&SSessionInfoWindow::GetCommandLineText,	&SSessionInfoWindow::IsVisibleCommandLineText, true);
	AddInfoLine(VerticalBox, LOCTEXT("VFSPaths_HeaderText",			"VFS Paths"),			&SSessionInfoWindow::GetVFSPathsText,		&SSessionInfoWindow::IsVisibleVFSPathsText, true);
	AddInfoLine(VerticalBox, LOCTEXT("OtherMetadata_HeaderText",	"Other Metadata"),		&SSessionInfoWindow::GetOtherMetadataText,	&SSessionInfoWindow::IsVisibleOtherMetadataText, true);
	AddInfoLine(VerticalBox, LOCTEXT("StreamType_HeaderText",		"Stream Type"),			&SSessionInfoWindow::GetStreamTypeText,		&SSessionInfoWindow::IsAlwaysVisible);

	EndSection(VerticalBox);

	BeginSection(VerticalBox, LOCTEXT("AnalysisStatus_SectionText", "Analysis Status"));
	AddSimpleInfoLine(VerticalBox, TAttribute<FText>(this, &SSessionInfoWindow::GetStatusText), true);
	EndSection(VerticalBox);

	BeginSection(VerticalBox, LOCTEXT("AnalysisModules_SectionText", "Analysis Modules"));
	AddSimpleInfoLine(VerticalBox, TAttribute<FText>(this, &SSessionInfoWindow::GetModulesText), true);
	EndSection(VerticalBox);

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SSessionInfoWindow::OnSessionInfoTabClosed));

	return DockTab;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSessionInfoWindow::OnSessionInfoTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSessionInfoWindow::FillMenu(FMenuBuilder& MenuBuilder, const TSharedPtr<FTabManager> TabManager)
{
	if (!TabManager.IsValid())
	{
		return;
	}

	FInsightsManager::Get()->GetInsightsMenuBuilder()->PopulateMenu(MenuBuilder);

	TabManager->PopulateLocalTabSpawnerMenu(MenuBuilder);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSessionInfoWindow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();

	if (Session != AnalysisSession)
	{
		// The session has changed. We need new info.
		AnalysisSession = Session;
		bIsSessionInfoSet = false;

		StreamTypeText = FText::GetEmpty();
		SessionNameText = FText::GetEmpty();
		UriText = FText::GetEmpty();

		PlatformText = FText::GetEmpty();
		AppNameText = FText::GetEmpty();
		ProjectNameText = FText::GetEmpty();
		BuildConfigurationTypeText = FText::GetEmpty();
		BuildTargetTypeText = FText::GetEmpty();
		BuildVersionText = FText::GetEmpty();
		BranchText = FText::GetEmpty();
		ChangelistText = FText::GetEmpty();
		EngineVersionText = FText::GetEmpty();
		CommandLineText = FText::GetEmpty();
		VFSPathsText = FText::GetEmpty();
		OtherMetadataText = FText::GetEmpty();
	}

	// If we already have the session info data, we no longer poll for it.
	if (!bIsSessionInfoSet && Session.IsValid())
	{
		if (StreamTypeText.IsEmpty())
		{
			UpdateStreamTypeAndSessionName(*Session.Get());
		}

		bIsSessionInfoSet = TryUpdateDiagnosticsInfo(*Session.Get());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSessionInfoWindow::UpdateStreamTypeAndSessionName(const TraceServices::IAnalysisSession& Session)
{
	ETraceStreamType TraceStreamType = FInsightsManager::Get()->GetTraceStreamType();

	switch (TraceStreamType)
	{
	case ETraceStreamType::TraceFile:
		StreamTypeText = LOCTEXT("TraceStreamType_TraceFile", "Trace File");
		break;

	case ETraceStreamType::TraceStore:
		StreamTypeText = FText::Format(LOCTEXT("TraceStreamType_TraceStore_Fmt", "Trace Store {0} -- TraceId={1}"),
			FText::FromString(FString::Printf(TEXT("%s:%u"), *FInsightsManager::Get()->GetLastStoreHost(), FInsightsManager::Get()->GetLastStorePort())),
			FText::FromString(FString::Printf(TEXT("0x%X"), Session.GetTraceId())));
		break;

	case ETraceStreamType::DirectTrace:
	case ETraceStreamType::DirectTraceSecure:
		StreamTypeText = FText::Format(LOCTEXT("TraceStreamType_DirectTrace_Fmt", "Direct Trace -- Port={0}"),
			FText::AsNumber(uint32(FInsightsManager::Get()->GetDirectTracePort()), &FNumberFormattingOptions::DefaultNoGrouping()));
		break;

	case ETraceStreamType::Custom:
		StreamTypeText = LOCTEXT("TraceStreamType_Custom", "Custom Stream");
		break;

	default:
		StreamTypeText = LOCTEXT("TraceStreamType_Unknown", "Unknown");
		break;
	}

	// We can quickly get the session name and uri.
	if (TraceStreamType == ETraceStreamType::TraceFile ||
		TraceStreamType == ETraceStreamType::TraceStore)
	{
		SessionNameText = FText::FromString(FPaths::GetBaseFilename(Session.GetName()));

		FString Uri(Session.GetName());
		FPaths::NormalizeFilename(Uri);
		UriText = FText::FromString(Uri);
	}
	else
	{
		SessionNameText = FText::FromString(Session.GetName());
		UriText = FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool SSessionInfoWindow::TryUpdateDiagnosticsInfo(const TraceServices::IAnalysisSession& Session)
{
	TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);

	const TraceServices::IDiagnosticsProvider* DiagnosticsProvider = TraceServices::ReadDiagnosticsProvider(Session);
	if (!DiagnosticsProvider || !DiagnosticsProvider->IsSessionInfoAvailable())
	{
		return false;
	}

	TraceServices::FSessionInfo SessionInfo = DiagnosticsProvider->GetSessionInfo();

	PlatformText = FText::FromString(SessionInfo.Platform);
	AppNameText = FText::FromString(SessionInfo.AppName);
	ProjectNameText = FText::FromString(SessionInfo.ProjectName);

	BuildConfigurationTypeText = FText::FromString(LexToString(SessionInfo.ConfigurationType));
	BuildTargetTypeText = FText::FromString(LexToString(SessionInfo.TargetType));
	BuildVersionText = FText::FromString(SessionInfo.BuildVersion);
	BranchText = FText::FromString(SessionInfo.Branch);
	ChangelistText = FText::AsNumber(SessionInfo.Changelist, &FNumberFormattingOptions::DefaultNoGrouping());

	EngineVersionText = FText::FromString(SessionInfo.EngineVersion);
	if (!SessionInfo.EngineVersion.IsEmpty() &&
		!SessionInfo.BuildVersion.IsEmpty())
	{
		// Trim the BuildVersion from the EngineVersion, if it is included and it is the same.
		int32 Index = SessionInfo.EngineVersion.Find(SessionInfo.BuildVersion, ESearchCase::CaseSensitive);
		if (Index > 0 && SessionInfo.EngineVersion[Index - 1] == TEXT('-'))
		{
			EngineVersionText = FText::FromString(SessionInfo.EngineVersion.Left(Index - 1));
		}
	}

	CommandLineText = FText::FromString(SessionInfo.CommandLine);

	TArray<FString> VFSPaths;
	SessionInfo.VFSPaths.ParseIntoArray(VFSPaths, TEXT(";"), true);
	int32 NumVFSPaths = 0;
	TStringBuilder<1024> VFSPathsBuilder;
	for (FString& Path : VFSPaths)
	{
		Path.TrimStartAndEndInline();
		if (!Path.IsEmpty())
		{
			if (NumVFSPaths & 1)
			{
				VFSPathsBuilder.Append(TEXT(" \u2192 "));
			}
			else if (NumVFSPaths > 0)
			{
				VFSPathsBuilder.Append(TEXT("\n"));
			}
			++NumVFSPaths;
			VFSPathsBuilder.Append(Path);
		}
	}
	VFSPathsText = FText::FromString(VFSPathsBuilder.ToString());

	TStringBuilder<1024> OtherMetadata;

	Session.EnumerateMetadata(
		[&OtherMetadata]
		(const TraceServices::FTraceSessionMetadata& Metadata)
		{
			static FName ExcludedMetadata[]
			{
				FName("Platform"),
				FName("AppName"),
				FName("ProjectName"),
				FName("ConfigurationType"),
				FName("TargetType"),
				FName("BuildVersion"),
				FName("Branch"),
				FName("Changelist"),
				FName("EngineVersion"),
				FName("CommandLine"),
				FName("VFSPaths"),
			};

			for (int32 Index = 0; Index < UE_ARRAY_COUNT(ExcludedMetadata); ++Index)
			{
				if (Metadata.Name == ExcludedMetadata[Index])
				{
					return;
				}
			}

			switch (Metadata.Type)
			{
				case TraceServices::FTraceSessionMetadata::EType::Int64:
				{
					if (OtherMetadata.Len() > 0)
					{
						OtherMetadata.Append(TEXT("\n"));
					}
					OtherMetadata.Append(Metadata.Name.GetPlainNameString());
					OtherMetadata.Append(TEXT("="));
					OtherMetadata.Appendf(TEXT("%lld"), Metadata.Int64Value);
					break;
				}
				case TraceServices::FTraceSessionMetadata::EType::Double:
				{
					if (OtherMetadata.Len() > 0)
					{
						OtherMetadata.Append(TEXT("\n"));
					}
					OtherMetadata.Append(Metadata.Name.GetPlainNameString());
					OtherMetadata.Append(TEXT("="));
					OtherMetadata.Appendf(TEXT("%f"), Metadata.DoubleValue);
					break;
				}
				case TraceServices::FTraceSessionMetadata::EType::String:
				{
					if (OtherMetadata.Len() > 0)
					{
						OtherMetadata.Append(TEXT("\n"));
					}
					OtherMetadata.Append(Metadata.Name.GetPlainNameString());
					OtherMetadata.Append(TEXT("="));
					OtherMetadata.Append(Metadata.StringValue);
					break;
				}
			}
		});

	OtherMetadataText = FText::FromString(FString(OtherMetadata.ToView()));

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EActiveTimerReturnType SSessionInfoWindow::UpdateActiveDuration(double InCurrentTime, float InDeltaTime)
{
	DurationActive += InDeltaTime;

	// The window will explicitly unregister this active timer when the mouse leaves.
	return EActiveTimerReturnType::Continue;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSessionInfoWindow::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);

	if (!ActiveTimerHandle.IsValid())
	{
		ActiveTimerHandle = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SSessionInfoWindow::UpdateActiveDuration));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void SSessionInfoWindow::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseLeave(MouseEvent);

	auto PinnedActiveTimerHandle = ActiveTimerHandle.Pin();
	if (PinnedActiveTimerHandle.IsValid())
	{
		UnRegisterActiveTimer(PinnedActiveTimerHandle.ToSharedRef());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SSessionInfoWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SSessionInfoWindow::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (FInsightsManager::Get()->OnDragOver(DragDropEvent))
	{
		return FReply::Handled();
	}

	return SCompoundWidget::OnDragOver(MyGeometry, DragDropEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SSessionInfoWindow::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (FInsightsManager::Get()->OnDrop(DragDropEvent))
	{
		return FReply::Handled();
	}

	return SCompoundWidget::OnDrop(MyGeometry, DragDropEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SSessionInfoWindow::GetFileSizeText() const
{
	//TODO
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SSessionInfoWindow::GetStatusText() const
{
	TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();
	InsightsManager->UpdateSessionDuration();

	FNumberFormattingOptions FormattingOptions;
	FormattingOptions.MinimumFractionalDigits = 1;
	FormattingOptions.MaximumFractionalDigits = 1;

	const int32 NumDigits = InsightsManager->IsAnalysisComplete() ? 2 : 0;

	using namespace UE::Insights;
	FText Status = FText::Format(LOCTEXT("StatusFmt", "{0}\nSession Duration: {1}\nAnalyzed in {2} at {3}X speed."),
		InsightsManager->IsAnalysisComplete() ?
			LOCTEXT("Status_AnalysisCompleted", "ANALYSIS COMPLETED.") :
			LOCTEXT("Status_Analyzing", "ANALYZING..."),
		FText::FromString(FormatTimeAuto(InsightsManager->GetSessionDuration(), NumDigits)),
		FText::FromString(FormatTimeAuto(InsightsManager->GetAnalysisDuration(), NumDigits)),
		FText::AsNumber(InsightsManager->GetAnalysisSpeedFactor(), &FormattingOptions));

	return Status;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText SSessionInfoWindow::GetModulesText() const
{
	FString ModulesStr;
	TArray<TraceServices::FModuleInfoEx> Modules;

	TSharedPtr<TraceServices::IModuleService> ModuleService = FInsightsManager::Get()->GetModuleService();
	if (ModuleService)
	{
		ModuleService->GetAvailableModulesEx(Modules);
	}

	Modules.Sort([](const TraceServices::FModuleInfoEx& LHS, const TraceServices::FModuleInfoEx& RHS)
		{
			return FCString::Strcmp(LHS.Info.DisplayName, RHS.Info.DisplayName) < 0;
		});

	bool bIsFirst = true;
	for (const TraceServices::FModuleInfoEx& Module : Modules)
	{
		if (bIsFirst)
		{
			bIsFirst = false;
		}
		else
		{
			ModulesStr += TEXT(", ");
		}
		if (!Module.bIsEnabled)
		{
			ModulesStr += TEXT("!");
		}
		ModulesStr += Module.Info.DisplayName;
	}

	return FText::FromString(ModulesStr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
