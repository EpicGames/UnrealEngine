// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Docking/TabManager.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Misc/Guid.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Views/SListView.h"

// TraceServices
#include "TraceServices/ModuleService.h"

// TraceInsights
#include "Insights/InsightsManager.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

class FActiveTimerHandle;
class SVerticalBox;
class SEditableTextBox;

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FSessionInfoTabs
{
	// Tab identifiers
	static const FName SessionInfoID;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Implements the Start Page window. */
class SSessionInfoWindow : public SCompoundWidget
{
public:
	/** Default constructor. */
	SSessionInfoWindow();

	/** Virtual destructor. */
	virtual ~SSessionInfoWindow();

	SLATE_BEGIN_ARGS(SSessionInfoWindow) {}
	SLATE_END_ARGS()

	/** Constructs this widget. */
	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);

private:
	/** Updates the amount of time the profiler has been active. */
	EActiveTimerReturnType UpdateActiveDuration(double InCurrentTime, float InDeltaTime);

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/**
	 * The system will use this event to notify a widget that the cursor has entered it. This event is NOT bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 */
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/**
	 * The system will use this event to notify a widget that the cursor has left it. This event is NOT bubbled.
	 *
	 * @param MouseEvent Information about the input event
	 */
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	/**
	 * Called after a key is pressed when this widget has focus
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param  InKeyEvent  Key event
	 *
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/**
	 * Called when the user is dropping something onto a widget; terminates drag and drop.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/**
	 * Called during drag and drop when the the mouse is being dragged over a widget.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)  override;

	/**
	 * Fill the main menu with menu items.
	 *
	 * @param MenuBuilder The multi-box builder that should be filled with content for this pull-down menu.
	 * @param TabManager A Tab Manager from which to populate tab spawner menu items.
	 */
	static void FillMenu(FMenuBuilder& MenuBuilder, const TSharedPtr<FTabManager> TabManager);

private:
	void BeginSection(TSharedPtr<SVerticalBox> InVerticalBox, const FText& InSectionName) const;
	void EndSection(TSharedPtr<SVerticalBox> InVerticalBox) const;
	TSharedRef<SWidget> CreateTextBox(const TAttribute<FText>& InText, bool bMultiLine) const;
	void AddInfoLine(TSharedPtr<SVerticalBox> InVerticalBox,
					 const FText& InHeader,
					 FText(SSessionInfoWindow::* InGetTextMethodPtr)() const,
					 EVisibility(SSessionInfoWindow::* InVisibilityMethodPtr)() const,
					 bool bMultiLine = false) const;
	void AddSimpleInfoLine(TSharedPtr<SVerticalBox> InVerticalBox, const TAttribute<FText>& InValue, bool bMultiLine = false) const;

	TSharedRef<SDockTab> SpawnTab_SessionInfo(const FSpawnTabArgs& Args);
	void OnSessionInfoTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	void UpdateStreamTypeAndSessionName(const TraceServices::IAnalysisSession& Session);
	bool TryUpdateDiagnosticsInfo(const TraceServices::IAnalysisSession& Session);

	FText GetStreamTypeText() const { return StreamTypeText; }
	FText GetSessionNameText() const { return SessionNameText; }
	FText GetUriText() const { return UriText; }

	FText GetPlatformText() const { return PlatformText; }
	FText GetAppNameText() const { return AppNameText; }
	FText GetProjectNameText() const { return ProjectNameText; }
	FText GetEngineVersionText() const { return EngineVersionText; }
	FText GetBuildVersionText() const { return BuildVersionText; }
	FText GetBranchText() const { return BranchText; }
	FText GetChangelistText() const { return ChangelistText; }
	FText GetBuildConfigText() const { return BuildConfigurationTypeText; }
	FText GetBuildTargetText() const { return BuildTargetTypeText; }
	FText GetCommandLineText() const { return CommandLineText; }
	FText GetVFSPathsText() const { return VFSPathsText; }
	FText GetOtherMetadataText() const { return OtherMetadataText; }

	EVisibility IsAlwaysVisible() const { return EVisibility::Visible; }

	EVisibility IsVisibleUriText() const { return UriText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }
	EVisibility IsVisiblePlatformText() const { return PlatformText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }
	EVisibility IsVisibleAppNameText() const { return AppNameText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }
	EVisibility IsVisibleProjectNameText() const { return ProjectNameText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }
	EVisibility IsVisibleEngineVersionText() const { return EngineVersionText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }
	EVisibility IsVisibleBranchText() const { return BranchText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }
	EVisibility IsVisibleBuildVersionText() const { return BuildVersionText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }
	EVisibility IsVisibleChangelistText() const { return ChangelistText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }
	EVisibility IsVisibleBuildConfigText() const { return BuildConfigurationTypeText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }
	EVisibility IsVisibleBuildTargetText() const { return BuildTargetTypeText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }
	EVisibility IsVisibleCommandLineText() const { return CommandLineText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }
	EVisibility IsVisibleVFSPathsText() const { return VFSPathsText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }
	EVisibility IsVisibleOtherMetadataText() const { return OtherMetadataText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }

	FText GetFileSizeText() const;
	FText GetStatusText() const;

	FText GetModulesText() const;

public:
	/** The number of seconds the profiler has been active */
	float DurationActive;

private:
	/** The handle to the active update duration tick */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	/** Holds the tab manager that manages the front-end's tabs. */
	TSharedPtr<FTabManager> TabManager;

	FText StreamTypeText;
	FText SessionNameText;
	FText UriText;

	FText PlatformText;
	FText AppNameText;
	FText ProjectNameText;
	FText EngineVersionText;
	FText BuildVersionText;
	FText BranchText;
	FText ChangelistText;
	FText BuildConfigurationTypeText;
	FText BuildTargetTypeText;
	FText CommandLineText;
	FText VFSPathsText;
	FText OtherMetadataText;

	TSharedPtr<const TraceServices::IAnalysisSession> AnalysisSession;
	bool bIsSessionInfoSet = false;
};

} // namespace UE::Insights
