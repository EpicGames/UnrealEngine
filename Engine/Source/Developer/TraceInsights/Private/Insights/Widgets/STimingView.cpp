// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimingView.h"

#include "DesktopPlatformModule.h"
#include "Containers/ArrayBuilder.h"
#include "Containers/MapBuilder.h"
#include "Features/IModularFeatures.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformTime.h"
#include "Layout/WidgetPath.h"
#include "Logging/MessageLog.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Rendering/DrawElements.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

// TraceAnalysis
#include "Trace/DataStream.h"
#include "Trace/OutDataStream.h"
#include "Trace/StoreClient.h"

// TraceServices
#include "Common/ProviderLock.h"
#include "TraceServices/Model/LoadTimeProfiler.h"
#include "TraceServices/Model/Regions.h"
#include "TraceServices/TraceTrimmer.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Filter/ViewModels/FilterConfigurator.h"
#include "InsightsCore/Filter/ViewModels/TimeFilterValueConverter.h"
#include "InsightsCore/Table/Widgets/STableTreeView.h"

// TraceInsights
#include "Insights/Common/InsightsMenuBuilder.h"
#include "Insights/Config.h"
#include "Insights/InsightsManager.h"
#include "Insights/InsightsStyle.h"
#include "Insights/ITimingViewExtender.h"
#include "Insights/LoadingProfiler/LoadingProfilerManager.h"
#include "Insights/LoadingProfiler/Tracks/LoadingTimingTrack.h"
#include "Insights/LoadingProfiler/ViewModels/LoadingSharedState.h"
#include "Insights/LoadingProfiler/Widgets/SLoadingProfilerWindow.h"
#include "Insights/Log.h"
#include "Insights/TaskGraphProfiler/TaskGraphProfilerManager.h"
#include "Insights/Tests/TimingProfilerTests.h"
#include "Insights/TimingProfiler/GraphTracks/TimingGraphTrack.h"
#include "Insights/TimingProfiler/TimingProfilerManager.h"
#include "Insights/TimingProfiler/Tracks/FileActivityTimingTrack.h"
#include "Insights/TimingProfiler/Tracks/MarkersTimingTrack.h"
#include "Insights/TimingProfiler/Tracks/RegionsTimingTrack.h"
#include "Insights/TimingProfiler/Tracks/ThreadTimingTrack.h"
#include "Insights/TimingProfiler/Tracks/TimeRulerTrack.h"
#include "Insights/TimingProfiler/UserAnnotationsTimingViewExtender.h"
#include "Insights/TraceInsightsModule.h"
#include "Insights/TimingProfiler/ViewModels/FileActivitySharedState.h"
#include "Insights/TimingProfiler/ViewModels/FrameTimingTrack.h"
#include "Insights/TimingProfiler/ViewModels/ThreadTimingSharedState.h"
#include "Insights/TimingProfiler/ViewModels/TimeMarker.h"
#include "Insights/TimingProfiler/ViewModels/TimerFilters.h"
#include "Insights/TimingProfiler/ViewModels/TimingRegionsSharedState.h"
#include "Insights/TimingProfiler/Widgets/SGoToFrameDialog.h"
#include "Insights/TimingProfiler/Widgets/SGoToTimeDialog.h"
#include "Insights/TimingProfiler/Widgets/SStatsView.h"
#include "Insights/TimingProfiler/Widgets/STimersView.h"
#include "Insights/TimingProfiler/Widgets/STimingProfilerWindow.h"
#include "Insights/TimingProfiler/Models/UserAnnotation.h"
#include "Insights/TimingProfiler/Models/UserAnnotationStore.h"
#include "Insights/TimingProfilerCommon.h"
#include "Insights/ViewModels/BaseTimingTrack.h"
#include "Insights/ViewModels/DrawHelpers.h"
#include "Insights/ViewModels/GraphSeries.h"
#include "Insights/ViewModels/GraphTrack.h"
#include "Insights/ViewModels/QuickFind.h"
#include "Insights/ViewModels/ThreadTrackEvent.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TimingViewDrawHelper.h"
#include "Insights/Widgets/SLogView.h"
#include "Insights/Widgets/SQuickFind.h"
#include "Insights/Widgets/STimingViewTrackList.h"

#include <limits>

#define LOCTEXT_NAMESPACE "UE::Insights::TimingProfiler::STimingView"

#define INSIGHTS_ACTIVATE_BENCHMARK 0

////////////////////////////////////////////////////////////////////////////////////////////////////

class FScopedLog
{
public:
	FScopedLog(const TCHAR* InName, double InMinDuration = 0.001)
		: Name(InName), MinDuration(InMinDuration)
	{
		Stopwatch.Start();
	}

	~FScopedLog()
	{
		Stopwatch.Stop();
		const double Duration = Stopwatch.GetAccumulatedTime();
		if (Duration > MinDuration)
		{
			UE_LOGF(LogTimingProfiler, Log, "%ls took %.3f seconds.", Name, Duration);
		}
	}

private:
	UE::Insights::FStopwatch Stopwatch;
	const TCHAR* Name;
	double MinDuration = 0.0;
};

#define INSIGHTS_SCOPED_LOG(Name) \
	FScopedLog ScopedLog(TEXT(#Name));

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights::Timing
{
	const FName TimingViewExtenderFeatureName(TEXT("TimingViewExtender"));
}
#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
namespace Insights
{
	const FName TimingViewExtenderFeatureName(TEXT("TimingViewExtenderOld"));
}
namespace UE::Insights
{
	const FName TimingViewExtenderFeatureName(TEXT("TimingViewExtenderOld"));
}
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54

namespace UE::Insights::TimingProfiler
{

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 STimingView::TimingViewId = 0;

////////////////////////////////////////////////////////////////////////////////////////////////////

STimingView::STimingView()
	: bScrollableTracksOrderIsDirty(false)
	, FrameSharedState(MakeShared<FFrameSharedState>(this))
	, ThreadTimingSharedState(MakeShared<FThreadTimingSharedState>(this))
	, LoadingSharedState(MakeShared<LoadingProfiler::FLoadingSharedState>(this))
	, FileActivitySharedState(MakeShared<FFileActivitySharedState>(this))
	, TimingRegionsSharedState(MakeShared<FTimingRegionsSharedState>(this))
	, TimeRulerTrack(MakeShared<FTimeRulerTrack>())
	, DefaultTimeMarker(MakeShared<FTimeMarker>())
	, MarkersTrack(MakeShared<FMarkersTimingTrack>())
	, bAllowPanningOnScreenEdges(false)
	, DPIScaleFactor(1.0f)
	, EdgeFrameCountX(0)
	, EdgeFrameCountY(0)
	, WhiteBrush(FInsightsStyle::Get().GetBrush("WhiteBrush"))
	, MainFont(FAppStyle::Get().GetFontStyle("SmallFont"))
	, QuickFindTabId(TEXT("InsightsQuickFind"), TimingViewId++)
{
	DefaultTimeMarker->SetName(TEXT(""));
	DefaultTimeMarker->SetColor(FLinearColor(0.85f, 0.5f, 0.03f, 0.5f));

	IModularFeatures::Get().RegisterModularFeature(Timing::TimingViewExtenderFeatureName, FrameSharedState.Get());
	IModularFeatures::Get().RegisterModularFeature(Timing::TimingViewExtenderFeatureName, ThreadTimingSharedState.Get());
	IModularFeatures::Get().RegisterModularFeature(Timing::TimingViewExtenderFeatureName, LoadingSharedState.Get());
	IModularFeatures::Get().RegisterModularFeature(Timing::TimingViewExtenderFeatureName, FileActivitySharedState.Get());
	IModularFeatures::Get().RegisterModularFeature(Timing::TimingViewExtenderFeatureName, TimingRegionsSharedState.Get());

	ExtensionOverlay = SNew(SOverlay).Visibility(EVisibility::SelfHitTestInvisible);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

STimingView::~STimingView()
{
	AllTracks.Reset();
	TopDockedTracks.Reset();
	BottomDockedTracks.Reset();
	ScrollableTracks.Reset();
	ForegroundTracks.Reset();

	SelectedEvent.Reset();
	CustomFilterConfigurator.Reset();

	for (Timing::ITimingViewExtender* Extender : GetExtenders())
	{
		Extender->OnEndSession(*this);
	}
#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	::Insights::ITimingViewSession* CurrentTimingViewSession = (::Insights::ITimingViewSession*)(Timing::ITimingViewSession*)this;
	for (::Insights::ITimingViewExtender* Extender : GetOldExtenders())
	{
		Extender->OnEndSession(*CurrentTimingViewSession);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54

	IModularFeatures::Get().UnregisterModularFeature(Timing::TimingViewExtenderFeatureName, TimingRegionsSharedState.Get());
	IModularFeatures::Get().UnregisterModularFeature(Timing::TimingViewExtenderFeatureName, FileActivitySharedState.Get());
	IModularFeatures::Get().UnregisterModularFeature(Timing::TimingViewExtenderFeatureName, LoadingSharedState.Get());
	IModularFeatures::Get().UnregisterModularFeature(Timing::TimingViewExtenderFeatureName, ThreadTimingSharedState.Get());
	IModularFeatures::Get().UnregisterModularFeature(Timing::TimingViewExtenderFeatureName, FrameSharedState.Get());

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(QuickFindTabId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::Construct(const FArguments& InArgs, FName InViewName)
{
	ViewName = InViewName;

	GraphTrack = MakeShared<FTimingGraphTrack>(SharedThis(this));
	GraphTrack->SetName(TEXT("Main Graph"));

	FSlimHorizontalToolBarBuilder LeftToolbar(nullptr, FMultiBoxCustomization::None);
	LeftToolbar.SetStyle(&FInsightsStyle::Get(), "SecondaryToolbar");

	LeftToolbar.BeginSection("Menus");
	LeftToolbar.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateSP(this, &STimingView::MakeAllTracksMenu),
		LOCTEXT("AllTracksMenu", "All Tracks"),
		LOCTEXT("AllTracksMenuToolTip", "The list of all available tracks"),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.AllTracksMenu.ToolBar"));
	LeftToolbar.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateSP(this, &STimingView::MakeCpuGpuTracksFilterMenu),
		LOCTEXT("CpuGpuTracksMenu", "CPU/GPU"),
		LOCTEXT("CpuGpuTracksMenuToolTip", "The CPU/GPU timing tracks"),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.CpuGpuTracksMenu.ToolBar"));
	LeftToolbar.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateSP(this, &STimingView::MakeOtherTracksFilterMenu),
		LOCTEXT("OtherTracksMenu", "Other"),
		LOCTEXT("OtherTracksMenuToolTip", "Other type of tracks"),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.OtherTracksMenu.ToolBar"));
	LeftToolbar.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateSP(this, &STimingView::MakePluginTracksFilterMenu),
		LOCTEXT("PluginTracksMenu", "Plugins"),
		LOCTEXT("PluginTracksMenuToolTip", "Tracks added by plugins"),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.PluginTracksMenu.ToolBar"));
	LeftToolbar.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateSP(this, &STimingView::MakeViewModeMenu),
		LOCTEXT("ViewModeMenu", "View Mode"),
		LOCTEXT("ViewModeMenuToolTip", "Various options for the Timing view"),
		FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.ViewModeMenu.ToolBar"));
	LeftToolbar.EndSection();

	//////////////////////////////////////////////////

	FSlimHorizontalToolBarBuilder RightToolbar(nullptr, FMultiBoxCustomization::None);
	RightToolbar.SetStyle(&FInsightsStyle::Get(), "SecondaryToolbar2");

	FUIAction AutoScrollToggleButtonAction;
	AutoScrollToggleButtonAction.GetActionCheckState.BindLambda([this]
	{
		return bAutoScroll ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	});
	AutoScrollToggleButtonAction.ExecuteAction.BindLambda([this]
	{
		SetAutoScroll(!bAutoScroll);
		Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HInvalidated);
	});

	RightToolbar.BeginSection("Auto-Scroll");
	RightToolbar.AddToolBarButton(
		AutoScrollToggleButtonAction,
		NAME_None,
		TAttribute<FText>(),
		LOCTEXT("AutoScrollToolTip", "Auto-Scroll"),
		FSlateIcon(FInsightsStyle::GetStyleSetName(),"Icons.AutoScroll"),
		EUserInterfaceActionType::ToggleButton);
	RightToolbar.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateSP(this, &STimingView::MakeAutoScrollOptionsMenu),
		TAttribute<FText>(),
		LOCTEXT("AutoScrollOptionsToolTip", "Auto-Scroll Options"),
		TAttribute<FSlateIcon>(),
		true);
	RightToolbar.EndSection();

	for (Timing::ITimingViewExtender* Extender : GetExtenders())
	{
		Extender->ExtendMenuToolbars(*this, LeftToolbar, RightToolbar);
	}

	//////////////////////////////////////////////////

	ChildSlot
	[
		SNew(SOverlay)
		.Visibility(EVisibility::SelfHitTestInvisible)

		+ SOverlay::Slot()
		.VAlign(VAlign_Bottom)
		.Padding(FMargin(0.0f, 0.0f, 10.0f, 0.0f))
		[
			SAssignNew(HorizontalScrollBar, SScrollBar)
			.Orientation(Orient_Horizontal)
			.AlwaysShowScrollbar(false)
			.Visibility(EVisibility::Visible)
			.OnUserScrolled(this, &STimingView::HorizontalScrollBar_OnUserScrolled)
		]

		+ SOverlay::Slot()
		.HAlign(HAlign_Right)
		.Padding(FMargin(0.0f, 0.0f, 2.0f, 0.0f))
		[
			SAssignNew(VerticalScrollBar, SScrollBar)
			.Orientation(Orient_Vertical)
			.AlwaysShowScrollbar(false)
			.Visibility(EVisibility::Visible)
			.OnUserScrolled(this, &STimingView::VerticalScrollBar_OnUserScrolled)
		]

		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		.Padding(FMargin(0.0f))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.FillWidth(1.0f)
			[
				LeftToolbar.MakeWidget()
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				RightToolbar.MakeWidget()
			]
		]

		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(FMargin(0.0f))
		[
			ExtensionOverlay.ToSharedRef()
		]
	];

	UpdateHorizontalScrollBar();
	UpdateVerticalScrollBar();

	BindCommands();

	FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(QuickFindTabId,
		FOnSpawnTab::CreateSP(this, &STimingView::SpawnQuickFindTab))
		.SetDisplayName(LOCTEXT("QuickFindTabTitle", "Quick Find"))
		.SetMenuType(ETabSpawnerMenuType::Hidden)
		.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.Find"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FFilterConfigurator> STimingView::MakeCustomFilterConfigurator(uint32 TimerId, double IntervalStart, double IntervalEnd)
{
	if (CustomFilterConfigurator.IsValid())
	{
		CustomFilterConfigurator.Reset();
	}

	CustomFilterConfigurator = MakeShared<FFilterConfigurator>();

	CustomFilterConfigurator->Add(MakeShared<FFilter>(
		static_cast<int32>(EFilterField::StartTime),
		LOCTEXT("StartTime", "Start Time"),
		LOCTEXT("StartTime", "Start Time"),
		EFilterDataType::Double,
		MakeShared<FTimeFilterValueConverter>(),
		FFilterService::Get()->GetDoubleOperators()));

	CustomFilterConfigurator->Add(MakeShared<FFilter>(
		static_cast<int32>(EFilterField::EndTime),
		LOCTEXT("EndTime", "End Time"),
		LOCTEXT("EndTime", "End Time"),
		EFilterDataType::Double,
		MakeShared<FTimeFilterValueConverter>(),
		FFilterService::Get()->GetDoubleOperators()));

	CustomFilterConfigurator->Add(MakeShared<FFilter>(
		static_cast<int32>(EFilterField::TimerId),
		LOCTEXT("TimerId", "Timer Id"),
		LOCTEXT("TimerId", "Timer Id"),
		EFilterDataType::Int64,
		nullptr,
		FFilterService::Get()->GetIntegerOperators()));

	FFilterConfiguratorNodePtr StartTimeNode = MakeShared<FFilterConfiguratorNode>(false);
	StartTimeNode->SetSelectedFilter(CustomFilterConfigurator->GetRootNode()->GetAvailableFilters()->GetData()[0]);
	StartTimeNode->SetTextBoxValue(FString::SanitizeFloat(IntervalStart));
	StartTimeNode->SetSelectedFilterOperator(MakeShared<FFilterOperator<double>>(EFilterOperator::Gte, TEXT(""), std::greater_equal<>()));

	FFilterConfiguratorNodePtr EndTimeNode = MakeShared<FFilterConfiguratorNode>(false);
	EndTimeNode->SetSelectedFilter(CustomFilterConfigurator->GetRootNode()->GetAvailableFilters()->GetData()[1]);
	EndTimeNode->SetTextBoxValue(FString::SanitizeFloat(IntervalEnd));

	EndTimeNode->SetSelectedFilterOperator(MakeShared<FFilterOperator<double>>(EFilterOperator::Lte, TEXT(""), std::less_equal<>()));
	FFilterConfiguratorNodePtr TimerIdNode = MakeShared<FFilterConfiguratorNode>(false);
	TimerIdNode->SetSelectedFilter(CustomFilterConfigurator->GetRootNode()->GetAvailableFilters()->GetData()[2]);
	TimerIdNode->SetTextBoxValue(FString::FromInt(TimerId));

	CustomFilterConfigurator->GetRootNode()->AddChildAndSetParent(StartTimeNode);
	CustomFilterConfigurator->GetRootNode()->AddChildAndSetParent(EndTimeNode);
	CustomFilterConfigurator->GetRootNode()->AddChildAndSetParent(TimerIdNode);
	CustomFilterConfigurator->GetRootNode()->ProcessFilter();

	return CustomFilterConfigurator;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::HideAllDefaultTracks()
{
	FrameSharedState->HideAllFrameTracks();
	LoadingSharedState->HideAllLoadingTracks();
	FileActivitySharedState->HideAllIoTracks();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::Reset(bool bIsFirstReset)
{
	LLM_SCOPE_BYTAG(Insights);

	const FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();

	if (!bIsFirstReset)
	{
		for (Timing::ITimingViewExtender* Extender : GetExtenders())
		{
			Extender->OnEndSession(*this);
		}
#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		::Insights::ITimingViewSession* CurrentTimingViewSession = (::Insights::ITimingViewSession*)(Timing::ITimingViewSession*)this;
		for (::Insights::ITimingViewExtender* Extender : GetOldExtenders())
		{
			Extender->OnEndSession(*CurrentTimingViewSession);
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
	}

	//////////////////////////////////////////////////

	Viewport.Reset();

	if (IsAutoHideEmptyTracksEnabled() != Settings.IsAutoHideEmptyTracksEnabled())
	{
		ToggleAutoHideEmptyTracks();
	}

	Viewport.SetScaleX(100.0 / FMath::Clamp(Settings.GetDefaultZoomLevel(), 0.00000001, 3600.0));

	//////////////////////////////////////////////////

	for (auto& KV : AllTracks)
	{
		KV.Value->SetLocation(ETimingTrackLocation::None);
	}

	AllTracks.Reset();
	TopDockedTracks.Reset();
	BottomDockedTracks.Reset();
	ScrollableTracks.Reset();
	ForegroundTracks.Reset();

	bScrollableTracksOrderIsDirty = false;

	FTimingEventsTrack::bUseDownSampling = true;

	//////////////////////////////////////////////////

	TimeRulerTrack->Reset();
	AddTopDockedTrack(TimeRulerTrack);
	TimeRulerTrack->AddTimeMarker(DefaultTimeMarker);
	SetTimeMarker(std::numeric_limits<double>::infinity());

#if 0 // test for multiple time markers
	TSharedRef<FTimeMarker> TimeMarkerA = DefaultTimeMarker;
	TimeMarkerA->SetName(TEXT("A"));
	TimeMarkerA->SetColor(FLinearColor(0.85f, 0.5f, 0.03f, 0.5f));

	TSharedRef<FTimeMarker> TimeMarkerB = MakeShared<FTimeMarker>();
	TimeRulerTrack->AddTimeMarker(TimeMarkerB);
	TimeMarkerB->SetName(TEXT("B"));
	TimeMarkerB->SetColor(FLinearColor(0.03f, 0.85f, 0.5f, 0.5f));

	TSharedRef<FTimeMarker> TimeMarkerC = MakeShared<FTimeMarker>();
	TimeRulerTrack->AddTimeMarker(TimeMarkerC);
	TimeMarkerC->SetName(TEXT("C"));
	TimeMarkerC->SetColor(FLinearColor(0.03f, 0.5f, 0.85f, 0.5f));

	TimeMarkerA->SetTime(0.0f);
	TimeMarkerB->SetTime(1.0f);
	TimeMarkerC->SetTime(2.0f);
#endif

	MarkersTrack->Reset();
	AddTopDockedTrack(MarkersTrack);

	GraphTrack->Reset();
	GraphTrack->SetOrder(FTimingTrackOrder::First);
	constexpr double GraphTrackHeight = 200.0;
	GraphTrack->SetHeight(static_cast<float>(GraphTrackHeight));
	GraphTrack->GetSharedValueViewport().SetBaselineY(GraphTrackHeight - 1.0);
	GraphTrack->GetSharedValueViewport().SetScaleY(GraphTrackHeight / 0.1); // 100ms
	GraphTrack->AddDefaultFrameSeries();
	GraphTrack->SetVisibilityFlag(false);
	AddTopDockedTrack(GraphTrack);

	//////////////////////////////////////////////////

	ExtensionOverlay->ClearChildren();

	//////////////////////////////////////////////////

	MousePosition = FVector2D::ZeroVector;

	MousePositionOnButtonDown = FVector2D::ZeroVector;
	ViewportStartTimeOnButtonDown = 0.0;
	ViewportScrollPosYOnButtonDown = 0.0f;

	MousePositionOnButtonUp = FVector2D::ZeroVector;

	LastScrollPosY = 0.0f;

	bIsLMB_Pressed = false;
	bIsRMB_Pressed = false;

	bIsSpaceBarKeyPressed = false;
	bIsDragging = false;

	bAutoScroll = Settings.IsAutoScrollEnabled();
	AutoScrollFrameAlignment = Settings.GetAutoScrollFrameAlignment();
	AutoScrollViewportOffsetPercent = Settings.GetAutoScrollViewportOffsetPercent();
	AutoScrollMinDelay = Settings.GetAutoScrollMinDelay();
	LastAutoScrollTime = 0;

	bIsPanning = false;
	bAllowPanningOnScreenEdges = Settings.IsPanningOnScreenEdgesEnabled();
	DPIScaleFactor = 1.0f;
	EdgeFrameCountX = 0;
	EdgeFrameCountY = 0;
	PanningMode = EPanningMode::None;

	OverscrollLeft = 0.0f;
	OverscrollRight = 0.0f;
	OverscrollTop = 0.0f;
	OverscrollBottom = 0.0f;

	bIsSelecting = false;
	SelectionStartTime = 0.0;
	SelectionEndTime = 0.0;
	RaiseSelectionChanged();

	if (HoveredTrack.IsValid())
	{
		HoveredTrack.Reset();
		OnHoveredTrackChangedDelegate.Broadcast(HoveredTrack);
	}
	if (HoveredEvent.IsValid())
	{
		HoveredEvent.Reset();
		OnHoveredEventChangedDelegate.Broadcast(HoveredEvent);
	}

	if (SelectedTrack.IsValid())
	{
		SelectedTrack.Reset();
		OnSelectedTrackChangedDelegate.Broadcast(SelectedTrack);
	}
	if (SelectedEvent.IsValid())
	{
		SelectedEvent.Reset();
		OnSelectedEventChangedDelegate.Broadcast(SelectedEvent);
	}

	if (TimingEventFilter.IsValid())
	{
		TimingEventFilter.Reset();
	}

	bPreventThrottling = false;

	Tooltip.Reset();

	LastSelectionType = ESelectionType::None;

	//ThisGeometry

	bDrawTopSeparatorLine = false;
	bDrawBottomSeparatorLine = false;

	//////////////////////////////////////////////////

	NumUpdatedEvents = 0;
	PreUpdateTracksDurationHistory.Reset();
	PreUpdateTracksDurationHistory.AddValue(0);
	UpdateTracksDurationHistory.Reset();
	UpdateTracksDurationHistory.AddValue(0);
	PostUpdateTracksDurationHistory.Reset();
	PostUpdateTracksDurationHistory.AddValue(0);
	TickDurationHistory.Reset();
	TickDurationHistory.AddValue(0);
	PreDrawTracksDurationHistory.Reset();
	PreDrawTracksDurationHistory.AddValue(0);
	DrawTracksDurationHistory.Reset();
	DrawTracksDurationHistory.AddValue(0);
	PostDrawTracksDurationHistory.Reset();
	PostDrawTracksDurationHistory.AddValue(0);
	OnPaintDeltaTimeHistory.Reset();
	OnPaintDeltaTimeHistory.AddValue(0);
	LastOnPaintTime = FPlatformTime::Cycles64();

	//////////////////////////////////////////////////

	for (Timing::ITimingViewExtender* Extender : GetExtenders())
	{
		Extender->OnBeginSession(*this);
	}
#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	::Insights::ITimingViewSession* CurrentTimingViewSession = (::Insights::ITimingViewSession*)(Timing::ITimingViewSession*)this;
	for (::Insights::ITimingViewExtender* Extender : GetOldExtenders())
	{
		Extender->OnBeginSession(*CurrentTimingViewSession);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::IsOldGpu1TrackVisible() const
{
	return ThreadTimingSharedState && ThreadTimingSharedState->IsOldGpu1TrackVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::IsOldGpu2TrackVisible() const
{
	return ThreadTimingSharedState && ThreadTimingSharedState->IsOldGpu2TrackVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::IsAnyGpuTrackVisible() const
{
	return ThreadTimingSharedState && ThreadTimingSharedState->IsAnyGpuTrackVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::IsGpuTrackVisible(uint32 InQueueId) const
{
	return ThreadTimingSharedState && ThreadTimingSharedState->IsGpuTrackVisible(InQueueId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::IsCpuTrackVisible(uint32 InThreadId) const
{
	return ThreadTimingSharedState && ThreadTimingSharedState->IsCpuTrackVisible(InThreadId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	//SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	FStopwatch TickStopwatch;
	TickStopwatch.Start();

	LLM_SCOPE_BYTAG(Insights);

	UpdateFilters();

	ThisGeometry = AllottedGeometry;

	Tooltip.SetFontScale(AllottedGeometry.Scale);

	bPreventThrottling = false;

	constexpr float OverscrollFadeSpeed = 2.0f;
	if (OverscrollLeft > 0.0f)
	{
		OverscrollLeft = FMath::Max(0.0f, OverscrollLeft - InDeltaTime * OverscrollFadeSpeed);
	}
	if (OverscrollRight > 0.0f)
	{
		OverscrollRight = FMath::Max(0.0f, OverscrollRight - InDeltaTime * OverscrollFadeSpeed);
	}
	if (OverscrollTop > 0.0f)
	{
		OverscrollTop = FMath::Max(0.0f, OverscrollTop - InDeltaTime * OverscrollFadeSpeed);
	}
	if (OverscrollBottom > 0.0f)
	{
		OverscrollBottom = FMath::Max(0.0f, OverscrollBottom - InDeltaTime * OverscrollFadeSpeed);
	}

	const float ViewWidth = static_cast<float>(AllottedGeometry.GetLocalSize().X);
	const float ViewHeight = static_cast<float>(AllottedGeometry.GetLocalSize().Y);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Update viewport.

	constexpr float ToolbarHeight = 32.0f;
	constexpr float HorizontalScrollBarHeight = 12.0f;

	Viewport.SetPosY(ToolbarHeight);
	Viewport.UpdateSize(FMath::RoundToFloat(ViewWidth), FMath::RoundToFloat(ViewHeight) - Viewport.GetPosY() - HorizontalScrollBarHeight);

	if (!bIsPanning && !bAutoScroll)
	{
		// Elastic snap to horizontal time limits.
		if (Viewport.EnforceHorizontalScrollLimits(0.5)) // 0.5 is the interpolation factor
		{
			UpdateHorizontalScrollBar();
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Check the analysis session time.

	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session)
	{
		double SessionTime = 0.0;
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			SessionTime = Session->GetDurationSeconds();
		}

		// Check if horizontal scroll area has changed.
		if (SessionTime > Viewport.GetMaxValidTime() &&
			SessionTime != DBL_MAX &&
			SessionTime != std::numeric_limits<double>::infinity())
		{
			const double PreviousSessionTime = Viewport.GetMaxValidTime();
			if ((PreviousSessionTime >= Viewport.GetStartTime() && PreviousSessionTime <= Viewport.GetEndTime()) ||
				(SessionTime >= Viewport.GetStartTime() && SessionTime <= Viewport.GetEndTime()))
			{
				Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HClippedSessionTimeChanged);
			}

			//UE_LOGF(LogTimingProfiler, Log, "Session Duration: %g", DT);
			Viewport.SetMaxValidTime(SessionTime);
			UpdateHorizontalScrollBar();
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	if (bIsPanning)
	{
		// Disable auto-scroll if user starts panning manually.
		SetAutoScroll(false);
	}

	if (bAutoScroll)
	{
		const uint64 CurrentTime = FPlatformTime::Cycles64();
		if (static_cast<double>(CurrentTime - LastAutoScrollTime) * FPlatformTime::GetSecondsPerCycle64() > AutoScrollMinDelay)
		{
			const double ViewportDuration = Viewport.GetEndTime() - Viewport.GetStartTime(); // width of the viewport in [seconds]
			const double AutoScrollViewportOffsetTime = ViewportDuration * AutoScrollViewportOffsetPercent;

			// By default, align the current session time with the right side of the viewport + offset.
			double MinStartTime = Viewport.GetMaxValidTime() - ViewportDuration + AutoScrollViewportOffsetTime;

			if (AutoScrollFrameAlignment >= 0)
			{
				if (Session.IsValid())
				{
					TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
					const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(*Session.Get());

					const ETraceFrameType FrameTpe = static_cast<ETraceFrameType>(AutoScrollFrameAlignment);
					const uint64 FrameCount = FramesProvider.GetFrameCount(FrameTpe);

					if (FrameCount > 0)
					{
						// Search the last frame with EndTime <= SessionTime.
						uint64 FrameIndex = FrameCount;
						while (FrameIndex > 0)
						{
							const TraceServices::FFrame* FramePtr = FramesProvider.GetFrame(FrameTpe, --FrameIndex);
							if (FramePtr && FramePtr->EndTime <= Viewport.GetMaxValidTime())
							{
								// Align the start time of the frame with the right side of the viewport.
								MinStartTime = FramePtr->EndTime - ViewportDuration + AutoScrollViewportOffsetTime;
								break;
							}
						}

						// Get the frame at the center of the viewport.
						TraceServices::FFrame Frame;
						const double ViewportCenter = MinStartTime + ViewportDuration / 2;
						if (FramesProvider.GetFrameFromTime(FrameTpe, ViewportCenter, Frame))
						{
							if (Frame.EndTime > ViewportCenter)
							{
								// Align the start time of the frame with the center of the viewport.
								MinStartTime = Frame.StartTime - ViewportDuration / 2;
							}
						}
					}
				}
			}

			ScrollAtTime(MinStartTime);
			LastAutoScrollTime = CurrentTime;
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	// Tick plugin extenders.
	// Each extender can add/remove tracks and/or change order of tracks.
	Timing::FTimingViewExtenderTickParams TickParams(*this, Session.Get());
	TickParams.DeltaTime = InDeltaTime;
	TickParams.CurrentTime = InCurrentTime;

	for (Timing::ITimingViewExtender* Extender : GetExtenders())
	{
		if (Session)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Extender->Tick(*this, *Session.Get());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		// New Tick version.
		Extender->Tick(TickParams);
	}

	if (Session)
	{
#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		::Insights::ITimingViewSession* CurrentTimingViewSession = (::Insights::ITimingViewSession*)(Timing::ITimingViewSession*)this;
		for (::Insights::ITimingViewExtender* Extender : GetOldExtenders())
		{
			Extender->Tick(*CurrentTimingViewSession, *Session.Get());
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54

		// Re-sort now (if we need to).
		UpdateScrollableTracksOrder();
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	// Animate the (vertical) layout transition (i.e. compact mode <-> normal mode).
	Viewport.UpdateLayout();

	TimeRulerTrack->SetSelection(bIsSelecting, SelectionStartTime, SelectionEndTime);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class FTimingTrackUpdateContext : public ITimingTrackUpdateContext
	{
	public:
		explicit FTimingTrackUpdateContext(STimingView* InTimingView, const FGeometry& InGeometry, double InCurrentTime, float InDeltaTime)
			: TimingView(InTimingView)
			, Geometry(InGeometry)
			, CurrentTime(InCurrentTime)
			, DeltaTime(InDeltaTime)
		{}

		virtual const FGeometry& GetGeometry() const override { return Geometry; }
		virtual const FTimingTrackViewport& GetViewport() const override { return TimingView->GetViewport(); }
		virtual const FVector2D& GetMousePosition() const override { return TimingView->GetMousePosition(); }
		virtual const TSharedPtr<const ITimingEvent> GetHoveredEvent() const override { return TimingView->GetHoveredEvent(); }
		virtual const TSharedPtr<const ITimingEvent> GetSelectedEvent() const override { return TimingView->GetSelectedEvent(); }
		virtual const TSharedPtr<ITimingEventFilter> GetEventFilter() const override { return TimingView->GetEventFilter(); }
		virtual const TArray<TUniquePtr<ITimingEventRelation>>& GetCurrentRelations() const override { return TimingView->GetCurrentRelations(); }
		virtual double GetCurrentTime() const override { return CurrentTime; }
		virtual float GetDeltaTime() const override { return DeltaTime; }

	public:
		STimingView* TimingView;
		const FGeometry& Geometry;
		double CurrentTime;
		float DeltaTime;
	};

	FTimingTrackUpdateContext UpdateContext(this, AllottedGeometry, InCurrentTime, InDeltaTime);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Pre-Update.
	// The tracks needs to update their size.

	{
		FStopwatch PreUpdateTracksStopwatch;
		PreUpdateTracksStopwatch.Start();

		for (TSharedPtr<FBaseTimingTrack>& TrackPtr : TopDockedTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PreUpdate(UpdateContext);
			}
		}

		for (TSharedPtr<FBaseTimingTrack>& TrackPtr : BottomDockedTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PreUpdate(UpdateContext);
			}
		}

		for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PreUpdate(UpdateContext);
			}
		}

		for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ForegroundTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PreUpdate(UpdateContext);
			}
		}

		PreUpdateTracksStopwatch.Stop();
		PreUpdateTracksDurationHistory.AddValue(PreUpdateTracksStopwatch.AccumulatedTime);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Update Y position for the visible top docked tracks.
	// Compute the total height of top docked areas.

	int32 NumVisibleTopDockedTracks = 0;
	float TopOffset = 0.0f;
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : TopDockedTracks)
	{
		SetTrackPosY(*TrackPtr, Viewport.GetPosY() + TopOffset);
		if (TrackPtr->IsVisible())
		{
			TopOffset += TrackPtr->GetHeight();
			NumVisibleTopDockedTracks++;
		}
	}
	if (NumVisibleTopDockedTracks > 0)
	{
		bDrawTopSeparatorLine = true;
		TopOffset += 2.0f;
	}
	else
	{
		bDrawTopSeparatorLine = false;
	}
	Viewport.SetTopOffset(TopOffset);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Update Y position for the visible bottom docked tracks.
	// Compute the total height of bottom docked areas.

	float BottomOffset = 0.0f;
	int32 NumVisibleBottomDockedTracks = 0;
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : BottomDockedTracks)
	{
		if (TrackPtr->IsVisible())
		{
			BottomOffset += TrackPtr->GetHeight();
			NumVisibleBottomDockedTracks++;
		}
	}
	if (NumVisibleBottomDockedTracks > 0)
	{
		BottomOffset += 2.0f;
		if (Viewport.GetTopOffset() + BottomOffset > Viewport.GetHeight())
		{
			BottomOffset = Viewport.GetHeight() - Viewport.GetTopOffset();
		}
		float BottomDockedTrackPosY = Viewport.GetPosY() + Viewport.GetHeight() - BottomOffset + 2.0f;
		for (TSharedPtr<FBaseTimingTrack>& TrackPtr : BottomDockedTracks)
		{
			SetTrackPosY(*TrackPtr, BottomDockedTrackPosY);
			if (TrackPtr->IsVisible())
			{
				BottomDockedTrackPosY += TrackPtr->GetHeight();
			}
		}
	}
	bDrawBottomSeparatorLine = (BottomOffset > 0.0f);
	Viewport.SetBottomOffset(BottomOffset);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Compute the total height of visible scrollable tracks.

	float ScrollHeight = 0.0f;
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
	{
		if (TrackPtr->IsVisible())
		{
			ScrollHeight += TrackPtr->GetHeight();
		}
	}
	ScrollHeight += 1.0f; // allow 1 pixel at the bottom (for last horizontal line)

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Check if vertical scroll area has changed.

	bool bScrollHeightChanged = false;
	if (ScrollHeight != Viewport.GetScrollHeight())
	{
		bScrollHeightChanged = true;
		Viewport.SetScrollHeight(ScrollHeight);
		UpdateVerticalScrollBar();
	}

	// Set the VerticalScrollBar padding so it is limited to the scrollable area.
	VerticalScrollBar->SetPadding(FMargin(0.0f, Viewport.GetPosY() + TopOffset + 2.0f, 0.0f, BottomOffset + HorizontalScrollBarHeight));

	////////////////////////////////////////////////////////////////////////////////////////////////////

	TSharedPtr<FBaseTimingTrack> PinnedTrack;
	float InitialPinnedTrackPosY = 0.0f;
	if (SelectedTrack.IsValid() && SelectedTrack->IsVisible() && DelayedVisibleTrackCounter <= 0)
	{
		if (ScrollableTracks.Contains(SelectedTrack))
		{
			PinnedTrack = SelectedTrack;
			InitialPinnedTrackPosY = PinnedTrack->GetPosY();
		}
	}

	// Update the Y position for visible scrollable tracks.
	UpdatePositionForScrollableTracks();

	// The selected track will be pinned (keeps Y pos fixed unless user scrolls vertically).
	if (PinnedTrack.IsValid())
	{
		const float InitialScrollPosY = Viewport.GetScrollPosY();
		const float AdjustmentDY = LastScrollPosY - InitialScrollPosY + InitialPinnedTrackPosY - PinnedTrack->GetPosY();
		const float NewScrollPosY = InitialScrollPosY - AdjustmentDY;
		if (!FMath::IsNearlyZero(AdjustmentDY, 0.5f))
		{
			ViewportScrollPosYOnButtonDown -= AdjustmentDY;
			ScrollAtPosY(NewScrollPosY);
			UpdatePositionForScrollableTracks();
		}
	}

	// Force the DelayedVisibleTrack to be visible.
	if (DelayedVisibleTrackCounter > 0)
	{
		if (DelayedVisibleTrack.IsValid() &&
			DelayedVisibleTrack->IsVisible() &&
			DelayedVisibleTrack->GetLocation() == ETimingTrackLocation::Scrollable)
		{
			float PreviousScrollPosY = Viewport.GetScrollPosY();

			// Try to bring into view the entire track (with 10% top/bottom margins).
			const float TrackHeight = DelayedVisibleTrack->GetHeight();
			const float TopScrollY = DelayedVisibleTrack->GetPosY() + Viewport.GetScrollPosY() - Viewport.GetTopOffset() - Viewport.GetPosY();
			const float ScrollableAreaHeight = Viewport.GetScrollableAreaHeight();
			BringIntoViewY(TopScrollY - ScrollableAreaHeight * 0.1f, TopScrollY + TrackHeight + ScrollableAreaHeight * 0.2f);

			// Try to bring into view the entire track.
			BringIntoViewY(TopScrollY, TopScrollY + TrackHeight);

			if (DelayedVisibleTrackLaneDepth >= 0)
			{
				// Try to bring into view only the lane at specified depth (with 10% top/bottom margins).
				const float LaneHeight = Viewport.GetLayout().EventH + Viewport.GetLayout().EventDY;
				const float ScrollY = TopScrollY + float(DelayedVisibleTrackLaneDepth) * LaneHeight;
				BringIntoViewY(ScrollY - ScrollableAreaHeight * 0.1f, ScrollY + LaneHeight + ScrollableAreaHeight * 0.2f);
			}

			UpdatePositionForScrollableTracks();

			if (FMath::IsNearlyZero(Viewport.GetScrollPosY() - PreviousScrollPosY, 0.5f) &&
				FMath::IsNearlyZero(Viewport.GetScrollHeight() - DelayedVisibleTrackLastScrollHeight, 0.5f))
			{
				if (--DelayedVisibleTrackCounter == 0)
				{
					DelayedVisibleTrack.Reset();
				}
			}
			else
			{
				DelayedVisibleTrackCounter = 3;
				DelayedVisibleTrackLastScrollHeight = Viewport.GetScrollHeight();
			}
		}
		else
		{
			DelayedVisibleTrack.Reset();
			DelayedVisibleTrackCounter = 0;
		}
	}

	// Elastic snap to vertical scroll limits.
	if (!bIsPanning && DelayedVisibleTrackCounter <= 0)
	{
		const float DY = Viewport.GetScrollHeight() - Viewport.GetScrollableAreaHeight();
		const float MinY = FMath::Min(DY, 0.0f);
		const float MaxY = DY - MinY;

		float ScrollPosY = Viewport.GetScrollPosY();

		if (ScrollPosY < MinY)
		{
			if (bScrollHeightChanged || Viewport.IsDirty(ETimingTrackViewportDirtyFlags::VLayoutChanged))
			{
				ScrollPosY = MinY;
			}
			else
			{
				constexpr float U = 0.5f;
				ScrollPosY = ScrollPosY * U + (1.0f - U) * MinY;
				if (FMath::IsNearlyEqual(ScrollPosY, MinY, 0.5f))
				{
					ScrollPosY = MinY;
				}
			}
		}
		else if (ScrollPosY > MaxY)
		{
			if (bScrollHeightChanged || Viewport.IsDirty(ETimingTrackViewportDirtyFlags::VLayoutChanged))
			{
				ScrollPosY = MaxY;
			}
			else
			{
				constexpr float U = 0.5f;
				ScrollPosY = ScrollPosY * U + (1.0f - U) * MaxY;
				if (FMath::IsNearlyEqual(ScrollPosY, MaxY, 0.5f))
				{
					ScrollPosY = MaxY;
				}
			}
			if (ScrollPosY < MinY)
			{
				ScrollPosY = MinY;
			}
		}

		if (ScrollPosY != Viewport.GetScrollPosY())
		{
			ScrollAtPosY(ScrollPosY);
			UpdatePositionForScrollableTracks();
		}
	}

	LastScrollPosY = Viewport.GetScrollPosY();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// At this point it is assumed all tracks have proper position and size.
	// Update.
	{
		FStopwatch UpdateTracksStopwatch;
		UpdateTracksStopwatch.Start();

		for (TSharedPtr<FBaseTimingTrack>& TrackPtr : TopDockedTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->Update(UpdateContext);
			}
		}

		for (TSharedPtr<FBaseTimingTrack>& TrackPtr : BottomDockedTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->Update(UpdateContext);
			}
		}

		for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->Update(UpdateContext);
			}
		}

		for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ForegroundTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->Update(UpdateContext);
			}
		}

		UpdateTracksStopwatch.Stop();
		UpdateTracksDurationHistory.AddValue(UpdateTracksStopwatch.AccumulatedTime);
	}
	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Post-Update.
	{
		FStopwatch PostUpdateTracksStopwatch;
		PostUpdateTracksStopwatch.Start();

		for (TSharedPtr<FBaseTimingTrack>& TrackPtr : TopDockedTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PostUpdate(UpdateContext);
			}
		}

		for (TSharedPtr<FBaseTimingTrack>& TrackPtr : BottomDockedTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PostUpdate(UpdateContext);
			}
		}

		for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PostUpdate(UpdateContext);
			}
		}

		for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ForegroundTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PostUpdate(UpdateContext);
			}
		}

		PostUpdateTracksStopwatch.Stop();
		PostUpdateTracksDurationHistory.AddValue(PostUpdateTracksStopwatch.AccumulatedTime);
	}
	////////////////////////////////////////////////////////////////////////////////////////////////////

	Tooltip.Update();
	if (!MousePosition.IsZero())
	{
		Tooltip.SetPosition(MousePosition, 0.0f, Viewport.GetWidth() - 12.0f, Viewport.GetPosY(), Viewport.GetPosY() + Viewport.GetHeight() - 12.0f); // -12.0f is to avoid overlapping the scrollbars
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Update "hovered" and "selected" flags for all visible tracks.
	//TODO: Move this before PreUpdate (so a track could adjust its size based on hovered/selected flags).

	// Reset hovered/selected flags for all tracks.
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : TopDockedTracks)
	{
		if (TrackPtr->IsVisible())
		{
			TrackPtr->SetHoveredState(false);
			TrackPtr->SetSelectedFlag(false);
		}
	}
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : BottomDockedTracks)
	{
		if (TrackPtr->IsVisible())
		{
			TrackPtr->SetHoveredState(false);
			TrackPtr->SetSelectedFlag(false);
		}
	}
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
	{
		if (TrackPtr->IsVisible())
		{
			TrackPtr->SetHoveredState(false);
			TrackPtr->SetSelectedFlag(false);
		}
	}
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ForegroundTracks)
	{
		if (TrackPtr->IsVisible())
		{
			TrackPtr->SetHoveredState(false);
			TrackPtr->SetSelectedFlag(false);
		}
	}

	// Set the hovered flag for the actual hovered track, if any.
	if (HoveredTrack.IsValid())
	{
		HoveredTrack->SetHoveredState(true);
	}

	// Set the selected flag for the actual selected track, if any.
	if (SelectedTrack.IsValid())
	{
		SelectedTrack->SetSelectedFlag(true);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	Viewport.ResetDirtyFlags();

	if (bBringSelectedEventIntoViewVerticallyOnNextTick && SelectedEvent.IsValid())
	{
		if (SelectedEvent->GetTrack()->GetLocation() == ETimingTrackLocation::Scrollable)
		{
			BringScrollableTrackIntoView(*SelectedEvent->GetTrack());
		}
	}

	bBringSelectedEventIntoViewVerticallyOnNextTick = false;

	TickStopwatch.Stop();
	TickDurationHistory.AddValue(TickStopwatch.AccumulatedTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::UpdatePositionForScrollableTracks()
{
	// Update the Y position for the visible scrollable tracks.
	float ScrollableTrackPosY = Viewport.GetPosY() + Viewport.GetTopOffset() - Viewport.GetScrollPosY();
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
	{
		SetTrackPosY(*TrackPtr, ScrollableTrackPosY);
		if (TrackPtr->IsVisible())
		{
			ScrollableTrackPosY += TrackPtr->GetHeight();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SetTrackPosY(FBaseTimingTrack& Track, float TrackPosY) const
{
	Track.SetPosY(TrackPosY);
	Track.UpdateChildTracksPosY(Viewport.GetLayout());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 STimingView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FStopwatch Stopwatch;
	Stopwatch.Start();

	const bool bEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	FDrawContext DrawContext(AllottedGeometry, MyCullingRect, InWidgetStyle, DrawEffects, OutDrawElements, LayerId);

#if 0 // Enabling this may further increase UI performance (TODO: profile if this is really needed again).
	// Avoids multiple resizes of Slate's draw elements buffers.
	OutDrawElements.GetRootDrawLayer().DrawElements.Reserve(50000);
#endif

	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const float FontScale = AllottedGeometry.Scale;

	const float ViewWidth = static_cast<float>(AllottedGeometry.GetLocalSize().X);
	const float ViewHeight = static_cast<float>(AllottedGeometry.GetLocalSize().Y);

#if 0 // Enabling this may further increase UI performance (TODO: profile if this is really needed again).
	// Warm up Slate vertex/index buffers to avoid initial freezes due to multiple resizes of those buffers.
	static bool bWarmingUp = false;
	if (!bWarmingUp)
	{
		bWarmingUp = true;

		FRandomStream RandomStream(0);
		const int32 Count = 1'000'000;
		for (int32 Index = 0; Index < Count; ++Index)
		{
			float X = ViewWidth * RandomStream.GetFraction();
			float Y = ViewHeight * RandomStream.GetFraction();
			FLinearColor Color(RandomStream.GetFraction(), RandomStream.GetFraction(), RandomStream.GetFraction(), 1.0f);
			DrawContext.DrawBox(X, Y, 1.0f, 1.0f, WhiteBrush, Color);
		}
		LayerId++;
		LayerId++;
	}
#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////

	class FTimingTrackDrawContext : public ITimingTrackDrawContext
	{
	public:
		explicit FTimingTrackDrawContext(const STimingView* InTimingView, FDrawContext& InDrawContext, const FTimingViewDrawHelper& InHelper)
			: TimingView(InTimingView)
			, DrawContext(InDrawContext)
			, Helper(InHelper)
		{}

		virtual const FTimingTrackViewport& GetViewport() const override { return TimingView->GetViewport(); }
		virtual const FVector2D& GetMousePosition() const override { return TimingView->GetMousePosition(); }
		virtual const TSharedPtr<const ITimingEvent> GetHoveredEvent() const override { return TimingView->GetHoveredEvent(); }
		virtual const TSharedPtr<const ITimingEvent> GetSelectedEvent() const override { return TimingView->GetSelectedEvent(); }
		virtual const TSharedPtr<ITimingEventFilter> GetEventFilter() const override { return TimingView->GetEventFilter(); }
		virtual FDrawContext& GetDrawContext() const override { return DrawContext; }
		virtual const ITimingViewDrawHelper& GetHelper() const override { return Helper; }

	public:
		const STimingView* TimingView;
		FDrawContext& DrawContext;
		const FTimingViewDrawHelper& Helper;
	};

	FTimingViewDrawHelper Helper(DrawContext, Viewport);
	FTimingTrackDrawContext TimingDrawContext(this, DrawContext, Helper);

	////////////////////////////////////////////////////////////////////////////////////////////////////

	// Draw background.
	Helper.DrawBackground();

	//////////////////////////////////////////////////
	// Pre-Draw
	{
		FStopwatch PreDrawTracksStopwatch;
		PreDrawTracksStopwatch.Start();

		for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PreDraw(TimingDrawContext);
			}
		}

		for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : TopDockedTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PreDraw(TimingDrawContext);
			}
		}

		for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : BottomDockedTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PreDraw(TimingDrawContext);
			}
		}

		for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : ForegroundTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PreDraw(TimingDrawContext);
			}
		}

		PreDrawTracksStopwatch.Stop();
		PreDrawTracksDurationHistory.AddValue(PreDrawTracksStopwatch.AccumulatedTime);
	}

	const FVector2f Position = FVector2f(AllottedGeometry.GetAbsolutePosition());
	const float Scale = AllottedGeometry.GetAccumulatedLayoutTransform().GetScale();

	//////////////////////////////////////////////////
	// Draw
	{
		FStopwatch DrawTracksStopwatch;
		DrawTracksStopwatch.Start();

		Helper.BeginDrawTracks();

		// Draw the scrollable tracks.
		{
			const float TopY = Viewport.GetPosY() + Viewport.GetTopOffset();
			const float BottomY = Viewport.GetPosY() + Viewport.GetHeight() - Viewport.GetBottomOffset();

			{
				const float L = Position.X;
				const float R = Position.X + Viewport.GetWidth() * Scale;
				const float T = Position.Y + TopY * Scale;
				const float B = Position.Y + BottomY * Scale;
				const FSlateClippingZone ClipZone(FSlateRect(L, T, R, B));
				DrawContext.ElementList.PushClip(ClipZone);
			}

			for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
			{
				if (TrackPtr->IsVisible())
				{
					if (TrackPtr->GetPosY() + TrackPtr->GetHeight() <= TopY)
					{
						continue;
					}
					if (TrackPtr->GetPosY() >= BottomY)
					{
						break;
					}
					TrackPtr->Draw(TimingDrawContext);
				}
			}

			// Draw relations between scrollable tracks.
			const FTimingViewDrawHelper& TimingHelper = *static_cast<const FTimingViewDrawHelper*>(&TimingDrawContext.GetHelper());
			TimingHelper.DrawRelations(CurrentRelations, ITimingEventRelation::EDrawFilter::BetweenScrollableTracks);

			DrawContext.ElementList.PopClip();
		}

		// Draw the top docked tracks.
		{
			const float TopY = Viewport.GetPosY();
			const float BottomY = Viewport.GetPosY() + Viewport.GetTopOffset();

			{
				const float L = Position.X;
				const float R = Position.X + Viewport.GetWidth() * Scale;
				const float T = Position.Y + TopY * Scale;
				const float B = Position.Y + BottomY * Scale;
				const FSlateClippingZone ClipZone(FSlateRect(L, T, R, B));
				DrawContext.ElementList.PushClip(ClipZone);
			}

			for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : TopDockedTracks)
			{
				if (TrackPtr->IsVisible())
				{
					TrackPtr->Draw(TimingDrawContext);
				}
			}

			if (bDrawTopSeparatorLine)
			{
				// Draw separator line between top docked tracks and scrollable tracks.
				DrawContext.DrawBox(0.0f, BottomY - 2.0f, Viewport.GetWidth(), 2.0f, WhiteBrush, FLinearColor(0.01f, 0.01f, 0.01f, 1.0f));
				++DrawContext.LayerId;
			}

			DrawContext.ElementList.PopClip();
		}

		// Draw the bottom docked tracks.
		{
			const float TopY = Viewport.GetPosY() + Viewport.GetHeight() - Viewport.GetBottomOffset();
			const float BottomY = Viewport.GetPosY() + Viewport.GetHeight();

			{
				const float L = Position.X;
				const float R = Position.X + Viewport.GetWidth() * Scale;
				const float T = Position.Y + TopY * Scale;
				const float B = Position.Y + BottomY * Scale;
				const FSlateClippingZone ClipZone(FSlateRect(L, T, R, B));
				DrawContext.ElementList.PushClip(ClipZone);
			}

			for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : BottomDockedTracks)
			{
				if (TrackPtr->IsVisible())
				{
					TrackPtr->Draw(TimingDrawContext);
				}
			}

			if (bDrawBottomSeparatorLine)
			{
				// Draw separator line between top docked tracks and scrollable tracks.
				DrawContext.DrawBox(0.0f, TopY, Viewport.GetWidth(), 2.0f, WhiteBrush, FLinearColor(0.01f, 0.01f, 0.01f, 1.0f));
				++DrawContext.LayerId;
			}

			DrawContext.ElementList.PopClip();
		}

		// Draw the foreground tracks.
		for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : ForegroundTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->Draw(TimingDrawContext);
			}
		}

		Helper.EndDrawTracks();

		DrawTracksStopwatch.Stop();
		DrawTracksDurationHistory.AddValue(DrawTracksStopwatch.AccumulatedTime);
	}

	//////////////////////////////////////////////////
	// Draw the selected and/or hovered event.

	if (ITimingEvent::AreValidAndEquals(SelectedEvent, HoveredEvent))
	{
		const TSharedRef<const FBaseTimingTrack> TrackPtr = SelectedEvent->GetTrack();

		// Highlight the selected and hovered timing event (if any).
		if (TrackPtr->IsVisible())
		{
			SelectedEvent->GetTrack()->DrawEvent(TimingDrawContext, *SelectedEvent, EDrawEventMode::SelectedAndHovered);
		}
	}
	else
	{
		// Highlight the selected timing event (if any).
		if (SelectedEvent.IsValid())
		{
			const TSharedRef<const FBaseTimingTrack> TrackPtr = SelectedEvent->GetTrack();
			if (TrackPtr->IsVisible())
			{
				SelectedEvent->GetTrack()->DrawEvent(TimingDrawContext, *SelectedEvent, EDrawEventMode::Selected);
			}
		}

		// Highlight the hovered timing event (if any).
		if (HoveredEvent.IsValid())
		{
			const TSharedRef<const FBaseTimingTrack> TrackPtr = HoveredEvent->GetTrack();
			if (TrackPtr->IsVisible())
			{
				HoveredEvent->GetTrack()->DrawEvent(TimingDrawContext, *HoveredEvent, EDrawEventMode::Hovered);
			}
		}
	}

	// Draw the time range selection.
	FDrawHelpers::DrawTimeRangeSelection(DrawContext, Viewport, SelectionStartTime, SelectionEndTime, WhiteBrush, MainFont);

	//////////////////////////////////////////////////
	// Post-Draw
	{
		FStopwatch PostDrawTracksStopwatch;
		PostDrawTracksStopwatch.Start();

		for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PostDraw(TimingDrawContext);
			}
		}

		for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : TopDockedTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PostDraw(TimingDrawContext);
			}
		}

		for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : BottomDockedTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PostDraw(TimingDrawContext);
			}
		}

		for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : ForegroundTracks)
		{
			if (TrackPtr->IsVisible())
			{
				TrackPtr->PostDraw(TimingDrawContext);
			}
		}

		PostDrawTracksStopwatch.Stop();
		PostDrawTracksDurationHistory.AddValue(PostDrawTracksStopwatch.AccumulatedTime);
	}

	//////////////////////////////////////////////////
	// Draw relations between docked tracks.
	{
		bool bIsClipZoneSet = false;
		if (GraphTrack->IsVisible() &&
			GraphTrack->GetLocation() == ETimingTrackLocation::TopDocked)
		{
			// Avoid overlapping the Main Graph track (when top docked).
			const float TopY = GraphTrack->GetPosY() + GraphTrack->GetHeight();
			const float BottomY = Viewport.GetPosY() + Viewport.GetHeight();

			const float L = Position.X;
			const float R = Position.X + Viewport.GetWidth() * Scale;
			const float T = Position.Y + TopY * Scale;
			const float B = Position.Y + BottomY * Scale;
			const FSlateClippingZone ClipZone(FSlateRect(L, T, R, B));
			DrawContext.ElementList.PushClip(ClipZone);
			bIsClipZoneSet = true;
		}

		const FTimingViewDrawHelper& TimingHelper = *static_cast<const FTimingViewDrawHelper*>(&TimingDrawContext.GetHelper());
		TimingHelper.DrawRelations(CurrentRelations, ITimingEventRelation::EDrawFilter::BetweenDockedTracks);

		if (bIsClipZoneSet)
		{
			DrawContext.ElementList.PopClip();
		}
	}
	//////////////////////////////////////////////////

	// Draw tooltip with info about hovered event.
	Tooltip.Draw(DrawContext);

	const FLinearColor BackgroundColor = FSlateColor(EStyleColor::Panel).GetSpecifiedColor();

	// Fill the background of the toolbar.
	DrawContext.DrawBox(0.0f, 0.0f, ViewWidth, Viewport.GetPosY(), WhiteBrush, BackgroundColor);

	// Fill the background of the vertical scrollbar.
	{
		const float ScrollBarHeight = Viewport.GetScrollableAreaHeight();
		if (ScrollBarHeight > 0)
		{
			constexpr float ScrollBarWidth = 12.0f;
			const     float ScrollBarPosX = ViewWidth - ScrollBarWidth;
			const     float ScrollBarPosY = Viewport.GetPosY() + Viewport.GetTopOffset();
			DrawContext.DrawBox(ScrollBarPosX, ScrollBarPosY, ScrollBarWidth, ScrollBarHeight, WhiteBrush, BackgroundColor);
		}
	}
	// Fill the background of the horizontal scrollbar.
	{
		const     float ScrollBarWidth = ViewWidth;
		constexpr float ScrollBarHeight = 12.0f;
		constexpr float ScrollBarPosX = 0.0f;
		const     float ScrollBarPosY = ViewHeight - ScrollBarHeight;
		DrawContext.DrawBox(ScrollBarPosX, ScrollBarPosY, ScrollBarWidth, ScrollBarHeight, WhiteBrush, BackgroundColor);
	}

	//////////////////////////////////////////////////
	// Draw the overscroll indication lines.

	constexpr float OverscrollLineSize = 1.0f;
	constexpr int32 OverscrollLineCount = 8;

	if (OverscrollLeft > 0.0f)
	{
		// TODO: single box with gradient opacity
		const float OverscrollLineY = Viewport.GetPosY();
		const float OverscrollLineH = Viewport.GetHeight();
		for (int32 LineIndex = 0; LineIndex < OverscrollLineCount; ++LineIndex)
		{
			const float Opacity = OverscrollLeft * static_cast<float>(OverscrollLineCount - LineIndex) / static_cast<float>(OverscrollLineCount);
			DrawContext.DrawBox(static_cast<float>(LineIndex) * OverscrollLineSize, OverscrollLineY, OverscrollLineSize, OverscrollLineH, WhiteBrush, FLinearColor(1.0f, 0.1f, 0.1f, Opacity));
		}
	}
	if (OverscrollRight > 0.0f)
	{
		const float OverscrollLineY = Viewport.GetPosY();
		const float OverscrollLineH = Viewport.GetHeight();
		for (int32 LineIndex = 0; LineIndex < OverscrollLineCount; ++LineIndex)
		{
			const float Opacity = OverscrollRight * static_cast<float>(OverscrollLineCount - LineIndex) / static_cast<float>(OverscrollLineCount);
			DrawContext.DrawBox(ViewWidth - static_cast<float>(1 + LineIndex) * OverscrollLineSize, OverscrollLineY, OverscrollLineSize, OverscrollLineH, WhiteBrush, FLinearColor(1.0f, 0.1f, 0.1f, Opacity));
		}
	}
	if (OverscrollTop > 0.0f)
	{
		const float OverscrollLineY = Viewport.GetPosY() + Viewport.GetTopOffset();
		for (int32 LineIndex = 0; LineIndex < OverscrollLineCount; ++LineIndex)
		{
			const float Opacity = OverscrollTop * static_cast<float>(OverscrollLineCount - LineIndex) / static_cast<float>(OverscrollLineCount);
			DrawContext.DrawBox(0.0f, OverscrollLineY + static_cast<float>(LineIndex) * OverscrollLineSize, ViewWidth, OverscrollLineSize, WhiteBrush, FLinearColor(1.0f, 0.1f, 0.1f, Opacity));
		}
	}
	if (OverscrollBottom > 0.0f)
	{
		const float OverscrollLineY = Viewport.GetPosY() + Viewport.GetHeight() - Viewport.GetBottomOffset();
		for (int32 LineIndex = 0; LineIndex < OverscrollLineCount; ++LineIndex)
		{
			const float Opacity = OverscrollBottom * static_cast<float>(OverscrollLineCount - LineIndex) / static_cast<float>(OverscrollLineCount);
			DrawContext.DrawBox(0.0f, OverscrollLineY - static_cast<float>(1 + LineIndex) * OverscrollLineSize, ViewWidth, OverscrollLineSize, WhiteBrush, FLinearColor(1.0f, 0.1f, 0.1f, Opacity));
		}
	}

	//////////////////////////////////////////////////

	const bool bShouldDisplayDebugInfo = FInsightsManager::Get()->IsDebugInfoEnabled();
	if (bShouldDisplayDebugInfo)
	{
		const FSlateFontInfo& SummaryFont = MainFont;

		const float MaxFontCharHeight = static_cast<float>(FontMeasureService->Measure(TEXT("!"), SummaryFont, FontScale).Y / FontScale);
		const float DbgDY = MaxFontCharHeight;

		const float DbgW = 320.0f;
		const float DbgH = DbgDY * 9.0f + 3.0f;
		const float DbgX = ViewWidth - DbgW - 20.0f;
		float DbgY = Viewport.GetPosY() + Viewport.GetTopOffset() + 10.0f;

		DrawContext.LayerId++;

		DrawContext.DrawBox(DbgX - 2.0f, DbgY - 2.0f, DbgW, DbgH, WhiteBrush, FLinearColor(1.0f, 1.0f, 1.0f, 0.9f));
		DrawContext.LayerId++;

		FLinearColor DbgTextColor(0.0f, 0.0f, 0.0f, 0.9f);

		//////////////////////////////////////////////////
		// Display the "Draw" performance info.

		// Time interval since last OnPaint call.
		const uint64 CurrentTime = FPlatformTime::Cycles64();
		const uint64 OnPaintDeltaTime = CurrentTime - LastOnPaintTime;
		LastOnPaintTime = CurrentTime;
		OnPaintDeltaTimeHistory.AddValue(OnPaintDeltaTime); // saved for last 32 OnPaint calls
		const uint64 AvgOnPaintDeltaTime = OnPaintDeltaTimeHistory.ComputeAverage();
		const uint64 AvgOnPaintDeltaTimeMs = FStopwatch::Cycles64ToMilliseconds(AvgOnPaintDeltaTime);
		const double AvgOnPaintFps = AvgOnPaintDeltaTimeMs != 0 ? 1.0 / FStopwatch::Cycles64ToSeconds(AvgOnPaintDeltaTime) : 0.0;

		const uint64 AvgPreDrawTracksDurationMs = FStopwatch::Cycles64ToMilliseconds(PreDrawTracksDurationHistory.ComputeAverage());
		const uint64 AvgDrawTracksDurationMs = FStopwatch::Cycles64ToMilliseconds(DrawTracksDurationHistory.ComputeAverage());
		const uint64 AvgPostDrawTracksDurationMs = FStopwatch::Cycles64ToMilliseconds(PostDrawTracksDurationHistory.ComputeAverage());
		const uint64 AvgTotalDrawDurationMs = FStopwatch::Cycles64ToMilliseconds(TotalDrawDurationHistory.ComputeAverage());

		DrawContext.DrawText
		(
			DbgX, DbgY,
			FString::Printf(TEXT("D: %" UINT64_FMT " ms + %" UINT64_FMT " ms + %" UINT64_FMT " ms + %" UINT64_FMT " ms = %" UINT64_FMT " ms | + %" UINT64_FMT " ms = %" UINT64_FMT " ms (%" INT64_FMT " fps)"),
				AvgPreDrawTracksDurationMs, // pre-draw tracks time
				AvgDrawTracksDurationMs, // draw tracks time
				AvgPostDrawTracksDurationMs, // post-draw tracks time
				AvgTotalDrawDurationMs - AvgPreDrawTracksDurationMs - AvgDrawTracksDurationMs - AvgPostDrawTracksDurationMs, // other draw code
				AvgTotalDrawDurationMs,
				AvgOnPaintDeltaTimeMs - AvgTotalDrawDurationMs, // other overhead to OnPaint calls
				AvgOnPaintDeltaTimeMs, // average time between two OnPaint calls
				FMath::RoundToInt(AvgOnPaintFps)), // framerate of OnPaint calls
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		//////////////////////////////////////////////////
		// Display the "update" performance info.

		const uint64 AvgPreUpdateTracksDurationMs = FStopwatch::Cycles64ToMilliseconds(PreUpdateTracksDurationHistory.ComputeAverage());
		const uint64 AvgUpdateTracksDurationMs = FStopwatch::Cycles64ToMilliseconds(UpdateTracksDurationHistory.ComputeAverage());
		const uint64 AvgPostUpdateTracksDurationMs = FStopwatch::Cycles64ToMilliseconds(PostUpdateTracksDurationHistory.ComputeAverage());
		const uint64 AvgTickDurationMs = FStopwatch::Cycles64ToMilliseconds(TickDurationHistory.ComputeAverage());

		DrawContext.DrawText
		(
			DbgX, DbgY,
			FString::Printf(TEXT("U avg: %llu ms + %llu ms + %llu ms + %llu ms = %llu ms"),
				AvgPreUpdateTracksDurationMs,
				AvgUpdateTracksDurationMs,
				AvgPostUpdateTracksDurationMs,
				AvgTickDurationMs - AvgPreUpdateTracksDurationMs - AvgUpdateTracksDurationMs - AvgPostUpdateTracksDurationMs,
				AvgTickDurationMs),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		//////////////////////////////////////////////////
		// Display timing events stats.

		DrawContext.DrawText
		(
			DbgX, DbgY,
			FString::Format(TEXT("{0} events : {1} ({2}) boxes, {3} borders, {4} texts"),
			{
				FText::AsNumber(Helper.GetNumEvents()).ToString(),
				FText::AsNumber(Helper.GetNumDrawBoxes()).ToString(),
				FText::AsPercent((double)Helper.GetNumDrawBoxes() / (Helper.GetNumDrawBoxes() + Helper.GetNumMergedBoxes())).ToString(),
				FText::AsNumber(Helper.GetNumDrawBorders()).ToString(),
				FText::AsNumber(Helper.GetNumDrawTexts()).ToString(),
				//OutDrawElements.GetRootDrawLayer().GetElementCount(),
			}),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		//////////////////////////////////////////////////
		// Display time markers stats.

		if (MarkersTrack->IsVisible())
		{
			DrawContext.DrawText
			(
				DbgX, DbgY,
				FString::Format(TEXT("{0} logs : {1} boxes, {2} texts"),
				{
					FText::AsNumber(MarkersTrack->GetNumLogMessages()).ToString(),
					FText::AsNumber(MarkersTrack->GetNumBoxes()).ToString(),
					FText::AsNumber(MarkersTrack->GetNumTexts()).ToString(),
				}),
				SummaryFont, DbgTextColor
			);
			DbgY += DbgDY;
		}

		//////////////////////////////////////////////////
		// Display Graph track stats.

		if (GraphTrack)
		{
			DrawContext.DrawText
			(
				DbgX, DbgY,
				FString::Format(TEXT("{0} events : {1} points, {2} lines, {3} boxes"),
					{
						FText::AsNumber(GraphTrack->GetNumAddedEvents()).ToString(),
						FText::AsNumber(GraphTrack->GetNumDrawPoints()).ToString(),
						FText::AsNumber(GraphTrack->GetNumDrawLines()).ToString(),
						FText::AsNumber(GraphTrack->GetNumDrawBoxes()).ToString(),
					}),
					SummaryFont, DbgTextColor
			);
			DbgY += DbgDY;
		}

		//////////////////////////////////////////////////
		// Display viewport's horizontal info.

		DrawContext.DrawText
		(
			DbgX, DbgY,
			FString::Printf(TEXT("SX: %g, ST: %g, ET: %s"),
				Viewport.GetScaleX(),
				Viewport.GetStartTime(),
				*FormatTimeAuto(Viewport.GetMaxValidTime())),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		//////////////////////////////////////////////////
		// Display viewport's vertical info.

		DrawContext.DrawText
		(
			DbgX, DbgY,
			FString::Printf(TEXT("Y: %.2f, H: %g, VH: %g"),
				Viewport.GetScrollPosY(),
				Viewport.GetScrollHeight(),
				Viewport.GetHeight()),
			SummaryFont, DbgTextColor
		);
		DbgY += DbgDY;

		//////////////////////////////////////////////////
		// Display input related debug info.

		FString InputStr = FString::Printf(TEXT("(%.0f, %.0f)"), MousePosition.X, MousePosition.Y);
		if (bIsSpaceBarKeyPressed) InputStr += " Space";
		if (bIsLMB_Pressed) InputStr += " LMB";
		if (bIsRMB_Pressed) InputStr += " RMB";
		if (bIsPanning) InputStr += " Panning";
		if (bIsSelecting) InputStr += " Selecting";
		if (bIsDragging) InputStr += " Dragging";
		if (TimeRulerTrack->IsScrubbing()) InputStr += " Scrubbing";
		DrawContext.DrawText(DbgX, DbgY, InputStr, SummaryFont, DbgTextColor);
		DbgY += DbgDY;
	}

	//////////////////////////////////////////////////

	Stopwatch.Stop();
	TotalDrawDurationHistory.AddValue(Stopwatch.AccumulatedTime);

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled && IsEnabled());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* STimingView::GetLocationName(ETimingTrackLocation Location)
{
	switch (Location)
	{
	case ETimingTrackLocation::TopDocked:		return TEXT("Top Docked");
	case ETimingTrackLocation::BottomDocked:	return TEXT("Bottom Docked");
	case ETimingTrackLocation::Scrollable:		return TEXT("Scrollable");
	case ETimingTrackLocation::Foreground:		return TEXT("Foreground");
	default:									return nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::AddTrack(TSharedPtr<FBaseTimingTrack> Track, ETimingTrackLocation Location)
{
	check(Track.IsValid());

	check(Location == ETimingTrackLocation::Scrollable ||
		  Location == ETimingTrackLocation::TopDocked ||
		  Location == ETimingTrackLocation::BottomDocked ||
		  Location == ETimingTrackLocation::Foreground);

	const TCHAR* LocationName = GetLocationName(Location);
	TArray<TSharedPtr<FBaseTimingTrack>>& TrackList = const_cast<TArray<TSharedPtr<FBaseTimingTrack>>&>(GetTrackList(Location));

	const int32 MaxNumTracks = 1000;
	if (TrackList.Num() >= MaxNumTracks)
	{
		using namespace UE::Insights::TimingProfiler;
		UE_LOGF(LogTimingProfiler, Warning, "Too many tracks already created (%d tracks)! Ignoring %ls track : %ls (\"%ls\")",
			TrackList.Num(),
			LocationName,
			*Track->GetTypeName().ToString(),
			*Track->GetName());
		return;
	}

#if 0
	UE_LOGF(LogTimingProfiler, Log, "New %ls Track (%d) : %ls (\"%ls\")",
		LocationName,
		TrackList.Num() + 1,
		*Track->GetTypeName().ToString(),
		*Track->GetName());
#endif

	ensure(Track->GetLocation() == ETimingTrackLocation::None);
	Track->SetLocation(Location);

	check(!AllTracks.Contains(Track->GetId()));
	AllTracks.Add(Track->GetId(), Track);

	TrackList.Add(Track);
	Algo::SortBy(TrackList, &FBaseTimingTrack::GetOrder);

	if (Location == ETimingTrackLocation::Scrollable)
	{
		InvalidateScrollableTracksOrder();
	}

	OnTrackAddedDelegate.Broadcast(Track);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::RemoveTrack(TSharedPtr<FBaseTimingTrack> Track)
{
	check(Track.IsValid());

	if (AllTracks.Remove(Track->GetId()) > 0)
	{
		const ETimingTrackLocation Location = Track->GetLocation();
		check(Location == ETimingTrackLocation::Scrollable ||
			  Location == ETimingTrackLocation::TopDocked ||
			  Location == ETimingTrackLocation::BottomDocked ||
			  Location == ETimingTrackLocation::Foreground);

		Track->SetLocation(ETimingTrackLocation::None);

		const TCHAR* LocationName = GetLocationName(Location);
		TArray<TSharedPtr<FBaseTimingTrack>>& TrackList = const_cast<TArray<TSharedPtr<FBaseTimingTrack>>&>(GetTrackList(Location));

		TrackList.Remove(Track);

		if (Location == ETimingTrackLocation::Scrollable)
		{
			InvalidateScrollableTracksOrder();
		}

		OnTrackRemovedDelegate.Broadcast(Track);

#if 0
		UE_LOGF(LogTimingProfiler, Log, "Removed %ls Track (%d) : %ls (\"%ls\")",
			LocationName,
			TrackList.Num(),
			*Track->GetTypeName().ToString(),
			*Track->GetName());
#endif

		return true;
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::HideAllScrollableTracks()
{
	for (TSharedPtr<FBaseTimingTrack>& Track : ScrollableTracks)
	{
		Track->Hide();
	}
	HandleTrackVisibilityChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::InvalidateScrollableTracksOrder()
{
	bScrollableTracksOrderIsDirty = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::UpdateScrollableTracksOrder()
{
	if (bScrollableTracksOrderIsDirty)
	{
		Algo::SortBy(ScrollableTracks, &FBaseTimingTrack::GetOrder);
		bScrollableTracksOrderIsDirty = false;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 STimingView::GetFirstScrollableTrackOrder() const
{
	return (ScrollableTracks.Num() > 0) ? ScrollableTracks[0]->GetOrder() : 1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 STimingView::GetLastScrollableTrackOrder() const
{
	return (ScrollableTracks.Num() > 0) ? ScrollableTracks.Last()->GetOrder() : -1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimingView::AllowTracksToProcessOnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : TopDockedTracks)
	{
		if (TrackPtr->IsVisible())
		{
			FReply Reply = TrackPtr->OnMouseButtonDown(MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
	}

	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : BottomDockedTracks)
	{
		if (TrackPtr->IsVisible())
		{
			FReply Reply = TrackPtr->OnMouseButtonDown(MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
	}

	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
	{
		if (TrackPtr->IsVisible())
		{
			FReply Reply = TrackPtr->OnMouseButtonDown(MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
	}

	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ForegroundTracks)
	{
		if (TrackPtr->IsVisible())
		{
			FReply Reply = TrackPtr->OnMouseButtonDown(MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
	}

	return FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimingView::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = AllowTracksToProcessOnMouseButtonDown(MyGeometry, MouseEvent);
	if (Reply.IsEventHandled())
	{
		return Reply;
	}

	MousePositionOnButtonDown = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	MousePosition = MousePositionOnButtonDown;

	if (bAllowPanningOnScreenEdges)
	{
		const FVector2f ScreenSpacePosition = FVector2f(MouseEvent.GetScreenSpacePosition());
		DPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(ScreenSpacePosition.X, ScreenSpacePosition.Y);

		EdgeFrameCountX = 0;
		EdgeFrameCountY = 0;
	}

	bool bStartPanningSelectingOrScrubbing = false;
	bool bStartPanning = false;
	bool bStartSelecting = false;
	bool bStartScrubbing = false;

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (!bIsRMB_Pressed)
		{
			bIsLMB_Pressed = true;
			bStartPanningSelectingOrScrubbing = true;
			SelectHoveredTimingTrack();
		}
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (!bIsLMB_Pressed)
		{
			bIsRMB_Pressed = true;
			bStartPanningSelectingOrScrubbing = true;
			SelectHoveredTimingTrack();
		}
	}

	TSharedPtr<FTimeMarker> ScrubbingTimeMarker = nullptr;

	if (bStartPanningSelectingOrScrubbing)
	{
		bool bIsHoveringTimeRulerTrack = false;
		if (TimeRulerTrack->IsVisible())
		{
			bIsHoveringTimeRulerTrack = MousePositionOnButtonDown.Y >= TimeRulerTrack->GetPosY() &&
										MousePositionOnButtonDown.Y < TimeRulerTrack->GetPosY() + TimeRulerTrack->GetHeight();
			if (bIsHoveringTimeRulerTrack)
			{
				if (MouseEvent.GetModifierKeys().IsControlDown())
				{
					ScrubbingTimeMarker = DefaultTimeMarker;
				}
				else
				{
					ScrubbingTimeMarker = TimeRulerTrack->GetTimeMarkerAtPos(MousePositionOnButtonDown, Viewport);
				}
			}
		}

		if (bIsSpaceBarKeyPressed)
		{
			bStartPanning = true;
		}
		else if (ScrubbingTimeMarker)
		{
			bStartScrubbing = true;
		}
		else if (bIsHoveringTimeRulerTrack || (MouseEvent.GetModifierKeys().IsControlDown() && MouseEvent.GetModifierKeys().IsShiftDown()))
		{
			bStartSelecting = true;
		}
		else
		{
			bStartPanning = true;
		}

		// Capture mouse, so we can drag outside this widget.
		if (bAllowPanningOnScreenEdges)
		{
			Reply = FReply::Handled().CaptureMouse(SharedThis(this)).UseHighPrecisionMouseMovement(SharedThis(this)).SetUserFocus(SharedThis(this), EFocusCause::Mouse);
		}
		else
		{
			Reply = FReply::Handled().CaptureMouse(SharedThis(this));
		}
	}

	if (bPreventThrottling)
	{
		Reply.PreventThrottling();
	}

	if (bStartScrubbing)
	{
		bIsPanning = false;
		bIsDragging = false;
		TimeRulerTrack->StartScrubbing(ScrubbingTimeMarker.ToSharedRef());
	}
	else if (bStartPanning)
	{
		bIsPanning = true;
		bIsDragging = false;
		TimeRulerTrack->StopScrubbing();

		ViewportStartTimeOnButtonDown = Viewport.GetStartTime();
		ViewportScrollPosYOnButtonDown = Viewport.GetScrollPosY();

		if (MouseEvent.GetModifierKeys().IsControlDown())
		{
			// Allow panning only horizontally.
			PanningMode = EPanningMode::Horizontal;
		}
		else if (MouseEvent.GetModifierKeys().IsShiftDown())
		{
			// Allow panning only vertically.
			PanningMode = EPanningMode::Vertical;
		}
		else
		{
			// Allow panning both horizontally and vertically.
			PanningMode = EPanningMode::HorizontalAndVertical;
		}
	}
	else if (bStartSelecting)
	{
		bIsSelecting = true;
		bIsDragging = false;
		TimeRulerTrack->StopScrubbing();

		SelectionStartTime = Viewport.SlateUnitsToTime(static_cast<float>(MousePositionOnButtonDown.X));
		SelectionEndTime = SelectionStartTime;
		LastSelectionType = ESelectionType::None;
		RaiseSelectionChanging();
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimingView::AllowTracksToProcessOnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : TopDockedTracks)
	{
		if (TrackPtr->IsVisible())
		{
			FReply Reply = TrackPtr->OnMouseButtonUp(MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
	}

	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : BottomDockedTracks)
	{
		if (TrackPtr->IsVisible())
		{
			FReply Reply = TrackPtr->OnMouseButtonUp(MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
	}

	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
	{
		if (TrackPtr->IsVisible())
		{
			FReply Reply = TrackPtr->OnMouseButtonUp(MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
	}

	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ForegroundTracks)
	{
		if (TrackPtr->IsVisible())
		{
			FReply Reply = TrackPtr->OnMouseButtonUp(MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
	}

	return FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimingView::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = AllowTracksToProcessOnMouseButtonUp(MyGeometry, MouseEvent);
	if (Reply.IsEventHandled())
	{
		return Reply;
	}

	MousePositionOnButtonUp = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	MousePosition = MousePositionOnButtonUp;

	bool bIsMouseClick = MousePositionOnButtonUp.Equals(MousePositionOnButtonDown, 2.0f);

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (bIsLMB_Pressed)
		{
			if (bIsPanning)
			{
				PanningMode = EPanningMode::None;
				bIsPanning = false;
			}
			else if (bIsSelecting)
			{
				SelectTimeInterval(SelectionStartTime, SelectionEndTime - SelectionStartTime);
				bIsSelecting = false;
				bIsMouseClick = false;
			}
			else if (TimeRulerTrack->IsScrubbing())
			{
				RaiseTimeMarkerChanged(TimeRulerTrack->GetScrubbingTimeMarker());
				TimeRulerTrack->StopScrubbing();
				bIsMouseClick = false;
			}

			if (bIsMouseClick)
			{
				UpdateHoveredTimingEvent(static_cast<float>(MousePositionOnButtonUp.X), static_cast<float>(MousePositionOnButtonUp.Y));

				if (MouseEvent.GetModifierKeys().IsControlDown())
				{
					if (SelectedEvent.IsValid() && HoveredEvent.IsValid())
					{
						// Select the time region that includes both the current selected event and the new event to be selected.
						const double RegionStartTime = FMath::Min(SelectedEvent->GetStartTime(), HoveredEvent->GetStartTime());
						const double RegionEndTime = FMath::Max(Viewport.RestrictEndTime(SelectedEvent->GetEndTime()), Viewport.RestrictEndTime(HoveredEvent->GetEndTime()));
						SelectTimeInterval(RegionStartTime, RegionEndTime - RegionStartTime);
					}
				}

				// Select the hovered timing event (if any).
				SelectHoveredTimingTrack();
				SelectHoveredTimingEvent();

				if (MouseEvent.GetModifierKeys().IsShiftDown())
				{
					ToggleGraphSeries(HoveredEvent);
				}

				// When clicking on an empty space...
				if (!SelectedEvent.IsValid())
				{
					// ...reset selection.
					SelectionEndTime = SelectionStartTime = 0.0;
					LastSelectionType = ESelectionType::None;
					RaiseSelectionChanged();
				}
			}

			bIsDragging = false;

			// Release mouse as we no longer drag.
			Reply = FReply::Handled().ReleaseMouseCapture();

			bIsLMB_Pressed = false;
		}
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (bIsRMB_Pressed)
		{
			if (bIsPanning)
			{
				PanningMode = EPanningMode::None;
				bIsPanning = false;
			}
			else if (bIsSelecting)
			{
				RaiseSelectionChanged();
				bIsSelecting = false;
			}
			else if (TimeRulerTrack->IsScrubbing())
			{
				RaiseTimeMarkerChanged(TimeRulerTrack->GetScrubbingTimeMarker());
				TimeRulerTrack->StopScrubbing();
				bIsMouseClick = false;
			}

			if (bIsMouseClick)
			{
				SelectHoveredTimingTrack();
				ShowContextMenu(MouseEvent);
			}

			bIsDragging = false;

			// Release mouse as we no longer drag.
			Reply = FReply::Handled().ReleaseMouseCapture();

			bIsRMB_Pressed = false;
		}
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimingView::AllowTracksToProcessOnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : TopDockedTracks)
	{
		if (TrackPtr->IsVisible())
		{
			FReply Reply = TrackPtr->OnMouseButtonDoubleClick(MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
	}

	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : BottomDockedTracks)
	{
		if (TrackPtr->IsVisible())
		{
			FReply Reply = TrackPtr->OnMouseButtonDoubleClick(MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
	}

	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
	{
		if (TrackPtr->IsVisible())
		{
			FReply Reply = TrackPtr->OnMouseButtonDoubleClick(MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
	}

	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ForegroundTracks)
	{
		if (TrackPtr->IsVisible())
		{
			FReply Reply = TrackPtr->OnMouseButtonDoubleClick(MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}
	}

	return FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimingView::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = AllowTracksToProcessOnMouseButtonDoubleClick(MyGeometry, MouseEvent);
	if (Reply.IsEventHandled())
	{
		return Reply;
	}

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (HoveredEvent.IsValid())
		{
			if (MouseEvent.GetModifierKeys().IsControlDown())
			{
				const double EndTime = Viewport.RestrictEndTime(HoveredEvent->GetEndTime());
				SelectTimeInterval(HoveredEvent->GetStartTime(), EndTime - HoveredEvent->GetStartTime());
			}
			else
			{
				if (HoveredEvent->Is<FTimingEvent>() &&
					IsFilterByEventType(HoveredEvent->As<FTimingEvent>().GetType()))
				{
					SetEventFilter(nullptr); // reset filter
				}
				else
				{
					SetEventFilter(HoveredEvent->GetTrack()->GetFilterByEvent(HoveredEvent));
				}
			}
		}
		else
		{
			if (TimingEventFilter.IsValid())
			{
				TimingEventFilter.Reset();
				Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HInvalidated);
			}
		}

		Reply = FReply::Handled();
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimingView::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	const FVector2D& CursorDelta = MouseEvent.GetCursorDelta();

	if (bIsPanning && bAllowPanningOnScreenEdges)
	{
		if (MouseEvent.GetScreenSpacePosition().X == MouseEvent.GetLastScreenSpacePosition().X)
		{
			++EdgeFrameCountX;
		}
		else
		{
			EdgeFrameCountX = 0;
		}

		if (EdgeFrameCountX > 10) // handle delay between high precision mouse movement and update of the actual cursor position
		{
			MousePositionOnButtonDown.X -= CursorDelta.X / DPIScaleFactor;
		}

		if (MouseEvent.GetScreenSpacePosition().Y == MouseEvent.GetLastScreenSpacePosition().Y)
		{
			++EdgeFrameCountY;
		}
		else
		{
			EdgeFrameCountY = 0;
		}

		if (EdgeFrameCountY > 10) // handle delay between high precision mouse movement and update of the actual cursor position
		{
			MousePositionOnButtonDown.Y -= CursorDelta.Y / DPIScaleFactor;
		}
	}

	if (!CursorDelta.IsZero())
	{
		if (bIsPanning)
		{
			if (HasMouseCapture())
			{
				bIsDragging = true;

				if ((int32)PanningMode & (int32)EPanningMode::Horizontal)
				{
					const double StartTime = ViewportStartTimeOnButtonDown + static_cast<double>(MousePositionOnButtonDown.X - MousePosition.X) / Viewport.GetScaleX();
					ScrollAtTime(StartTime);
				}

				if ((int32)PanningMode & (int32)EPanningMode::Vertical)
				{
					const float ScrollPosY = ViewportScrollPosYOnButtonDown + static_cast<float>(MousePositionOnButtonDown.Y - MousePosition.Y);
					ScrollAtPosY(ScrollPosY);
				}
			}
		}
		else if (bIsSelecting)
		{
			if (HasMouseCapture())
			{
				bIsDragging = true;

				SelectionStartTime = Viewport.SlateUnitsToTime(static_cast<float>(MousePositionOnButtonDown.X));
				SelectionEndTime = Viewport.SlateUnitsToTime(static_cast<float>(MousePosition.X));
				if (SelectionStartTime > SelectionEndTime)
				{
					double Temp = SelectionStartTime;
					SelectionStartTime = SelectionEndTime;
					SelectionEndTime = Temp;
				}
				LastSelectionType = ESelectionType::TimeRange;
				RaiseSelectionChanging();
			}
		}
		else if (TimeRulerTrack->IsScrubbing())
		{
			if (HasMouseCapture())
			{
				bIsDragging = true;
				double Time = Viewport.SlateUnitsToTime(static_cast<float>(MousePosition.X));

				// Snap to markers.
				const bool bSnapTimeMarkers = MarkersTrack->IsVisible();
				if (bSnapTimeMarkers)
				{
					const double SnapTolerance = 5.0 / Viewport.GetScaleX(); // +/- 5 pixels
					Time = MarkersTrack->Snap(Time, SnapTolerance);
				}

				// Snap to user annotations.
				{
					FUserAnnotationsTimingViewExtender* AnnotationExt = FindAnnotationExtender();
					if (AnnotationExt)
					{
						const double SnapTolerance = 5.0 / Viewport.GetScaleX();
						Time = AnnotationExt->Snap(*this, Time, SnapTolerance);
					}
				}

				TSharedRef<FTimeMarker> ScrubbingTimeMarker = TimeRulerTrack->GetScrubbingTimeMarker();
				ScrubbingTimeMarker->SetTime(Time);
				RaiseTimeMarkerChanging(ScrubbingTimeMarker);
			}
		}
		else
		{
			UpdateHoveredTimingEvent(static_cast<float>(MousePosition.X), static_cast<float>(MousePosition.Y));
		}

		Reply = FReply::Handled();
	}

	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	if (!HasMouseCapture())
	{
		// No longer dragging (unless we have mouse capture).
		bIsDragging = false;
		bIsPanning = false;
		bIsSelecting = false;

		bIsLMB_Pressed = false;
		bIsRMB_Pressed = false;

		MousePosition = FVector2D::ZeroVector;

		if (HoveredTrack.IsValid())
		{
			HoveredTrack.Reset();
			OnHoveredTrackChangedDelegate.Broadcast(HoveredTrack);
		}
		if (HoveredEvent.IsValid())
		{
			HoveredEvent.Reset();
			OnHoveredEventChangedDelegate.Broadcast(HoveredEvent);
		}
		Tooltip.SetDesiredOpacity(0.0f);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimingView::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetModifierKeys().IsShiftDown())
	{
		if (GraphTrack->IsVisible() &&
			MousePosition.Y >= GraphTrack->GetPosY() &&
			MousePosition.Y < GraphTrack->GetPosY() + GraphTrack->GetHeight())
		{
			// Zoom in/out vertically.
			const double Delta = MouseEvent.GetWheelDelta();
			constexpr double ZoomStep = 0.25; // as percent
			constexpr double MinScaleY = 0.0001;
			constexpr double MaxScaleY = 1.0e10;
			double ScaleY = GraphTrack->GetSharedValueViewport().GetScaleY();
			if (Delta > 0)
			{
				ScaleY *= FMath::Pow(1.0 + ZoomStep, Delta);
				if (ScaleY > MaxScaleY)
				{
					ScaleY = MaxScaleY;
				}
			}
			else
			{
				ScaleY *= FMath::Pow(1.0 / (1.0 + ZoomStep), -Delta);
				if (ScaleY < MinScaleY)
				{
					ScaleY = MinScaleY;
				}
			}
			GraphTrack->GetSharedValueViewport().SetScaleY(ScaleY);

			for (const TSharedPtr<FGraphSeries>& Series : GraphTrack->GetSeries())
			{
				if (Series->IsUsingSharedViewport())
				{
					Series->SetScaleY(ScaleY);
					Series->SetDirtyFlag();
				}
			}
		}
		else
		{
			// Scroll vertically.
			constexpr float ScrollSpeedY = 16.0f * 3;
			const float NewScrollPosY = Viewport.GetScrollPosY() - ScrollSpeedY * MouseEvent.GetWheelDelta();
			ScrollAtPosY(EnforceVerticalScrollLimits(NewScrollPosY));
		}
	}
	else if (MouseEvent.GetModifierKeys().IsControlDown())
	{
		if (HoveredTrack.IsValid())
		{
			SelectHoveredTimingTrack();
		}

		// Scroll horizontally.
		const double ScrollSpeedX = Viewport.GetDurationForViewportDX(16.0 * 3);
		const double NewStartTime = Viewport.GetStartTime() - ScrollSpeedX * MouseEvent.GetWheelDelta();
		ScrollAtTime(EnforceHorizontalScrollLimits(NewStartTime));
	}
	else
	{
		if (HoveredTrack.IsValid())
		{
			SelectHoveredTimingTrack();
		}

		// Zoom in/out horizontally.
		const float Delta = MouseEvent.GetWheelDelta();
		if (Viewport.RelativeZoomWithFixedX(Delta, static_cast<float>(MousePosition.X)))
		{
			UpdateHorizontalScrollBar();
		}
	}

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SCompoundWidget::OnDragEnter(MyGeometry, DragDropEvent);

	//TSharedPtr<FStatIDDragDropOp> Operation = DragDropEvent.GetOperationAs<FStatIDDragDropOp>();
	//if (Operation.IsValid())
	//{
	//	Operation->ShowOK();
	//}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	SCompoundWidget::OnDragLeave(DragDropEvent);

	//TSharedPtr<FStatIDDragDropOp> Operation = DragDropEvent.GetOperationAs<FStatIDDragDropOp>();
	//if (Operation.IsValid())
	//{
	//	Operation->ShowError();
	//}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimingView::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	return SCompoundWidget::OnDragOver(MyGeometry,DragDropEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimingView::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	//TSharedPtr<FStatIDDragDropOp> Operation = DragDropEvent.GetOperationAs<FStatIDDragDropOp>();
	//if (Operation.IsValid())
	//{
	//	return FReply::Handled();
	//}

	return SCompoundWidget::OnDrop(MyGeometry,DragDropEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FCursorReply STimingView::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	if (bIsPanning)
	{
		if (bIsDragging)
		{
			//return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
			return FCursorReply::Cursor(EMouseCursor::GrabHand);
		}
	}
	else if (bIsSelecting)
	{
		if (bIsDragging)
		{
			return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
		}
	}
	else if (bIsSpaceBarKeyPressed)
	{
		return FCursorReply::Cursor(EMouseCursor::GrabHand);
	}

	return FCursorReply::Unhandled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimingView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::B)
	{
		if (!InKeyEvent.GetModifierKeys().IsControlDown() && !InKeyEvent.GetModifierKeys().IsShiftDown())
		{
			// Toggle Bookmarks.
			if (MarkersTrack->IsVisible())
			{
				if (!MarkersTrack->IsBookmarksTrack())
				{
					SetDrawOnlyBookmarks(true);
				}
				else
				{
					SetTimeMarkersVisible(false);
				}
			}
			else
			{
				SetTimeMarkersVisible(true);
				SetDrawOnlyBookmarks(true);
			}
			return FReply::Handled();
		}
	}
	else if (InKeyEvent.GetKey() == EKeys::M)
	{
		if (!InKeyEvent.GetModifierKeys().IsControlDown() && !InKeyEvent.GetModifierKeys().IsShiftDown())
		{
			// Toggle Time Markers.
			if (MarkersTrack->IsVisible())
			{
				if (MarkersTrack->IsBookmarksTrack())
				{
					SetDrawOnlyBookmarks(false);
				}
				else
				{
					SetTimeMarkersVisible(false);
				}
			}
			else
			{
				SetTimeMarkersVisible(true);
				SetDrawOnlyBookmarks(false);
			}

			return FReply::Handled();
		}
	}
	else if (InKeyEvent.GetKey() == EKeys::F)
	{
		if (!InKeyEvent.GetModifierKeys().IsControlDown() && !InKeyEvent.GetModifierKeys().IsShiftDown())
		{
			FrameSelection();
			return FReply::Handled();
		}
	}
	else if (InKeyEvent.GetKey() == EKeys::C)
	{
		if (InKeyEvent.GetModifierKeys().IsControlDown())
		{
			if (SelectedEvent.IsValid())
			{
				SelectedEvent->GetTrack()->OnClipboardCopyEvent(*SelectedEvent);
			}
			return FReply::Handled();
		}
	}
	else if (InKeyEvent.GetKey() == EKeys::Equals ||
			 InKeyEvent.GetKey() == EKeys::Add)
	{
		// Zoom In
		const double ScaleX = Viewport.GetScaleX() * 1.25;
		if (Viewport.ZoomWithFixedX(ScaleX, Viewport.GetWidth() / 2))
		{
			UpdateHorizontalScrollBar();
		}
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Hyphen ||
			 InKeyEvent.GetKey() == EKeys::Subtract)
	{
		// Zoom Out
		const double ScaleX = Viewport.GetScaleX() / 1.25;
		if (Viewport.ZoomWithFixedX(ScaleX, Viewport.GetWidth() / 2))
		{
			UpdateHorizontalScrollBar();
		}
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Left)
	{
		if (InKeyEvent.GetModifierKeys().IsControlDown())
		{
			// Scroll Left
			const double NewStartTime = Viewport.GetStartTime() - Viewport.GetDuration() * 0.05;
			ScrollAtTime(EnforceHorizontalScrollLimits(NewStartTime));
		}
		else
		{
			SelectLeftTimingEvent();
		}
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Right)
	{
		if (InKeyEvent.GetModifierKeys().IsControlDown())
		{
			// Scroll Right
			const double NewStartTime = Viewport.GetStartTime() + Viewport.GetDuration() * 0.05;
			ScrollAtTime(EnforceHorizontalScrollLimits(NewStartTime));
		}
		else
		{
			SelectRightTimingEvent();
		}
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Up)
	{
		if (InKeyEvent.GetModifierKeys().IsControlDown())
		{
			// Scroll Up
			const float NewScrollPosY = Viewport.GetScrollPosY() - 16.0f * 3;
			ScrollAtPosY(EnforceVerticalScrollLimits(NewScrollPosY));
		}
		else
		{
			SelectUpTimingEvent();
		}
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Down)
	{
		if (InKeyEvent.GetModifierKeys().IsControlDown())
		{
			// Scroll Down
			const float NewScrollPosY = Viewport.GetScrollPosY() + 16.0f * 3;
			ScrollAtPosY(EnforceVerticalScrollLimits(NewScrollPosY));
		}
		else
		{
			SelectDownTimingEvent();
		}
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Enter)
	{
		// Enter: Selects the time range of the currently selected timing event.
		if (SelectedEvent.IsValid())
		{
			const double Duration = Viewport.RestrictDuration(SelectedEvent->GetStartTime(), SelectedEvent->GetEndTime());
			SelectTimeInterval(SelectedEvent->GetStartTime(), Duration);
		}
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::SpaceBar)
	{
		bIsSpaceBarKeyPressed = true;
		FSlateApplication::Get().QueryCursor();
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::D)
	{
		if (InKeyEvent.GetModifierKeys().IsControlDown() && InKeyEvent.GetModifierKeys().IsShiftDown())
		{
			// Ctrl+Shift+D: Toggles down-sampling on/off (for debugging purposes only).
			FTimingEventsTrack::bUseDownSampling = !FTimingEventsTrack::bUseDownSampling;
			Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HInvalidated);
			return FReply::Handled();
		}
	}
	else if (InKeyEvent.GetKey() == EKeys::A)
	{
		if (InKeyEvent.GetModifierKeys().IsControlDown())
		{
			// Ctrl+A: Selects the entire time range of session.
			SelectTimeInterval(0, Viewport.GetMaxValidTime());
			return FReply::Handled();
		}
	}
	else if (InKeyEvent.GetKey() == EKeys::One)
	{
		LoadingSharedState->SetColorSchema(0);
		Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HInvalidated);
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Two)
	{
		LoadingSharedState->SetColorSchema(1);
		Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HInvalidated);
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Three)
	{
		LoadingSharedState->SetColorSchema(2);
		Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HInvalidated);
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Four)
	{
		LoadingSharedState->SetColorSchema(3);
		Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HInvalidated);
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::X)
	{
		ChooseNextEventDepthLimitMax();
		return FReply::Handled();
	}
#if INSIGHTS_ACTIVATE_BENCHMARK
	else if (InKeyEvent.GetKey() == EKeys::Z)
	{
		FTimingProfilerTests::FCheckValues CheckValues;
		FTimingProfilerTests::RunEnumerateBenchmark(FTimingProfilerTests::FEnumerateTestParams(), CheckValues);
		return FReply::Handled();
	}
#endif

	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STimingView::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::SpaceBar)
	{
		bIsSpaceBarKeyPressed = false;
		FSlateApplication::Get().QueryCursor();
		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyUp(MyGeometry, InKeyEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ShowContextMenu(const FPointerEvent& MouseEvent)
{
	using namespace UE::Insights::TimingProfiler;
	const FTimingViewCommands& Commands = FTimingViewCommands::Get();

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	bool bHasAnyActions = false;

	if (HoveredTrack.IsValid())
	{
		MenuBuilder.BeginSection(TEXT("Track"), FText::FromString(HoveredTrack->GetName()));
		CreateTrackLocationMenu(MenuBuilder, HoveredTrack.ToSharedRef());
		MenuBuilder.EndSection();

		HoveredTrack->BuildContextMenu(MenuBuilder);
		MenuBuilder.AddSeparator();
		bHasAnyActions = true;
	}

	MenuBuilder.BeginSection(TEXT("Misc"));
	{
		MenuBuilder.AddMenuEntry(Commands.QuickFind,
			FName("QuickFind"),
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.Find"));

		MenuBuilder.AddMenuEntry(Commands.GoToFrame,
			FName("GoToFrame"),
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.FramesTrack"));

		MenuBuilder.AddMenuEntry(Commands.GoToTime,
			FName("GoToTime"),
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.TimersView"));

		TSharedPtr<SLogView> LogView = GetLogView();
		const double MousePosTime = Viewport.SlateUnitsToTime(static_cast<float>(MousePosition.X));
		const FText MousePosTimeText = FText::FromString(FormatTimeAuto(MousePosTime, 2));
		const FText Label = FText::Format(LOCTEXT("ContextMenu_ScrollLogView_Fmt", "Scroll Log View (\u2192 {0})"), MousePosTimeText);
		MenuBuilder.AddMenuEntry(
			Label,
			FText::Format(LOCTEXT("ContextMenu_ScrollLogView_Desc_Fmt", "Scrolls the Log View at the message with the closest timestamp to the time of the current mouse position ({0})."), MousePosTimeText),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.LogView"),
			FUIAction(
				FExecuteAction::CreateLambda([LogView, MousePosTime]()
				{
					if (LogView.IsValid())
					{
						LogView->SelectLogMessageByClosestTime(MousePosTime);
					}
				}),
				FCanExecuteAction::CreateLambda([LogView]() -> bool
				{
					return LogView.IsValid();
				})),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		if (HoveredEvent)
		{
			double RangeStart = HoveredEvent->GetStartTime();
			double RangeDuration = Viewport.RestrictEndTime(HoveredEvent->GetEndTime()) - RangeStart;

			MenuBuilder.AddMenuEntry(
				LOCTEXT("ContextMenu_SelectEventTimeRange", "Select Time Range of Event"),
				FText(),
				FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.SelectEventRange"),
				FUIAction(FExecuteAction::CreateLambda([this, RangeStart, RangeDuration]()
					{
						SelectTimeInterval(RangeStart, RangeDuration);
					})),
				NAME_None,
				EUserInterfaceActionType::Button
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("ContextMenu_Copy", "Copy Event To Clipboard"),
				FText(),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(FExecuteAction::CreateLambda([Event=HoveredEvent]()
					{
						Event->GetTrack()->OnClipboardCopyEvent(*Event);
					})),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}

		MenuBuilder.AddMenuEntry(Commands.TrimAndSaveAs,
			FName("TrimAndSaveAs"),
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FInsightsStyle::GetStyleSetName(), "Icons.Save"));

		bHasAnyActions = true;
	}
	MenuBuilder.EndSection();

	for (Timing::ITimingViewExtender* Extender : GetExtenders())
	{
		bHasAnyActions |= Extender->ExtendGlobalContextMenu(*this, MenuBuilder);
	}
#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	::Insights::ITimingViewSession* CurrentTimingViewSession = (::Insights::ITimingViewSession*)(Timing::ITimingViewSession*)this;
	for (::Insights::ITimingViewExtender* Extender : GetOldExtenders())
	{
		bHasAnyActions |= Extender->ExtendGlobalContextMenu(*CurrentTimingViewSession, MenuBuilder);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54

	if (!bHasAnyActions)
	{
		MenuBuilder.BeginSection(TEXT("Empty"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ContextMenu_NA", "N/A"),
				LOCTEXT("ContextMenu_NA_Desc", "No action available."),
				FSlateIcon(),
				FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([](){ return false; })),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		MenuBuilder.EndSection();
	}

	TSharedRef<SWidget> MenuWidget = MenuBuilder.MakeWidget();

	FWidgetPath EventPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
	const FVector2D ScreenSpacePosition = MouseEvent.GetScreenSpacePosition();
	FSlateApplication::Get().PushMenu(SharedThis(this), EventPath, MenuWidget, ScreenSpacePosition, FPopupTransitionEffect::ContextMenu);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::CreateTrackLocationMenu(FMenuBuilder& MenuBuilder, TSharedRef<FBaseTimingTrack> Track)
{
	if (EnumHasAnyFlags(Track->GetValidLocations(), ETimingTrackLocation::TopDocked))
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("TrackLocationMenu_DockToTop", "Top Docked"),
			LOCTEXT("TrackLocationMenu_DockToTop_Desc", "Dock this track to the top."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STimingView::ChangeTrackLocation, Track, ETimingTrackLocation::TopDocked),
				FCanExecuteAction::CreateSP(this, &STimingView::CanChangeTrackLocation, Track, ETimingTrackLocation::TopDocked),
				FIsActionChecked::CreateSP(this, &STimingView::CheckTrackLocation, Track, ETimingTrackLocation::TopDocked)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}

	if (EnumHasAnyFlags(Track->GetValidLocations(), ETimingTrackLocation::Scrollable))
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("TrackLocationMenu_MoveToScrollable", "Scrollable"),
			LOCTEXT("TrackLocationMenu_MoveToScrollable_Desc", "Move this track to the list of scrollable tracks."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STimingView::ChangeTrackLocation, Track, ETimingTrackLocation::Scrollable),
				FCanExecuteAction::CreateSP(this, &STimingView::CanChangeTrackLocation, Track, ETimingTrackLocation::Scrollable),
				FIsActionChecked::CreateSP(this, &STimingView::CheckTrackLocation, Track, ETimingTrackLocation::Scrollable)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}

	if (EnumHasAnyFlags(Track->GetValidLocations(), ETimingTrackLocation::BottomDocked))
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("TrackLocationMenu_DockToBottom", "Bottom Docked"),
			LOCTEXT("TrackLocationMenu_DockToBottom_Desc", "Dock this track to the bottom."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STimingView::ChangeTrackLocation, Track, ETimingTrackLocation::BottomDocked),
				FCanExecuteAction::CreateSP(this, &STimingView::CanChangeTrackLocation, Track, ETimingTrackLocation::BottomDocked),
				FIsActionChecked::CreateSP(this, &STimingView::CheckTrackLocation, Track, ETimingTrackLocation::BottomDocked)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}

	if (EnumHasAnyFlags(Track->GetValidLocations(), ETimingTrackLocation::Foreground))
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("TrackLocationMenu_MoveToForeground", "Foreground"),
			LOCTEXT("TrackLocationMenu_MoveToForeground_Desc", "Move this track to the list of foreground tracks."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STimingView::ChangeTrackLocation, Track, ETimingTrackLocation::Foreground),
				FCanExecuteAction::CreateSP(this, &STimingView::CanChangeTrackLocation, Track, ETimingTrackLocation::Foreground),
				FIsActionChecked::CreateSP(this, &STimingView::CheckTrackLocation, Track, ETimingTrackLocation::Foreground)),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::EnumerateAllTracks(TFunctionRef<bool(TSharedPtr<FBaseTimingTrack>&)> Callback)
{
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : TopDockedTracks)
	{
		if (!Callback(TrackPtr))
		{
			return;
		}
	}
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
	{
		if (!Callback(TrackPtr))
		{
			return;
		}
	}
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : BottomDockedTracks)
	{
		if (!Callback(TrackPtr))
		{
			return;
		}
	}
	for (TSharedPtr<FBaseTimingTrack>& TrackPtr : ForegroundTracks)
	{
		if (!Callback(TrackPtr))
		{
			return;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ChangeTrackLocation(TSharedRef<FBaseTimingTrack> Track, ETimingTrackLocation NewLocation)
{
	if (EnumHasAnyFlags(Track->GetValidLocations(), NewLocation) &&
		Track->GetLocation() != NewLocation)
	{
		switch (Track->GetLocation())
		{
		case ETimingTrackLocation::Scrollable:
			ensure(RemoveScrollableTrack(Track));
			break;

		case ETimingTrackLocation::TopDocked:
			ensure(RemoveTopDockedTrack(Track));
			break;

		case ETimingTrackLocation::BottomDocked:
			ensure(RemoveBottomDockedTrack(Track));
			break;

		case ETimingTrackLocation::Foreground:
			ensure(RemoveForegroundTrack(Track));
			break;
		}

		switch (NewLocation)
		{
		case ETimingTrackLocation::Scrollable:
			AddScrollableTrack(Track);
			break;

		case ETimingTrackLocation::TopDocked:
			AddTopDockedTrack(Track);
			break;

		case ETimingTrackLocation::BottomDocked:
			AddBottomDockedTrack(Track);
			break;

		case ETimingTrackLocation::Foreground:
			AddForegroundTrack(Track);
			break;
		}

		HandleTrackVisibilityChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::CanChangeTrackLocation(TSharedRef<FBaseTimingTrack> Track, ETimingTrackLocation NewLocation) const
{
	return EnumHasAnyFlags(Track->GetValidLocations(), NewLocation);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::CheckTrackLocation(TSharedRef<FBaseTimingTrack> Track, ETimingTrackLocation Location) const
{
	return Track->GetLocation() == Location;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::BindCommands()
{
	using namespace UE::Insights::TimingProfiler;
	FTimingViewCommands::Register();

	const FTimingViewCommands& Commands = FTimingViewCommands::Get();

	CommandList = MakeShared<FUICommandList>();

	CommandList->MapAction(
		Commands.AutoHideEmptyTracks,
		FExecuteAction::CreateSP(this, &STimingView::ToggleAutoHideEmptyTracks),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &STimingView::IsAutoHideEmptyTracksEnabled));

	CommandList->MapAction(
		Commands.PanningOnScreenEdges,
		FExecuteAction::CreateSP(this, &STimingView::TogglePanningOnScreenEdges),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &STimingView::IsPanningOnScreenEdgesEnabled));

	CommandList->MapAction(
		Commands.ToggleCompactMode,
		FExecuteAction::CreateSP(this, &STimingView::ToggleCompactMode),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &STimingView::IsCompactModeEnabled));

	CommandList->MapAction(
		Commands.ShowMainGraphTrack,
		FExecuteAction::CreateSP(this, &STimingView::ShowHideGraphTrack_Execute),
		//FCanExecuteAction(),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateSP(this, &STimingView::ShowHideGraphTrack_IsChecked));

	CommandList->MapAction(
		Commands.QuickFind,
		FExecuteAction::CreateSP(this, &STimingView::QuickFind_Execute),
		FCanExecuteAction::CreateSP(this, &STimingView::QuickFind_CanExecute));

	CommandList->MapAction(
		Commands.GoToFrame,
		FExecuteAction::CreateSP(this, &STimingView::GoToFrame_Execute),
		FCanExecuteAction::CreateSP(this, &STimingView::GoToFrame_CanExecute));

	CommandList->MapAction(
		Commands.GoToTime,
		FExecuteAction::CreateSP(this, &STimingView::GoToTime_Execute),
		FCanExecuteAction::CreateSP(this, &STimingView::GoToTime_CanExecute));

	CommandList->MapAction(
		Commands.TrimAndSaveAs,
		FExecuteAction::CreateSP(this, &STimingView::TrimAndSaveAs_Execute),
		FCanExecuteAction::CreateSP(this, &STimingView::TrimAndSaveAs_CanExecute));

	// User Annotation shortcuts
	CommandList->MapAction(
		Commands.AddPointAnnotation,
		FExecuteAction::CreateSP(this, &STimingView::AddPointAnnotation_Execute),
		FCanExecuteAction::CreateSP(this, &STimingView::AddPointAnnotation_CanExecute));

	CommandList->MapAction(
		Commands.AddRangeAnnotation,
		FExecuteAction::CreateSP(this, &STimingView::AddRangeAnnotation_Execute),
		FCanExecuteAction::CreateSP(this, &STimingView::AddRangeAnnotation_CanExecute));

	CommandList->MapAction(
		Commands.AddEventAnnotation,
		FExecuteAction::CreateSP(this, &STimingView::AddEventAnnotation_Execute),
		FCanExecuteAction::CreateSP(this, &STimingView::AddEventAnnotation_CanExecute));

	CommandList->MapAction(
		Commands.ToggleAllAnnotationVisibility,
		FExecuteAction::CreateSP(this, &STimingView::ToggleAllAnnotationVisibility_Execute),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &STimingView::ToggleAllAnnotationVisibility_IsChecked));

	CommandList->MapAction(
		Commands.NextAnnotation,
		FExecuteAction::CreateSP(this, &STimingView::NextAnnotation_Execute),
		FCanExecuteAction::CreateSP(this, &STimingView::HasAnyAnnotations));

	CommandList->MapAction(
		Commands.PrevAnnotation,
		FExecuteAction::CreateSP(this, &STimingView::PrevAnnotation_Execute),
		FCanExecuteAction::CreateSP(this, &STimingView::HasAnyAnnotations));

	CommandList->MapAction(
		Commands.ToggleFloatingAnnotations,
		FExecuteAction::CreateSP(this, &STimingView::ToggleFloatingAnnotations_Execute),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &STimingView::ToggleFloatingAnnotations_IsChecked));

	CommandList->MapAction(Commands.FindPreviousInstance,
		FExecuteAction::CreateSP(this, &STimingView::FindEventInstance_Execute, ESelectEventType::Previous),
		FCanExecuteAction::CreateSP(this, &STimingView::FindEventInstance_CanExecute));
	CommandList->MapAction(Commands.FindNextInstance,
		FExecuteAction::CreateSP(this, &STimingView::FindEventInstance_Execute, ESelectEventType::Next),
		FCanExecuteAction::CreateSP(this, &STimingView::FindEventInstance_CanExecute));
	CommandList->MapAction(Commands.FindMaxInstance,
		FExecuteAction::CreateSP(this, &STimingView::FindEventInstance_Execute, ESelectEventType::Max),
		FCanExecuteAction::CreateSP(this, &STimingView::FindEventInstance_CanExecute));
	CommandList->MapAction(Commands.FindPreviousInstanceInSelection,
		FExecuteAction::CreateSP(this, &STimingView::FindEventInstanceInSelection_Execute, ESelectEventType::Previous),
		FCanExecuteAction::CreateSP(this, &STimingView::FindEventInstanceInSelection_CanExecute));
	CommandList->MapAction(Commands.FindNextInstanceInSelection,
		FExecuteAction::CreateSP(this, &STimingView::FindEventInstanceInSelection_Execute, ESelectEventType::Next),
		FCanExecuteAction::CreateSP(this, &STimingView::FindEventInstanceInSelection_CanExecute));
	CommandList->MapAction(Commands.FindMaxInstanceInSelection,
		FExecuteAction::CreateSP(this, &STimingView::FindEventInstanceInSelection_Execute, ESelectEventType::Max),
		FCanExecuteAction::CreateSP(this, &STimingView::FindEventInstanceInSelection_CanExecute));

	FrameSharedState->BindCommands();
	ThreadTimingSharedState->BindCommands();
	LoadingSharedState->BindCommands();
	FileActivitySharedState->BindCommands();
	TimingRegionsSharedState->BindCommands();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SetAutoScroll(bool bOnOff)
{
	bAutoScroll = bOnOff;

	// Persistent option. Save it to the config file.
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
	Settings.SetAndSaveAutoScroll(bAutoScroll);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::AutoScroll_OnCheckStateChanged(ECheckBoxState NewRadioState)
{
	SetAutoScroll(NewRadioState == ECheckBoxState::Checked);
	Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HInvalidated);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState STimingView::AutoScroll_IsChecked() const
{
	return bAutoScroll ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SetAutoScrollFrameAlignment(int32 FrameType)
{
	AutoScrollFrameAlignment = FrameType;

	// Persistent option. Save it to the config file.
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
	Settings.SetAndSaveAutoScrollFrameAlignment(FrameType);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::CompareAutoScrollFrameAlignment(int32 FrameType) const
{
	return AutoScrollFrameAlignment == FrameType;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SetAutoScrollViewportOffset(double Percent)
{
	AutoScrollViewportOffsetPercent = Percent;

	// Persistent option. Save it to the config file.
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
	Settings.SetAndSaveAutoScrollViewportOffsetPercent(Percent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::CompareAutoScrollViewportOffset(double Percent) const
{
	return AutoScrollViewportOffsetPercent == Percent;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SetAutoScrollDelay(double Delay)
{
	AutoScrollMinDelay = Delay;

	// Persistent option. Save it to the config file.
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
	Settings.SetAndSaveAutoScrollMinDelay(Delay);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::CompareAutoScrollDelay(double Delay) const
{
	return AutoScrollMinDelay == Delay;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double STimingView::EnforceHorizontalScrollLimits(const double InStartTime)
{
	double NewStartTime = InStartTime;

	double MinT, MaxT;
	Viewport.GetHorizontalScrollLimits(MinT, MaxT);

	if (NewStartTime > MaxT)
	{
		NewStartTime = MaxT;
		OverscrollRight = 1.0f;
	}

	if (NewStartTime < MinT)
	{
		NewStartTime = MinT;
		OverscrollLeft = 1.0f;
	}

	return NewStartTime;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

float STimingView::EnforceVerticalScrollLimits(const float InScrollPosY)
{
	float NewScrollPosY = InScrollPosY;

	const float DY = Viewport.GetScrollHeight() - Viewport.GetScrollableAreaHeight();
	const float MinY = FMath::Min(DY, 0.0f);
	const float MaxY = DY - MinY;

	if (NewScrollPosY > MaxY)
	{
		NewScrollPosY = MaxY;
		OverscrollBottom = 1.0f;
	}

	if (NewScrollPosY < MinY)
	{
		NewScrollPosY = MinY;
		OverscrollTop = 1.0f;
	}

	return NewScrollPosY;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::HorizontalScrollBar_OnUserScrolled(float ScrollOffset)
{
	// Disable auto-scroll if user starts scrolling with horizontal scrollbar.
	SetAutoScroll(false);

	Viewport.OnUserScrolled(HorizontalScrollBar, ScrollOffset);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::UpdateHorizontalScrollBar()
{
	Viewport.UpdateScrollBar(HorizontalScrollBar);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::VerticalScrollBar_OnUserScrolled(float ScrollOffset)
{
	Viewport.OnUserScrolledY(VerticalScrollBar, ScrollOffset);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::UpdateVerticalScrollBar()
{
	Viewport.UpdateScrollBarY(VerticalScrollBar);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ScrollAtPosY(float ScrollPosY)
{
	if (ScrollPosY != Viewport.GetScrollPosY())
	{
		Viewport.SetScrollPosY(ScrollPosY);
		UpdateVerticalScrollBar();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::BringIntoViewY(float InTopScrollY, float InBottomScrollY)
{
	const float ScrollY = Viewport.GetScrollPosY();
	const float ScrollH = Viewport.GetScrollableAreaHeight();
	if (ScrollY > InTopScrollY)
	{
		ScrollAtPosY(InTopScrollY);
	}
	else if (ScrollY + ScrollH < InBottomScrollY)
	{
		ScrollAtPosY(FMath::Min(InTopScrollY, InBottomScrollY - ScrollH));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::BringScrollableTrackIntoView(const FBaseTimingTrack& InTrack)
{
	if (ensure(InTrack.GetLocation() == ETimingTrackLocation::Scrollable))
	{
		const float TopScrollY = InTrack.GetPosY() + Viewport.GetScrollPosY() - Viewport.GetTopOffset() - Viewport.GetPosY();
		BringIntoViewY(TopScrollY, TopScrollY + InTrack.GetHeight());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::BringScrollableTrackIntoViewDelayed(TSharedPtr<FBaseTimingTrack> Track, int32 LaneDepth)
{
	DelayedVisibleTrack = Track;
	DelayedVisibleTrackLaneDepth = LaneDepth;
	DelayedVisibleTrackCounter = 3;
	DelayedVisibleTrackLastScrollHeight = 0.0f;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ScrollAtTime(double StartTime)
{
	if (Viewport.ScrollAtTime(StartTime))
	{
		UpdateHorizontalScrollBar();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::CenterOnTimeInterval(double IntervalStartTime, double IntervalDuration)
{
	if (Viewport.CenterOnTimeInterval(IntervalStartTime, IntervalDuration))
	{
		Viewport.EnforceHorizontalScrollLimits(1.0); // 1.0 is to disable interpolation
		UpdateHorizontalScrollBar();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ZoomOnTimeInterval(double IntervalStartTime, double IntervalDuration)
{
	if (Viewport.ZoomOnTimeInterval(IntervalStartTime, IntervalDuration))
	{
		Viewport.EnforceHorizontalScrollLimits(1.0); // 1.0 is to disable interpolation
		UpdateHorizontalScrollBar();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::BringIntoView(double StartTime, double EndTime)
{
	EndTime = Viewport.RestrictEndTime(EndTime);

	// Increase interval with 8% (of view size) on each side.
	const double DT = Viewport.GetDuration() * 0.08;
	StartTime -= DT;
	EndTime += DT;

	double NewStartTime = Viewport.GetStartTime();

	if (EndTime > Viewport.GetEndTime())
	{
		NewStartTime += EndTime - Viewport.GetEndTime();
	}

	if (StartTime < NewStartTime)
	{
		NewStartTime = StartTime;
	}

	ScrollAtTime(NewStartTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SelectTimeInterval(double IntervalStartTime, double IntervalDuration)
{
	SelectionStartTime = IntervalStartTime;
	SelectionEndTime = IntervalStartTime + IntervalDuration;

	if (GetFrameTypeToSnapTo() != ETraceFrameType::TraceFrameType_Count && IsInTimingProfiler())
	{
		SnapToFrameBound(SelectionStartTime, SelectionEndTime);
	}

	LastSelectionType = ESelectionType::TimeRange;
	RaiseSelectionChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SnapToFrameBound(double& StartTime, double& EndTime)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();

	if (!Session.IsValid())
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
	const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(*Session.Get());

	ETraceFrameType FrameTypeToSnapTo = GetFrameTypeToSnapTo();
	uint32 FrameNum = FramesProvider.GetFrameNumberForTimestamp(FrameTypeToSnapTo, StartTime);
	const TraceServices::FFrame* StartFrame = FramesProvider.GetFrame(FrameTypeToSnapTo, FrameNum);

	if (StartFrame == nullptr)
	{
		return;
	}

	if (StartFrame->EndTime < StartTime)
	{
		if (FrameNum + 1 < FramesProvider.GetFrameCount(FrameTypeToSnapTo))
		{
			StartFrame = FramesProvider.GetFrame(FrameTypeToSnapTo, FrameNum + 1);
		}
	}

	FrameNum = FramesProvider.GetFrameNumberForTimestamp(FrameTypeToSnapTo, EndTime);
	const TraceServices::FFrame* EndFrame = FramesProvider.GetFrame(FrameTypeToSnapTo, FrameNum);

	if (EndFrame == nullptr)
	{
		EndFrame = FramesProvider.GetFrame(FrameTypeToSnapTo, FrameNum - 1);
		if (EndFrame == nullptr)
		{
			return;
		}
	}

	// If the interval is before the first frame or after the last frame.
	if (StartFrame->StartTime > EndTime || EndFrame->EndTime < StartTime)
	{
		return;
	}

	// If the interval is between frames.
	if (EndFrame->Index < StartFrame->Index)
	{
		return;
	}

	StartTime = StartFrame->StartTime;
	EndTime = FMath::Min(EndFrame->EndTime, Session->GetDurationSeconds());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::RaiseSelectionChanging()
{
	OnSelectionChangedDelegate.Broadcast(Timing::ETimeChangedFlags::Interactive, SelectionStartTime, SelectionEndTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::RaiseSelectionChanged()
{
	OnSelectionChangedDelegate.Broadcast(Timing::ETimeChangedFlags::None, SelectionStartTime, SelectionEndTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::RaiseTimeMarkerChanging(TSharedRef<FTimeMarker> InTimeMarker)
{
	if (InTimeMarker == DefaultTimeMarker)
	{
		const double Time = DefaultTimeMarker->GetTime();
		OnTimeMarkerChangedDelegate.Broadcast(Timing::ETimeChangedFlags::Interactive, Time);
	}
	OnCustomTimeMarkerChangedDelegate.Broadcast(Timing::ETimeChangedFlags::Interactive, InTimeMarker);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::RaiseTimeMarkerChanged(TSharedRef<FTimeMarker> InTimeMarker)
{
	if (InTimeMarker == DefaultTimeMarker)
	{
		const double Time = DefaultTimeMarker->GetTime();
		OnTimeMarkerChangedDelegate.Broadcast(Timing::ETimeChangedFlags::None, Time);
	}
	OnCustomTimeMarkerChangedDelegate.Broadcast(Timing::ETimeChangedFlags::None, InTimeMarker);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

double STimingView::GetTimeMarker() const
{
	return DefaultTimeMarker->GetTime();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SetTimeMarker(double InMarkerTime)
{
	DefaultTimeMarker->SetTime(InMarkerTime);
	RaiseTimeMarkerChanged(DefaultTimeMarker);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::AddOverlayWidget(const TSharedRef<SWidget>& InWidget)
{
	if (ExtensionOverlay.IsValid())
	{
		ExtensionOverlay->AddSlot()
		[
			InWidget
		];
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SetAndCenterOnTimeMarker(double Time)
{
	SetTimeMarker(Time);

	double MinT, MaxT;
	Viewport.GetHorizontalScrollLimits(MinT, MaxT);
	const double ViewportDuration = Viewport.GetDuration();
	MinT += ViewportDuration / 2;
	MaxT += ViewportDuration / 2;
	Time = FMath::Clamp<double>(Time, MinT, MaxT);

	Time = Viewport.AlignTimeToPixel(Time);
	CenterOnTimeInterval(Time, 0.0);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SelectToTimeMarker(double Time)
{
	const double TimeMarker = GetTimeMarker();
	if (TimeMarker < Time)
	{
		SelectTimeInterval(TimeMarker, Time - TimeMarker);
	}
	else
	{
		SelectTimeInterval(Time, TimeMarker - Time);
	}

	SetTimeMarker(Time);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SetTimeMarkersVisible(bool bIsMarkersTrackVisible)
{
	if (MarkersTrack->IsVisible() != bIsMarkersTrackVisible)
	{
		MarkersTrack->SetVisibilityFlag(bIsMarkersTrackVisible);

		if (MarkersTrack->IsVisible())
		{
			if (Viewport.GetScrollPosY() != 0.0f)
			{
				UE_LOGF(LogTimingProfiler, Log, "SetTimeMarkersVisible!!!");
				Viewport.SetScrollPosY(Viewport.GetScrollPosY() + MarkersTrack->GetHeight());
			}

			MarkersTrack->SetDirtyFlag();
		}
		else
		{
			UE_LOGF(LogTimingProfiler, Log, "SetTimeMarkersVisible!!!");
			Viewport.SetScrollPosY(Viewport.GetScrollPosY() - MarkersTrack->GetHeight());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SetDrawOnlyBookmarks(bool bIsBookmarksTrack)
{
	if (MarkersTrack->IsBookmarksTrack() != bIsBookmarksTrack)
	{
		const float PrevHeight = MarkersTrack->GetHeight();
		MarkersTrack->SetBookmarksTrackFlag(bIsBookmarksTrack);

		if (MarkersTrack->IsVisible())
		{
			if (Viewport.GetScrollPosY() != 0.0f)
			{
				UE_LOGF(LogTimingProfiler, Log, "SetDrawOnlyBookmarks!!!");
				Viewport.SetScrollPosY(Viewport.GetScrollPosY() + MarkersTrack->GetHeight() - PrevHeight);
			}

			MarkersTrack->SetDirtyFlag();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TSharedPtr<FBaseTimingTrack> STimingView::GetTrackAt(float InPosX, float InPosY) const
{
	TSharedPtr<FBaseTimingTrack> FoundTrack;

	if (InPosY < Viewport.GetPosY())
	{
		// above viewport
	}
	else if (InPosY < Viewport.GetPosY() + Viewport.GetTopOffset())
	{
		// Top Docked Tracks
		for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : TopDockedTracks)
		{
			const FBaseTimingTrack& Track = *TrackPtr;
			if (TrackPtr->IsVisible())
			{
				if (InPosY >= Track.GetPosY() && InPosY < Track.GetPosY() + Track.GetHeight())
				{
					FoundTrack = TrackPtr;
					break;
				}
			}
		}
	}
	else if (InPosY < Viewport.GetPosY() + Viewport.GetHeight() - Viewport.GetBottomOffset())
	{
		// Scrollable Tracks
		for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
		{
			const FBaseTimingTrack& Track = *TrackPtr;
			if (Track.IsVisible())
			{
				if (InPosY >= Track.GetPosY() && InPosY < Track.GetPosY() + Track.GetHeight())
				{
					FoundTrack = TrackPtr;
					break;
				}
			}
		}
	}
	else if (InPosY < Viewport.GetPosY() + Viewport.GetHeight())
	{
		// Bottom Docked Tracks
		for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : BottomDockedTracks)
		{
			const FBaseTimingTrack& Track = *TrackPtr;
			if (TrackPtr->IsVisible())
			{
				if (InPosY >= Track.GetPosY() && InPosY < Track.GetPosY() + Track.GetHeight())
				{
					FoundTrack = TrackPtr;
					break;
				}
			}
		}
	}
	else
	{
		// below viewport
	}

	return FoundTrack;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static bool EventBelongsToTrack(const ITimingEvent& Event, const FBaseTimingTrack* Track)
{
	if (Track == nullptr)
	{
		return false;
	}

	for (const TSharedRef<FBaseTimingTrack>& ChildTrack : Track->GetChildTracks())
	{
		if (EventBelongsToTrack(Event, &*ChildTrack))
		{
			return true;
		}
	}

	return &Event.GetTrack().Get() == Track;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::UpdateHoveredTimingEvent(float InMousePosX, float InMousePosY)
{
	TSharedPtr<FBaseTimingTrack> NewHoveredTrack = GetTrackAt(InMousePosX, InMousePosY);
	if (NewHoveredTrack != HoveredTrack)
	{
		HoveredTrack = NewHoveredTrack;
		OnHoveredTrackChangedDelegate.Broadcast(HoveredTrack);
	}

	TSharedPtr<const ITimingEvent> NewHoveredEvent;
	if (HoveredTrack.IsValid())
	{
		FStopwatch Stopwatch;
		Stopwatch.Start();

		NewHoveredEvent = HoveredTrack->GetEvent(InMousePosX, InMousePosY, Viewport);

		Stopwatch.Stop();
		const double DT = Stopwatch.GetAccumulatedTime();
		if (DT > 0.001)
		{
			UE_LOGF(LogTimingProfiler, Log, "HoveredTrack [%g, %g] GetEvent: %.1f ms", InMousePosX, InMousePosY, DT * 1000.0);
		}
	}

	if (NewHoveredEvent.IsValid())
	{
		if (!HoveredEvent.IsValid() || !NewHoveredEvent->Equals(*HoveredEvent))
		{
			FStopwatch Stopwatch;
			Stopwatch.Start();

			HoveredEvent = NewHoveredEvent;
			ensure(EventBelongsToTrack(*HoveredEvent, HoveredTrack.Get()));
			HoveredTrack->UpdateEventStats(const_cast<ITimingEvent&>(*HoveredEvent));

			Stopwatch.Update();
			const double T1 = Stopwatch.GetAccumulatedTime();

			HoveredTrack->InitTooltip(Tooltip, *HoveredEvent);

			Stopwatch.Update();
			const double T2 = Stopwatch.GetAccumulatedTime();

			OnHoveredEventChangedDelegate.Broadcast(HoveredEvent);

			Stopwatch.Update();
			const double T3 = Stopwatch.GetAccumulatedTime();
			if (T3 > 0.001)
			{
				UE_LOGF(LogTimingProfiler, Log, "HoveredTrack [%g, %g] Tooltip: %.1f ms (%.1f + %.1f + %.1f)",
					InMousePosX, InMousePosY, T3 * 1000.0, T1 * 1000.0, (T2 - T1) * 1000.0, (T3 - T2) * 1000.0);
			}
		}
		Tooltip.SetDesiredOpacity(1.0f);
	}
	else
	{
		if (HoveredEvent.IsValid())
		{
			HoveredEvent.Reset();
			OnHoveredEventChangedDelegate.Broadcast(HoveredEvent);
		}
		Tooltip.SetDesiredOpacity(0.0f);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::OnSelectedTimingEventChanged()
{
	if (SelectedEvent.IsValid())
	{
		SelectedEvent->GetTrack()->UpdateEventStats(const_cast<ITimingEvent&>(*SelectedEvent));
		SelectedEvent->GetTrack()->OnEventSelected(*SelectedEvent);
	}

	OnSelectedEventChangedDelegate.Broadcast(SelectedEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SelectHoveredTimingTrack()
{
	SelectTimingTrack(HoveredTrack, false);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SelectHoveredTimingEvent()
{
	SelectTimingEvent(HoveredEvent, false, false);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SelectTimingTrack(const TSharedPtr<FBaseTimingTrack> InTrack, bool bBringTrackIntoView)
{
	if (SelectedTrack != InTrack)
	{
		SelectedTrack = InTrack;

		if (SelectedTrack.IsValid())
		{
			if (bBringTrackIntoView &&
				SelectedTrack->GetLocation() == ETimingTrackLocation::Scrollable)
			{
				TSharedPtr<FBaseTimingTrack> ParentTrack = SelectedTrack->GetParentTrack().Pin();
				if (ParentTrack.IsValid())
				{
					BringScrollableTrackIntoView(*ParentTrack);
				}
				else
				{
					BringScrollableTrackIntoView(*SelectedTrack);
				}
			}
		}

		OnSelectedTrackChangedDelegate.Broadcast(SelectedTrack);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SelectTimingEvent(const TSharedPtr<const ITimingEvent> InEvent, bool bBringEventIntoViewHorizontally, bool bBringEventIntoViewVertically)
{
	if (SelectedEvent != InEvent)
	{
		SelectedEvent = InEvent;

		if (SelectedEvent.IsValid())
		{
			LastSelectionType = ESelectionType::TimingEvent;
			if (bBringEventIntoViewHorizontally)
			{
				BringIntoView(SelectedEvent->GetStartTime(), SelectedEvent->GetEndTime());
			}
			if (bBringEventIntoViewVertically)
			{
				bBringSelectedEventIntoViewVerticallyOnNextTick = true;
				// We need the layout to be calculated in one frame, no animations, otherwise the event might not be in view at the end of the animation.
				Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::VLayoutChanged);
			}
		}

		OnSelectedTimingEventChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ToggleGraphSeries(const TSharedPtr<const ITimingEvent> InEvent)
{
	if (InEvent.Get() && InEvent.Get()->Is<FThreadTrackEvent>() && IsInTimingProfiler())
	{
		const FThreadTrackEvent& TrackEvent = InEvent.Get()->As<FThreadTrackEvent>();

		using namespace UE::Insights::TimingProfiler;
		FTimingProfilerManager::Get()->ToggleTimingViewMainGraphEventSeries(TrackEvent.GetTimerId());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SelectLeftTimingEvent()
{
	if (SelectedEvent.IsValid())
	{
		const uint32 Depth = SelectedEvent->GetDepth();
		const double StartTime = SelectedEvent->GetStartTime();
		const double EndTime = SelectedEvent->GetEndTime();

		auto EventFilter = [Depth, StartTime, EndTime](double EventStartTime, double EventEndTime, uint32 EventDepth)
		{
			return EventDepth == Depth
				&& (EventStartTime < StartTime || EventEndTime < EndTime);
		};

		const TSharedPtr<const ITimingEvent> LeftEvent = SelectedEvent->GetTrack()->SearchEvent(
			FTimingEventSearchParameters(0.0, StartTime, ETimingEventSearchFlags::SearchAll, EventFilter));

		if (LeftEvent.IsValid())
		{
			SelectedEvent = LeftEvent;
			BringIntoView(SelectedEvent->GetStartTime(), SelectedEvent->GetEndTime());
			OnSelectedTimingEventChanged();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SelectRightTimingEvent()
{
	if (SelectedEvent.IsValid())
	{
		const uint32 Depth = SelectedEvent->GetDepth();
		const double StartTime = SelectedEvent->GetStartTime();
		const double EndTime = SelectedEvent->GetEndTime();

		auto EventFilter = [Depth, StartTime, EndTime](double EventStartTime, double EventEndTime, uint32 EventDepth)
		{
			return EventDepth == Depth
				&& (EventStartTime > StartTime || EventEndTime > EndTime);
		};

		const TSharedPtr<const ITimingEvent> RightEvent = SelectedEvent->GetTrack()->SearchEvent(
			FTimingEventSearchParameters(EndTime, Viewport.GetMaxValidTime(), ETimingEventSearchFlags::StopAtFirstMatch, EventFilter));

		if (RightEvent.IsValid())
		{
			SelectedEvent = RightEvent;
			BringIntoView(SelectedEvent->GetStartTime(), SelectedEvent->GetEndTime());
			OnSelectedTimingEventChanged();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SelectUpTimingEvent()
{
	if (SelectedEvent.IsValid() &&
		SelectedEvent->GetDepth() > 0)
	{
		const uint32 Depth = SelectedEvent->GetDepth() - 1;
		const double StartTime = SelectedEvent->GetStartTime();
		const double EndTime = SelectedEvent->GetEndTime();

		auto EventFilter = [Depth, StartTime, EndTime](double EventStartTime, double EventEndTime, uint32 EventDepth)
		{
			return EventDepth == Depth
				&& EventStartTime <= EndTime
				&& EventEndTime >= StartTime;
		};

		const TSharedPtr<const ITimingEvent> UpEvent = SelectedEvent->GetTrack()->SearchEvent(
			FTimingEventSearchParameters(StartTime, EndTime, ETimingEventSearchFlags::StopAtFirstMatch, EventFilter));

		if (UpEvent.IsValid())
		{
			SelectedEvent = UpEvent;
			BringIntoView(SelectedEvent->GetStartTime(), SelectedEvent->GetEndTime());
			OnSelectedTimingEventChanged();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SelectDownTimingEvent()
{
	if (SelectedEvent.IsValid())
	{
		const uint32 Depth = SelectedEvent->GetDepth() + 1;
		const double StartTime = SelectedEvent->GetStartTime();
		const double EndTime = SelectedEvent->GetEndTime();
		double LargestDuration = 0.0;

		auto EventFilter = [Depth, StartTime, EndTime, &LargestDuration](double EventStartTime, double EventEndTime, uint32 EventDepth)
		{
			const double Duration = EventEndTime - EventStartTime;
			return Duration > LargestDuration
				&& EventDepth == Depth
				&& EventStartTime <= EndTime
				&& EventEndTime >= StartTime;
		};

		auto EventMatched = [&LargestDuration](double EventStartTime, double EventEndTime, uint32 EventDepth)
		{
			const double Duration = EventEndTime - EventStartTime;
			LargestDuration = Duration;
		};

		const TSharedPtr<const ITimingEvent> DownEvent = SelectedEvent->GetTrack()->SearchEvent(
			FTimingEventSearchParameters(StartTime, EndTime, ETimingEventSearchFlags::SearchAll, EventFilter, EventMatched));

		if (DownEvent.IsValid())
		{
			SelectedEvent = DownEvent;
			BringIntoView(SelectedEvent->GetStartTime(), SelectedEvent->GetEndTime());
			OnSelectedTimingEventChanged();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::FrameSelection()
{
	double StartTime, EndTime;

	ESelectionType Type = ESelectionType::None;

	if (LastSelectionType == ESelectionType::TimingEvent)
	{
		// Try framing the selected timing event.
		if (SelectedEvent.IsValid())
		{
			Type = ESelectionType::TimingEvent;
		}

		// Next time, try framing the selected time range.
		LastSelectionType = ESelectionType::TimeRange;
	}
	else if (LastSelectionType == ESelectionType::TimeRange)
	{
		// Try framing the selected time range.
		if (SelectionEndTime > SelectionStartTime)
		{
			Type = ESelectionType::TimeRange;
		}

		// Next time, try framing the selected timing event.
		LastSelectionType = ESelectionType::TimingEvent;
	}

	// If no last selection or last selection is empty...
	if (LastSelectionType == ESelectionType::None || Type == ESelectionType::None)
	{
		// First, try framing the selected timing event...
		if (SelectedEvent.IsValid())
		{
			Type = ESelectionType::TimingEvent;
		}
		else // ...otherwise, try framing the selected time range
		{
			Type = ESelectionType::TimeRange;
		}
	}

	if (Type == ESelectionType::TimingEvent)
	{
		// Frame the selected event.
		StartTime = SelectedEvent->GetStartTime();
		EndTime = Viewport.RestrictEndTime(SelectedEvent->GetEndTime());
		if (EndTime == StartTime)
		{
			EndTime += 1.0 / Viewport.GetScaleX(); // +1px
		}
		if (SelectedEvent->GetTrack()->GetLocation() == ETimingTrackLocation::Scrollable)
		{
			BringScrollableTrackIntoView(*SelectedEvent->GetTrack());
		}
	}
	else
	{
		// Frame the selected time range.
		StartTime = SelectionStartTime;
		EndTime = Viewport.RestrictEndTime(SelectionEndTime);
	}

	if (EndTime > StartTime)
	{
		const double Duration = EndTime - StartTime;
		if (Viewport.ZoomOnTimeInterval(StartTime - Duration * 0.1, Duration * 1.2))
		{
			UpdateHorizontalScrollBar();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SetEventFilter(const TSharedPtr<ITimingEventFilter> InEventFilter)
{
	TimingEventFilter = InEventFilter;
	Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HInvalidated);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ToggleEventFilterByEventType(const uint64 EventType)
{
	if (IsFilterByEventType(EventType))
	{
		SetEventFilter(nullptr); // reset filter
	}
	else
	{
		LLM_SCOPE_BYTAG(Insights);
		TSharedRef<FTimingEventFilterByEventType> NewEventFilter = MakeShared<FTimingEventFilterByEventType>(EventType);
		NewEventFilter->SetFilterByTrackTypeName(true);
		NewEventFilter->SetTrackTypeName(FThreadTimingTrack::GetStaticTypeName());
		SetEventFilter(NewEventFilter); // set new filter
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::IsFilterByEventType(const uint64 EventType) const
{
	if (TimingEventFilter.IsValid() &&
		TimingEventFilter->Is<FTimingEventFilterByEventType>())
	{
		const FTimingEventFilterByEventType& EventFilterByEventType = TimingEventFilter->As<FTimingEventFilterByEventType>();
		return EventFilterByEventType.GetEventType() == EventType;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::CreateCompactMenuLine(FMenuBuilder& MenuBuilder, FText Label, TSharedRef<SWidget> InnerWidget) const
{
	TSharedRef<SWidget> Widget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(-8.0f, 0.0f, 0.0f, 0.0f))
		.AutoWidth()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(140.0f)
			.HAlign(HAlign_Right)
			.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(Label)
			]
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 1.0f, 8.0f, 1.0f))
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			InnerWidget
		]
	;

	MenuBuilder.AddWidget(Widget, FText(), true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimingView::MakeCompactAutoScrollOptionsMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, CommandList);

	MenuBuilder.BeginSection("AutoScrollOptions", LOCTEXT("CompactAutoScrollOptionsMenu_Section", "Auto-Scroll Options"));
	{
		CreateCompactMenuLine(MenuBuilder,
			LOCTEXT("FrameAlignment", "Frame Alignment:"),
			SNew(SSegmentedControl<int32>)
			.OnValueChanged_Lambda([this](int32 InValue) { SetAutoScrollFrameAlignment(InValue); })
			.Value_Lambda([this] { return AutoScrollFrameAlignment; })
			+ SSegmentedControl<int32>::Slot(-1)
			.Text(LOCTEXT("None", "None"))
			.ToolTip(LOCTEXT("AutoScrollNoFrameAlignment_Tooltip", "Disables the frame alignment (when auto-scrolling)."))
			+ SSegmentedControl<int32>::Slot((int32)TraceFrameType_Game)
			.Text(LOCTEXT("Game", "Game"))
			.ToolTip(LOCTEXT("AutoScrollAlignWithGameFrames_Tooltip", "Aligns the viewport's center position with the start time of a Game frame (when auto-scrolling)."))
			+ SSegmentedControl<int32>::Slot((int32)TraceFrameType_Rendering)
			.Text(LOCTEXT("Rendering", "Rendering"))
			.ToolTip(LOCTEXT("AutoScrollAlignWithRenderingFrames_Tooltip", "Aligns the viewport's center position with the start time of a Rendering frame (when auto-scrolling)."))
		);

		CreateCompactMenuLine(MenuBuilder,
			LOCTEXT("ViewportOffset", "Viewport Offset:"),
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SEditableTextBox)
				.MinDesiredWidth(40.0f)
				.HintText(LOCTEXT("ViewportOffsetCustomHint", "Custom"))
				.Text_Lambda([this]
				{
					const FString ValueStr = (AutoScrollViewportOffsetPercent == 0.0) ? FString(TEXT("0")) : FString::Printf(TEXT("%g%%"), AutoScrollViewportOffsetPercent * 100.0);
					return FText::FromString(ValueStr);
				})
				.OnTextChanged_Lambda([this](const FText& InText)
				{
					double OffsetPercent = FCString::Atof(*InText.ToString());
					SetAutoScrollViewportOffset(OffsetPercent * 0.01);
				})
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SSegmentedControl<double>)
				.OnValueChanged_Lambda([this](double InValue) { SetAutoScrollViewportOffset(InValue); })
				.Value_Lambda([this] { return AutoScrollViewportOffsetPercent; })
				+ SSegmentedControl<double>::Slot(-0.1)
				.Text(LOCTEXT("AutoScrollViewportOffset-10", "-10%"))
				.ToolTip(LOCTEXT("AutoScrollViewportOffset-10_Tooltip", "Sets the viewport offset to -10% (i.e. backward) of the viewport's width (when auto-scrolling).\nAvoids flickering as the end of the session will be outside of the viewport."))
				+ SSegmentedControl<double>::Slot(0.0)
				.Text(LOCTEXT("AutoScrollViewportOffset0", "0"))
				.ToolTip(LOCTEXT("AutoScrollViewportOffset0_Tooltip", "Sets the viewport offset to 0 (when auto-scrolling).\nThe right side of the viewport will correspond to the current session time."))
				+ SSegmentedControl<double>::Slot(+0.1)
				.Text(LOCTEXT("AutoScrollViewportOffset+10", "+10%"))
				.ToolTip(LOCTEXT("AutoScrollViewportOffset+10_Tooltip", "Sets the viewport offset to +10% (i.e. forward) of the viewport's width (when auto-scrolling).\nAllows 10% empty space on the right side of the viewport."))
			]
		);

		CreateCompactMenuLine(MenuBuilder,
			LOCTEXT("Delay", "Delay:"),
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SEditableTextBox)
				.MinDesiredWidth(40.0f)
				.HintText(LOCTEXT("AutoScrollDelayCustomHint", "Custom"))
				.Text_Lambda([this]
				{
					const FString ValueStr = (AutoScrollMinDelay == 0.0) ? FString(TEXT("0")) : FString::Printf(TEXT("%gs"), AutoScrollMinDelay);
					return FText::FromString(ValueStr);
				})
				.OnTextChanged_Lambda([this](const FText& InText)
				{
					double Delay = FCString::Atof(*InText.ToString());
					SetAutoScrollDelay(Delay);
				})
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SSegmentedControl<double>)
				.OnValueChanged_Lambda([this](double InValue) { SetAutoScrollDelay(InValue); })
				.Value_Lambda([this] { return AutoScrollMinDelay; })
				+ SSegmentedControl<double>::Slot(0.0)
				.Text(LOCTEXT("AutoScrollDelay0", "0"))
				.ToolTip(LOCTEXT("AutoScrollDelay0_Tooltip", "Sets the time delay of the auto-scroll update to 0."))
				+ SSegmentedControl<double>::Slot(0.3)
				.Text(LOCTEXT("AutoScrollDelay300ms", "300ms"))
				.ToolTip(LOCTEXT("AutoScrollDelay300ms_Tooltip", "Sets the time delay of the auto-scroll update to 300ms."))
				+ SSegmentedControl<double>::Slot(1.0)
				.Text(LOCTEXT("AutoScrollDelay1s", "1s"))
				.ToolTip(LOCTEXT("AutoScrollDelay1s_Tooltip", "Sets the time delay of the auto-scroll update to 1s."))
				+ SSegmentedControl<double>::Slot(3.0)
				.Text(LOCTEXT("AutoScrollDelay3s", "3s"))
				.ToolTip(LOCTEXT("AutoScrollDelay3s_Tooltip", "Sets the time delay of the auto-scroll update to 3s."))
			]
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimingView::MakeAutoScrollOptionsMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, CommandList);
	MenuBuilder.SetSearchable(false);

	MenuBuilder.BeginSection("Alignment", LOCTEXT("AutoScrollOptionsMenu_Section_Alignment", "Alignment"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoScrollNoFrameAlignment", "None"),
			LOCTEXT("AutoScrollNoFrameAlignment_Tooltip", "Disables the frame alignment (when auto-scrolling)."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &STimingView::SetAutoScrollFrameAlignment, -1),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CompareAutoScrollFrameAlignment, -1)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoScrollAlignWithGameFrames", "Game Frames"),
			LOCTEXT("AutoScrollAlignWithGameFrames_Tooltip", "Aligns the viewport's center position with the start time of a Game frame (when auto-scrolling)."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &STimingView::SetAutoScrollFrameAlignment, (int32)TraceFrameType_Game),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CompareAutoScrollFrameAlignment, (int32)TraceFrameType_Game)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoScrollAlignWithRenderingFrames", "Rendering Frames"),
			LOCTEXT("AutoScrollAlignWithRenderingFrames_Tooltip", "Aligns the viewport's center position with the start time of a Rendering frame (when auto-scrolling)."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &STimingView::SetAutoScrollFrameAlignment, (int32)TraceFrameType_Rendering),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CompareAutoScrollFrameAlignment, (int32)TraceFrameType_Rendering)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("ViewportOffset", LOCTEXT("AutoScrollOptionsMenu_Section_ViewportOffset", "Viewport Offset"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoScrollViewportOffset-10", "-10%"),
			LOCTEXT("AutoScrollViewportOffset-10_Tooltip", "Sets the viewport offset to -10% (i.e. backward) of the viewport's width (when auto-scrolling).\nAvoids flickering as the end of the session will be outside of the viewport."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &STimingView::SetAutoScrollViewportOffset, -0.1),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CompareAutoScrollViewportOffset, -0.1)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoScrollViewportOffset0", "0"),
			LOCTEXT("AutoScrollViewportOffset0_Tooltip", "Sets the viewport offset to 0 (when auto-scrolling).\nThe right side of the viewport will correspond to the current session time."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &STimingView::SetAutoScrollViewportOffset, 0.0),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CompareAutoScrollViewportOffset, 0.0)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoScrollViewportOffset+10", "+10%"),
			LOCTEXT("AutoScrollViewportOffset+10_Tooltip", "Sets the viewport offset to +10% (i.e. forward) of the viewport's width (when auto-scrolling).\nAllows 10% empty space on the right side of the viewport."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &STimingView::SetAutoScrollViewportOffset, +0.1),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CompareAutoScrollViewportOffset, +0.1)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			FUIAction(FExecuteAction(), FCanExecuteAction(), FIsActionChecked::CreateLambda(
				[this]() -> bool
				{
					return	AutoScrollViewportOffsetPercent != 0.0 &&
							AutoScrollViewportOffsetPercent != -0.1 &&
							AutoScrollViewportOffsetPercent != +0.1;
				})),
			SNew(SBox)
			.Padding(FMargin(0.0f, 0.0f, 12.0f, 0.0f))
			[
				SNew(SEditableTextBox)
				.HintText(LOCTEXT("ViewportOffsetCustomHint", "Custom"))
				.Text_Lambda([this]
				{
					const FString ValueStr = (AutoScrollViewportOffsetPercent == 0.0) ? FString(TEXT("0")) : FString::Printf(TEXT("%g%%"), AutoScrollViewportOffsetPercent * 100.0);
					return FText::FromString(ValueStr);
				})
				.OnTextChanged_Lambda([this](const FText& InText)
				{
					double OffsetPercent = FCString::Atof(*InText.ToString());
					SetAutoScrollViewportOffset(OffsetPercent * 0.01);
				})
			],
			NAME_None,
			LOCTEXT("AutoScrollViewportOffsetCustom_Tooltip", "Sets a custom value for the viewport offset as percent from viewport's width (when auto-scrolling)."),
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Delay", LOCTEXT("AutoScrollOptionsMenu_Section_Delay", "Delay"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoScrollDelay0", "0"),
			LOCTEXT("AutoScrollDelay0_Tooltip", "Sets the time delay of the auto-scroll update to 0."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &STimingView::SetAutoScrollDelay, 0.0),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CompareAutoScrollDelay, 0.0)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoScrollDelay300ms", "300ms"),
			LOCTEXT("AutoScrollDelay300ms_Tooltip", "Sets the time delay of the auto-scroll update to 300ms."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &STimingView::SetAutoScrollDelay, 0.3),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CompareAutoScrollDelay, 0.3)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoScrollDelay1s", "1s"),
			LOCTEXT("AutoScrollDelay1s_Tooltip", "Sets the time delay of the auto-scroll update to 1s."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &STimingView::SetAutoScrollDelay, 1.0),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CompareAutoScrollDelay, 1.0)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoScrollDelay3s", "3s"),
			LOCTEXT("AutoScrollDelay3s_Tooltip", "Sets the time delay of the auto-scroll update to 3s."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &STimingView::SetAutoScrollDelay, 3.0),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CompareAutoScrollDelay, 3.0)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			FUIAction(FExecuteAction(), FCanExecuteAction(), FIsActionChecked::CreateLambda(
				[this]() -> bool
				{
					return AutoScrollMinDelay != 0.0 &&
						   AutoScrollMinDelay != 0.3 &&
						   AutoScrollMinDelay != 1.0 &&
						   AutoScrollMinDelay != 3.0;
				})),
			SNew(SBox)
			.Padding(FMargin(0.0f, 0.0f, 12.0f, 0.0f))
			[
				SNew(SEditableTextBox)
				.HintText(LOCTEXT("AutoScrollDelayCustomHint", "Custom"))
				.Text_Lambda([this]
				{
					const FString ValueStr = (AutoScrollMinDelay == 0.0) ? FString(TEXT("0")) : FString::Printf(TEXT("%gs"), AutoScrollMinDelay);
					return FText::FromString(ValueStr);
				})
				.OnTextChanged_Lambda([this](const FText& InText)
				{
					double Delay = FCString::Atof(*InText.ToString());
					SetAutoScrollDelay(Delay);
				})
			],
			NAME_None,
			LOCTEXT("AutoScrollDelayCustom_Tooltip", "Sets a custom time delay (in seconds) for the auto-scroll update."),
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimingView::MakeAllTracksMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, CommandList);
	MenuBuilder.SetSearchable(false);

	CreateAllTracksMenu(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::CreateAllTracksMenu(FMenuBuilder& MenuBuilder)
{
	constexpr float MaxDesiredHeight = 24.0f + 8 * 18.0f;
	constexpr float MaxDesiredHeightScrollableTracks = 24.0f + 16 * 18.0f;
	constexpr float MinDesiredWidth = 300.0f;
	constexpr float MaxDesiredWidth = 300.0f;

	if (TopDockedTracks.Num() > 0)
	{
		MenuBuilder.BeginSection("TopDockedTracks", LOCTEXT("ContextMenu_Section_TopDockedTracks", "Top Docked Tracks"));
		//MenuBuilder.AddWidget(
		//	SNew(STextBlock)
		//	.Text(LOCTEXT("TopDockedTracks", "Top Docked Tracks")),
		//	FText(), true);

		MenuBuilder.AddWidget(
			SNew(SBox)
			.MaxDesiredHeight(MaxDesiredHeight)
			.MinDesiredWidth(MinDesiredWidth)
			.MaxDesiredWidth(MaxDesiredWidth)
			[
				SNew(STimingViewTrackList, SharedThis(this), ETimingTrackLocation::TopDocked)
			],
			FText(), true);

		MenuBuilder.EndSection();
	}

	if (ScrollableTracks.Num() > 0)
	{
		MenuBuilder.BeginSection("ScrollableTracks", LOCTEXT("ContextMenu_Section_ScrollableTracks", "Scrollable Tracks"));
		//MenuBuilder.AddWidget(
		//	SNew(STextBlock)
		//	.Text(LOCTEXT("ScrollableTracks", "Scrollable Tracks")),
		//	FText(), true);

		MenuBuilder.AddWidget(
			SNew(SBox)
			.Padding(FMargin(0.0f, 0.0f))
			.MaxDesiredHeight(MaxDesiredHeightScrollableTracks)
			.MinDesiredWidth(MinDesiredWidth)
			.MaxDesiredWidth(MaxDesiredWidth)
			[
				SNew(STimingViewTrackList, SharedThis(this), ETimingTrackLocation::Scrollable)
			],
			FText(), true);

		MenuBuilder.EndSection();
	}

	if (BottomDockedTracks.Num() > 0)
	{
		MenuBuilder.BeginSection("BottomDockedTracks", LOCTEXT("ContextMenu_Section_BottomDockedTracks", "Bottom Docked Tracks"));
		//MenuBuilder.AddWidget(
		//	SNew(STextBlock)
		//	.Text(LOCTEXT("BottomDockedTracks", "Bottom Docked Tracks")),
		//	FText(), true);

		MenuBuilder.AddWidget(
			SNew(SBox)
			.MaxDesiredHeight(MaxDesiredHeight)
			.MinDesiredWidth(MinDesiredWidth)
			.MaxDesiredWidth(MaxDesiredWidth)
			[
				SNew(STimingViewTrackList, SharedThis(this), ETimingTrackLocation::BottomDocked)
			],
			FText(), true);

		MenuBuilder.EndSection();
	}

	if (ForegroundTracks.Num() > 0)
	{
		MenuBuilder.BeginSection("ForegroundTracks", LOCTEXT("ContextMenu_Section_ForegroundTracks", "Foreground Tracks"));
		//MenuBuilder.AddWidget(
		//	SNew(STextBlock)
		//	.Text(LOCTEXT("ForegroundTracks", "Foreground Tracks")),
		//	FText(), true);

		MenuBuilder.AddWidget(
			SNew(SBox)
			.MaxDesiredHeight(MaxDesiredHeight)
			.MinDesiredWidth(MinDesiredWidth)
			.MaxDesiredWidth(MaxDesiredWidth)
			[
				SNew(STimingViewTrackList, SharedThis(this), ETimingTrackLocation::Foreground)
			],
			FText(), true);

		MenuBuilder.EndSection();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimingView::MakeCpuGpuTracksFilterMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, CommandList);

	// Let any plugin extend the GPU Tracks Filter menu.
	for (Timing::ITimingViewExtender* Extender : GetExtenders())
	{
		Extender->ExtendGpuTracksFilterMenu(*this, MenuBuilder);
	}
#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	::Insights::ITimingViewSession* CurrentTimingViewSession = (::Insights::ITimingViewSession*)(Timing::ITimingViewSession*)this;
	for (::Insights::ITimingViewExtender* Extender : GetOldExtenders())
	{
		Extender->ExtendGpuTracksFilterMenu(*CurrentTimingViewSession, MenuBuilder);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54

	// Let any plugin extend the CPU Tracks Filter menu.
	for (Timing::ITimingViewExtender* Extender : GetExtenders())
	{
		Extender->ExtendCpuTracksFilterMenu(*this, MenuBuilder);
	}
#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (::Insights::ITimingViewExtender* Extender : GetOldExtenders())
	{
		Extender->ExtendCpuTracksFilterMenu(*CurrentTimingViewSession, MenuBuilder);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimingView::MakeOtherTracksFilterMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, CommandList);

	const FTimingViewCommands& Commands = FTimingViewCommands::Get();

	MenuBuilder.BeginSection("GraphTracks", LOCTEXT("ContextMenu_Section_GraphTracks", "Main Graph Track"));
	{
		MenuBuilder.AddMenuEntry(Commands.ShowMainGraphTrack);
	}
	MenuBuilder.EndSection();

	// Let any plugin extend the Other Tracks Filter menu.
	for (Timing::ITimingViewExtender* Extender : GetExtenders())
	{
		Extender->ExtendOtherTracksFilterMenu(*this, MenuBuilder);
	}
#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	::Insights::ITimingViewSession* CurrentTimingViewSession = (::Insights::ITimingViewSession*)(Timing::ITimingViewSession*)this;
	for (::Insights::ITimingViewExtender* Extender : GetOldExtenders())
	{
		Extender->ExtendOtherTracksFilterMenu(*CurrentTimingViewSession, MenuBuilder);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::ShowHideGraphTrack_IsChecked() const
{
	return GraphTrack->IsVisible();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ShowHideGraphTrack_Execute()
{
	GraphTrack->ToggleVisibility();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimingView::MakePluginTracksFilterMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, CommandList);

	// Let any plugin extend the filter menu.
	for (Timing::ITimingViewExtender* Extender : GetExtenders())
	{
		Extender->ExtendFilterMenu(*this, MenuBuilder);
	}
#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	::Insights::ITimingViewSession* CurrentTimingViewSession = (::Insights::ITimingViewSession*)(Timing::ITimingViewSession*)this;
	for (::Insights::ITimingViewExtender* Extender : GetOldExtenders())
	{
		Extender->ExtendFilterMenu(*CurrentTimingViewSession, MenuBuilder);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STimingView::MakeViewModeMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, CommandList);

	const FTimingViewCommands& Commands = FTimingViewCommands::Get();

	MenuBuilder.BeginSection("ViewMode", LOCTEXT("ContextMenu_Section_ViewMode", "View Mode"));
	{
		MenuBuilder.AddMenuEntry(Commands.ToggleCompactMode);
		MenuBuilder.AddMenuEntry(Commands.AutoHideEmptyTracks);
	}
	MenuBuilder.EndSection();

	CreateDepthLimitMenu(MenuBuilder);

	CreateCpuThreadTrackColoringModeMenu(MenuBuilder);

	MenuBuilder.BeginSection("Misc", LOCTEXT("ContextMenu_Section_Misc", "Misc Settings"));
	{
		MenuBuilder.AddMenuEntry(Commands.PanningOnScreenEdges);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::IsCompactModeEnabled() const
{
	return Viewport.IsLayoutCompactModeEnabled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ToggleCompactMode()
{
	Viewport.SwitchLayoutCompactMode();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::IsAutoHideEmptyTracksEnabled() const
{
	return (Viewport.GetLayout().TargetMinTimelineH == 0.0f);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ToggleAutoHideEmptyTracks()
{
	Viewport.ToggleLayoutMinTrackHeight();

	for (const TSharedPtr<FBaseTimingTrack>& TrackPtr : ScrollableTracks)
	{
		if (TrackPtr->Is<FTimingEventsTrack>())
		{
			TrackPtr->SetHeight(0.0f);
		}
	}

	// Persistent option. Save it to the config file.
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
	const bool bIsEnabled = IsAutoHideEmptyTracksEnabled();
	Settings.SetAndSaveAutoHideEmptyTracks(bIsEnabled);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::IsPanningOnScreenEdgesEnabled() const
{
	return bAllowPanningOnScreenEdges;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::TogglePanningOnScreenEdges()
{
	bAllowPanningOnScreenEdges = !bAllowPanningOnScreenEdges;

	// Persistent option. Save it to the config file.
	FInsightsSettings& Settings = FInsightsManager::Get()->GetSettings();
	Settings.SetAndSavePanningOnScreenEdges(bAllowPanningOnScreenEdges);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::CreateDepthLimitMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("TrackDepthLimits", LOCTEXT("ContextMenu_Section_TrackDepthLimits", "Track Depth Limits"));
	{
#if 0 // TODO
		MenuBuilder.AddMenuEntry(
			FUIAction(),
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(-32.0f, 0.0f, 0.0f, 0.0f))
			[
				SNew(STextBlock)
				.MinDesiredWidth(34.0f)
				.Text(LOCTEXT("TrackDepthLimitMin_Text", "Min"))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
			[
				SNew(SSegmentedControl<uint32>)
				.Value_Lambda([this]
				{
					return FTimingProfilerManager::Get()->GetEventDepthLimitMin();
				})
				.OnValueChanged_Lambda([this](uint32 InValue)
				{
					SetEventDepthLimitMin(InValue);
				})
				+ SSegmentedControl<uint32>::Slot(0u)
				.Text(FText::FromString("0"))
				+ SSegmentedControl<uint32>::Slot(1u)
				.Text(FText::FromString("1"))
				+ SSegmentedControl<uint32>::Slot(2u)
				.Text(FText::FromString("2"))
				+ SSegmentedControl<uint32>::Slot(4u)
				.Text(FText::FromString("4"))
				+ SSegmentedControl<uint32>::Slot(8u)
				.Text(FText::FromString("8"))
				+ SSegmentedControl<uint32>::Slot(16u)
				.Text(FText::FromString("16"))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			.Padding(FMargin(0.0f, 0.0f, 8.0f, 0.0f))
			[
				SNew(SSpinBox<uint32>)
				.MinValue(TOptional<uint32>(0))
				.MaxValue(TOptional<uint32>(100))
				.MinSliderValue(TOptional<uint32>(0))
				.MaxSliderValue(TOptional<uint32>(16))
				.Delta(1)
				.MinDesiredWidth(60.0f)
				.Value_Lambda([this]() -> uint32
				{
					return FTimingProfilerManager::Get()->GetEventDepthLimitMin();
				})
				.OnValueChanged_Lambda([this](uint32 InValue)
				{
					SetEventDepthLimitMin(InValue);
				})
			],
			NAME_None,
			LOCTEXT("TrackDepthLimitMin_Tooltip", "The minimum depth (number of lanes) for a Timing Events track (like a CPU Thread track).\nValid values: [0 .. 100]\nThis option is persistent to the UnrealInsightsSettings.ini file."),
			EUserInterfaceActionType::None
		);
#endif

		MenuBuilder.AddMenuEntry(
			FUIAction(),
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(-32.0f, 0.0f, 0.0f, 0.0f))
			[
				SNew(STextBlock)
				.MinDesiredWidth(34.0f)
				.Text(LOCTEXT("TrackDepthLimitMax_Text", "Max"))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
			[
				SNew(SSegmentedControl<uint32>)
				.Value_Lambda([this]
				{
					return FTimingProfilerManager::Get()->GetEventDepthLimitMax();
				})
				.OnValueChanged_Lambda([this](uint32 InValue)
				{
					SetEventDepthLimitMax(InValue);
				})
				+ SSegmentedControl<uint32>::Slot(1000u)
				[
					SNew(STextBlock)
					.Margin(FMargin(0.0f, -1.0f, 0.0f, 0.0f))
					.Text(LOCTEXT("DepthLimitMax_Unlimited", "\u221E"))
				]
				+ SSegmentedControl<uint32>::Slot(1u)
				.Text(FText::FromString("1"))
				+ SSegmentedControl<uint32>::Slot(2u)
				.Text(FText::FromString("2"))
				+ SSegmentedControl<uint32>::Slot(4u)
				.Text(FText::FromString("4"))
				+ SSegmentedControl<uint32>::Slot(8u)
				.Text(FText::FromString("8"))
				+ SSegmentedControl<uint32>::Slot(16u)
				.Text(FText::FromString("16"))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
			[
				SNew(SSpinBox<uint32>)
				.MinValue(TOptional<uint32>(1))
				.MaxValue(TOptional<uint32>(1000))
				.MinSliderValue(TOptional<uint32>(1))
				.MaxSliderValue(TOptional<uint32>(16))
				.Delta(1)
				.MinDesiredWidth(60.0f)
				.Value_Lambda([this]() -> uint32
				{
					return FTimingProfilerManager::Get()->GetEventDepthLimitMax();
				})
				.OnValueChanged_Lambda([this](uint32 InValue)
				{
					SetEventDepthLimitMax(InValue);
				})
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(4.0f, 0.0f))
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "Menu.Keybinding")
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text(LOCTEXT("TrackDepthLimitMax_Keybinding", "X"))
			],
			NAME_None,
			LOCTEXT("TrackDepthLimitMax_Tooltip", "The maximum depth (number of lanes) for a Timing Events track (like a CPU Thread track).\nValid values: [1 .. 1000]\nThis option is persistent to the UnrealInsightsSettings.ini file."),
			EUserInterfaceActionType::None
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STimingView::GetEventDepthLimitKeybindingText(uint32 InDepthLimitMax) const
{
	uint32 CurrentDepthLimit = FTimingProfilerManager::Get()->GetEventDepthLimitMax();
	uint32 NextDepthLimit = GetNextEventDepthLimitMax(CurrentDepthLimit);
	return InDepthLimitMax == NextDepthLimit ? LOCTEXT("DepthLimitKeybinding", "X") : FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::CheckEventDepthLimitMax(uint32 InDepthLimitMax) const
{
	return InDepthLimitMax == FTimingProfilerManager::Get()->GetEventDepthLimitMax();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 STimingView::GetNextEventDepthLimitMax(uint32 DepthLimit) const
{
	// Cycles between: 1, 2, 4, 8, 16, Unlimited
	if (DepthLimit == FTimingProfilerManager::UnlimitedEventDepth)
	{
		return 1;
	}
	DepthLimit = FMath::RoundUpToPowerOfTwo(DepthLimit * 2);
	return (DepthLimit <= 16) ? DepthLimit : FTimingProfilerManager::UnlimitedEventDepth;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ChooseNextEventDepthLimitMax()
{
	const uint32 CurrentDepthLimit = FTimingProfilerManager::Get()->GetEventDepthLimitMax();
	const uint32 NextDepthLimit = GetNextEventDepthLimitMax(CurrentDepthLimit);
	FTimingProfilerManager::Get()->SetEventDepthLimitMax(NextDepthLimit);
	Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HInvalidated);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SetEventDepthLimitMax(uint32 InDepthLimitMax)
{
	FTimingProfilerManager::Get()->SetEventDepthLimitMax(InDepthLimitMax);
	Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HInvalidated);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SetEventDepthLimitMin(uint32 InDepthLimitMin)
{
	FTimingProfilerManager::Get()->SetEventDepthLimitMin(InDepthLimitMin);
	Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HInvalidated);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::CreateCpuThreadTrackColoringModeMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("CpuThreadTrackColoringMode", LOCTEXT("ContextMenu_Section_CpuThreadTrackColoringMode", "Coloring Mode (CPU Thread Tracks)"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CpuThreadTrackColoringMode_ByTimerName", "By Timer Name"),
			LOCTEXT("CpuThreadTrackColoringMode_ByTimerName_Desc", "Assign a color to CPU/GPU timing events based on their timer name."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STimingView::SetCpuThreadTrackColoringMode, ETimingEventsColoringMode::ByTimerName),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CheckCpuThreadTrackColoringMode, ETimingEventsColoringMode::ByTimerName)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CpuThreadTrackColoringMode_ByTimerNameAndMetadata", "By Timer Name And Metadata"),
			LOCTEXT("CpuThreadTrackColoringMode_ByTimerNameAndMetadata_Desc", "Assign a color to CPU/GPU timing events based on their combined timer name and metadata values."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STimingView::SetCpuThreadTrackColoringMode, ETimingEventsColoringMode::ByTimerNameAndMetadata),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CheckCpuThreadTrackColoringMode, ETimingEventsColoringMode::ByTimerNameAndMetadata)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CpuThreadTrackColoringMode_ByTimerId", "By Timer Id"),
			LOCTEXT("CpuThreadTrackColoringMode_ByTimerId_Desc", "Assign a color to CPU/GPU timing events based on their timer id."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STimingView::SetCpuThreadTrackColoringMode, ETimingEventsColoringMode::ByTimerId),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CheckCpuThreadTrackColoringMode, ETimingEventsColoringMode::ByTimerId)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CpuThreadTrackColoringMode_BySourceFile", "By Source File"),
			LOCTEXT("CpuThreadTrackColoringMode_BySourceFile_Desc", "Assign a color to CPU/GPU timing events based on their source file."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STimingView::SetCpuThreadTrackColoringMode, ETimingEventsColoringMode::BySourceFile),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CheckCpuThreadTrackColoringMode, ETimingEventsColoringMode::BySourceFile)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CpuThreadTrackColoringMode_ByDuration", "By Duration"),
			LOCTEXT("CpuThreadTrackColoringMode_ByDuration_Desc", "Assign a color to CPU/GPU timing events based on their duration (inclusive time).\n\t≥ 10ms : red\n\t≥ 1ms : yellow\n\t≥ 100μs : green\n\t≥ 10μs : cyan\n\t≥ 1μs : blue\n\t< 1μs : gray"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STimingView::SetCpuThreadTrackColoringMode, ETimingEventsColoringMode::ByDuration),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimingView::CheckCpuThreadTrackColoringMode, ETimingEventsColoringMode::ByDuration)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ChooseNextCpuThreadTrackColoringMode()
{
	uint32 Mode = (uint32)FTimingProfilerManager::Get()->GetColoringMode();
	Mode = (Mode + 1) % (uint32)ETimingEventsColoringMode::Count;
	FTimingProfilerManager::Get()->SetColoringMode((ETimingEventsColoringMode)Mode);
	Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HInvalidated);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SetCpuThreadTrackColoringMode(UE::Insights::ETimingEventsColoringMode Mode)
{
	FTimingProfilerManager::Get()->SetColoringMode(Mode);
	Viewport.AddDirtyFlags(ETimingTrackViewportDirtyFlags::HInvalidated);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::CheckCpuThreadTrackColoringMode(UE::Insights::ETimingEventsColoringMode Mode)
{
	return Mode == FTimingProfilerManager::Get()->GetColoringMode();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::ToggleTrackVisibility_IsChecked(uint64 InTrackId) const
{
	const TSharedPtr<FBaseTimingTrack>* const TrackPtrPtr = AllTracks.Find(InTrackId);
	if (TrackPtrPtr)
	{
		return (*TrackPtrPtr)->IsVisible();
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ToggleTrackVisibility_Execute(uint64 InTrackId)
{
	const TSharedPtr<FBaseTimingTrack>* TrackPtrPtr = AllTracks.Find(InTrackId);
	if (TrackPtrPtr)
	{
		(*TrackPtrPtr)->ToggleVisibility();
		HandleTrackVisibilityChanged();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::QuickFind_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::QuickFind_Execute()
{
	using namespace UE::Insights;

	LLM_SCOPE_BYTAG(Insights);

	if (!QuickFindVm.IsValid())
	{
		TSharedPtr<FFilterConfigurator> NewFilterConfigurator = MakeShared<FFilterConfigurator>();

		NewFilterConfigurator->Add(MakeShared<FFilter>(
			static_cast<int32>(EFilterField::StartTime),
			LOCTEXT("StartTime", "Start Time"),
			LOCTEXT("StartTime", "Start Time"),
			EFilterDataType::Double,
			MakeShared<FTimeFilterValueConverter>(),
			FFilterService::Get()->GetDoubleOperators()));

		NewFilterConfigurator->Add(MakeShared<FFilter>(
			static_cast<int32>(EFilterField::EndTime),
			LOCTEXT("EndTime", "End Time"),
			LOCTEXT("EndTime", "End Time"),
			EFilterDataType::Double,
			MakeShared<FTimeFilterValueConverter>(),
			FFilterService::Get()->GetDoubleOperators()));

		NewFilterConfigurator->Add(MakeShared<FFilter>(
			static_cast<int32>(EFilterField::Duration),
			LOCTEXT("Duration", "Duration"),
			LOCTEXT("Duration", "Duration"),
			EFilterDataType::Double,
			MakeShared<FTimeFilterValueConverter>(),
			FFilterService::Get()->GetDoubleOperators()));

		TSharedRef<FFilterWithSuggestions> TrackFilter = MakeShared<FFilterWithSuggestions>(
			static_cast<int32>(EFilterField::TrackName),
			LOCTEXT("Track", "Track"),
			LOCTEXT("Track", "Track"),
			EFilterDataType::String,
			nullptr,
			FFilterService::Get()->GetStringOperators());
		TrackFilter->SetCallback([this](const FString& Text, TArray<FString>& OutSuggestions)
		{
			this->PopulateTrackSuggestionList(Text, OutSuggestions);
		});
		NewFilterConfigurator->Add(TrackFilter);

		NewFilterConfigurator->Add(MakeShared<FFilter>(
			static_cast<int32>(EFilterField::TimerId),
			LOCTEXT("TimerId", "Timer Id"),
			LOCTEXT("TimerId", "Timer Id"),
			EFilterDataType::Int64,
			nullptr,
			FFilterService::Get()->GetIntegerOperators()));

		NewFilterConfigurator->Add(MakeShared<FTimerNameFilter>());
		NewFilterConfigurator->Add(MakeShared<FMetadataFilter>());

		TSharedRef<FFilterWithSuggestions> RegionNameFilter = MakeShared<FFilterWithSuggestions>(
			static_cast<int32>(EFilterField::RegionName),
			LOCTEXT("RegionName", "Region Name"),
			LOCTEXT("RegionName", "Region Name"),
			EFilterDataType::String,
			nullptr,
			FFilterService::Get()->GetStringOperators());
		RegionNameFilter->SetCallback([this](const FString& Text, TArray<FString>& OutSuggestions)
		{
			this->PopulateRegionNameSuggestionList(Text, OutSuggestions);
		});
		NewFilterConfigurator->Add(RegionNameFilter);

		TSharedRef<FFilterWithSuggestions> RegionCategoryFilter = MakeShared<FFilterWithSuggestions>(
			static_cast<int32>(EFilterField::RegionCategory),
			LOCTEXT("RegionCategory", "Region Category"),
			LOCTEXT("RegionCategory", "Region Category"),
			EFilterDataType::String,
			nullptr,
			FFilterService::Get()->GetStringOperators());
		RegionCategoryFilter->SetCallback([this](const FString& Text, TArray<FString>& OutSuggestions)
		{
			this->PopulateRegionCategorySuggestionList(Text, OutSuggestions);
		});
		NewFilterConfigurator->Add(RegionCategoryFilter);

		for (Timing::ITimingViewExtender* Extender : GetExtenders())
		{
			Extender->AddQuickFindFilters(NewFilterConfigurator);
		}
#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (::Insights::ITimingViewExtender* Extender : GetOldExtenders())
		{
			Extender->AddQuickFindFilters(NewFilterConfigurator);
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54

		QuickFindVm = MakeShared<::Insights::FQuickFind>(NewFilterConfigurator);
		constexpr bool bUseCustomFilter = false;
		QuickFindVm->GetOnFindFirstEvent().AddSP(this, &STimingView::FindFirstEvent, bUseCustomFilter);
		QuickFindVm->GetOnFindPreviousEvent().AddSP(this, &STimingView::FindPrevEvent, bUseCustomFilter);
		QuickFindVm->GetOnFindNextEvent().AddSP(this, &STimingView::FindNextEvent, bUseCustomFilter);
		QuickFindVm->GetOnFindLastEvent().AddSP(this, &STimingView::FindLastEvent, bUseCustomFilter);
		QuickFindVm->GetOnFindMaxEvent().AddSP(this, &STimingView::FindMaxEvent);
		QuickFindVm->GetOnFindMinEvent().AddSP(this, &STimingView::FindMinEvent);
		QuickFindVm->GetOnFilterAllEvent().AddSP(this, &STimingView::FilterAllTracks);
		QuickFindVm->GetOnClearFiltersEvent().AddSP(this, &STimingView::ClearFilters);
	}

	SAssignNew(QuickFindWidgetSharedPtr, ::Insights::SQuickFind, QuickFindVm);

	if (FGlobalTabmanager::Get()->HasTabSpawner(QuickFindTabId))
	{
		FGlobalTabmanager::Get()->TryInvokeTab(QuickFindTabId);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::GoToFrame_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::GoToFrame_Execute()
{
	using namespace UE::Insights::TimingProfiler;

	// Spawn container window
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("GoToFrameTitle", "Go to Frame"))
		.SizingRule(ESizingRule::Autosized)
		.SupportsMaximize(false)
		.SupportsMinimize(false);

	// Retrieve saved preferences
	const FInsightsSettings& Settings = FInsightsManager::GetSettings();

	// Spawn window contents
	TSharedRef<SGoToFrameDialog> Dialog = SNew(SGoToFrameDialog)
		.ParentWindow(Window)
		.InitialFrameType(Settings.GetGoToFrameType())
		.InitialFrameNumber(Settings.GetGoToFrameFrameNumber())
		.InitialSelectTimeRange(Settings.IsGoToFrameSelectTimeRangeEnabled())
		.InitialZoomTimingView(Settings.IsGoToFrameZoomTimingViewEnabled())
		.OnNavigate_Lambda([this](double StartTime, double EndTime, uint64 FrameNumber, bool bSelectTimeRange, bool bZoomTimingView, ETraceFrameType FrameType)
		{
			// Whenever navigate is called with valid parameters, save the values to disk
			FInsightsSettings& MutableSettings = FInsightsManager::GetSettings();
			MutableSettings.SetGoToFrameType(FrameType);
			MutableSettings.SetGoToFrameFrameNumber((int64)FrameNumber);
			MutableSettings.SetGoToFrameSelectTimeRange(bSelectTimeRange);
			MutableSettings.SetGoToFrameZoomTimingView(bZoomTimingView);
			MutableSettings.SaveToConfig();

			// When navigating to a Game Frame, ensure Game Frames track is visible. Same for Render Frames.
			if (TSharedPtr<FFrameSharedState> FrameState = GetFrameSharedState())
			{
				if (TSharedPtr<FFrameTimingTrack> Track = FrameState->GetFrameTrack((uint32)FrameType))
				{
					if (!Track->IsVisible())
					{
						Track->Show();
						HandleTrackVisibilityChanged();
					}
				}
			}

			// Zoom or center based on navigate parameters
			const double Duration = EndTime - StartTime;
			if (bZoomTimingView)
			{
				ZoomOnTimeInterval(StartTime - Duration * 0.1, Duration * 1.2);
			}
			else
			{
				CenterOnTimeInterval(StartTime, Duration);
			}

			if (bSelectTimeRange)
			{
				SelectTimeInterval(StartTime, Duration);
			}
		});

	// Spawn the modal and focus the frame number input field. The modal will keep focus while opened.
	Window->SetContent(Dialog);
	Window->SetWidgetToFocusOnActivate(Dialog->GetFrameNumberBox());
	FSlateApplication::Get().AddModalWindow(Window, SharedThis(this));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::GoToTime_CanExecute() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::GoToTime_Execute()
{
	using namespace UE::Insights::TimingProfiler;

	// Spawn container window
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("GoToTimeTitle", "Go to Time"))
		.SizingRule(ESizingRule::Autosized)
		.SupportsMaximize(false)
		.SupportsMinimize(false);

	// Retrieve saved preferences
	const FInsightsSettings& Settings = FInsightsManager::GetSettings();

	// Spawn window contents
	TSharedRef<SGoToTimeDialog> Dialog = SNew(SGoToTimeDialog)
		.ParentWindow(Window)
		.InitialTime(Settings.GetGoToTimeTime())
		.InitialSelectTimeRange(Settings.IsGoToTimeSelectTimeRangeEnabled())
		.InitialZoomTimingView(Settings.IsGoToTimeZoomTimingViewEnabled())
		.InitialTimeRangeDuration(Settings.GetGoToTimeTimeRangeDuration())
		.OnNavigate_Lambda([this](double TargetTime, bool bSelectTimeRange, bool bZoomTimingView, double TimeRangeDuration)
		{
			// Whenever navigate is called with valid parameters, save the values to disk
			FInsightsSettings& MutableSettings = FInsightsManager::GetSettings();
			MutableSettings.SetGoToTimeTime(TargetTime);
			MutableSettings.SetGoToTimeSelectTimeRange(bSelectTimeRange);
			MutableSettings.SetGoToTimeZoomTimingView(bZoomTimingView);
			MutableSettings.SetGoToTimeTimeRangeDuration(TimeRangeDuration);
			MutableSettings.SaveToConfig();

			// TargetTime is the start of the interval
			if (bSelectTimeRange)
			{
				SelectTimeInterval(TargetTime, TimeRangeDuration);
			}

			if (bZoomTimingView)
			{
				ZoomOnTimeInterval(TargetTime - TimeRangeDuration * 0.1, TimeRangeDuration * 1.2);
			}
			else
			{
				CenterOnTimeInterval(TargetTime, 0.0);
			}
		});

	// Spawn the modal and focus the timestamp input field. The modal will keep focus while opened.
	Window->SetContent(Dialog);
	Window->SetWidgetToFocusOnActivate(Dialog->GetTimeBox());
	FSlateApplication::Get().AddModalWindow(Window, SharedThis(this));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> STimingView::SpawnQuickFindTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	const TSharedPtr<SWindow>& OwnerWindow = Args.GetOwnerWindow();
	if (OwnerWindow.IsValid() && OwnerWindow != FSlateApplication::Get().FindWidgetWindow(SharedThis(this)))
	{
		TSharedPtr<SWindow> TopmostAncestor = OwnerWindow->GetTopmostAncestor();
		if (TopmostAncestor != OwnerWindow)
		{
			const FVector2D DPIProbePoint = TopmostAncestor->GetPositionInScreen() + FVector2D(10.0, 10.0);
			const float LocalDPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(static_cast<float>(DPIProbePoint.X), static_cast<float>(DPIProbePoint.Y));
			OwnerWindow->Resize(FVector2D(600 * LocalDPIScaleFactor, 400 * LocalDPIScaleFactor));
		}
	}

	if (!QuickFindWidgetSharedPtr.IsValid())
	{
		return DockTab;
	}

	DockTab->SetContent(QuickFindWidgetSharedPtr.ToSharedRef());
	QuickFindWidgetSharedPtr->SetParentTab(DockTab);
	QuickFindWidgetWeakPtr = QuickFindWidgetSharedPtr;
	QuickFindWidgetSharedPtr.Reset();
	return DockTab;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::CloseQuickFindTab()
{
	TSharedPtr<::Insights::SQuickFind> QuickFindWidget = QuickFindWidgetWeakPtr.Pin();
	if (QuickFindWidget)
	{
		TSharedPtr<SDockTab> QuickFindTab = QuickFindWidget->GetParentTab().Pin();
		if (QuickFindTab.IsValid())
		{
			QuickFindTab->RequestCloseTab();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::TrimAndSaveAs_CanExecute() const
{
	if (SelectionStartTime >= SelectionEndTime)
	{
		return false;
	}

	TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();

	ETraceStreamType TraceStreamType = InsightsManager->GetTraceStreamType();
	if (TraceStreamType != ETraceStreamType::TraceFile &&
		TraceStreamType != ETraceStreamType::TraceStore)
	{
		return false;
	}

	TSharedPtr<const TraceServices::IAnalysisSession> Session = InsightsManager->GetSession();
	if (!Session)
	{
		return false;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::TrimAndSaveAs_Execute()
{
	if (SelectionStartTime >= SelectionEndTime)
	{
		return;
	}

	TraceServices::FTrimParameters TrimParams;
	TrimParams.StartTime = SelectionStartTime;
	TrimParams.EndTime = SelectionEndTime;

	//////////////////////////////////////////////////

	TSharedPtr<FInsightsManager> InsightsManager = FInsightsManager::Get();

	TSharedPtr<const TraceServices::IAnalysisSession> Session = InsightsManager->GetSession();
	if (!Session)
	{
		UE_LOGF(LogTimingProfiler, Error, "[TraceTrimmer] Invalid session!");
		return;
	}

	//////////////////////////////////////////////////
	// InputDataStream

	TUniquePtr<UE::Trace::IInDataStream> InputDataStream;
	FString InputTraceName;

	ETraceStreamType TraceStreamType = InsightsManager->GetTraceStreamType();
	if (TraceStreamType == ETraceStreamType::TraceFile)
	{
		InputTraceName = Session->GetName();
		FPaths::NormalizeFilename(InputTraceName);

		InputDataStream = MakeUnique<UE::Trace::FFileDataStream>();
		UE::Trace::FFileDataStream* FileDataStream = (UE::Trace::FFileDataStream*)InputDataStream.Get();
		if (!FileDataStream->Open(*InputTraceName))
		{
			UE_LOGF(LogTimingProfiler, Error, "[TraceTrimmer] Cannot open the input trace file ('%ls')!", *InputTraceName);
			return;
		}
	}
	else if (TraceStreamType == ETraceStreamType::TraceStore)
	{
		InputTraceName = Session->GetName();
		FPaths::NormalizeFilename(InputTraceName);

		const uint32 TraceId = Session->GetTraceId();

		UE::Trace::FStoreClient* StoreClient = InsightsManager->GetStoreClient();
		if (StoreClient)
		{
			InputDataStream = StoreClient->ReadTrace(TraceId);
		}

		if (!InputDataStream)
		{
			UE_LOGF(LogTimingProfiler, Error, "[TraceTrimmer] Cannot open the input trace stream ('%ls')!", *InputTraceName);
			return;
		}
	}
	else
	{
		UE_LOGF(LogTimingProfiler, Error, "[TraceTrimmer] Unsupported input trace stream ('%ls')!", *InsightsManager->GetTraceName());
		return;
	}

	//////////////////////////////////////////////////
	// Choose the output trace file

	TArray<FString> SaveFilenames;
	bool bDialogResult = false;

	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		const FString DefaultPath = FPaths::ProjectSavedDir();
		const FString DefaultFile = FPaths::GetBaseFilename(InputTraceName) + TEXT("_trim.utrace");

		bDialogResult = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("SaveTrimmedTraceAs", "Save Trimmed Trace As").ToString(),
			DefaultPath,
			DefaultFile,
			TEXT("Trace Files (*.utrace)|*.utrace|All Files (*.*)|*.*"),
			EFileDialogFlags::None,
			SaveFilenames
		);
	}

	if (!bDialogResult || SaveFilenames.Num() == 0)
	{
		return;
	}

	const FString& OutputTraceFile = SaveFilenames[0];

	//////////////////////////////////////////////////
	// OutputDataStream

	TUniquePtr<UE::Trace::FFileOutDataStream> OutputDataStream = MakeUnique<UE::Trace::FFileOutDataStream>();
	if (!OutputDataStream->Open(*OutputTraceFile))
	{
		UE_LOGF(LogTimingProfiler, Error, "[TraceTrimmer] Cannot write the output trace file ('%ls')!", *OutputTraceFile);
		return;
	}

	//////////////////////////////////////////////////
	// Trim

	int32 Result = TraceServices::Trim(TrimParams, *InputDataStream, *OutputDataStream);
	if (Result == 0)
	{
		UE_LOGF(LogTimingProfiler, Log, "[TraceTrimmer] Success!");
	}
	else
	{
		UE_LOGF(LogTimingProfiler, Error, "[TraceTrimmer] Failed!");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::HandleTrackVisibilityChanged()
{
	if (HoveredTrack.IsValid())
	{
		HoveredTrack.Reset();
		OnHoveredTrackChangedDelegate.Broadcast(HoveredTrack);
	}
	if (HoveredEvent.IsValid())
	{
		HoveredEvent.Reset();
		OnHoveredEventChangedDelegate.Broadcast(HoveredEvent);
	}
	if (SelectedTrack.IsValid())
	{
		SelectedTrack.Reset();
		OnSelectedTrackChangedDelegate.Broadcast(SelectedTrack);
	}
	if (SelectedEvent.IsValid())
	{
		SelectedEvent.Reset();
		OnSelectedEventChangedDelegate.Broadcast(SelectedEvent);
	}
	Tooltip.SetDesiredOpacity(0.0f);

	OnTrackVisibilityChangedDelegate.Broadcast();
	FTimingProfilerManager::Get()->OnThreadFilterChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::PreventThrottling()
{
	bPreventThrottling = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FBaseTimingTrack> STimingView::FindTrack(uint64 InTrackId)
{
	TSharedPtr<FBaseTimingTrack>* TrackPtrPtr = AllTracks.Find(InTrackId);
	return TrackPtrPtr ? *TrackPtrPtr : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::EnumerateTracks(TFunctionRef<void(TSharedPtr<FBaseTimingTrack> Track)> Callback)
{
	for (auto& Entry : AllTracks)
	{
		Callback(Entry.Value);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TArray<Timing::ITimingViewExtender*> STimingView::GetExtenders() const
{
	return IModularFeatures::Get().GetModularFeatureImplementations<Timing::ITimingViewExtender>(Timing::TimingViewExtenderFeatureName);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
PRAGMA_DISABLE_DEPRECATION_WARNINGS
TArray<::Insights::ITimingViewExtender*> STimingView::GetOldExtenders() const
{
	return IModularFeatures::Get().GetModularFeatureImplementations<::Insights::ITimingViewExtender>(::Insights::TimingViewExtenderFeatureName);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ClearRelations()
{
	CurrentRelations.Empty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::FindFirstEvent(bool bUseCustomFilter)
{
	INSIGHTS_SCOPED_LOG(FindFirstEvent);

	if (SelectedEvent.IsValid())
	{
		SelectedEvent.Reset();
		OnSelectedEventChangedDelegate.Broadcast(SelectedEvent);
	}
	FindNextEvent(bUseCustomFilter);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::FindPrevEvent(bool bUseCustomFilter)
{
	INSIGHTS_SCOPED_LOG(FindPrevEvent);

	TSharedPtr<const ITimingEvent> BestMatchEvent;
	double StartTime = SelectedEvent.IsValid() ? SelectedEvent->GetStartTime() : std::numeric_limits<double>::max();

	TSharedPtr<FFilterConfigurator> Filter = bUseCustomFilter ? CustomFilterConfigurator : QuickFindVm->GetFilterConfigurator();

	auto EventFilter = [StartTime](double EventStartTime, double EventEndTime, uint32 EventDepth)
	{
		return EventStartTime < StartTime;
	};

	FTimingEventSearchParameters Params(std::numeric_limits<double>::lowest(), StartTime, ETimingEventSearchFlags::StopAtFirstMatch, EventFilter);
	Params.FilterExecutor = Filter;
	Params.SearchDirection = FTimingEventSearchParameters::ESearchDirection::Backward;

	TSharedPtr<const FBaseTimingTrack> PriorityTrack = SelectedEvent.IsValid() ? SelectedEvent->GetTrack().ToSharedPtr() : nullptr;
	if (!PriorityTrack.IsValid() && Filter.IsValid())
	{
		PriorityTrack = FindPriorityTrack(*Filter->GetRootNode());
	}

	EnumerateFilteredTracks(Filter, PriorityTrack,
		[&Params, &BestMatchEvent]
		(TSharedPtr<const FBaseTimingTrack> Track)
		{
			Params.StartTime = BestMatchEvent.IsValid() ? BestMatchEvent->GetStartTime() : std::numeric_limits<double>::lowest();

			TSharedPtr<const ITimingEvent> FoundEvent = Track->SearchEvent(Params);
			if (FoundEvent.IsValid())
			{
				if (!BestMatchEvent.IsValid() || BestMatchEvent->GetStartTime() < FoundEvent->GetStartTime())
				{
					BestMatchEvent = FoundEvent;
				}
			}
		});

	PostFindEvent(BestMatchEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::FindNextEvent(bool bUseCustomFilter)
{
	INSIGHTS_SCOPED_LOG(FindNextEvent);

	TSharedPtr<const ITimingEvent> BestMatchEvent;
	double StartTime = SelectedEvent.IsValid() ? SelectedEvent->GetStartTime() : std::numeric_limits<double>::lowest();

	TSharedPtr<FFilterConfigurator> Filter = bUseCustomFilter ? CustomFilterConfigurator : QuickFindVm->GetFilterConfigurator();

	auto EventFilter = [StartTime](double EventStartTime, double EventEndTime, uint32 EventDepth)
	{
		return EventStartTime > StartTime;
	};

	FTimingEventSearchParameters Params(StartTime, std::numeric_limits<double>::max(), ETimingEventSearchFlags::StopAtFirstMatch, EventFilter);
	Params.FilterExecutor = Filter;

	TSharedPtr<const FBaseTimingTrack> PriorityTrack = SelectedEvent.IsValid() ? SelectedEvent->GetTrack().ToSharedPtr() : nullptr;
	if (!PriorityTrack.IsValid() && Filter.IsValid())
	{
		PriorityTrack = FindPriorityTrack(*Filter->GetRootNode());
	}

	EnumerateFilteredTracks(Filter, PriorityTrack,
		[&Params, &BestMatchEvent]
		(TSharedPtr<const FBaseTimingTrack> Track)
		{
			Params.EndTime = BestMatchEvent.IsValid() ? BestMatchEvent->GetStartTime() : std::numeric_limits<double>::max();

			TSharedPtr<const ITimingEvent> FoundEvent = Track->SearchEvent(Params);
			if (FoundEvent.IsValid())
			{
				if (BestMatchEvent.IsValid())
				{
					ensure(FoundEvent->GetStartTime() < BestMatchEvent->GetStartTime());
				}
				BestMatchEvent = FoundEvent;
			}
		});

	PostFindEvent(BestMatchEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::FindLastEvent(bool bUseCustomFilter)
{
	INSIGHTS_SCOPED_LOG(FindLastEvent);

	if (SelectedEvent.IsValid())
	{
		SelectedEvent.Reset();
		OnSelectedEventChangedDelegate.Broadcast(SelectedEvent);
	}
	FindPrevEvent(bUseCustomFilter);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::FindMaxEvent()
{
	INSIGHTS_SCOPED_LOG(FindMaxEvent);

	if (SelectedEvent.IsValid())
	{
		SelectedEvent.Reset();
		OnSelectedEventChangedDelegate.Broadcast(SelectedEvent);
	}

	const double MaxValidTime = Viewport.GetMaxValidTime();
	TSharedPtr<const ITimingEvent> BestMatchEvent;
	double CurrentDuration = -1.0;

	TSharedPtr<FFilterConfigurator> Filter = QuickFindVm->GetFilterConfigurator();

	auto EventFilter = [&CurrentDuration, MaxValidTime](double EventStartTime, double EventEndTime, uint32 EventDepth)
	{
		const double Duration = EventEndTime == std::numeric_limits<double>::infinity() ?
			MaxValidTime - EventStartTime : EventEndTime - EventStartTime;
		return Duration > CurrentDuration;
	};

	auto EventMatched = [&CurrentDuration, MaxValidTime](double EventStartTime, double EventEndTime, uint32 EventDepth)
	{
		const double Duration = EventEndTime == std::numeric_limits<double>::infinity() ?
			MaxValidTime - EventStartTime : EventEndTime - EventStartTime;
		CurrentDuration = Duration;
	};

	FTimingEventSearchParameters Params(0.0, std::numeric_limits<double>::max(), ETimingEventSearchFlags::SearchAll, EventFilter, EventMatched);
	Params.FilterExecutor = Filter;

	TSharedPtr<const FBaseTimingTrack> PriorityTrack = SelectedEvent.IsValid() ? SelectedEvent->GetTrack().ToSharedPtr() : nullptr;
	if (!PriorityTrack.IsValid() && Filter.IsValid())
	{
		PriorityTrack = FindPriorityTrack(*Filter->GetRootNode());
	}

	EnumerateFilteredTracks(Filter, PriorityTrack,
		[&Params, &BestMatchEvent]
		(TSharedPtr<const FBaseTimingTrack> Track)
		{
			TSharedPtr<const ITimingEvent> FoundEvent = Track->SearchEvent(Params);
			if (FoundEvent.IsValid())
			{
				if (!BestMatchEvent.IsValid() || BestMatchEvent->GetDuration() < FoundEvent->GetDuration())
				{
					BestMatchEvent = FoundEvent;
				}
			}
		});

	PostFindEvent(BestMatchEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::FindMinEvent()
{
	INSIGHTS_SCOPED_LOG(FindMinEvent);

	if (SelectedEvent.IsValid())
	{
		SelectedEvent.Reset();
		OnSelectedEventChangedDelegate.Broadcast(SelectedEvent);
	}

	const double MaxValidTime = Viewport.GetMaxValidTime();
	TSharedPtr<const ITimingEvent> BestMatchEvent;
	double CurrentDuration = std::numeric_limits<double>::max();

	TSharedPtr<FFilterConfigurator> Filter = QuickFindVm->GetFilterConfigurator();

	auto EventFilter = [&CurrentDuration, MaxValidTime](double EventStartTime, double EventEndTime, uint32 EventDepth)
	{
		const double Duration = EventEndTime == std::numeric_limits<double>::infinity() ?
			MaxValidTime - EventStartTime : EventEndTime - EventStartTime;
		return Duration < CurrentDuration;
	};

	auto EventMatched = [&CurrentDuration, MaxValidTime](double EventStartTime, double EventEndTime, uint32 EventDepth)
	{
		const double Duration = EventEndTime == std::numeric_limits<double>::infinity() ?
			MaxValidTime - EventStartTime : EventEndTime - EventStartTime;
		CurrentDuration = Duration;
	};

	FTimingEventSearchParameters Params(0.0, std::numeric_limits<double>::max(), ETimingEventSearchFlags::SearchAll, EventFilter, EventMatched);
	Params.FilterExecutor = Filter;

	TSharedPtr<const FBaseTimingTrack> PriorityTrack = SelectedEvent.IsValid() ? SelectedEvent->GetTrack().ToSharedPtr() : nullptr;
	if (!PriorityTrack.IsValid() && Filter.IsValid())
	{
		PriorityTrack = FindPriorityTrack(*Filter->GetRootNode());
	}

	EnumerateFilteredTracks(Filter, PriorityTrack,
		[&Params, &BestMatchEvent]
		(TSharedPtr<const FBaseTimingTrack> Track)
		{
			TSharedPtr<const ITimingEvent> FoundEvent = Track->SearchEvent(Params);
			if (FoundEvent.IsValid())
			{
				if (!BestMatchEvent.IsValid() || BestMatchEvent->GetDuration() > FoundEvent->GetDuration())
				{
					BestMatchEvent = FoundEvent;
				}
			}
		});

	PostFindEvent(BestMatchEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::PostFindEvent(TSharedPtr<const ITimingEvent> BestMatchEvent)
{
	if (BestMatchEvent)
	{
		SelectedEvent = BestMatchEvent;
		bool bZoomOnFoundEvent = true;
		if (bZoomOnFoundEvent)
		{
			const double Duration = Viewport.RestrictDuration(SelectedEvent->GetStartTime(), SelectedEvent->GetEndTime());
			if (Duration > 0.0)
			{
				ZoomOnTimeInterval(SelectedEvent->GetStartTime() - Duration * 0.1, Duration * 1.2);
			}
		}
		BringIntoView(SelectedEvent->GetStartTime(), SelectedEvent->GetEndTime());
		TSharedPtr<FBaseTimingTrack> Track = ConstCastSharedPtr<FBaseTimingTrack>(SelectedEvent->GetTrack().ToSharedPtr());
		BringScrollableTrackIntoViewDelayed(Track, int32(SelectedEvent->GetDepth()));
		SelectTimingTrack(Track, true);
		OnSelectedTimingEventChanged();
	}
	else
	{
		FMessageLog ReportMessageLog(FTimingProfilerManager::Get()->GetLogListingName());
		ReportMessageLog.Error(LOCTEXT("NoEventFound", "No event found!"));
		ReportMessageLog.Notify();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<const FBaseTimingTrack> STimingView::FindPriorityTrack(FFilterConfiguratorNode& Node)
{
	for (FBaseTreeNodePtr Child : Node.GetChildren())
	{
		check(Child.IsValid() && Child->Is<FFilterConfiguratorNode>());
		FFilterConfiguratorNode& ChildNode = Child->As<FFilterConfiguratorNode>();
		TSharedPtr<FFilter> Filter = ChildNode.GetSelectedFilter();
		if (Filter.IsValid())
		{
			if (Filter->GetKey() == static_cast<int32>(EFilterField::RegionName) ||
				Filter->GetKey() == static_cast<int32>(EFilterField::RegionCategory))
			{
				for (auto& Entry : AllTracks)
				{
					if (Entry.Value->IsVisible() &&
						Entry.Value->Is<FTimingRegionsTrack>())
					{
						return Entry.Value;
					}
				}
				return nullptr;
			}
		}

		TSharedPtr<const FBaseTimingTrack> Result = FindPriorityTrack(ChildNode);
		if (Result.IsValid())
		{
			return Result;
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::FilterAllTracks()
{
	LLM_SCOPE_BYTAG(Insights);

	FilterConfigurator = MakeShared<UE::Insights::FFilterConfigurator>(*QuickFindVm->GetFilterConfigurator());

	for (auto& Entry : AllTracks)
	{
		Entry.Value->SetFilterConfigurator(FilterConfigurator);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ClearFilters()
{
	LLM_SCOPE_BYTAG(Insights);

	FilterConfigurator.Reset();

	for (auto& Entry : AllTracks)
	{
		Entry.Value->SetFilterConfigurator(nullptr);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::PopulateTrackSuggestionList(const FString& Text, TArray<FString>& OutSuggestions)
{
	for (auto& Entry : AllTracks)
	{
		if (Text.IsEmpty() || Entry.Value->GetName().Contains(Text))
		{
			OutSuggestions.Add(Entry.Value->GetName());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::PopulateTimerNameSuggestionList(const FString& Text, TArray<FString>& OutSuggestions)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (!Session.IsValid())
	{
		return;
	}

	const TraceServices::ITimingProfilerProvider* TimingProfilerProvider = TraceServices::ReadTimingProfilerProvider(*Session.Get());
	if (TimingProfilerProvider)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());

		const TraceServices::ITimingProfilerTimerReader& TimerReader = TimingProfilerProvider->GetTimerReader();

		uint32 TimerCount = TimerReader.GetTimerCount();
		for (uint32 TimerIndex = 0; TimerIndex < TimerCount; ++TimerIndex)
		{
			const TraceServices::FTimingProfilerTimer* Timer = TimerReader.GetTimer(TimerIndex);
			if (Timer && Timer->Name)
			{
				if (Text.IsEmpty())
				{
					OutSuggestions.Add(Timer->Name);
					continue;
				}
				const TCHAR* FoundString = FCString::Stristr(Timer->Name, *Text);
				if (FoundString)
				{
					OutSuggestions.Add(Timer->Name);
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::PopulateRegionNameSuggestionList(const FString& Text, TArray<FString>& OutSuggestions)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::IRegionProvider& RegionProvider = TraceServices::ReadRegionProvider(*Session);
		TraceServices::FProviderReadScopeLock RegionProviderScopedLock(RegionProvider);

		RegionProvider.EnumerateTimers(
			[&Text, &OutSuggestions]
			(const TraceServices::FRegionTimer& Timer) -> bool
			{
				if (Timer.Name && (FCString::Stristr(Timer.Name, *Text) || Text.IsEmpty()))
				{
					OutSuggestions.Add(Timer.Name);
				}
				return true;
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::PopulateRegionCategorySuggestionList(const FString& Text, TArray<FString>& OutSuggestions)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
	if (Session.IsValid())
	{
		const TraceServices::IRegionProvider& RegionProvider = TraceServices::ReadRegionProvider(*Session);
		TraceServices::FProviderReadScopeLock RegionProviderScopedLock(RegionProvider);

		RegionProvider.EnumerateCategories(
			[&Text, &OutSuggestions]
			(const TraceServices::FRegionCategory& Category) -> bool
			{
				if (Category.Name && (FCString::Stristr(Category.Name, *Text) || Text.IsEmpty()))
				{
					OutSuggestions.Add(Category.Name);
				}
				return true;
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

class FBaseTimingTrackFilterContext : public FBaseFilterContext
{
public:
	static constexpr int32 TrackNameKey = static_cast<int32>(UE::Insights::EFilterField::TrackName);

public:
	FBaseTimingTrackFilterContext() = default;
	virtual ~FBaseTimingTrackFilterContext() = default;

	virtual bool GetDefaultReturnValue() const override
	{
		return true;
	}

	virtual bool HasData(int32 Key) const override
	{
		return Key == TrackNameKey;
	}

	virtual EFilterDataType GetDataType(int32 Key) const override
	{
		return Key == TrackNameKey ? EFilterDataType::String : EFilterDataType::Void;
	}

	virtual FStringView GetDataAsStringView(int32 Key) const override
	{
		return Track && Key == TrackNameKey ? Track->GetName() : FStringView();
	}
	virtual const TCHAR* GetDataAsCString(int32 Key) const override
	{
		return Track && Key == TrackNameKey ? *Track->GetName() : nullptr;
	}
	virtual const FString& GetDataAsString(int32 Key) const override
	{
		return Track && Key == TrackNameKey ? Track->GetName() : EmptyString;
	}

	void SetTrack(const FBaseTimingTrack* InTrack)
	{
		Track = InTrack;
	}

private:
	const FBaseTimingTrack* Track = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::EnumerateFilteredTracks(TSharedPtr<UE::Insights::FFilterConfigurator> InFilterConfigurator, TSharedPtr<const FBaseTimingTrack> PriorityTrack, EnumerateFilteredTracksCallback Callback)
{
	FBaseTimingTrackFilterContext FilterContext;

	// Call the callback for the PriorityTrack first if it passes the filters.
	// This is an optimization because in many cases, the next/prev event will be on the same track
	// and searching this one first will potentially avoid searching all events on other tracks.
	uint64 SkipId = std::numeric_limits<uint64>::max();
	if (PriorityTrack.IsValid())
	{
		SkipId = PriorityTrack->GetId();
		if (PriorityTrack->IsVisible())
		{
			FilterContext.SetTrack(PriorityTrack.Get());
			if (InFilterConfigurator->ApplyFilters(FilterContext))
			{
				Callback(PriorityTrack);
			}
		}
	}

	for (auto& Entry : AllTracks)
	{
		if (Entry.Value->GetId() != SkipId && Entry.Value->IsVisible())
		{
			FilterContext.SetTrack(Entry.Value.Get());
			if (InFilterConfigurator->ApplyFilters(FilterContext))
			{
				Callback(Entry.Value);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ETraceFrameType STimingView::GetFrameTypeToSnapTo()
{
	TSharedPtr<STimersView> TimersView = GetTimersView();
	if (TimersView.IsValid())
	{
		ETimerAggregationMode AggregationMode = TimersView->GetAggregationMode();
		switch (AggregationMode)
		{
		case ETimerAggregationMode::GameFrame:
			return ETraceFrameType::TraceFrameType_Game;
		case ETimerAggregationMode::RenderingFrame:
			return ETraceFrameType::TraceFrameType_Rendering;
		}
	}

	return ETraceFrameType::TraceFrameType_Count; // no frame snap
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::FindEventInstance_CanExecute() const
{
	return SelectedEvent.IsValid() && SelectedEvent->Is<FThreadTrackEvent>();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::FindEventInstance_Execute(ESelectEventType EventType)
{
	if (!FindEventInstance_CanExecute())
	{
		return;
	}
	const FThreadTrackEvent ThreadTrackEvent = SelectedEvent->As<FThreadTrackEvent>();
	SelectEventInstance(ThreadTrackEvent.GetTimerId(), EventType, false);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::FindEventInstanceInSelection_CanExecute() const
{
	return FindEventInstance_CanExecute() && SelectionEndTime > SelectionStartTime;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::FindEventInstanceInSelection_Execute(ESelectEventType EventType)
{
	if (!FindEventInstanceInSelection_CanExecute())
	{
		return;
	}

	const FThreadTrackEvent ThreadTrackEvent = SelectedEvent->As<FThreadTrackEvent>();
	SelectEventInstance(ThreadTrackEvent.GetTimerId(), EventType, true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::SelectEventInstance(uint32 TimerId, ESelectEventType Type, bool bUseSelection)
{
	INSIGHTS_SCOPED_LOG(FindEvent);

	double IntervalStart = 0.0;
	double IntervalEnd = std::numeric_limits<double>::max();

	if (bUseSelection && SelectionEndTime > SelectionStartTime)
	{
		IntervalStart = SelectionStartTime;
		IntervalEnd = SelectionEndTime;
	}

	const double MaxValidTime = Viewport.GetMaxValidTime();

	TSharedPtr<const ITimingEvent> TimingEvent;

	// Create a temporary filter object using the TimerId and search range as filter criteria
	MakeCustomFilterConfigurator(TimerId, IntervalStart, IntervalEnd);

	switch (Type)
	{
	case ESelectEventType::First:
		FindFirstEvent(true);
		TimingEvent = SelectedEvent;
		break;
	case ESelectEventType::Previous:
		FindPrevEvent(true);
		TimingEvent = SelectedEvent;
		break;
	case ESelectEventType::Next:
		FindNextEvent(true);
		TimingEvent = SelectedEvent;
		break;
	case ESelectEventType::Last:
		FindLastEvent(true);
		TimingEvent = SelectedEvent;
		break;
	case ESelectEventType::Min:
		SelectTimingEvent(nullptr, false, false);
		TimingEvent = ThreadTimingSharedState->FindMinEventInstance(TimerId, IntervalStart, IntervalEnd, MaxValidTime);
		break;
	case ESelectEventType::Max:
		SelectTimingEvent(nullptr, false, false);
		TimingEvent = ThreadTimingSharedState->FindMaxEventInstance(TimerId, IntervalStart, IntervalEnd, MaxValidTime);
		break;
	}

	PostFindEvent(TimingEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::UpdateFilters()
{
	if (!bUpdateFilters)
	{
		return;
	}

	if (FInsightsManager::Get()->IsAnalysisComplete())
	{
		// This will be the final update.
		bUpdateFilters = false;
	}

	if (FilterConfigurator.IsValid())
	{
		FilterConfigurator->Update();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::IsInTimingProfiler() const
{
	return GetName() == FInsightsManagerTabs::TimingProfilerTabId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<STimersView> STimingView::GetTimersView() const
{
	if (IsInTimingProfiler())
	{
		TSharedPtr<STimingProfilerWindow> TimingWindow = FTimingProfilerManager::Get()->GetProfilerWindow();
		if (TimingWindow.IsValid())
		{
			return TimingWindow->GetTimersView();
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SLogView> STimingView::GetLogView() const
{
	if (IsInTimingProfiler())
	{
		TSharedPtr<STimingProfilerWindow> TimingWindow = FTimingProfilerManager::Get()->GetProfilerWindow();
		if (TimingWindow.IsValid())
		{
			return TimingWindow->GetLogView();
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// User Annotation shortcut handlers
////////////////////////////////////////////////////////////////////////////////////////////////////

FUserAnnotationsTimingViewExtender* STimingView::FindAnnotationExtender() const
{
	// Cache the result — module is process-lifetime and the extender lives as a member of the
	// module, so the pointer is stable. Hot path (CanExecute predicates fire many times per frame).
	if (CachedAnnotationExtender)
	{
		return CachedAnnotationExtender;
	}
	FTraceInsightsModule* Module = static_cast<FTraceInsightsModule*>(
		FModuleManager::GetModulePtr<IUnrealInsightsModule>("TraceInsights"));
	if (Module)
	{
		CachedAnnotationExtender = &Module->GetUserAnnotationsExtender();
	}
	return CachedAnnotationExtender;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::AddPointAnnotation_Execute()
{
	FUserAnnotationsTimingViewExtender* AnnotationExt = FindAnnotationExtender();
	if (!AnnotationExt)
	{
		return;
	}

	TSharedPtr<FUserAnnotationStore> Store = AnnotationExt->GetAnnotationStoreForCurrentSession();
	if (!Store.IsValid())
	{
		return;
	}

	const double Time = Viewport.SlateUnitsToTime(static_cast<float>(MousePosition.X));
	FUserAnnotationsTimingViewExtender::FAnnotationContext Ctx =
		FUserAnnotationsTimingViewExtender::BuildPointContext(*this, Time);
	AnnotationExt->ShowAnnotationDialog(Store, Ctx);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STimingView::AddPointAnnotation_CanExecute() const
{
	if (SelectionEndTime > SelectionStartTime) // Only when no range selection active
	{
		return false;
	}
	FUserAnnotationsTimingViewExtender* AnnotationExt = FindAnnotationExtender();
	return AnnotationExt
		&& AnnotationExt->IsAnnotationsTrackVisible(*this)
		&& AnnotationExt->CanPersistAnnotations(*this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::AddRangeAnnotation_Execute()
{
	FUserAnnotationsTimingViewExtender* AnnotationExt = FindAnnotationExtender();
	if (!AnnotationExt)
	{
		return;
	}

	TSharedPtr<FUserAnnotationStore> Store = AnnotationExt->GetAnnotationStoreForCurrentSession();
	if (!Store.IsValid())
	{
		return;
	}

	FUserAnnotationsTimingViewExtender::FAnnotationContext Ctx =
		FUserAnnotationsTimingViewExtender::BuildRangeContext(*this, SelectionStartTime, SelectionEndTime);
	AnnotationExt->ShowAnnotationDialog(Store, Ctx);
}

bool STimingView::AddRangeAnnotation_CanExecute() const
{
	if (SelectionEndTime <= SelectionStartTime)
	{
		return false;
	}
	FUserAnnotationsTimingViewExtender* AnnotationExt = FindAnnotationExtender();
	return AnnotationExt
		&& AnnotationExt->IsAnnotationsTrackVisible(*this)
		&& AnnotationExt->CanPersistAnnotations(*this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::AddEventAnnotation_Execute()
{
	FUserAnnotationsTimingViewExtender* AnnotationExt = FindAnnotationExtender();
	if (!AnnotationExt)
	{
		return;
	}

	TSharedPtr<FUserAnnotationStore> Store = AnnotationExt->GetAnnotationStoreForCurrentSession();
	if (!Store.IsValid())
	{
		return;
	}

	if (!HoveredEvent.IsValid() || !HoveredEvent->Is<FThreadTrackEvent>())
	{
		return;
	}

	FUserAnnotationsTimingViewExtender::FAnnotationContext Ctx =
		FUserAnnotationsTimingViewExtender::BuildEventContext(*this, *HoveredEvent);
	AnnotationExt->ShowAnnotationDialog(Store, Ctx);
}

bool STimingView::AddEventAnnotation_CanExecute() const
{
	if (!HoveredEvent.IsValid() || !HoveredEvent->Is<FThreadTrackEvent>())
	{
		return false;
	}
	FUserAnnotationsTimingViewExtender* AnnotationExt = FindAnnotationExtender();
	return AnnotationExt
		&& AnnotationExt->IsAnnotationsTrackVisible(*this)
		&& AnnotationExt->CanPersistAnnotations(*this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ToggleAllAnnotationVisibility_Execute()
{
	FUserAnnotationsTimingViewExtender* AnnotationExt = FindAnnotationExtender();
	if (AnnotationExt)
	{
		AnnotationExt->ToggleAllAnnotationVisibility();
	}
}

bool STimingView::ToggleAllAnnotationVisibility_IsChecked() const
{
	FUserAnnotationsTimingViewExtender* AnnotationExt = FindAnnotationExtender();
	if (!AnnotationExt)
	{
		return false;
	}

	TSharedPtr<FUserAnnotationStore> Store = AnnotationExt->GetAnnotationStoreForCurrentSession();
	if (!Store.IsValid())
	{
		return false;
	}

	// Returns true if any annotations are currently visible
	for (const FUserAnnotation& Annotation : Store->GetAllAnnotations())
	{
		if (Annotation.bVisible)
		{
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::NextAnnotation_Execute()
{
	FUserAnnotationsTimingViewExtender* AnnotationExt = FindAnnotationExtender();
	if (AnnotationExt)
	{
		AnnotationExt->NavigateToNextAnnotation(*this, /*bForward=*/true);
	}
}

void STimingView::PrevAnnotation_Execute()
{
	FUserAnnotationsTimingViewExtender* AnnotationExt = FindAnnotationExtender();
	if (AnnotationExt)
	{
		AnnotationExt->NavigateToNextAnnotation(*this, /*bForward=*/false);
	}
}

bool STimingView::HasAnyAnnotations() const
{
	FUserAnnotationsTimingViewExtender* AnnotationExt = FindAnnotationExtender();
	if (!AnnotationExt)
	{
		return false;
	}

	TSharedPtr<FUserAnnotationStore> Store = AnnotationExt->GetAnnotationStoreForCurrentSession();
	return Store.IsValid() && Store->GetAllAnnotations().Num() > 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingView::ToggleFloatingAnnotations_Execute()
{
	FUserAnnotationsTimingViewExtender* AnnotationExt = FindAnnotationExtender();
	if (AnnotationExt)
	{
		AnnotationExt->ToggleFloatingAnnotations();
	}
}

bool STimingView::ToggleFloatingAnnotations_IsChecked() const
{
	FUserAnnotationsTimingViewExtender* AnnotationExt = FindAnnotationExtender();
	return AnnotationExt ? AnnotationExt->GetShowFloatingAnnotations() : true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::TimingProfiler

#undef INSIGHTS_ACTIVATE_BENCHMARK
#undef INSIGHTS_SCOPED_LOG
#undef LOCTEXT_NAMESPACE
