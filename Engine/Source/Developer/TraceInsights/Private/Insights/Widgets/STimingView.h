// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Geometry.h"
#include "Styling/SlateTypes.h"
#include "Templates/Function.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

// TraceInsightsCore
#include "InsightsCore/Common/FixedCircularBuffer.h"

// TraceInsights
#include "Insights/Config.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/TimingProfiler/ViewModels/TimerNode.h"
#include "Insights/ViewModels/BaseTimingTrack.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingEventsTrack.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TimingViewDrawHelper.h"
#include "Insights/ViewModels/TooltipDrawState.h"

#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
#include "Insights/ITimingViewExtender.h"
#endif

class FMenuBuilder;
class FUICommandList;
class SDockTab;
class SOverlay;
class SScrollBar;
class FSpawnTabArgs;

namespace Insights
{
	class FQuickFind;
	class SQuickFind;
}

namespace UE::Insights
{
	class FFilterConfigurator;
	enum class ETimingEventsColoringMode : uint32;
	class SLogView;
	class FFilterConfiguratorNode;
}

namespace UE::Insights::Timing
{
	class ITimingViewExtender;
}

class FTimingViewDrawHelper;

namespace UE::Insights::LoadingProfiler
{
	class FLoadingSharedState;
}

namespace UE::Insights::TimingProfiler
{

class FFrameSharedState;
class FThreadTimingSharedState;
class FTimingRegionsSharedState;
class FFileActivitySharedState;
class FUserAnnotationsTimingViewExtender;

class FTimeRulerTrack;
class FTimeMarker;
class FMarkersTimingTrack;
class FTimingGraphTrack;

class STimersView;

enum class ESelectEventType : uint32
{
	First,
	Previous,
	Next,
	Last,
	Min,
	Max
};

/** A custom widget used to display timing events. */
class STimingView : public SCompoundWidget, public UE::Insights::Timing::ITimingViewSession
{
public:
	/** Default constructor. */
	STimingView();

	/** Virtual destructor. */
	virtual ~STimingView();

	SLATE_BEGIN_ARGS(STimingView)
		{
			_Clipping = EWidgetClipping::ClipToBounds;
		}
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs, FName InViewName = NAME_None);

	/** Gets the name of the view. */
	virtual const FName& GetName() const { return ViewName; }

	bool IsCompactModeEnabled() const;
	void ToggleCompactMode();

	bool IsAutoHideEmptyTracksEnabled() const;
	void ToggleAutoHideEmptyTracks();

	bool IsPanningOnScreenEdgesEnabled() const;
	void TogglePanningOnScreenEdges();

	bool QuickFind_CanExecute() const;
	void QuickFind_Execute();

	bool GoToFrame_CanExecute() const;
	void GoToFrame_Execute();

	bool GoToTime_CanExecute() const;
	void GoToTime_Execute();

	bool TrimAndSaveAs_CanExecute() const;
	void TrimAndSaveAs_Execute();

	bool ToggleTrackVisibility_IsChecked(uint64 InTrackId) const;
	void ToggleTrackVisibility_Execute(uint64 InTrackId);

	TSharedPtr<FFrameSharedState> GetFrameSharedState() const { return FrameSharedState; }
	TSharedPtr<FThreadTimingSharedState> GetThreadTimingSharedState() const { return ThreadTimingSharedState; }
	TSharedPtr<LoadingProfiler::FLoadingSharedState> GetLoadingSharedState() const { return LoadingSharedState; }
	TSharedPtr<FFileActivitySharedState> GetFileActivitySharedState() const { return FileActivitySharedState; }

	TSharedPtr<FFilterConfigurator> MakeCustomFilterConfigurator(uint32 TimerId, double IntervalStart, double IntervalEnd);

	void HideAllDefaultTracks();

	/** Gets the time ruler track. It includes the custom time markers (ones user can drag with mouse). */
	TSharedRef<FTimeRulerTrack> GetTimeRulerTrack() { return TimeRulerTrack; }
	const TSharedRef<const FTimeRulerTrack> GetTimeRulerTrack() const { return TimeRulerTrack; }

	/** Gets the default (custom) time marker (for backward compatibility). */
	TSharedRef<FTimeMarker> GetDefaultTimeMarker() { return DefaultTimeMarker; }
	const TSharedRef<const FTimeMarker> GetDefaultTimeMarker() const { return DefaultTimeMarker; }

	/** Resets internal widget's data to the default one. */
	void Reset(bool bIsFirstReset = false);

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	/**
	 * The system calls this method to notify the widget that a mouse button was pressed within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/**
	 * The system calls this method to notify the widget that a mouse button was release within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/**
	 * Called when a mouse button is double clicked.  Override this in derived classes.
	 *
	 * @param  InMyGeometry  Widget geometry
	 * @param  InMouseEvent  Mouse button event
	 *
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/**
	 * The system calls this method to notify the widget that a mouse moved within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/**
	 * The system will use this event to notify a widget that the cursor has entered it. This event is NOT bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 */
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/**
	 * The system will use this event to notify a widget that the cursor has left it. This event is NOT bubbled.
	 *
	 * @param MouseEvent Information about the input event
	 */
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	/**
	 * Called when the mouse wheel is spun. This event is bubbled.
	 *
	 * @param  MouseEvent  Mouse event
	 *
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/**
	 * Called when the user is dropping something onto a widget; terminates drag and drop.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/**
	 * Called during drag and drop when the drag enters a widget.
	 *
	 * Enter/Leave events in slate are meant as lightweight notifications.
	 * So we do not want to capture mouse or set focus in response to these.
	 * However, OnDragEnter must also support external APIs (e.g. OLE Drag/Drop)
	 * Those require that we let them know whether we can handle the content
	 * being dragged OnDragEnter.
	 *
	 * The concession is to return a can_handled/cannot_handle
	 * boolean rather than a full FReply.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether the contents of the DragDropEvent can potentially be processed by this widget.
	 */
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/**
	 * Called during drag and drop when the drag leaves a widget.
	 *
	 * @param DragDropEvent   The drag and drop event.
	 */
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;

	/**
	 * Called during drag and drop when the mouse is being dragged over a widget.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/**
	 * Called when the system wants to know which cursor to display for this Widget.  This event is bubbled.
	 *
	 * @return  The cursor requested (can be None.)
	 */
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// ITimingViewSession interface

	virtual void AddTopDockedTrack(TSharedPtr<FBaseTimingTrack> Track) override { AddTrack(Track, ETimingTrackLocation::TopDocked); }
	virtual bool RemoveTopDockedTrack(TSharedPtr<FBaseTimingTrack> Track) override { return RemoveTrack(Track); }

	virtual void AddBottomDockedTrack(TSharedPtr<FBaseTimingTrack> Track) override { AddTrack(Track, ETimingTrackLocation::BottomDocked); }
	virtual bool RemoveBottomDockedTrack(TSharedPtr<FBaseTimingTrack> Track) override { return RemoveTrack(Track); }

	virtual void AddScrollableTrack(TSharedPtr<FBaseTimingTrack> Track) override { AddTrack(Track, ETimingTrackLocation::Scrollable); }
	virtual bool RemoveScrollableTrack(TSharedPtr<FBaseTimingTrack> Track) override { return RemoveTrack(Track); }
	virtual void InvalidateScrollableTracksOrder() override;

	virtual void AddForegroundTrack(TSharedPtr<FBaseTimingTrack> Track) override { AddTrack(Track, ETimingTrackLocation::Foreground); }
	virtual bool RemoveForegroundTrack(TSharedPtr<FBaseTimingTrack> Track) override { return RemoveTrack(Track); }

	virtual void AddTrack(TSharedPtr<FBaseTimingTrack> Track, ETimingTrackLocation Location) override;
	virtual bool RemoveTrack(TSharedPtr<FBaseTimingTrack> Track) override;

	virtual TSharedPtr<FBaseTimingTrack> FindTrack(uint64 InTrackId) override;
	virtual void EnumerateTracks(TFunctionRef<void(TSharedPtr<FBaseTimingTrack> Track)> Callback) override;

	virtual double GetTimeMarker() const override;
	virtual void SetTimeMarker(double InTimeMarker) override;
	virtual void SetAndCenterOnTimeMarker(double InTimeMarker) override;

	virtual Timing::FSelectionChangedDelegate& OnSelectionChanged() override { return OnSelectionChangedDelegate; }
	virtual Timing::FTimeMarkerChangedDelegate& OnTimeMarkerChanged() override { return OnTimeMarkerChangedDelegate; }
	virtual Timing::FCustomTimeMarkerChangedDelegate& OnCustomTimeMarkerChanged() override { return OnCustomTimeMarkerChangedDelegate; }
	virtual Timing::FHoveredTrackChangedDelegate& OnHoveredTrackChanged() override { return OnHoveredTrackChangedDelegate; }
	virtual Timing::FHoveredEventChangedDelegate& OnHoveredEventChanged() override { return OnHoveredEventChangedDelegate; }
	virtual Timing::FSelectedTrackChangedDelegate& OnSelectedTrackChanged() override { return OnSelectedTrackChangedDelegate; }
	virtual Timing::FSelectedEventChangedDelegate& OnSelectedEventChanged() override { return OnSelectedEventChangedDelegate; }
	virtual Timing::FTrackVisibilityChangedDelegate& OnTrackVisibilityChanged() override { return OnTrackVisibilityChangedDelegate; }
	virtual Timing::FTrackAddedDelegate& OnTrackAdded() override { return OnTrackAddedDelegate; }
	virtual Timing::FTrackRemovedDelegate& OnTrackRemoved() override { return OnTrackRemovedDelegate; }

	virtual void ResetSelectedEvent() override
	{
		if (SelectedEvent)
		{
			SelectedEvent.Reset();
			OnSelectedEventChanged();
		}
	}

	virtual void ResetEventFilter() override { SetEventFilter(nullptr); }

	virtual void PreventThrottling() override;
	virtual void AddOverlayWidget(const TSharedRef<SWidget>& InWidget) override;

	virtual const TArray<TUniquePtr<ITimingEventRelation>>& GetCurrentRelations() const override { return CurrentRelations; }
	virtual TArray<TUniquePtr<ITimingEventRelation>>& EditCurrentRelations() override { return CurrentRelations; }
	virtual void AddRelation(TUniquePtr<ITimingEventRelation>& Relation) override { CurrentRelations.Add(MoveTemp(Relation)); }
	virtual void ClearRelations() override;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	// The callback should return 'true' to continue the enumeration.
	void EnumerateAllTracks(TFunctionRef<bool(TSharedPtr<FBaseTimingTrack>&)> Callback);

	const TArray<TSharedPtr<FBaseTimingTrack>>& GetTrackList(ETimingTrackLocation TrackLocation) const
	{
		static const TArray<TSharedPtr<FBaseTimingTrack>> EmptyTrackList;
		switch (TrackLocation)
		{
			case ETimingTrackLocation::Scrollable:   return ScrollableTracks;
			case ETimingTrackLocation::TopDocked:    return TopDockedTracks;
			case ETimingTrackLocation::BottomDocked: return BottomDockedTracks;
			case ETimingTrackLocation::Foreground:   return ForegroundTracks;
			default:                                 return EmptyTrackList;
		}
	}

	static const TCHAR* GetLocationName(ETimingTrackLocation Location);

	void ChangeTrackLocation(TSharedRef<FBaseTimingTrack> Track, ETimingTrackLocation NewLocation);
	bool CanChangeTrackLocation(TSharedRef<FBaseTimingTrack> Track, ETimingTrackLocation NewLocation) const;
	bool CheckTrackLocation(TSharedRef<FBaseTimingTrack> Track, ETimingTrackLocation Location) const;

	void UpdateScrollableTracksOrder();
	int32 GetFirstScrollableTrackOrder() const;
	int32 GetLastScrollableTrackOrder() const;

	void HideAllScrollableTracks();

	void HandleTrackVisibilityChanged();

	bool IsOldGpu1TrackVisible() const;
	bool IsOldGpu2TrackVisible() const;
	bool IsAnyGpuTrackVisible() const;
	bool IsGpuTrackVisible(uint32 InQueueId) const;
	bool IsCpuTrackVisible(uint32 InThreadId) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	const FTimingTrackViewport& GetViewport() const { return Viewport; }

	const FVector2D& GetMousePosition() const { return MousePosition; }

	double GetSelectionStartTime() const { return SelectionStartTime; }
	double GetSelectionEndTime() const { return SelectionEndTime; }

	bool IsPanning() const { return bIsPanning; }
	bool IsSelecting() const { return bIsSelecting; }

	bool IsTimeSelected(double Time) const { return Time >= SelectionStartTime && Time < SelectionEndTime; }
	bool IsTimeSelectedInclusive(double Time) const { return Time >= SelectionStartTime && Time <= SelectionEndTime; }

	bool IsAutoScrollEnabled() const { return bAutoScroll; }
	void SetAutoScroll(bool bOnOff);

	void ScrollAtPosY(float ScrollPosY);
	void BringIntoViewY(float InTopY, float InBottomY);
	void BringScrollableTrackIntoView(const FBaseTimingTrack& Track);
	void BringScrollableTrackIntoViewDelayed(TSharedPtr<FBaseTimingTrack> Track, int32 LaneDepth = -1);
	void ScrollAtTime(double StartTime);
	void CenterOnTimeInterval(double IntervalStartTime, double IntervalDuration);
	void ZoomOnTimeInterval(double IntervalStartTime, double IntervalDuration);
	void BringIntoView(double StartTime, double EndTime);
	void SelectTimeInterval(double IntervalStartTime, double IntervalDuration);
	void SnapToFrameBound(double& IntervalStartTime, double& IntervalDuration);
	void SelectToTimeMarker(double InTimeMarker);

	//bool AreTimeMarkersVisible() { return MarkersTrack->IsVisible(); }
	void SetTimeMarkersVisible(bool bOnOff);
	//bool IsDrawOnlyBookmarksEnabled() { return MarkersTrack->IsBookmarksTrack(); }
	void SetDrawOnlyBookmarks(bool bOnOff);

	TSharedPtr<FTimingGraphTrack> GetMainTimingGraphTrack() { return GraphTrack; }

	const TSharedPtr<FBaseTimingTrack> GetHoveredTrack() const { return HoveredTrack; }
	const TSharedPtr<const ITimingEvent> GetHoveredEvent() const { return HoveredEvent; }

	const TSharedPtr<FBaseTimingTrack> GetSelectedTrack() const { return SelectedTrack; }
	const TSharedPtr<const ITimingEvent> GetSelectedEvent() const { return SelectedEvent; }

	void SelectTimingTrack(const TSharedPtr<FBaseTimingTrack> InTrack, bool bBringTrackIntoView);
	void SelectTimingEvent(const TSharedPtr<const ITimingEvent> InEvent, bool bBringEventIntoViewHorizontally, bool bBringEventIntoViewVertically = false);
	void ToggleGraphSeries(const TSharedPtr<const ITimingEvent> InEvent);

	const TSharedPtr<ITimingEventFilter> GetEventFilter() const { return TimingEventFilter; }
	void SetEventFilter(const TSharedPtr<ITimingEventFilter> InEventFilter);

	void ToggleEventFilterByEventType(const uint64 EventType);

	bool IsFilterByEventType(const uint64 EventType) const;

	const TSharedPtr<FBaseTimingTrack> GetTrackAt(float InPosX, float InPosY) const;

	TSharedPtr<FUICommandList> GetCommandList() { return CommandList; }

	void CloseQuickFindTab();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// User Annotation shortcut handlers

private:
	// TODO: STimingView here reaches into a specific extender by type, and does so through
	// FTraceInsightsModule — both of which run against the ITimingViewExtender contract. When
	// STimingView is eventually split into a trace-agnostic base class (TraceInsightsCore) and a
	// TimingProfiler-specific derived class, annotations belong in the base as a generic feature.
	// The intended direction is to register the annotations extender as an IModularFeatures
	// implementation alongside other plugin extenders (see GetExtenders) and drive it through
	// generic ITimingViewExtender hooks or a dedicated interface, removing both the type-specific
	// access and the module dependency. The current direct access is scoped to these private
	// handlers and the TraceInsights module, so it can be migrated without touching external
	// callers.
	UE::Insights::TimingProfiler::FUserAnnotationsTimingViewExtender* FindAnnotationExtender() const;

	/** Cached extender pointer — avoids per-call FModuleManager::GetModulePtr (slow) from CanExecute predicates.
	 *  Safe because FTraceInsightsModule is process-lifetime: loaded once at IMPLEMENT_MODULE and never
	 *  unloaded in standalone UnrealInsights or the editor. The extender lives as a value member of the
	 *  module, so the address is stable for the widget's lifetime. If the module ever becomes unload/
	 *  reload-capable at runtime, this needs FModuleManager::OnModulesChanged invalidation. */
	mutable UE::Insights::TimingProfiler::FUserAnnotationsTimingViewExtender* CachedAnnotationExtender = nullptr;

public:
	void AddPointAnnotation_Execute();
	bool AddPointAnnotation_CanExecute() const;

	void AddRangeAnnotation_Execute();
	bool AddRangeAnnotation_CanExecute() const;

	void AddEventAnnotation_Execute();
	bool AddEventAnnotation_CanExecute() const;

	void ToggleAllAnnotationVisibility_Execute();
	bool ToggleAllAnnotationVisibility_IsChecked() const;

	void NextAnnotation_Execute();
	void PrevAnnotation_Execute();
	bool HasAnyAnnotations() const;

	void ToggleFloatingAnnotations_Execute();
	bool ToggleFloatingAnnotations_IsChecked() const;

	TSharedPtr<UE::Insights::FFilterConfigurator> GetFilterConfigurator() { return FilterConfigurator; }

	TMap<uint64, TSharedPtr<FBaseTimingTrack>>& GetAllTracks() { return AllTracks; }

	void SelectEventInstance(uint32 TimerId, ESelectEventType Type, bool bUseSelection);

	bool FindEventInstance_CanExecute() const;
	void FindEventInstance_Execute(ESelectEventType EventType);
	bool FindEventInstanceInSelection_CanExecute() const;
	void FindEventInstanceInSelection_Execute(ESelectEventType EventType);

protected:
	virtual FVector2D ComputeDesiredSize(float) const override
	{
		return FVector2D(16.0f, 16.0f);
	}

	/** Binds our UI commands to delegates. */
	void BindCommands();

	void CreateCompactMenuLine(FMenuBuilder& MenuBuilder, FText Label, TSharedRef<SWidget> InnerWidget) const;
	TSharedRef<SWidget> MakeCompactAutoScrollOptionsMenu();
	TSharedRef<SWidget> MakeAutoScrollOptionsMenu();

	TSharedRef<SWidget> MakeAllTracksMenu();
	void CreateAllTracksMenu(FMenuBuilder& MenuBuilder);

	TSharedRef<SWidget> MakeCpuGpuTracksFilterMenu();

	TSharedRef<SWidget> MakeOtherTracksFilterMenu();
	bool ShowHideGraphTrack_IsChecked() const;
	void ShowHideGraphTrack_Execute();

	TSharedRef<SWidget> MakePluginTracksFilterMenu();

	TSharedRef<SWidget> MakeViewModeMenu();

	void CreateDepthLimitMenu(FMenuBuilder& MenuBuilder);
	FText GetEventDepthLimitKeybindingText(uint32 InDepthLimitMax) const;
	uint32 GetNextEventDepthLimitMax(uint32 InDepthLimitMax) const;
	bool CheckEventDepthLimitMax(uint32 InDepthLimitMax) const;
	void ChooseNextEventDepthLimitMax();
	void SetEventDepthLimitMax(uint32 InDepthLimitMax);
	void SetEventDepthLimitMin(uint32 InDepthLimitMin);

	void CreateCpuThreadTrackColoringModeMenu(FMenuBuilder& MenuBuilder);
	void ChooseNextCpuThreadTrackColoringMode();
	void SetCpuThreadTrackColoringMode(UE::Insights::ETimingEventsColoringMode Mode);
	bool CheckCpuThreadTrackColoringMode(UE::Insights::ETimingEventsColoringMode Mode);

	void ShowContextMenu(const FPointerEvent& MouseEvent);
	void CreateTrackLocationMenu(FMenuBuilder& MenuBuilder, TSharedRef<FBaseTimingTrack> Track);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Auto-Scroll

	void AutoScroll_OnCheckStateChanged(ECheckBoxState NewRadioState);
	ECheckBoxState AutoScroll_IsChecked() const;

	void SetAutoScrollFrameAlignment(int32 FrameType);
	bool CompareAutoScrollFrameAlignment(int32 FrameType) const;

	void SetAutoScrollViewportOffset(double Percent);
	bool CompareAutoScrollViewportOffset(double Percent) const;

	void SetAutoScrollDelay(double Delay);
	bool CompareAutoScrollDelay(double Delay) const;

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void UpdatePositionForScrollableTracks();

	double EnforceHorizontalScrollLimits(const double InStartTime);
	float EnforceVerticalScrollLimits(const float InScrollPosY);

	/**
	 * Called when the user scrolls the horizontal scrollbar.
	 *
	 * @param ScrollOffset Scroll offset as a fraction between 0 and 1.
	 */
	void HorizontalScrollBar_OnUserScrolled(float ScrollOffset);
	void UpdateHorizontalScrollBar();

	/**
	 * Called when the user scrolls the vertical scrollbar.
	 *
	 * @param ScrollOffset Scroll offset as a fraction between 0 and 1.
	 */
	void VerticalScrollBar_OnUserScrolled(float ScrollOffset);
	void UpdateVerticalScrollBar();

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void RaiseSelectionChanging();
	void RaiseSelectionChanged();

	void RaiseTimeMarkerChanging(TSharedRef<FTimeMarker> InTimeMarker);
	void RaiseTimeMarkerChanged(TSharedRef<FTimeMarker> InTimeMarker);

	void UpdateHoveredTimingEvent(float InMousePosX, float InMousePosY);

	void OnSelectedTimingEventChanged();

	void SelectHoveredTimingTrack();
	void SelectHoveredTimingEvent();

	void SelectLeftTimingEvent();
	void SelectRightTimingEvent();
	void SelectUpTimingEvent();
	void SelectDownTimingEvent();

	void FrameSelection();

	// Get all the plugin extenders we care about
	TArray<UE::Insights::Timing::ITimingViewExtender*> GetExtenders() const;
#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<::Insights::ITimingViewExtender*> GetOldExtenders() const;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54

	FReply AllowTracksToProcessOnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	FReply AllowTracksToProcessOnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	FReply AllowTracksToProcessOnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	void SetTrackPosY(FBaseTimingTrack& Track, float TrackPosY) const;

	void FindFirstEvent(bool bUseCustomFilter);
	void FindPrevEvent(bool bUseCustomFilter);
	void FindNextEvent(bool bUseCustomFilter);
	void FindLastEvent(bool bUseCustomFilter);
	void FindMaxEvent();
	void FindMinEvent();
	void PostFindEvent(TSharedPtr<const ITimingEvent> BestMatchEvent);
	TSharedPtr<const FBaseTimingTrack> FindPriorityTrack(FFilterConfiguratorNode& Node);
	void FilterAllTracks();
	void ClearFilters();

	TSharedRef<SDockTab> SpawnQuickFindTab(const FSpawnTabArgs& Args);
	void PopulateTrackSuggestionList(const FString& Text, TArray<FString>& OutSuggestions);
	void PopulateTimerNameSuggestionList(const FString& Text, TArray<FString>& OutSuggestions);
	void PopulateRegionNameSuggestionList(const FString& Text, TArray<FString>& OutSuggestions);
	void PopulateRegionCategorySuggestionList(const FString& Text, TArray<FString>& OutSuggestions);

	typedef TFunctionRef<void(TSharedPtr<const FBaseTimingTrack> Track)> EnumerateFilteredTracksCallback;
	void EnumerateFilteredTracks(TSharedPtr<UE::Insights::FFilterConfigurator> FilterConfigurator, TSharedPtr<const FBaseTimingTrack> PriorityTrack, EnumerateFilteredTracksCallback Callback);

	ETraceFrameType GetFrameTypeToSnapTo();

	/** The FilterConfigurator needs to be updated so that custom filters work correctly when analysis is still running
	  * and the timer list can change.
	  */
	void UpdateFilters();

	bool IsInTimingProfiler() const;
	TSharedPtr<STimersView> GetTimersView() const;
	TSharedPtr<SLogView> GetLogView() const;

protected:
	/** The name of the view. */
	FName ViewName;

	/** The track's viewport. Encapsulates info about position and scale. */
	FTimingTrackViewport Viewport;

	////////////////////////////////////////////////////////////

	/** All created tracks.
	  * Maps track id to track pointer.
	  */
	TMap<uint64, TSharedPtr<FBaseTimingTrack>> AllTracks;

	TArray<TSharedPtr<FBaseTimingTrack>> TopDockedTracks; /**< tracks docked on top, in the order to be displayed (top to bottom) */
	TArray<TSharedPtr<FBaseTimingTrack>> BottomDockedTracks; /**< tracks docked on bottom, in the order to be displayed (top to bottom) */
	TArray<TSharedPtr<FBaseTimingTrack>> ScrollableTracks; /**< tracks in scrollable area, in the order to be displayed (top to bottom) */
	TArray<TSharedPtr<FBaseTimingTrack>> ForegroundTracks; /**< tracks to draw over top/scrollable/bottom tracks (can use entire area), in the order to be displayed (back to front) */

	/** Whether the order of scrollable tracks is dirty and list need to be re-sorted */
	bool bScrollableTracksOrderIsDirty;

	////////////////////////////////////////////////////////////

	// Shared state for Frame Thread tracks
	TSharedPtr<FFrameSharedState> FrameSharedState;

	// Shared state for CPU/GPU Thread tracks
	TSharedPtr<FThreadTimingSharedState> ThreadTimingSharedState;

	// Shared state for Asset Loading tracks
	TSharedPtr<LoadingProfiler::FLoadingSharedState> LoadingSharedState;

	// Shared state for File Activity (I/O) tracks
	TSharedPtr<FFileActivitySharedState> FileActivitySharedState;

	// Shared state for Regions tracks
	TSharedPtr<FTimingRegionsSharedState> TimingRegionsSharedState;

	////////////////////////////////////////////////////////////

	/** The time ruler track. It includes the custom time markers (ones user can drag with mouse). */
	TSharedRef<FTimeRulerTrack> TimeRulerTrack;

	/** The default time marker (for backward compatibility). */
	TSharedRef<FTimeMarker> DefaultTimeMarker;

	/** The time markers track. It displays fixed time markers based on bookmarks and log messages. */
	TSharedRef<FMarkersTimingTrack> MarkersTrack;

	/** A graph track for frame times and CPU/GPU timing graphs. */
	TSharedPtr<FTimingGraphTrack> GraphTrack;

	////////////////////////////////////////////////////////////

	/** The extension overlay containing external sub-widgets */
	TSharedPtr<SOverlay> ExtensionOverlay;

	/** Horizontal scroll bar, used for scrolling timing events' viewport. */
	TSharedPtr<SScrollBar> HorizontalScrollBar;

	/** Vertical scroll bar, used for scrolling timing events' viewport. */
	TSharedPtr<SScrollBar> VerticalScrollBar;

	////////////////////////////////////////////////////////////

	/** The current mouse position. */
	FVector2D MousePosition;

	/** Mouse position during the call on mouse button down. */
	FVector2D MousePositionOnButtonDown;
	double ViewportStartTimeOnButtonDown;
	float ViewportScrollPosYOnButtonDown;

	/** Mouse position during the call on mouse button up. */
	FVector2D MousePositionOnButtonUp;

	float LastScrollPosY;

	bool bIsLMB_Pressed;
	bool bIsRMB_Pressed;

	bool bIsSpaceBarKeyPressed;
	bool bIsDragging;

	////////////////////////////////////////////////////////////
	// Auto-Scroll

	/** True if the viewport scrolls automatically. */
	bool bAutoScroll;

	/**
	 * Frame Alignment. Controls if auto-scroll should align center of the viewport with start of a frame or not.
	 * Valid options: -1 to disable frame alignment or the type of frame to align with (0 = Game or 1 = Rendering; see ETraceFrameType).
	 */
	int32 AutoScrollFrameAlignment;

	/**
	 * Viewport offset while auto-scrolling, as percent of viewport width.
	 * If positive, it offsets the viewport forward, allowing an empty space at the right side of the viewport (i.e. after end of session).
	 * If negative, it offsets the viewport backward (i.e. end of session will be outside viewport).
	 */
	double AutoScrollViewportOffsetPercent;

	/** Minimum time between two auto-scroll updates, in [seconds]. */
	double AutoScrollMinDelay;

	/** Timestamp of last auto-scroll update, in [cycle64]. */
	uint64 LastAutoScrollTime;

	////////////////////////////////////////////////////////////
	// Panning

	/** True, if the user is currently interactively panning the view (horizontally and/or vertically). */
	bool bIsPanning;

	/** If enabled, the panning is allowed to continue when mouse cursor reaches the edges of the screen. */
	bool bAllowPanningOnScreenEdges;

	float DPIScaleFactor;

	uint32 EdgeFrameCountX;
	uint32 EdgeFrameCountY;

	/** How to pan. */
	enum class EPanningMode : uint8
	{
		None = 0,
		Horizontal = 0x01,
		Vertical = 0x02,
		HorizontalAndVertical = Horizontal | Vertical,
	};
	EPanningMode PanningMode;

	float OverscrollLeft;
	float OverscrollRight;
	float OverscrollTop;
	float OverscrollBottom;

	////////////////////////////////////////////////////////////
	// Selection

	/** True, if the user is currently changing the selection. */
	bool bIsSelecting;

	double SelectionStartTime;
	double SelectionEndTime;

	TSharedPtr<FBaseTimingTrack> HoveredTrack;
	TSharedPtr<const ITimingEvent> HoveredEvent;

	TSharedPtr<FBaseTimingTrack> SelectedTrack;
	TSharedPtr<const ITimingEvent> SelectedEvent;

	TSharedPtr<FBaseTimingTrack> DelayedVisibleTrack;
	int32 DelayedVisibleTrackLaneDepth = -1;
	float DelayedVisibleTrackLastScrollHeight = 0.0f;
	int32 DelayedVisibleTrackCounter = 0;

	TSharedPtr<ITimingEventFilter> TimingEventFilter;

	FTooltipDrawState Tooltip;

	enum class ESelectionType
	{
		None,
		TimeRange,
		TimingEvent
	};
	ESelectionType LastSelectionType;

	/** Throttle flag, allowing tracks to control whether Slate throttle should take place */
	bool bPreventThrottling;

	////////////////////////////////////////////////////////////
	// Misc

	FGeometry ThisGeometry;

	const FSlateBrush* WhiteBrush;
	const FSlateFontInfo MainFont;

	bool bDrawTopSeparatorLine;
	bool bDrawBottomSeparatorLine;

	bool bBringSelectedEventIntoViewVerticallyOnNextTick = false;

	// Debug stats
	int32 NumUpdatedEvents;
	typedef TFixedCircularBuffer<uint64, 32> FHistoryBuffer;
	FHistoryBuffer PreUpdateTracksDurationHistory;
	FHistoryBuffer UpdateTracksDurationHistory;
	FHistoryBuffer PostUpdateTracksDurationHistory;
	FHistoryBuffer TickDurationHistory;
	mutable FHistoryBuffer PreDrawTracksDurationHistory;
	mutable FHistoryBuffer DrawTracksDurationHistory;
	mutable FHistoryBuffer PostDrawTracksDurationHistory;
	mutable FHistoryBuffer TotalDrawDurationHistory;
	mutable FHistoryBuffer OnPaintDeltaTimeHistory;
	mutable uint64 LastOnPaintTime;

	////////////////////////////////////////////////////////////
	// Delegates

	Timing::FSelectionChangedDelegate OnSelectionChangedDelegate;
	Timing::FTimeMarkerChangedDelegate OnTimeMarkerChangedDelegate;
	Timing::FCustomTimeMarkerChangedDelegate OnCustomTimeMarkerChangedDelegate;
	Timing::FHoveredTrackChangedDelegate OnHoveredTrackChangedDelegate;
	Timing::FHoveredEventChangedDelegate OnHoveredEventChangedDelegate;
	Timing::FSelectedTrackChangedDelegate OnSelectedTrackChangedDelegate;
	Timing::FSelectedEventChangedDelegate OnSelectedEventChangedDelegate;
	Timing::FTrackVisibilityChangedDelegate OnTrackVisibilityChangedDelegate;
	Timing::FTrackAddedDelegate OnTrackAddedDelegate;
	Timing::FTrackRemovedDelegate OnTrackRemovedDelegate;

	TSharedPtr<FUICommandList> CommandList;

	TArray<TUniquePtr<ITimingEventRelation>> CurrentRelations;

	TSharedPtr<::Insights::FQuickFind> QuickFindVm;
	TSharedPtr<FFilterConfigurator> FilterConfigurator;
	// Temporary custom filter to run timer search queries from context menus.
	TSharedPtr<FFilterConfigurator> CustomFilterConfigurator;
	static uint32 TimingViewId;
	const FName QuickFindTabId;
	bool bUpdateFilters = true;

	// Used only between the creation of the widget and the spawning of the owning tab. When the tab is spawned, we relinquish ownership.
	TSharedPtr<::Insights::SQuickFind> QuickFindWidgetSharedPtr;
	TWeakPtr<::Insights::SQuickFind> QuickFindWidgetWeakPtr;
};

} // namespace UE::Insights::TimingProfiler
