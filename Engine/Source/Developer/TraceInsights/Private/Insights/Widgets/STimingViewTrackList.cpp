// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimingViewTrackList.h"

#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SHeaderRow.h"

// TraceInsights
#include "Insights/ViewModels/BaseTimingTrack.h"
#include "Insights/Widgets/STimingView.h"

#define LOCTEXT_NAMESPACE "STimingViewTrackList"

////////////////////////////////////////////////////////////////////////////////////////////////////
// STimingViewTrackListEntry
////////////////////////////////////////////////////////////////////////////////////////////////////

// A list entry widget for a track
class STimingViewTrackListEntry : public STableRow<TSharedPtr<FBaseTimingTrack>>
{
public:
	SLATE_BEGIN_ARGS(STimingViewTrackListEntry) {}

	SLATE_ARGUMENT(TSharedPtr<UE::Insights::TimingProfiler::STimingView>, TimingView)
	SLATE_ARGUMENT(ETimingTrackLocation, TrackLocation)
	SLATE_ARGUMENT(TSharedPtr<FBaseTimingTrack>, Track)

	SLATE_ATTRIBUTE(FText, SearchText)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		TimingView = InArgs._TimingView;
		TrackLocation = InArgs._TrackLocation;
		Track = InArgs._Track;
		SearchText = InArgs._SearchText;

		STableRow<TSharedPtr<FBaseTimingTrack>>::Construct(STableRow<TSharedPtr<FBaseTimingTrack>>::FArguments(), InOwnerTable);
	}

	virtual void ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent) override
	{
		ChildSlot
		[
			//SNew(SHorizontalBox)
			//+ SHorizontalBox::Slot()
			//.AutoWidth()
			//.Padding(2.0f)
			//.HAlign(HAlign_Center)
			//.VAlign(VAlign_Center)
			//[
				SNew(SCheckBox)
				.IsChecked_Lambda([this](){ return Track.Pin()->IsVisible() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
				{
					Track.Pin()->SetVisibilityFlag(InCheckBoxState == ECheckBoxState::Checked);
					Track.Pin()->SetDirtyFlag();
					TimingView.Pin()->HandleTrackVisibilityChanged();
				})
				.Content()
				[
					SNew(STextBlock)
					.Text(FText::FromString(Track.Pin()->GetName()))
					.HighlightText(SearchText)
				]
			//]
			//+ SHorizontalBox::Slot()
			//.FillWidth(1.0f)
			//.Padding(2.0f)
			//.HAlign(HAlign_Right)
			//.VAlign(VAlign_Center)
			//[
			//	SNew(SBox)
			//	.HAlign(HAlign_Center)
			//	.VAlign(VAlign_Center)
			//	.WidthOverride(12.0f)
			//	.HeightOverride(12.0f)
			//	[
			//		SNew(SImage)
			//		.ColorAndOpacity_Lambda([this]() { return FSlateColor(GraphSeries.Pin()->GetColor()); })
			//		.Image(FAppStyle::Get().GetBrush("Icons.Circle"))
			//	]
			//]
		];
	}

	// The widget containing the track we represent
	TWeakPtr<UE::Insights::TimingProfiler::STimingView> TimingView;

	// The location of the track we represent
	ETimingTrackLocation TrackLocation;

	// The track we represent
	TWeakPtr<FBaseTimingTrack> Track;

	// The search text to highlight
	TAttribute<FText> SearchText;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// STimingViewTrackList
////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingViewTrackList::Construct(const FArguments& InArgs, const TSharedRef<UE::Insights::TimingProfiler::STimingView>& InTimingView, ETimingTrackLocation InTrackLocation)
{
	TimingView = InTimingView;
	TrackLocation = InTrackLocation;

	FilteredTracks = InTimingView->GetTrackList(TrackLocation);

	ListView = SNew(SListView<TSharedPtr<FBaseTimingTrack>>)
		.IsFocusable(true)
		.SelectionMode(ESelectionMode::None)
		.ListItemsSource(&FilteredTracks)
		.OnGenerateRow(this, &STimingViewTrackList::OnGenerateRow);

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			[
				// Checkbox for bulk operations
				SNew(SCheckBox)
				.IsChecked_Lambda([this]()
				{
					int32 NumVisible = 0;
					for (TSharedPtr<FBaseTimingTrack> Track : FilteredTracks)
					{
						if (Track->IsVisible())
						{
							NumVisible++;
						}
					}

					if (NumVisible == 0)
					{
						return ECheckBoxState::Unchecked;
					}
					else if (NumVisible == FilteredTracks.Num())
					{
						return ECheckBoxState::Checked;
					}

					return ECheckBoxState::Undetermined;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
				{
					const bool bVisible = InCheckBoxState != ECheckBoxState::Unchecked;
					for (TSharedPtr<FBaseTimingTrack> Track : FilteredTracks)
					{
						Track->SetVisibilityFlag(bVisible);
						Track->SetDirtyFlag();
					}
					TimingView.Pin()->HandleTrackVisibilityChanged();
				})
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
			.FillWidth(1.0f)
			[
				// Search box allows for filtering
				SAssignNew(SearchBox, SSearchBox)
				.OnTextChanged_Lambda([this](const FText& InText){ SearchText = InText; RefreshFilter(); })
			]
		]
		+ SVerticalBox::Slot()
		.Padding(FMargin(2.0f, 0.0f))
		.FillHeight(1.0f)
		[
			SNew(SScrollBorder, ListView.ToSharedRef())
			[
				ListView.ToSharedRef()
			]
		]
	];

	// Set focus to the search box on creation
	FSlateApplication::Get().SetKeyboardFocus(SearchBox);
	FSlateApplication::Get().SetUserFocus(0, SearchBox);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableRow> STimingViewTrackList::OnGenerateRow(TSharedPtr<FBaseTimingTrack> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STimingViewTrackListEntry, OwnerTable)
		.TimingView(TimingView.Pin())
		.TrackLocation(TrackLocation)
		.Track(Item)
		.SearchText_Lambda([this](){ return SearchText; });
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STimingViewTrackList::RefreshFilter()
{
	FilteredTracks.Reset();

	for (TSharedPtr<FBaseTimingTrack> Track : TimingView.Pin()->GetTrackList(TrackLocation))
	{
		if (SearchText.IsEmpty() || Track->GetName().Contains(SearchText.ToString()))
		{
			FilteredTracks.Add(Track);
		}
	}

	ListView->RequestListRefresh();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
