// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

#include "Insights/ViewModels/BaseTimingTrack.h"

class SSearchBox;
namespace UE::Insights::TimingProfiler { class STimingView; }

// A widget representing all the series in a graph track, allowing management of their visibility
class STimingViewTrackList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STimingViewTrackList) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<UE::Insights::TimingProfiler::STimingView>& InTimingView, ETimingTrackLocation InTrackLocation);

private:
	// Generate a row for the list
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FBaseTimingTrack> Item, const TSharedRef<STableViewBase>& OwnerTable);

	// Refresh the list as the filter has changed
	void RefreshFilter();

private:
	// The timing view widget we are operating on
	TWeakPtr<UE::Insights::TimingProfiler::STimingView> TimingView;

	// The location of tracks we are operating on (scrollable tracks, top docked tracks, etc.)
	ETimingTrackLocation TrackLocation;

	// The search widget
	TSharedPtr<SSearchBox> SearchBox;

	// The list view widget
	TSharedPtr<SListView<TSharedPtr<FBaseTimingTrack>>> ListView;

	// Text we are searching for
	FText SearchText;

	// Filtered list of tracks
	TArray<TSharedPtr<FBaseTimingTrack>> FilteredTracks;
};
