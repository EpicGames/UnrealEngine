// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"
UE_DEPRECATED_HEADER(5.5, "Use InsightsCore/Common/AvailabilityCheck.h from the TraceInsightsCore module instead.")

// TraceInsightsCore
#include "InsightsCore/Common/AvailabilityCheck.h"

// TraceInsights
#include "Insights/Config.h"

#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
namespace Insights { using FAvailabilityCheck = UE::Insights::FAvailabilityCheck; }
#endif
