// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"
#include "Math/UnrealMathUtility.h"

namespace UE::Insights
{

// Perceptually uniform, colorblind-friendly color maps for spatial visualization.
// Maps a scalar T in [0,1] to FLinearColor via piecewise-linear interpolation.
// Viridis: purple -> teal -> yellow.
// Inferno: black -> purple -> orange -> yellow.

inline FLinearColor ColorMapViridis(float T)
{
	T = FMath::Clamp(T, 0.0f, 1.0f);

	// 5 control points at T = {0.0, 0.25, 0.5, 0.75, 1.0}
	static constexpr FLinearColor Colors[] = {
		FLinearColor(FColor( 68,   1,  84)),
		FLinearColor(FColor( 59,  82, 139)),
		FLinearColor(FColor( 33, 145, 140)),
		FLinearColor(FColor( 94, 201,  98)),
		FLinearColor(FColor(253, 231,  37))
	};

	constexpr int32 NumColors = UE_ARRAY_COUNT(Colors);
	const float ScaledT = T * (NumColors - 1);
	const int32 Index = FMath::Min(FMath::FloorToInt(ScaledT), NumColors - 2);
	const float Fraction = ScaledT - Index;

	return FMath::Lerp(Colors[Index], Colors[Index + 1], Fraction);
}

inline FLinearColor ColorMapInferno(float T)
{
	T = FMath::Clamp(T, 0.0f, 1.0f);

	// 9 control points at T = {0.0, 0.125, 0.25, 0.375, 0.5, 0.625, 0.75, 0.875, 1.0}
	static constexpr FLinearColor Colors[] = {
		FLinearColor(FColor(  0,   0,   4)),
		FLinearColor(FColor( 33,  12,  74)),
		FLinearColor(FColor(163,  44,  97)),
		FLinearColor(FColor(230,  93,  47)),
		FLinearColor(FColor(249, 141,  10)),
		FLinearColor(FColor(251, 157,   7)),
		FLinearColor(FColor(252, 172,  17)),
		FLinearColor(FColor(250, 196,  42)),
		FLinearColor(FColor(252, 255, 165))
	};

	constexpr int32 NumColors = UE_ARRAY_COUNT(Colors);
	const float ScaledT = T * (NumColors - 1);
	const int32 Index = FMath::Min(FMath::FloorToInt(ScaledT), NumColors - 2);
	const float Fraction = ScaledT - Index;

	return FMath::Lerp(Colors[Index], Colors[Index + 1], Fraction);
}

inline FLinearColor ColorMapGrayscale(float T)
{
	T = FMath::Clamp(T, 0.0f, 1.0f);
	return FLinearColor(T, T, T, 1.0f);
}

} // namespace UE::Insights
