// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"
UE_DEPRECATED_HEADER(5.5, "Use InsightsCore/Common/SimpleRtti.h from the TraceInsightsCore module instead.")

// TraceInsightsCore
#include "InsightsCore/Common/SimpleRtti.h"
