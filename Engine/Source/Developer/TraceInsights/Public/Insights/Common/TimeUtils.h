// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"
UE_DEPRECATED_HEADER(5.5, "Use InsightsCore/Common/TimeUtils.h from the TraceInsightsCore module instead.")

// TraceInsightsCore
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/Config.h"

#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54

namespace TimeUtils
{

static constexpr double UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace") Picosecond = 0.000000000001;
static constexpr double UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace") Nanosecond = 0.000000001;
static constexpr double UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace") Microsecond = 0.000001;
static constexpr double UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace") Milisecond = 0.001; // !!!
static constexpr double UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace") Second = 1.0;
static constexpr double UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace") Minute = 60.0;
static constexpr double UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace") Hour = 3600.0;
static constexpr double UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace") Day = 86400.0;
static constexpr double UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace") Week = 604800.0;

struct UE_DEPRECATED(5.5, "FTimeSplit was moved into UE::Insights namespace") FTimeSplit;
struct FTimeSplit : public UE::Insights::FTimeSplit {};

UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace")
inline FString FormatTimeValue(const double Duration, const int32 NumDigits = 1) { return UE::Insights::FormatTimeValue(Duration, NumDigits); }

UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace")
inline FString FormatTimeAuto(const double Duration, const int32 NumDigits = 1) { return UE::Insights::FormatTimeAuto(Duration, NumDigits); }

UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace")
inline FString FormatTimeMs(const double Duration, const int32 NumDigits = 2, bool bAddTimeUnit = false) { return UE::Insights::FormatTimeMs(Duration, NumDigits, bAddTimeUnit); }

UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace")
inline FString FormatTime(const double Time, const double Precision = 0.0) { return UE::Insights::FormatTime(Time, Precision); }

UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace")
inline FString FormatTimeHMS(const double Time, const double Precision = 0.0) { return UE::Insights::FormatTimeHMS(Time, Precision); }

UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace")
inline void SplitTime(const double Time, UE::Insights::FTimeSplit& OutTimeSplit) { UE::Insights::SplitTime(Time, OutTimeSplit); }

UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace")
inline FString FormatTimeSplit(const UE::Insights::FTimeSplit& TimeSplit, const double Precision = 0.0) { return UE::Insights::FormatTimeSplit(TimeSplit, Precision); }

UE_DEPRECATED(5.5, "TimeUtils namespace was merged into UE::Insights namespace")
inline FString FormatTimeSplit(const double Time, const double Precision = 0.0) { return UE::Insights::FormatTimeSplit(Time, Precision); }

} // namespace TimeUtils

#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
