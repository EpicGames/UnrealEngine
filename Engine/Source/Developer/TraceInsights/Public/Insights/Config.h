// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#ifndef UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
#define UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54 1
#endif
