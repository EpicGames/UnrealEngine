// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"

namespace Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Interface for an Insights manager. */
class IInsightsManager : public IInsightsComponent
{
public:
	/** The event to execute when the session has changed. */
	DECLARE_MULTICAST_DELEGATE(FSessionChangedEvent);
	virtual FSessionChangedEvent& GetSessionChangedEvent() = 0;

	/** The event to execute when session analysis is complete. */
	DECLARE_MULTICAST_DELEGATE(FSessionAnalysisCompletedEvent);
	virtual FSessionAnalysisCompletedEvent& GetSessionAnalysisCompletedEvent() = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace Insights
