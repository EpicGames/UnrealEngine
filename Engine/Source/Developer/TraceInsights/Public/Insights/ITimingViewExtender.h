// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Features/IModularFeature.h"
#include "Templates/SharedPointer.h"

#include "Insights/Config.h"

#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
#include "Insights/ITimingViewSession.h"
#endif

class FMenuBuilder;
class FToolBarBuilder;

namespace TraceServices { class IAnalysisSession; }
namespace UE::Insights { class FFilterConfigurator; }

namespace UE::Insights::Timing
{

class ITimingViewSession;

extern TRACEINSIGHTS_API const FName TimingViewExtenderFeatureName;

struct FTimingViewExtenderTickParams
{
	FTimingViewExtenderTickParams(ITimingViewSession& InSession, const TraceServices::IAnalysisSession* InAnalysisSession)
		: Session(InSession)
		, AnalysisSession(InAnalysisSession)
	{}

	ITimingViewSession& Session;
	const TraceServices::IAnalysisSession* AnalysisSession;
	double CurrentTime;
	float DeltaTime;
};

class ITimingViewExtender : public IModularFeature
{
public:
	virtual ~ITimingViewExtender() = default;

	/** Called to set up any data at the end of the timing view session */
	virtual void OnBeginSession(ITimingViewSession& InSession) = 0;

	/** Called to clear out any data at the end of the timing view session */
	virtual void OnEndSession(ITimingViewSession& InSession) = 0;

	/** Called each frame. If any new tracks are created they can be added via ITimingViewSession::Add*Track() */
	UE_DEPRECATED(5.7, "Use newer Tick overload taking TimingViewExtenderTickParams. Be advised that one can be called with a null AnalysisSession.")
	virtual void Tick(ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) {};

	/** Called each frame. If any new tracks are created they can be added via ITimingViewSession::Add*Track() */
	virtual void Tick(const FTimingViewExtenderTickParams& Params) {};

	/** Extension hook for the 'CPU Tracks Filter' menu */
	virtual void ExtendCpuTracksFilterMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) {}
	/** Extension hook for the 'GPU Tracks Filter' menu */
	virtual void ExtendGpuTracksFilterMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) {}
	/** Extension hook for the 'Other Tracks Filter' menu */
	virtual void ExtendOtherTracksFilterMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) {}
	/** Extension hook for the 'Plugins' menu */
	virtual void ExtendFilterMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) {}

	/** Extension hook for the context menu for all tracks
	@return True if any menu option was added and False if no option was added */
	virtual bool ExtendGlobalContextMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) { return false; }

	/** Allows extender to add filters to the Quick Find widget. */
	virtual void AddQuickFindFilters(TSharedPtr<FFilterConfigurator> FilterConfigurator) {}

	/** Allows extender to add functionality to the menu toolbars. */
	virtual void ExtendMenuToolbars(ITimingViewSession& InSession, FToolBarBuilder& LeftToolbar, FToolBarBuilder& RightToolbar) {}
};

} // namespace UE::Insights::Timing

#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54

namespace Insights
{

UE_DEPRECATED(5.5, "TimingViewExtenderFeatureName was moved inside UE::Insights::Timing namespace")
extern TRACEINSIGHTS_API const FName TimingViewExtenderFeatureName;

class UE_DEPRECATED(5.5, "ITimingViewExtender class was moved inside UE::Insights::Timing namespace") ITimingViewExtender;
class ITimingViewExtender : public IModularFeature
{
public:
	virtual ~ITimingViewExtender() = default;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual void OnBeginSession(ITimingViewSession& InSession) = 0;
	virtual void OnEndSession(ITimingViewSession& InSession) = 0;
	virtual void Tick(ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) = 0;
	virtual void ExtendCpuTracksFilterMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) {}
	virtual void ExtendGpuTracksFilterMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) {}
	virtual void ExtendOtherTracksFilterMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) {}
	virtual void ExtendFilterMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) {}
	virtual bool ExtendGlobalContextMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) { return false; }
	virtual void AddQuickFindFilters(TSharedPtr<UE::Insights::FFilterConfigurator> FilterConfigurator) {}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

} // namespace Insights

namespace UE::Insights
{

UE_DEPRECATED(5.5, "TimingViewExtenderFeatureName was moved inside UE::Insights::Timing namespace")
extern TRACEINSIGHTS_API const FName TimingViewExtenderFeatureName;

class UE_DEPRECATED(5.5, "ITimingViewExtender class was moved inside UE::Insights::Timing namespace") ITimingViewExtender;
class ITimingViewExtender : public IModularFeature
{
public:
	virtual ~ITimingViewExtender() = default;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual void OnBeginSession(ITimingViewSession& InSession) = 0;
	virtual void OnEndSession(ITimingViewSession& InSession) = 0;
	virtual void Tick(ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) = 0;
	virtual void ExtendCpuTracksFilterMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) {}
	virtual void ExtendGpuTracksFilterMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) {}
	virtual void ExtendOtherTracksFilterMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) {}
	virtual void ExtendFilterMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) {}
	virtual bool ExtendGlobalContextMenu(ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) { return false; }
	virtual void AddQuickFindFilters(TSharedPtr<UE::Insights::FFilterConfigurator> FilterConfigurator) {}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

} // namespace UE::Insights

#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
