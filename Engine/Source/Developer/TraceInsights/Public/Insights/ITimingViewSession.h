// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/SharedPointer.h"

// TraceInsights
#include "Insights/Config.h"

class SWidget;

class FBaseTimingTrack;
class FTimingEventsTrack;
class ITimingEvent;
class ITimingEventRelation;
enum class ETimingTrackLocation : uint32; // Insights/ViewModels/BaseTimingTrack.h

#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
namespace Insights { enum class ETimeChangedFlags : int32; }
namespace Insights { class ITimeMarker; }
namespace Insights { class ITimingViewSession; }
namespace UE::Insights { using ETimeChangedFlags = ::Insights::ETimeChangedFlags; }
namespace UE::Insights { using ITimeMarker = ::Insights::ITimeMarker; }
namespace UE::Insights { using ITimingViewSession = ::Insights::ITimingViewSession; }
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54

namespace UE::Insights::Timing
{

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class ETimeChangedFlags : int32
{
	None,

	// The event fired in response to an interactive change from the user. Will be followed by a non-interactive change finished.
	Interactive = (1 << 0)
};
ENUM_CLASS_FLAGS(ETimeChangedFlags);

////////////////////////////////////////////////////////////////////////////////////////////////////

class ITimeMarker
{
public:
	virtual ~ITimeMarker() = default;

	virtual double GetTime() const = 0;
	virtual void SetTime(const double InTime) = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** The delegate to be invoked when the selection have been changed */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FSelectionChangedDelegate, ETimeChangedFlags /*InFlags*/, double /*StartTime*/, double /*EndTime*/);

/** The delegate to be invoked when the time marker has changed */
DECLARE_MULTICAST_DELEGATE_TwoParams(FTimeMarkerChangedDelegate, ETimeChangedFlags /*InFlags*/, double /*TimeMarker*/);

/** The delegate to be invoked when a custom time marker has changed */
DECLARE_MULTICAST_DELEGATE_TwoParams(FCustomTimeMarkerChangedDelegate, ETimeChangedFlags /*InFlags*/, TSharedRef<ITimeMarker> /*TimeMarker*/);

/** The delegate to be invoked when the timing track being hovered by the mouse has changed */
DECLARE_MULTICAST_DELEGATE_OneParam(FHoveredTrackChangedDelegate, const TSharedPtr<FBaseTimingTrack> /*InTrack*/);

/** The delegate to be invoked when the timing event being hovered by the mouse has changed */
DECLARE_MULTICAST_DELEGATE_OneParam(FHoveredEventChangedDelegate, const TSharedPtr<const ITimingEvent> /*InEvent*/);

/** The delegate to be invoked when the selected timing track has changed */
DECLARE_MULTICAST_DELEGATE_OneParam(FSelectedTrackChangedDelegate, const TSharedPtr<FBaseTimingTrack> /*InTrack*/);

/** The delegate to be invoked when the selected timing event has changed */
DECLARE_MULTICAST_DELEGATE_OneParam(FSelectedEventChangedDelegate, const TSharedPtr<const ITimingEvent> /*InEvent*/);

/** The delegate to be invoked when a track visibility has changed */
DECLARE_MULTICAST_DELEGATE(FTrackVisibilityChangedDelegate);

/** The delegate to be invoked when a track is added. */
DECLARE_MULTICAST_DELEGATE_OneParam(FTrackAddedDelegate, const TSharedPtr<const FBaseTimingTrack> /*Track*/);

/** The delegate to be invoked when a track is removed. */
DECLARE_MULTICAST_DELEGATE_OneParam(FTrackRemovedDelegate, const TSharedPtr<const FBaseTimingTrack> /*Track*/);

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Hosts a number of timing view visualizers, represents a session of the timing view. */
class ITimingViewSession
{
public:
	virtual ~ITimingViewSession() = default;

	/** Gets the name of the view. */
	virtual const FName& GetName() const = 0;

	/** Adds a new top docked track. */
	virtual void AddTopDockedTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;
	/** Removes a top docked track. Returns whether the track was removed or not. */
	virtual bool RemoveTopDockedTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;

	/** Adds a new bottom docked track. */
	virtual void AddBottomDockedTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;
	/** Removes a bottom docked track. Returns whether the track was removed or not. */
	virtual bool RemoveBottomDockedTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;

	/** Adds a new scrollable track. */
	virtual void AddScrollableTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;
	/** Removes a scrollable track. Returns whether the track was removed or not. */
	virtual bool RemoveScrollableTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;
	/** Marks the scrollable tracks as not being in the correct order, so they will be re-sorted. */
	virtual void InvalidateScrollableTracksOrder() = 0;

	/** Adds a new foreground track. */
	virtual void AddForegroundTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;
	/** Removes a foreground track. Returns whether the track was removed or not. */
	virtual bool RemoveForegroundTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;

	/** Adds a new track, specifying the location. */
	virtual void AddTrack(TSharedPtr<FBaseTimingTrack> Track, ETimingTrackLocation Location) = 0;
	/** Removes a track. Returns whether the track was removed or not. */
	virtual bool RemoveTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;

	/** Finds a track has been added via Add*Track(). */
	virtual TSharedPtr<FBaseTimingTrack> FindTrack(uint64 InTrackId) = 0;

	/** Enumerate all available tracks (for all locations). */
	virtual void EnumerateTracks(TFunctionRef<void(TSharedPtr<FBaseTimingTrack> Track)> Callback) = 0;

	//////////////////////////////////////////////////

	/** Gets the current marker time. */
	virtual double GetTimeMarker() const = 0;
	/** Sets the current marker time. */
	virtual void SetTimeMarker(double InTimeMarker) = 0;
	/** Sets the current marker time and center the view on it. */
	virtual void SetAndCenterOnTimeMarker(double InTimeMarker) = 0;

	//////////////////////////////////////////////////

	/** Gets the delegate to be invoked when the selection have been changed. */
	virtual FSelectionChangedDelegate& OnSelectionChanged() = 0;
	/** Gets the delegate to be invoked when the time marker has changed. */
	virtual FTimeMarkerChangedDelegate& OnTimeMarkerChanged() = 0;
	/** Gets the delegate to be invoked when a custom time marker has changed. */
	virtual FCustomTimeMarkerChangedDelegate& OnCustomTimeMarkerChanged() = 0;

	/** Gets the delegate to be invoked when the timing track being hovered by the mouse has changed. */
	virtual FHoveredTrackChangedDelegate& OnHoveredTrackChanged() = 0;
	/** Gets the delegate to be invoked when the timing event being hovered by the mouse has changed. */
	virtual FHoveredEventChangedDelegate& OnHoveredEventChanged() = 0;

	/** Gets the delegate to be invoked when the selected timing track has changed. */
	virtual FSelectedTrackChangedDelegate& OnSelectedTrackChanged() = 0;
	/** Gets the delegate to be invoked when the selected timing event has changed. */
	virtual FSelectedEventChangedDelegate& OnSelectedEventChanged() = 0;

	/** Gets the delegate to be invoked when the track visibility has changed. */
	virtual FTrackVisibilityChangedDelegate& OnTrackVisibilityChanged() = 0;

	/** Gets the delegate to be invoked when a new track is added. */
	virtual FTrackAddedDelegate& OnTrackAdded() = 0;

	/** Gets the delegate to be invoked when a track is removed. */
	virtual FTrackRemovedDelegate& OnTrackRemoved() = 0;

	//////////////////////////////////////////////////

	/** Resets the selected event back to empty. */
	virtual void ResetSelectedEvent() = 0;

	/** Resets the event filter back to empty. */
	virtual void ResetEventFilter() = 0;

	//////////////////////////////////////////////////

	/** Prevents mouse movements from throttling application updates. */
	virtual void PreventThrottling() = 0;

	/** Adds a slot to the overlay. */
	virtual void AddOverlayWidget(const TSharedRef<SWidget>& InWidget) = 0;

	//////////////////////////////////////////////////

	/** Get the active relations for this view. */
	virtual const TArray<TUniquePtr<ITimingEventRelation>>& GetCurrentRelations() const = 0;

	/** Get an editable array of relations for this view. */
	virtual TArray<TUniquePtr<ITimingEventRelation>>& EditCurrentRelations() = 0;

	/** Add a new relation to this view. */
	virtual void AddRelation(TUniquePtr<ITimingEventRelation>& Relation) = 0;

	/** Remove all relations from this view. */
	virtual void ClearRelations() = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights::Timing

////////////////////////////////////////////////////////////////////////////////////////////////////

#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
namespace Insights
{

enum class UE_DEPRECATED(5.5, "ITimeMarker class was moved inside UE::Insights::Timing namespace") ETimeChangedFlags : int32
{
	None,
	Interactive = (1 << 0)
};
PRAGMA_DISABLE_DEPRECATION_WARNINGS
ENUM_CLASS_FLAGS(ETimeChangedFlags);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

class UE_DEPRECATED(5.5, "ITimeMarker class was moved inside UE::Insights::Timing namespace") ITimeMarker
{
public:
	virtual ~ITimeMarker() = default;

	virtual double GetTime() const = 0;
	virtual void SetTime(const double  InTime) = 0;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
DECLARE_MULTICAST_DELEGATE_ThreeParams(FSelectionChangedDelegate, ETimeChangedFlags /*InFlags*/, double /*StartTime*/, double /*EndTime*/);
DECLARE_MULTICAST_DELEGATE_TwoParams(FTimeMarkerChangedDelegate, ETimeChangedFlags /*InFlags*/, double /*TimeMarker*/);
DECLARE_MULTICAST_DELEGATE_TwoParams(FCustomTimeMarkerChangedDelegate, ETimeChangedFlags /*InFlags*/, TSharedRef<ITimeMarker> /*TimeMarker*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FHoveredTrackChangedDelegate, const TSharedPtr<FBaseTimingTrack> /*InTrack*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FHoveredEventChangedDelegate, const TSharedPtr<const ITimingEvent> /*InEvent*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FSelectedTrackChangedDelegate, const TSharedPtr<FBaseTimingTrack> /*InTrack*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FSelectedEventChangedDelegate, const TSharedPtr<const ITimingEvent> /*InEvent*/);
DECLARE_MULTICAST_DELEGATE(FTrackVisibilityChangedDelegate);
DECLARE_MULTICAST_DELEGATE_OneParam(FTrackAddedDelegate, const TSharedPtr<const FBaseTimingTrack> /*Track*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FTrackRemovedDelegate, const TSharedPtr<const FBaseTimingTrack> /*Track*/);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

class UE_DEPRECATED(5.5, "ITimingViewSession class was moved inside UE::Insights::Timing namespace") ITimingViewSession
{
public:
	virtual ~ITimingViewSession() = default;
	virtual const FName& GetName() const = 0;
	virtual void AddTopDockedTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;
	virtual bool RemoveTopDockedTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;
	virtual void AddBottomDockedTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;
	virtual bool RemoveBottomDockedTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;
	virtual void AddScrollableTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;
	virtual bool RemoveScrollableTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;
	virtual void InvalidateScrollableTracksOrder() = 0;
	virtual void AddForegroundTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;
	virtual bool RemoveForegroundTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;
	virtual void AddTrack(TSharedPtr<FBaseTimingTrack> Track, ETimingTrackLocation Location) = 0;
	virtual bool RemoveTrack(TSharedPtr<FBaseTimingTrack> Track) = 0;
	virtual TSharedPtr<FBaseTimingTrack> FindTrack(uint64 InTrackId) = 0;
	virtual double GetTimeMarker() const = 0;
	virtual void SetTimeMarker(double InTimeMarker) = 0;
	virtual void SetAndCenterOnTimeMarker(double InTimeMarker) = 0;
	virtual FSelectionChangedDelegate& OnSelectionChanged() = 0;
	virtual FTimeMarkerChangedDelegate& OnTimeMarkerChanged() = 0;
	virtual FCustomTimeMarkerChangedDelegate& OnCustomTimeMarkerChanged() = 0;
	virtual FHoveredTrackChangedDelegate& OnHoveredTrackChanged() = 0;
	virtual FHoveredEventChangedDelegate& OnHoveredEventChanged() = 0;
	virtual FSelectedTrackChangedDelegate& OnSelectedTrackChanged() = 0;
	virtual FSelectedEventChangedDelegate& OnSelectedEventChanged() = 0;
	virtual FTrackVisibilityChangedDelegate& OnTrackVisibilityChanged() = 0;
	virtual FTrackAddedDelegate& OnTrackAdded() = 0;
	virtual FTrackRemovedDelegate& OnTrackRemoved() = 0;
	virtual void ResetSelectedEvent() = 0;
	virtual void ResetEventFilter() = 0;
	virtual void PreventThrottling() = 0;
	virtual void AddOverlayWidget(const TSharedRef<SWidget>& InWidget) = 0;
};

} // namespace Insights
#endif // UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
