// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Internationalization/Text.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

// TraceAnalysis
#include "Trace/DataStream.h"

// TraceInsights
#include "Insights/Widgets/IObjectProfilerMemoryView.h"

#define UE_API TRACEINSIGHTS_API

class FExtender;
class SWidget;
class FWorkspaceItem;

namespace UE::Trace { class FStoreClient; }
namespace TraceServices { class IAnalysisSession; }
namespace Insights { class IInsightsManager; }

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Major tab IDs for Insights tools */
struct FInsightsManagerTabs
{
	static UE_API const FName SessionInfoTabId;
	static UE_API const FName TimingProfilerTabId;
	static UE_API const FName LoadingProfilerTabId;
	static UE_API const FName MemoryProfilerTabId;
	static UE_API const FName NetworkingProfilerTabId;

	static UE_API const FName AutomationWindowTabId;
	static UE_API const FName MessageLogTabId;
	static UE_API const FName TraceControlTabId;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Tab IDs for the timing profiler */
struct FTimingProfilerTabs
{
	// Tab identifiers
	static UE_API const FName ToolbarID; // DEPRECATED
	static UE_API const FName FramesTrackID;
	static UE_API const FName TimingViewID;
	static UE_API const FName TimersID;
	static UE_API const FName CallersID;
	static UE_API const FName CalleesID;
	static UE_API const FName StatsCountersID;
	static UE_API const FName LogViewID;
	static UE_API const FName ModulesViewID;
	static UE_API const FName UserAnnotationsID;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FMemoryProfilerTabs
{
	// Tab identifiers
	static UE_API const FName TimingViewID;
	static UE_API const FName MemInvestigationViewID;
	static UE_API const FName MemTagTreeViewID;
	static UE_API const FName MemAllocTableTreeViewID; // base id
	static UE_API const FName ModulesViewID;
	static UE_API const FName UserAnnotationsID;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FLoadingProfilerTabs
{
	// Tab identifiers
	static UE_API const FName TimingViewID;
	static UE_API const FName EventAggregationTreeViewID;
	static UE_API const FName ObjectTypeAggregationTreeViewID;
	static UE_API const FName PackageDetailsTreeViewID;
	static UE_API const FName ExportDetailsTreeViewID;
	static UE_API const FName RequestsTreeViewID;
	static UE_API const FName UserAnnotationsID;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FNetworkingProfilerTabs
{
	// Tab identifiers
	static UE_API const FName PacketViewID;
	static UE_API const FName PacketContentViewID;
	static UE_API const FName NetStatsViewID;
	static UE_API const FName NetStatsCountersViewID;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Configuration for an Insights major tab */
struct FInsightsMajorTabConfig
{
	FInsightsMajorTabConfig()
		: Layout(nullptr)
		, WorkspaceGroup(nullptr)
		, bIsAvailable(true)
	{}

	/** Helper function for creating unavailable tab configs */
	static FInsightsMajorTabConfig Unavailable()
	{
		FInsightsMajorTabConfig Config;
		Config.bIsAvailable = false;
		return Config;
	}

	bool ShouldRegisterMinorTab(FName MinorTabId) const
	{
		return !DisabledMinorTabIds.Contains(MinorTabId);
	}

	/** Identifier for this config */
	FName ConfigId;

	/** Display name for this config */
	FText ConfigDisplayName;

	/** Label for the tab. If this is not set the default will be used */
	TOptional<FText> TabLabel;

	/** Tooltip for the tab. If this is not set the default will be used */
	TOptional<FText> TabTooltip;

	/** Icon for the tab. If this is not set the default will be used */
	TOptional<FSlateIcon> TabIcon;

	/** The tab layout to use. If not specified, the default will be used. */
	TSharedPtr<FTabManager::FLayout> Layout;

	/** The menu workspace group to use. If not specified, the default will be used. */
	TSharedPtr<FWorkspaceItem> WorkspaceGroup;

	/** A denylist that can be used to suppress default minor tabs. */
	TSet<FName> DisabledMinorTabIds;

	/** If true, show the tab spawner entries in a submenu. Otherwise show them as individual buttons in the toolbar. */
	bool bUseCompactTabSpawners = false;

	/** Whether the tab is available for selection (i.e. registered with the tab manager) */
	bool bIsAvailable;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Combination of extenders applied to the individual major tabs within Insights */
struct FInsightsMajorTabExtender
{
	FInsightsMajorTabExtender(TSharedPtr<FTabManager>& InTabManager, TSharedRef<FWorkspaceItem> InWorkspaceGroup)
		: MenuExtender(MakeShared<FExtender>())
		, TabManager(InTabManager)
		, WorkspaceGroup(InWorkspaceGroup)
	{}

	TSharedPtr<FExtender>& GetMenuExtender() { return MenuExtender; }
	FLayoutExtender& GetLayoutExtender() { return LayoutExtender; }
	FMinorTabConfig& AddMinorTabConfig() { return MinorTabs.AddDefaulted_GetRef(); }
	TSharedPtr<FTabManager> GetTabManager() const { return TabManager; }
	TSharedRef<FWorkspaceItem> GetWorkspaceGroup() { return WorkspaceGroup; }
	const TArray<FMinorTabConfig>& GetMinorTabs() const { return MinorTabs; }

protected:
	/** Extender used to add to the menu for this tab */
	TSharedPtr<FExtender> MenuExtender;

	/** Any additional minor tabs to add */
	TArray<FMinorTabConfig> MinorTabs;

	/** Extender used when creating the layout for this tab */
	FLayoutExtender LayoutExtender;

	/** Tab manager for this major tab*/
	TSharedPtr<FTabManager> TabManager;

	TSharedRef<FWorkspaceItem> WorkspaceGroup;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Called back to register common layout extensions */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnRegisterMajorTabExtensions, FInsightsMajorTabExtender& /*MajorTabExtender*/);

/** Delegate invoked when a major tab is created */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnInsightsMajorTabCreated, FName /*MajorTabId*/, TSharedRef<FTabManager> /*TabManager*/)

class IInsightsComponent;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Interface for an Unreal Insights module. */
class IUnrealInsightsModule : public IModuleInterface
{
public:
	virtual ~IUnrealInsightsModule() = default;

	/**
	 * Registers an IInsightsComponent. The component will Initialize().
	 */
	virtual void RegisterComponent(TSharedPtr<IInsightsComponent> Component) = 0;

	/**
	 * Unregisters an IInsightsComponent. The component will Shutdown().
	 */
	virtual void UnregisterComponent(TSharedPtr<IInsightsComponent> Component) = 0;

	//////////////////////////////////////////////////

	/**
	 * Creates the default trace store (for "Browser" mode).
	 */
	virtual void CreateDefaultStore() = 0;

	/**
	 * Gets the default trace store (for "Browser" mode).
	*/
	virtual FString GetDefaultStoreDir() = 0;

	/**
	 * Gets the store client.
	 */
	virtual UE::Trace::FStoreClient* GetStoreClient() = 0;

	/**
	 * Connects to a specified store.
	 *
	 * @param InStoreHost The host of the store to connect to.
	 * @param InStorePort The port of the store to connect to.
	 * @return If connected successfully or not.
	 */
	virtual bool ConnectToStore(const TCHAR* InStoreHost, uint32 InStorePort=0) = 0;

	//////////////////////////////////////////////////

	/**
	 * Gets the Insights manager.
	*/
	virtual TSharedPtr<::Insights::IInsightsManager> GetInsightsManager() = 0;

	//////////////////////////////////////////////////

	/**
	 * Gets the current analysis session.
	 */
	virtual TSharedPtr<const TraceServices::IAnalysisSession> GetAnalysisSession() const = 0;

	/**
	 * Sets the AutoQuit behavior when analysis is completed.
	 *
	 * @param bInAutoQuit - The Application will close when session analysis is complete or fails to start
	 * @param bInWaitForSymbolResolver - The Application will automatically close when both session analysis is complete and symbols have been resolved.Must be set together with the InAutoQuit parameter.
	 */
	virtual void SetAutoQuit(bool bInAutoQuit, bool bInWaitForSymbolResolver = false) = 0;

	/**
	 * Starts analysis of the specified trace. Called when the application starts in "Viewer" mode.
	 *
	 * @param InTraceId The id of the trace to analyze.
	 */
	virtual void StartAnalysisForTrace(uint32 InTraceId) = 0;

	UE_DEPRECATED(5.7, "Use explicit SetAutoQuit()")
	virtual void StartAnalysisForTrace(uint32 InTraceId, bool bInAutoQuit, bool bInWaitForSymbolResolver = false)
	{
		SetAutoQuit(bInAutoQuit, bInWaitForSymbolResolver);
		StartAnalysisForTrace(InTraceId);
	}

	/**
	 * Starts analysis of the last live session. Called when the application starts in "Viewer" mode.
	 * On failure, if InRetryTime is > 0, retry connecting every frame for RetryTime seconds.
	 *
	 * @param InRetryTime How many seconds to retry connecting asynchronously
	 */
	virtual void StartAnalysisForLastLiveSession(float InRetryTime = 1.0f) = 0;

	/**
	 * Starts analysis of the specified *.utrace file. Called when the application starts in "Viewer" mode.
	 *
	 * @param InTraceFile The filename (*.utrace) of the trace to analyze.
	 */
	virtual void StartAnalysisForTraceFile(const TCHAR* InTraceFile) = 0;

	UE_DEPRECATED(5.7, "Use explicit SetAutoQuit()")
	virtual void StartAnalysisForTraceFile(const TCHAR* InTraceFile, bool bInAutoQuit, bool bInWaitForSymbolResolver = false)
	{
		SetAutoQuit(bInAutoQuit, bInWaitForSymbolResolver);
		StartAnalysisForTraceFile(InTraceFile);
	}

	/**
	 * Starts analysis for a direct connection from the tracing application (might be self). Notice that
	 * analysis doesn't actually start until a connection has been made.
	 *
	 * @param InStreamName User facing display name for this stream
	 * @param InPort The specific port number to listen to or 0 to use any available port
	 * @return The actual port used for listening to incoming connections. 0 if failed to open port.
	 */
	virtual uint16 StartAnalysisWithDirectTrace(const TCHAR* InStreamName = nullptr, uint16 InPort = 0) = 0;

	/**
	 * Starts analysis for a secure trace trace connection.
	 * @param Host Host to connect to.
	 * @param Port Port to connect to.
	 * @param AuthorizationToken Optional authorization token to use when connecting.
	 * @param CertificateFile Optional certificate to use when connecting. Excludes usage of CertificateAuthority.
	 * @param CertificateAuthority Optional certificate to use as trusted. CertificateFile should be unset to use.
	 * @param Identity Identity string used to connect when using pre-shared keys.
	 * @note All view-type parameters are copied and does not have to outlive this function call.
	 */
	virtual void StartAnalysisWithSecureTrace(FStringView Host, uint16 Port, TConstArrayView<uint8> AuthorizationToken, FStringView CertificateFile = FStringView(), TConstArrayView<uint8> CertificateAuthority = TConstArrayView<uint8>(), FAnsiStringView Identity = FAnsiStringView()) = 0;

	/**
	 * Start analysis given a data stream.
	 *
	 * @param InStream Stream to read from
	 * @param InStreamName User facing display name for this stream
	 */
	virtual void StartAnalysisWithStream(TUniquePtr<UE::Trace::IInDataStream>&& InStream, const TCHAR* InStreamName = nullptr) = 0;

	//////////////////////////////////////////////////

	/**
	 * Registers a major tab layout. This defines how the major tab will appear when spawned.
	 * If this is not called prior to tabs being spawned then the built-in default layout will be used.
	 * @param InMajorTabId The major tab ID we are supplying a layout to
	 * @param InConfig The config to use when spawning the major tab
	 */
	virtual void RegisterMajorTabConfig(const FName& InMajorTabId, const FInsightsMajorTabConfig& InConfig) = 0;

	/**
	 * Unregisters a major tab layout. This will revert the major tab to spawning with its default layout
	 * @param InMajorTabId The major tab ID we are supplying a layout to
	 */
	virtual void UnregisterMajorTabConfig(const FName& InMajorTabId) = 0;

	/**
	 * Allows for registering a delegate callback for populating a FInsightsMajorTabExtender structure.
	 * @param InMajorTabId The major tab ID to register the delegate for
	 */
	virtual FOnRegisterMajorTabExtensions& OnRegisterMajorTabExtension(const FName& InMajorTabId) = 0;

	/** Callback invoked when a major tab is created */
	virtual FOnInsightsMajorTabCreated& OnMajorTabCreated() = 0;

	/** Finds a major tab config for the specified id. */
	virtual const FInsightsMajorTabConfig& FindMajorTabConfig(const FName& InMajorTabId) const = 0;

	virtual FOnRegisterMajorTabExtensions* FindMajorTabLayoutExtension(const FName& InMajorTabId) = 0;

	/** Sets the ini path for saving persistent layout data. */
	virtual void SetUnrealInsightsLayoutIni(const FString& InIniPath) = 0;

	/**
	 * Called when the application starts in "Viewer" mode.
	 */
	virtual void CreateSessionViewer(bool bAllowDebugTools = false) = 0;

	/**
	 * Called when the application shuts-down.
	 */
	virtual void ShutdownUserInterface() = 0;

	/**
	* Called to schedule a command to run after session analysis is complete. Intended for running Automation RunTests commands.
	*/
	virtual void ScheduleCommand(const FString& InCmd) = 0;

	/**
	* Called to initialize testing in stand alone Insights.
	 * @param InInitAutomationModules If true Insights will initialize the modules required for running automation tests.
	 * @param InAutoQuit If true Insights will close after completing session analysis and running any tests started using the ScheduleCommand function.
	*/
	virtual void InitializeTesting(bool InInitAutomationModules, bool InAutoQuit) = 0;

	/** Execute command. */
	virtual bool Exec(const TCHAR* Cmd, FOutputDevice& Ar) = 0;

	/**
	 * Sets an IAssetInfoProvider to be applied to the ObjectProfiler window.
	 * If the window is already open, the provider is applied immediately.
	 * Otherwise it is stored and applied when the window next opens.
	 */
	virtual void SetAssetInfoProvider(TSharedPtr<UE::Insights::ObjectProfiler::IAssetInfoProvider> InProvider) = 0;

	/** Returns the currently registered ObjectProfiler IAssetInfoProvider, or nullptr. */
	virtual TSharedPtr<UE::Insights::ObjectProfiler::IAssetInfoProvider> GetAssetInfoProvider() const = 0;

	/**
	 * Opens the Memory Profiler's allocations view at the specified time with optional package filtering.
	 * @param InTimeA Query time for live allocations (aAf rule).
	 * @param InTabDisplayName The new tab's title.
	 * @param InPackageFilter If non-empty, restricts displayed allocations to these package names.
	 */
	UE_EXPERIMENTAL(5.8, "OpenMemoryAllocationsView may change or be removed in future releases.")
	virtual void OpenMemoryAllocationsView(double InTimeA, const FText& InTabDisplayName, const TSet<FString>& InPackageFilter = {}) { }
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class IInsightsComponent
{
public:
	/** Initializes this component. Called by TraceInsights module when this component is registered. */
	virtual void Initialize(IUnrealInsightsModule& Module) = 0;

	/** Shuts-down this component. Called by TraceInsights module when this component is unregistered. */
	virtual void Shutdown() = 0;

	/* Allows this component to register major tabs. */
	virtual void RegisterMajorTabs(IUnrealInsightsModule& InsightsModule) = 0;

	/* Requests this component to unregister its major tabs. */
	virtual void UnregisterMajorTabs() = 0;

	/* Called by the TraceInsights module when it receives the OnWindowClosedEvent. Can be used to close any panels that should not persist in the layout. */
	virtual void OnWindowClosedEvent() {}

	/** Execute command. */
	virtual bool Exec(const TCHAR* Cmd, FOutputDevice& Ar) { return false; }
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
