// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseSet.h"
#include "Containers/UnrealString.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPath.h"

struct FAssetData;
struct FSlateBrush;

namespace UE::Insights
{
	class FTableTreeNode;
}

namespace UE::Insights::ObjectProfiler
{

class FObjectNode;
struct FAssetInfoNode;

class FActorInfo
{
public:
	FActorInfo() = default;
	FActorInfo(FSoftObjectPath&& InPath, FGuid&& InGuid) : Path(InPath), Guid(InGuid) { UpdateHashInternal(); }

	const FSoftObjectPath& GetPath() const { return Path; }
	const FGuid& GetGuid() const { return Guid; }

	friend uint32 GetTypeHash(const FActorInfo& Info)
	{
		return Info.Hash;
	}
	friend bool operator==(const FActorInfo& Left, const FActorInfo& Right)
	{
		return Left.Hash == Right.Hash
			&& Left.Path == Right.Path
			&& Left.Guid == Right.Guid;
	}
	friend bool operator!=(const FActorInfo& Left, const FActorInfo& Right)
	{
		return !(Left == Right);
	}

private:
	void UpdateHashInternal()
	{
		Hash = Guid.IsValid() ? GetTypeHash(Guid) : GetTypeHash(Path);
	}

private:
	FSoftObjectPath Path;
	FGuid Guid;
	uint32 Hash = 0;
};

struct FActorSet
{
public:
	FActorSet() = default;
	FActorSet(TSet<FActorInfo>&& InActors)
		: Actors(MoveTemp(InActors))
	{
	}

	const TSet<FActorInfo>& GetActors() const { return Actors; }
	int32 Num() const { return Actors.Num(); }

private:
	TSet<FActorInfo> Actors;
};

class IAssetInfoCategory
{
public:
	IAssetInfoCategory() = default;
	virtual ~IAssetInfoCategory() = default;

	virtual uint32 GetId() const = 0;

	virtual FText GetDisplayName() const = 0;
	virtual FLinearColor GetColor() const = 0;
	virtual const FSlateBrush* GetIcon() const = 0;
};

class IAssetInfoProvider
{
public:
	IAssetInfoProvider() = default;
	virtual ~IAssetInfoProvider() = default;

	virtual const IAssetInfoCategory& GetClassCategory(const FName InClassName) const = 0;
	virtual const IAssetInfoCategory& GetObjectCategory(const FName InClassName, const TCHAR* InObjectName, const TCHAR* InObjectPath = nullptr) const = 0;

	virtual FText GetDisplayName(const FAssetData& InAssetData, const FName InClassName) const = 0;
	virtual FLinearColor GetColor(const FAssetData& InAssetData, const FName InClassName) const = 0;
	virtual const FSlateBrush* GetIcon(const FAssetData& InAssetData, const FName InClassName) const = 0;
	virtual const FSlateBrush* GetThumbnail(const FAssetData& InAssetData, const FName InClassName) const = 0;

	virtual bool GetAssetData(const FObjectNode& InObjectNode, FAssetData& OutAsset) const { return false; }
	virtual bool FindMatchedAsset(const TArray<TSharedPtr<FTableTreeNode>>& InSelectedNodes, FAssetData& OutAsset) const { return false; }

	virtual bool GetActors(const FAssetData& InAssetData, TArray<FSoftObjectPath>& OutActors) const { return false; }
	virtual TMap<FName, TSharedRef<FActorSet>> MatchNamesToActors(const TArray<FName>& PackageNames) const { return TMap<FName, TSharedRef<FActorSet>>(); }

	virtual void GetValidationIssues(const FAssetData& InAssetData, TArray<FText>& OutIssues) const {}

	virtual bool BrowseToAsset(const FAssetInfoNode& InAssetInfo) const { return false; }
	virtual bool BrowseToActor(const FAssetInfoNode& InAssetInfo) const { return false; }

	virtual void ConvertRuntimePathToEditorPath(FString& ObjectPath) const {}

	virtual bool ShouldMaskAssetIdentity(const FAssetData& MatchedAsset) const { return false; }

	virtual bool IsAssetOwnedByCurrentProject(const FAssetData& MatchedAsset) const { return true; }
};

} // namespace UE::Insights::ObjectProfiler
