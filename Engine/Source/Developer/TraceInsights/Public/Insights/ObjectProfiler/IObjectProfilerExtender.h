// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Features/IModularFeature.h"
#include "UObject/NameTypes.h"

#include "Insights/Config.h"
#include "Insights/ObjectProfiler/IObjectProfilerSession.h"

namespace TraceServices { class IAnalysisSession; }

namespace UE::Insights::ObjectProfiler
{

extern TRACEINSIGHTS_API const FName ObjectProfilerExtenderFeatureName;
extern TRACEINSIGHTS_API const FName ObjectProfilerTabId;

struct FObjectProfilerExtenderTickParams
{
	FObjectProfilerExtenderTickParams(const TraceServices::IAnalysisSession* InAnalysisSession)
		: AnalysisSession(InAnalysisSession)
	{}

	const TraceServices::IAnalysisSession* AnalysisSession = nullptr;
	double CurrentTime = 0.0;
	float DeltaTime = 0.0f;
};

struct FOnSnapshotTreeRebuiltParams
{
	uint32 SnapshotId = 0;
	double RebuildDurationSeconds = 0.0;
};

class IObjectProfilerExtender : public IModularFeature
{
public:
	virtual ~IObjectProfilerExtender() = default;

	/** Called once after SObjectProfilerWindow construction finishes. Capture InSession if you need access during the window's lifetime. */
	virtual void OnBeginSession(IObjectProfilerSession& InSession) {};

	/** Called once before SObjectProfilerWindow destruction. Drop any reference to InSession. */
	virtual void OnEndSession(IObjectProfilerSession& InSession) {};

	virtual void Tick(const FObjectProfilerExtenderTickParams& Params) {};

	/** Called after SObjectTableTreeView finishes a RebuildTree pass. */
	virtual void OnSnapshotTreeRebuilt(const FOnSnapshotTreeRebuiltParams& Params) {};
};

} // namespace UE::Insights::ObjectProfiler