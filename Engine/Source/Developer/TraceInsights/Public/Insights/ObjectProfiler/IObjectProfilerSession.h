// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::Insights::ObjectProfiler
{

/**
 * Session handle passed to IObjectProfilerExtender during OnBeginSession / OnEndSession.
 * Valid only between those two calls.
 */
class IObjectProfilerSession
{
public:
	virtual ~IObjectProfilerSession() = default;

	/** Toggle the "hide objects external to the current project" tree-view filter. */
	virtual void SetHideObjectsExternalToProject(bool bHide) = 0;
	virtual bool IsHideObjectsExternalToProject() const = 0;
};

} // namespace UE::Insights::ObjectProfiler
