// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Delegates/DelegateCombinations.h"
#include "Misc/Optional.h"
#include "Features/IModularFeature.h"
#include "Internationalization/Text.h"
#include "Math/Box2D.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Templates/FunctionFwd.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"

#define UE_API TRACEINSIGHTS_API

namespace TraceServices { class IAnalysisSession; }
class FTooltipDrawState;
class FMenuBuilder;
class SWidget;

namespace UE::Insights::SpatialProfiler
{

UE_EXPERIMENTAL(5.8, "ISpatialPlotViewExtender API is experimental and subject to change.")
extern UE_API const FName SpatialPlotViewExtenderFeatureName;
UE_EXPERIMENTAL(5.8, "ISpatialPlotViewExtender API is experimental and subject to change.")
extern UE_API const FName SpatialProfilerTabId;

struct UE_EXPERIMENTAL(5.8, "ISpatialPlotViewExtender API is experimental and subject to change.") FSpatialPlotRegion
{
	uint64 Id;
	FBox2D Bounds;
	FLinearColor FillColor;
	FLinearColor BorderColor;
	FText Label;
};

struct UE_EXPERIMENTAL(5.8, "ISpatialPlotViewExtender API is experimental and subject to change.") FSpatialPlotMarker
{
	uint64 Id;
	FVector2D Position;
	FLinearColor Color;
	FText Label;
};

struct UE_EXPERIMENTAL(5.8, "ISpatialPlotViewExtender API is experimental and subject to change.") FSpatialPlotHitTestResult
{
	TArray<FSpatialPlotRegion> Regions;
	TArray<FSpatialPlotMarker> Markers;
};

struct UE_EXPERIMENTAL(5.8, "ISpatialPlotViewExtender API is experimental and subject to change.") FSpatialPlotViewExtenderTickParams
{
	double CurrentTraceTime;
	float DeltaTime;
};

enum class UE_EXPERIMENTAL(5.8, "ISpatialPlotViewExtender API is experimental and subject to change.") ESpatialPlotLegendType : uint8
{
	Discrete,
	Gradient
};

struct UE_EXPERIMENTAL(5.8, "ISpatialPlotViewExtender API is experimental and subject to change.") FSpatialPlotLegendEntry
{
	FLinearColor Color;
	FText Label;
};

struct UE_EXPERIMENTAL(5.8, "ISpatialPlotViewExtender API is experimental and subject to change.") FSpatialPlotLegend
{
	FText Title;
	ESpatialPlotLegendType Type = ESpatialPlotLegendType::Discrete;

	/** Discrete: each entry is a color swatch + label, all rendered.
	 *  Gradient: entries are color stops with even spacing; labels shown only where non-empty.
	 *            Entries[0] renders at the top of the bar. */
	TArray<FSpatialPlotLegendEntry> Entries;
};

class UE_EXPERIMENTAL(5.8, "ISpatialPlotViewExtender API is experimental and subject to change.") ISpatialPlotViewExtender : public IModularFeature
{
public:
	virtual ~ISpatialPlotViewExtender() = default;

	virtual void OnBeginSession(const TraceServices::IAnalysisSession& InAnalysisSession) = 0;
	virtual void OnEndSession() = 0;

	/** Must be unique across all registered extenders. */
	virtual FName GetLayerName() const = 0;
	virtual FText GetLayerDisplayName() const = 0;
	virtual FSlateIcon GetLayerIcon() const { return FSlateIcon(); }

	virtual void EnumerateRegions(TFunctionRef<void(const FSpatialPlotRegion&)> InCallback) { }
	virtual void EnumerateMarkers(TFunctionRef<void(const FSpatialPlotMarker&)> InCallback) { }

	/** Returns a legend describing the color mapping for this extender's regions. */
	virtual TOptional<FSpatialPlotLegend> GetLegend() const { return {}; }

	/** Appends tooltip content for hit-tested regions/markers. Return true if content was added. */
	virtual bool AppendTooltip(FTooltipDrawState& InOutTooltip, const FSpatialPlotHitTestResult& InHitTestResult) const { return false; }
	/** Appends context-menu entries for hit-tested regions/markers. Return true if entries were added. */
	virtual bool ExtendContextMenu(FMenuBuilder& InOutMenuBuilder, const FSpatialPlotHitTestResult& InHitTestResult) const { return false; }

	virtual void Tick(const FSpatialPlotViewExtenderTickParams& InTickParams) { }

	/** Returns a serial that changes when extender data changes. 0 means caching is disabled (always rebuild). */
	virtual uint32 GetChangeSerial() const { return 0; }

	/** Whether this extender has data available for the given session. Used for tab availability detection. */
	virtual bool HasDataForSession(const TraceServices::IAnalysisSession& InAnalysisSession) const = 0;

	/** Whether this extender provides a control panel widget, auto-docked in the right panel. */
	virtual bool HasControlPanel() const { return false; }

	/** Creates the control panel widget. Called from FOnSpawnTab when the tab is opened. */
	virtual TSharedPtr<SWidget> CreateControlPanel() { return nullptr; }
};

} // namespace UE::Insights::SpatialProfiler

#undef UE_API