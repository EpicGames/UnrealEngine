// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Trace/StoreClient.h"
#include "Containers/UnrealString.h"
#include "ProfilingDebugging/TraceAuxiliary.h" // for FTraceAuxiliary::EConnectionType
#include "TraceServices/Model/AnalysisSession.h"

#define UE_API TRACEINSIGHTS_API

namespace UE::Insights::Automation::TestNames
{
	const FStringView EditorTests = TEXTVIEW("UnrealInsights.Editor.");
	const FStringView ClientTests = TEXTVIEW("UnrealInsights.Client.");
	const FStringView ViewerStaticTraceTests = TEXTVIEW("UnrealInsights.Trace.Analysis.");
	const FStringView ViewerLiveTraceTests = TEXTVIEW("UnrealInsights.Analysis.");
	const FStringView HubTests = TEXTVIEW("UnrealInsights.Hub.");
	const FStringView UnitTests = TEXTVIEW("UnrealInsights.UnitTest.");
}

class FAutomationTestBase;

class FInsightsTestUtils
{
public:
	UE_API FInsightsTestUtils(FAutomationTestBase* Test);

	UE_API bool AnalyzeTrace(const TCHAR* Path) const;
	UE_API bool AnalyzeTrace(TSharedPtr<const TraceServices::IAnalysisSession> Session) const;
	UE_API bool FileContainsString(const FString& PathToFile, const FString& ExpectedString, double Timeout) const;
	UE_API bool SetupUTS(double Timeout = 30.0, bool bUseFork = false) const;
	UE_API bool KillUTS(double Timeout = 30.0) const;
	UE_API bool StartTracing(FTraceAuxiliary::EConnectionType ConnectionType, double Timeout = 10.0) const;
	UE_API bool IsUnrealTraceServerReady(const TCHAR* Host = TEXT("localhost"), int32 Port = 0U) const;
	UE_API bool IsTraceHasLiveStatus(const FString& TraceName, const TCHAR* Host = TEXT("localhost"), int32 Port = 0U) const;
	UE_API bool GetMetadata(const FString& TraceFilePath, TMap<FString, FString>& OutMetadata) const;
	UE_API void ResetSession() const;
	UE_API FString GetLiveTrace(const TCHAR* Host = TEXT("localhost"), int32 Port = 0U) const;
	UE_API FString GetUTSPath() const;

private:
	TSharedPtr<const TraceServices::IAnalysisSession> GetSession(const TCHAR* Path) const;
	TSharedPtr<UE::Trace::FStoreClient> CreateStoreClient(const TCHAR* Host, int32 Port) const;
	uint32 GetValidSessionCount(TSharedPtr<UE::Trace::FStoreClient> StoreClient) const;
	const UE::Trace::FStoreClient::FTraceInfo* FindTraceInfoByName(TSharedPtr<UE::Trace::FStoreClient> StoreClient, const FString& TraceName) const;

	FAutomationTestBase* Test;
};

#undef UE_API
