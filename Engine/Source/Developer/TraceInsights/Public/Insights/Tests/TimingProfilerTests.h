// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Logging/LogMacros.h"

// TraceServices
#include "TraceServices/Model/TimingProfiler.h" // for TraceServices::EEventSortOrder

#define UE_API TRACEINSIGHTS_API

DECLARE_LOG_CATEGORY_EXTERN(TimingProfilerTests, Log, All);

class FAutomationTestBase;

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * A class containing code for parametric tests for Insight functionality
 * Intended to be called from automatic or user triggered tests
 */
class FTimingProfilerTests
{
public:
	struct FCheckValues
	{
		double TotalEventDuration = 0.0;
		uint64 EventCount = 0;
		uint32 SumDepth = 0;
		uint32 SumTimerIndex = 0;
		double SessionDuration = 0.0;
		double EnumerationDuration = 0.0;
	};

	struct FEnumerateTestParams
	{
		double Interval = 0.01;
		int32 NumEnumerations = 10000;
		TraceServices::EEventSortOrder SortOrder = TraceServices::EEventSortOrder::ByEndTime;
	};

	static UE_API void RunEnumerateBenchmark(const FEnumerateTestParams& InParams, FCheckValues& OutCheckValues);
	static UE_API void RunEnumerateAsyncBenchmark(const FEnumerateTestParams& InParams, FCheckValues& OutCheckValues);
	static UE_API void RunEnumerateAllTracksBenchmark(const FEnumerateTestParams& InParams, FCheckValues& OutCheckValues);
	static UE_API void RunEnumerateAsyncAllTracksBenchmark(const FEnumerateTestParams& InParams, FCheckValues& OutCheckValues);
	static UE_API bool RunEnumerateSyncAsyncComparisonTest(FAutomationTestBase& Test, const FEnumerateTestParams& InParam, bool bGameThreadOnly);

	static UE_API uint32 GetTimelineIndex(const TCHAR* InName);
	static UE_API void VerifyCheckValues(FAutomationTestBase& Test, FCheckValues First, FCheckValues Second);
};

#undef UE_API
