// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// TraceInsights
#include "Insights/ViewModels/TimingEventsTrack.h"

namespace UE::Insights::TimingProfiler
{

class FThreadTimingTrack : public FTimingEventsTrack
{
	INSIGHTS_DECLARE_RTTI(FThreadTimingTrack, FTimingEventsTrack, TRACEINSIGHTS_API)

public:
	explicit FThreadTimingTrack() {}
	explicit FThreadTimingTrack(const FString& InTrackName) : FTimingEventsTrack(InTrackName) {}
	virtual ~FThreadTimingTrack() = default;

	virtual uint32 GetThreadId() const = 0;
	virtual int32 GetDepthAt(double Time) const = 0;
};

} // namespace UE::Insights::TimingProfiler
