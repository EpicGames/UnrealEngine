// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API TRACEINSIGHTS_API

class SScrollBar;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FAxisViewportInt32
{
private:
	static constexpr float SLATE_UNITS_TOLERANCE = 0.1f;

public:
	FAxisViewportInt32()
	{
		Reset();
	}

	void Reset()
	{
		Size = 0.0f;

		MinValue = 0;
		MaxValue = 0;

		MinPosition = 0.0f;
		MaxPosition = 0.0f;
		Position = 0.0f;

		MinScale = 0.0001f; // 10000 [sample/px]; 1 [px/sample]
		MaxScale = 16.0f; // 1 [sample/px]; 16 [px/sample]
		Scale = MaxScale;
	}

	FString ToDebugString(const TCHAR* Sufix, const TCHAR* Unit) const
	{
		if (Scale >= 1.0f)
		{
			return FString::Printf(TEXT("Scale%s: %.3f (%d pixels/%s), Pos%s: %.2f"),
				Sufix, Scale, FMath::RoundToInt(Scale), Unit, Sufix, Position);
		}
		else
		{
			return FString::Printf(TEXT("Scale%s: %.3f (%d %ss/pixel), Pos%s: %.2f"),
				Sufix, Scale, FMath::RoundToInt(1.0f / Scale), Unit, Sufix, Position);
		}
	}

	/**
	 * Size of viewport, in pixels (Slate units).
	 * The viewport's width if this is a horizontal axis or the viewport's height if this is a vertical axis.
	 */
	float GetSize() const { return Size; }
	UE_API bool SetSize(const float InSize);

	int32 GetMinValue() const { return MinValue; }
	int32 GetMaxValue() const { return MaxValue; }

	float GetMinPos() const { return MinPosition; }
	float GetMaxPos() const { return MaxPosition; }
	float GetPos() const { return Position; }

	float GetMinScale() const { return MinScale; }
	float GetMaxScale() const { return MaxScale; }

	/** Current scale factor between Value units (samples) and Slate units (pixels), in pixels per sample. [px/sample]
	 * If Scale > 1, it represents number of pixels for one sample,
	 * otherwise, the inverse of Scale represents number of samples in one pixel.
	 */
	float GetScale() const { return Scale; }

	/** Returns the number of pixels (Slate units) for one viewport sample. [px] */
	float GetSampleSize() const { return FMath::Max(1.0f, FMath::RoundToFloat(Scale)); }

	/** Returns the number of samples in one pixel (Slate unit). [sample] */
	int32 GetNumSamplesPerPixel() const { return FMath::Max(1, FMath::RoundToInt(1.0f / Scale)); }

	void SetMinMaxInterval(const int32 InMinValue, const int32 InMaxValue)
	{
		MinValue = InMinValue;
		MaxValue = InMaxValue;
		UpdateMinMax();
	}

	int32 GetValueAtPos(const float Pos) const
	{
		return FMath::RoundToInt(Pos / Scale);
	}

	int32 GetValueAtOffset(const float Offset) const
	{
		return FMath::RoundToInt((Position + Offset) / Scale);
	}

	float GetPosForValue(const int32 Value) const
	{
		return static_cast<float>(Value) * Scale;
	}

	float GetOffsetForValue(const int32 Value) const
	{
		return static_cast<float>(Value) * Scale - Position;
	}

	float GetRoundedOffsetForValue(const int32 Value) const
	{
		return FMath::RoundToFloat(static_cast<float>(Value) * Scale - Position);
	}

	bool ScrollAtPos(const float Pos)
	{
		if (Position != Pos)
		{
			Position = Pos;
			OnPositionChanged();
			return true;
		}
		return false;
	}

	bool ScrollAtValue(const int32 Value)
	{
		return ScrollAtPos(GetPosForValue(Value));
	}

	bool CenterOnValue(const int32 Value)
	{
		return ScrollAtPos(static_cast<float>(Value) * Scale - Size / 2.0f);
	}

	bool CenterOnValueInterval(const int32 IntervalStartValue, const int32 IntervalEndValue)
	{
		const float IntervalSize = Scale * static_cast<float>(IntervalEndValue - IntervalStartValue);
		if (IntervalSize > Size)
		{
			return ScrollAtValue(IntervalStartValue);
		}
		else
		{
			return ScrollAtPos(static_cast<float>(IntervalStartValue) * Scale - (Size - IntervalSize) / 2.0f);
		}
	}

	void SetScaleLimits(const float InMinScale, const float InMaxScale)
	{
		MinScale = InMinScale;
		MaxScale = InMaxScale;
	}

	UE_API bool SetScale(const float NewScale);
	UE_API bool ZoomWithFixedOffset(const float NewScale, const float Offset);
	UE_API bool RelativeZoomWithFixedOffset(const float Delta, const float Offset);

	UE_API void GetScrollLimits(float& OutMinPos, float& OutMaxPos);
	UE_API bool EnforceScrollLimits(const float InMinPos, const float InMaxPos, const float InterpolationFactor);
	UE_API bool UpdatePosWithinLimits();

	UE_API void OnUserScrolled(TSharedPtr<SScrollBar> ScrollBar, float ScrollOffset);
	UE_API void UpdateScrollBar(TSharedPtr<SScrollBar> ScrollBar) const;

private:
	void OnSizeChanged()
	{
	}

	void OnPositionChanged()
	{
	}

	void OnScaleChanged()
	{
		UpdateMinMax();
	}

	void UpdateMinMax()
	{
		MinPosition = GetPosForValue(MinValue);
		MaxPosition = GetPosForValue(MaxValue);

		//const float NewPosition = FMath::Clamp(Position, MinPosition, MaxPosition);
		//if (FMath::IsNearlyEqual(NewPosition, Position, SLATE_UNITS_TOLERANCE))
		//{
		//	Position = NewPosition;
		//	OnPositionChanged();
		//}
	}

private:
	float Size; // size of viewport (ex.: the viewport's width if this is a horizontal axis), in pixels (Slate units); [px]

	int32 MinValue; // minimum value (inclusive); [sample]
	int32 MaxValue; // maximum value (exclusive); [sample]

	//int32 StartValue; // value of viewport's left side; [value_unit]; StartValue = GetValueAtPos(Position); computed when Position or Scale changes
	//int32 EndValue; // value of viewport's right side; [value_unit]; EndValue = GetValueAtPos(Position + Size); computed when Position, Size or Scale changes

	float MinPosition; // minimum position (corresponding to MinValue), in pixels (Slate units); [px]
	float MaxPosition; // maximum position (corresponding to MaxValue), in pixels (Slate units); [px]
	float Position; // current position of the viewport, in pixels (Slate units); [px]

	float MinScale; // minimum scale factor; [px/sample]
	float MaxScale; // maximum scale factor; [px/sample]
	float Scale; // current scale factor between Value units (samples) and Slate units (pixels); [px/sample]
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
