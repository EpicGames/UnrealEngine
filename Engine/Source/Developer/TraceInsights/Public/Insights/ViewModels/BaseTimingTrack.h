// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Common/SimpleRtti.h"

// TraceInsights
#include "Insights/Config.h"

#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
#include "Insights/ITimingViewSession.h"
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMenuBuilder;
struct FGeometry;

namespace UE::Insights { class FDrawContext; }
namespace UE::Insights { class FFilterConfigurator; }

class FTimingEventSearchParameters;
class FTimingTrackViewport;
class ITimingViewDrawHelper;
class FTooltipDrawState;
class ITimingEvent;
class ITimingEventRelation;
class ITimingEventFilter;
struct FTimingViewLayout;

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class ETimingTrackLocation : uint32
{
	None         = 0,
	Scrollable   = (1 << 0),
	TopDocked    = (1 << 1),
	BottomDocked = (1 << 2),
	Foreground   = (1 << 3),
	All          = Scrollable | TopDocked | BottomDocked | Foreground
};
ENUM_CLASS_FLAGS(ETimingTrackLocation);

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FTimingTrackOrder
{
	static constexpr int32 GroupRange = 100000;
	static constexpr int32 TimeRuler  = -2 * GroupRange;
	static constexpr int32 Markers    = -1 * GroupRange;
	static constexpr int32 First      = 0;
	static constexpr int32 Task       = 1 * GroupRange;
	static constexpr int32 Memory     = 2 * GroupRange;
	static constexpr int32 Gpu        = 3 * GroupRange;
	static constexpr int32 Cpu        = 4 * GroupRange;
	static constexpr int32 Last       = 5 * GroupRange;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class ETimingTrackFlags : uint32
{
	None            = 0,
	IsVisible       = (1 << 0),
	IsDirty         = (1 << 1),
	IsSelected      = (1 << 2),
	IsHovered       = (1 << 3),
	IsHeaderHovered = (1 << 4),
};
ENUM_CLASS_FLAGS(ETimingTrackFlags);

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class EDrawEventMode : uint32
{
	None = 0,

	/** Draw the content of the event. This flag can be omitted in order to draw only the hovered/selected highlights. */
	Content = (1 << 0),

	/** Draw the highlights for a hovered event. */
	Hovered = (1 << 1),

	/** Draw the highlights for a selected event. */
	Selected = (1 << 2),

	/** Draw the highlights for an event that is both selected and hovered. */
	SelectedAndHovered = Hovered | Selected,
};
ENUM_CLASS_FLAGS(EDrawEventMode);

////////////////////////////////////////////////////////////////////////////////////////////////////

class ITimingTrackUpdateContext
{
public:
	virtual const FGeometry& GetGeometry() const = 0;
	virtual const FTimingTrackViewport& GetViewport() const = 0;
	virtual const FVector2D& GetMousePosition() const = 0;
	virtual const TSharedPtr<const ITimingEvent> GetHoveredEvent() const = 0;
	virtual const TSharedPtr<const ITimingEvent> GetSelectedEvent() const = 0;
	virtual const TSharedPtr<ITimingEventFilter> GetEventFilter() const = 0;
	virtual const TArray<TUniquePtr<ITimingEventRelation>>& GetCurrentRelations() const = 0;
	virtual double GetCurrentTime() const = 0;
	virtual float GetDeltaTime() const = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class ITimingTrackDrawContext
{
public:
	virtual const FTimingTrackViewport& GetViewport() const = 0;
	virtual const FVector2D& GetMousePosition() const = 0;
	virtual const TSharedPtr<const ITimingEvent> GetHoveredEvent() const = 0;
	virtual const TSharedPtr<const ITimingEvent> GetSelectedEvent() const = 0;
	virtual const TSharedPtr<ITimingEventFilter> GetEventFilter() const = 0;
	virtual UE::Insights::FDrawContext& GetDrawContext() const = 0;
	virtual const ITimingViewDrawHelper& GetHelper() const = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FBaseTimingTrack : public TSharedFromThis<FBaseTimingTrack>
{
	friend class FTimingViewDrawHelper;

	INSIGHTS_DECLARE_RTTI_BASE(FBaseTimingTrack, TRACEINSIGHTS_API)

protected:
	FBaseTimingTrack()
		: Id(GenerateId())
	{
	}

	explicit FBaseTimingTrack(const FString& InName)
		: Id(GenerateId())
		, Name(InName)
	{
	}

	FBaseTimingTrack(const FBaseTimingTrack&) = delete;

	TRACEINSIGHTS_API virtual ~FBaseTimingTrack();

public:
	virtual void Reset()
	{
		PosY = 0.0f;
		Height = 0.0f;
		Flags = ETimingTrackFlags::IsVisible | ETimingTrackFlags::IsDirty;
	}

	uint64 GetId() const { return Id; }

	const FString& GetName() const { return Name; }
	void SetName(const FString& InName) { Name = InName; }

	ETimingTrackLocation GetValidLocations() const { return ValidLocations; }
	ETimingTrackLocation GetLocation() const { return Location; }
	TRACEINSIGHTS_API void SetLocation(ETimingTrackLocation InLocation);
	virtual void OnLocationChanged() { SetDirtyFlag(); }

	int32 GetOrder() const { return Order; }
	void SetOrder(int32 InOrder) { Order = InOrder; }

	float GetPosY() const { return PosY; }
	virtual void SetPosY(float InPosY) { PosY = InPosY; }

	float GetHeight() const { return Height; }
	virtual void SetHeight(float InHeight) { Height = InHeight; }

	bool IsVisible() const { return EnumHasAnyFlags(Flags, ETimingTrackFlags::IsVisible); }
	void Show() { Flags |= ETimingTrackFlags::IsVisible; OnVisibilityChanged(); }
	void Hide() { Flags &= ~ETimingTrackFlags::IsVisible; OnVisibilityChanged(); }
	void ToggleVisibility() { Flags ^= ETimingTrackFlags::IsVisible; OnVisibilityChanged(); }
	void SetVisibilityFlag(bool bIsVisible) { bIsVisible ? Show() : Hide(); }
	virtual void OnVisibilityChanged() { if (IsVisible()) SetDirtyFlag(); }

	bool IsDirty() const { return EnumHasAnyFlags(Flags, ETimingTrackFlags::IsDirty); }
	void SetDirtyFlag() { Flags |= ETimingTrackFlags::IsDirty; OnDirtyFlagChanged(); }
	void ClearDirtyFlag() { Flags &= ~ETimingTrackFlags::IsDirty; OnDirtyFlagChanged(); }
	virtual void OnDirtyFlagChanged() {}

	bool IsSelected() const { return EnumHasAnyFlags(Flags, ETimingTrackFlags::IsSelected); }
	void Select() { Flags |= ETimingTrackFlags::IsSelected; OnSelectedFlagChanged(); }
	void Unselect() { Flags &= ~ETimingTrackFlags::IsSelected; OnSelectedFlagChanged(); }
	void ToggleSelectedFlag() { Flags ^= ETimingTrackFlags::IsSelected; OnSelectedFlagChanged(); }
	void SetSelectedFlag(bool bIsSelected) { bIsSelected ? Select() : Unselect(); }
	virtual void OnSelectedFlagChanged() {}

	bool IsHovered() const { return EnumHasAnyFlags(Flags, ETimingTrackFlags::IsHovered); }
	void SetHoveredState(bool bIsHovered) { bIsHovered ? Flags |= ETimingTrackFlags::IsHovered : Flags &= ~ETimingTrackFlags::IsHovered; }
	bool IsHeaderHovered() const { return EnumHasAllFlags(Flags, ETimingTrackFlags::IsHovered | ETimingTrackFlags::IsHeaderHovered); }
	void SetHeaderHoveredState(bool bIsHeaderHovered) { bIsHeaderHovered ? Flags |= ETimingTrackFlags::IsHeaderHovered : Flags &= ~ETimingTrackFlags::IsHeaderHovered; }

	//////////////////////////////////////////////////

	// PreUpdate callback called each frame, but only if the track is visible.
	// In this update, neither the position nor the size of the track is yet computed.
	// Track should update here its height.
	virtual void PreUpdate(const ITimingTrackUpdateContext& Context) {}

	// Update callback called each frame, but only if the track is visible.
	// In this update, it is assumed the track position and the track size are valid.
	virtual void Update(const ITimingTrackUpdateContext& Context) {}

	// PostUpdate callback called each frame, but only if the track is visible.
	// Track should update here its "hovered" state.
	virtual void PostUpdate(const ITimingTrackUpdateContext& Context) {}

	//////////////////////////////////////////////////

	// PreDraw callback (called from OnPaint) to draw something in the background.
	virtual void PreDraw(const ITimingTrackDrawContext& Context) const {}

	// Draw callback (called from OnPaint) to draw the track's content.
	virtual void Draw(const ITimingTrackDrawContext& Context) const {}

	// Draw a single event (can be used to draw only the highlight for a selected and/or hovered event).
	virtual void DrawEvent(const ITimingTrackDrawContext& Context, const ITimingEvent& InTimingEvent, EDrawEventMode InDrawMode) const {}

	// PostDraw callback (called from OnPaint) to draw something in the foreground.
	virtual void PostDraw(const ITimingTrackDrawContext& Context) const {}

	//////////////////////////////////////////////////

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return FReply::Unhandled(); }
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return FReply::Unhandled(); }
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return FReply::Unhandled(); }

	TRACEINSIGHTS_API virtual void BuildContextMenu(FMenuBuilder& MenuBuilder);

	/**
	 * Gets the event at a specified position. Leaves the InOutEvent unchanged if no event is found at specified position.
	 * @param X The horizontal coordinate of the point tested; in Slate pixels (viewport coordinates).
	 * @param Y The vertical coordinate of the point tested; in Slate pixels (viewport coordinates).
	 * @param Viewport The timing viewport used to transform time in viewport coordinates.
	 * @return The event located at (PosX, PosY) coordinates, if any; nullptr otherwise.
	 */
	virtual const TSharedPtr<const ITimingEvent> GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const { return nullptr; }

	// Search for an event using custom parameters.
	virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const { return nullptr; }

	// Get the filter object for filtering all events similar with a specified event. Used when double clicked on an event.
	virtual TSharedPtr<ITimingEventFilter> GetFilterByEvent(const TSharedPtr<const ITimingEvent> InTimingEvent) const { return nullptr; }

	// Allows tracks to update event stats that are slower to compute (called at a lower frequency than GetEventAtPosition or Search or SearchTimingEvent).
	virtual void UpdateEventStats(ITimingEvent& InOutEvent) const {}

	// Called back from the timing view when an event is hovered by mouse.
	virtual void OnEventHovered(const ITimingEvent& InHoveredEvent) const {}

	// Called back from the timing view when an event is selected.
	virtual void OnEventSelected(const ITimingEvent& InSelectedEvent) const {}

	/**
	 * Called to initialize the tooltip's content with info from a timing event.
	 *
	 * @note In most cases, this should begin by calling `InOutTooltip.ResetContent()` before adding
	 *       any content to the tooltip. However, in case the track may be used as a child track,
	 *       the tooltip content should not be reset, and instead should only be appended to
	 *       (because the parent track will reset the tooltip's content.)
	 *       This is because other child tracks may want to append to the tooltip's content,
	 *       and their appended content should not be reset.
	 */
	virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const {}

	// Called back from the timing view when an event is copied to the clipboard with Ctrl+C.
	virtual void OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const {}

	// Adding children to tracks is a two-step process:
	//
	//  1. add the child track into the parent track using `AddChildTrack`
	//  2. set the child track's parent track using `SetParentTrack`
	//
	// Both steps are needed to ensure the track renders properly.
	// If a track is a child track, it must not also be added as a docked or scrollable track
	// into the view.
	//
	// Note that child tracks do not respect the order established with `SetOrder`.
	// Instead, you can reorder the elements individually in `GetChildTracks()`.

	/** Gets a view into the array of child tracks, for enumeration and reordering. */
	TArrayView<const TSharedRef<FBaseTimingTrack>> GetChildTracks() const { return ChildTracks; }
	TArrayView<TSharedRef<FBaseTimingTrack>> GetChildTracks() { return ChildTracks; }

	/**
	 * Adds a child track to this track.
	 * Note that this operation is idempotent. Calling this with the same track twice will only
	 * add the track once.
	 *
	 * @note This invalidates the result of `GetChildTracks`.
	 */
	TRACEINSIGHTS_API void AddChildTrack(TSharedRef<FBaseTimingTrack> Track);

	/**
	 * Inserts a child track to this track at the specified index.
	 * @note This invalidates the result of `GetChildTracks`.
	 */
	TRACEINSIGHTS_API void AddChildTrack(TSharedRef<FBaseTimingTrack> Track, int32 Index);

	/**
	 * Removes the provided track from this track's child tracks.
	 *
	 * @note This invalidates the result of `GetChildTracks`.
	 */
	TRACEINSIGHTS_API void RemoveChildTrack(TSharedRef<FBaseTimingTrack> Track);

	// The reason why FindChildTrackOfType does not consider the inheritance hierarchy is because
	// if it did, downstream inheritors of specific public track types could break plugins defining
	// them. Certain parts of Insights assume that there exists at least one instance of a child
	// track of a given type (e.g. there must be at most one FContextSwitchesTimingTrack per
	// FCpuCoreTimingTrack.)

	/**
	 * Returns the first child track with the same type name.
	 * Note that the type name is matched exactly (without considering inheritance.)
	 * You should generally prefer the templated version over this.
	 */
	TRACEINSIGHTS_API TSharedPtr<FBaseTimingTrack> FindChildTrackOfType(FName TrackTypeName);

	/**
	 * Returns the first child track of the given type.
	 * Note that the type is matched exactly (without considering inheritance.)
	 * `T` must belong to the SimpleRTTI hierarchy.
	 */
	template<typename T>
	TSharedPtr<T> FindChildTrackOfType()
	{
		return StaticCastSharedPtr<T>(FindChildTrackOfType(T::GetStaticTypeName()));
	}

	void SetParentTrack(TWeakPtr<FBaseTimingTrack> InTrack) { ParentTrack = InTrack; }
	TWeakPtr<FBaseTimingTrack> GetParentTrack() const { return ParentTrack; }
	bool IsChildTrack() const { return ParentTrack.IsValid(); }

	TRACEINSIGHTS_API float GetChildTracksTopHeight(const FTimingViewLayout& Layout) const;
	TRACEINSIGHTS_API void UpdateChildTracksPosY(const FTimingViewLayout& Layout);

	// Legacy API supporting a single track. Do not use in new code.
	UE_DEPRECATED(5.6, "Loop over GetChildTracks instead")
	TRACEINSIGHTS_API TSharedPtr<FBaseTimingTrack> GetChildTrack() const;
	UE_DEPRECATED(5.6, "Use AddChildTrack/RemoveChildTrack instead")
	TRACEINSIGHTS_API void SetChildTrack(TSharedPtr<FBaseTimingTrack> InTrack);

	virtual void SetFilterConfigurator(TSharedPtr<UE::Insights::FFilterConfigurator> InFilterConfigurator) {}

	// Returns number of text lines needed to display the debug string.
	//TODO: virtual int GetDebugStringLineCount() const { return 0; }

	// Builds a string with debug text information.
	//TODO: virtual void BuildDebugString(FString& OutStr) const {}

protected:
	void SetValidLocations(ETimingTrackLocation InValidLocations) { ValidLocations = InValidLocations; }
	static uint64 GenerateId() { return IdGenerator++; }

protected:
	UE_DEPRECATED(5.6, "Use the ChildTracks array instead")
	TSharedPtr<FBaseTimingTrack> ChildTrack;

	TArray<TSharedRef<FBaseTimingTrack>> ChildTracks;
	TWeakPtr<FBaseTimingTrack> ParentTrack;

private:
	const uint64 Id;
	FString Name;
	ETimingTrackLocation ValidLocations = ETimingTrackLocation::Scrollable;
	ETimingTrackLocation Location = ETimingTrackLocation::None;
	int32 Order = 0;
	float PosY = 0.0f; // y position, in Slate units
	float Height = 0.0f; // height, in Slate units
	ETimingTrackFlags Flags = ETimingTrackFlags::IsVisible | ETimingTrackFlags::IsDirty;

	static TRACEINSIGHTS_API uint64 IdGenerator;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
