// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"

// TraceInsightsCore
#include "InsightsCore/Common/SimpleRtti.h"

// TraceInsights
#include "Insights/ViewModels/BaseTimingTrack.h"
#include "Insights/ViewModels/GraphTrackEvent.h"

#include <limits>

#define UE_API TRACEINSIGHTS_API

class FTimingTrackViewport;
struct FGraphSeriesEvent;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FGraphValueViewport
{
public:
	/**
	 * @return Y position (in viewport local space) of the baseline (with Value == 0); in pixels (Slate units).
	 * Y == 0 at the top of the graph track, positive values are downward.
	 */
	double GetBaselineY() const { return BaselineY; }
	void SetBaselineY(const double InBaselineY) { BaselineY = InBaselineY; }

	/**
	 * @return The scale between Value units and viewport units; in pixels (Slate units) / Value unit.
	 */
	double GetScaleY() const { return ScaleY; }
	void SetScaleY(const double InScaleY) { ScaleY = InScaleY; }

	/**
	 * @param Value a value; in Value units
	 * @return Y position (in viewport local space) for a Value; in pixels (Slate units).
	 * Y == 0 at the top of the graph track, positive values are downward.
	 */
	float GetYForValue(double Value) const
	{
		return static_cast<float>(BaselineY - Value * ScaleY);
	}
	float GetRoundedYForValue(double Value) const
	{
		return FMath::RoundToFloat(FMath::Clamp<float>(GetYForValue(Value), -FLT_MAX, FLT_MAX));
	}

	/**
	 * @param Y a Y position (in viewport local space); in pixels (Slate units).
	 * @return Value for specified Y position.
	 */
	double GetValueForY(float Y) const
	{
		return (BaselineY - static_cast<double>(Y)) / ScaleY;
	}

private:
	double BaselineY = 0.0; // Y position (in viewport local space) of the baseline (with Value == 0); in pixels (Slate units)
	double ScaleY = 1.0; // scale between Value units and viewport units; in pixels (Slate units) / Value unit
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FGraphSeries
{
	friend class FGraphTrack;
	friend class FGraphTrackBuilder;

	INSIGHTS_DECLARE_RTTI_BASE(FGraphSeries, UE_API)

public:
	struct FBox
	{
		float X;
		float W;
		float Y;
	};

public:
	UE_API FGraphSeries();
	UE_API virtual ~FGraphSeries();

	const FText& GetName() const { return Name; }
	void SetName(const TCHAR* InName) { Name = FText::FromString(InName); }
	void SetName(const FString& InName) { Name = FText::FromString(InName); }
	void SetName(const FText& InName) { Name = InName; }

	const FText& GetDescription() const { return Description; }
	void SetDescription(const TCHAR* InDescription) { Description = FText::FromString(InDescription); }
	void SetDescription(const FString& InDescription) { Description = FText::FromString(InDescription); }
	void SetDescription(const FText& InDescription) { Description = InDescription; }

	bool IsVisible() const { return bIsVisible; }
	virtual void SetVisibility(bool bOnOff) { bIsVisible = bOnOff; }

	bool IsDirty() const { return bIsDirty; }
	void SetDirtyFlag() { bIsDirty = true; }
	void ClearDirtyFlag() { bIsDirty = false; }

	const FLinearColor& GetColor() const { return Color; }
	const FLinearColor& GetFillColor() const { return FillColor; }
	const FLinearColor& GetBorderColor() const { return BorderColor; }

	void SetColor(const FLinearColor& InColor)
	{
		Color = InColor;
		FillColor = InColor.CopyWithNewOpacity(0.1f);
		BorderColor = FLinearColor(FMath::Min(InColor.R + 0.4f, 1.0f),
								   FMath::Min(InColor.G + 0.4f, 1.0f),
								   FMath::Min(InColor.B + 0.4f, 1.0f),
								   InColor.A);
	}

	void SetColor(const FLinearColor& InColor, const FLinearColor& InBorderColor)
	{
		Color = InColor;
		FillColor = InColor.CopyWithNewOpacity(0.5f);
		BorderColor = InBorderColor;
	}

	void SetColor(const FLinearColor& InColor, const FLinearColor& InBorderColor, const FLinearColor& InFillColor)
	{
		Color = InColor;
		FillColor = InFillColor;
		BorderColor = InBorderColor;
	}

	bool HasEventDuration() const { return bHasEventDuration; }
	void SetHasEventDuration(bool bOnOff) { bHasEventDuration = bOnOff; }

	bool IsAutoZoomEnabled() const { return bAutoZoom; }
	void EnableAutoZoom() { bAutoZoom = true; }
	void DisableAutoZoom() { bAutoZoom = false; }

	bool IsAutoZoomDirty() const { return bIsAutoZoomDirty; }
	void SetAutoZoomDirty() { bIsAutoZoomDirty = true; }
	void ResetAutoZoomDirty() { bIsAutoZoomDirty = false; }

	bool IsUsingSharedViewport() const { return bUseSharedViewport; }
	void EnableSharedViewport() { bUseSharedViewport = true; }

	virtual bool HasHighThresholdValue() const { return false; }
	virtual double GetHighThresholdValue() const { return +std::numeric_limits<double>::infinity(); }
	virtual void SetHighThresholdValue(double InValue) {}
	virtual void ResetHighThresholdValue() {}

	virtual bool HasLowThresholdValue() const { return false; }
	virtual double GetLowThresholdValue() const { return -std::numeric_limits<double>::infinity(); }
	virtual void SetLowThresholdValue(double InValue) {}
	virtual void ResetLowThresholdValue() {}

	//////////////////////////////////////////////////

	/**
	 * @return Y position (in viewport local space) of the baseline (with Value == 0); in pixels (Slate units).
	 * Y == 0 at the top of the graph track, positive values are downward.
	 */
	double GetBaselineY() const { return ValueViewport.GetBaselineY(); }
	void SetBaselineY(const double InBaselineY) { ValueViewport.SetBaselineY(InBaselineY); }

	/**
	 * @return The scale between Value units and viewport units; in pixels (Slate units) / Value unit.
	 */
	double GetScaleY() const { return ValueViewport.GetScaleY(); }
	void SetScaleY(const double InScaleY) { ValueViewport.SetScaleY(FMath::Max(InScaleY, DBL_EPSILON)); }

	/**
	 * @param Value a value; in Value units
	 * @return Y position (in viewport local space) for a Value; in pixels (Slate units).
	 * Y == 0 at the top of the graph track, positive values are downward.
	 */
	float GetYForValue(double Value) const { return ValueViewport.GetYForValue(Value); }
	float GetRoundedYForValue(double Value) const { return ValueViewport.GetRoundedYForValue(Value); }

	/**
	 * @param Y a Y position (in viewport local space); in pixels (Slate units).
	 * @return Value for specified Y position.
	 */
	double GetValueForY(float Y) const { return ValueViewport.GetValueForY(Y); }

	/**
	 * Compute BaselineY and ScaleY so the [Low, High] Value range will correspond to [Top, Bottom] Y position range.
	 * GetYForValue(InHighValue) == InTopY
	 * GetYForValue(InLowValue) == InBottomY
	 */
	void ComputeBaselineAndScale(const double InLowValue, const double InHighValue, const float InTopY, const float InBottomY, double& OutBaselineY, double& OutScaleY) const
	{
		ensure(InLowValue < InHighValue);
		ensure(InTopY <= InBottomY);
		const double InvRange = 1.0 / (InHighValue - InLowValue);
		OutScaleY = static_cast<double>(InBottomY - InTopY) * InvRange;
		//OutBaselineY = (InHighValue * static_cast<double>(InBottomY) - InLowValue * static_cast<double>(InTopY)) * InvRange;
		OutBaselineY = static_cast<double>(InTopY) + InHighValue * OutScaleY;
	}

	//////////////////////////////////////////////////

	/**
	 * @param X The horizontal coordinate of the point tested; in Slate pixels (local graph coordinates)
	 * @param Y The vertical coordinate of the point tested; in Slate pixels (local graph coordinates)
	 * @param Viewport The timing viewport used to transform time in local graph coordinates
	 * @param bCheckLine If needs to check the bounding box of the horizontal line (determined by duration of event and value) or only the bounding box of the visual point
	 * @param bCheckBox If needs to check the bounding box of the entire visual box (determined by duration of event, value and baseline)
	 * @return A pointer to an Event located at (X, Y) coordinates, if any; nullptr if no event is located at respective coordinates
	 * The returned pointer is valid only temporary until next Reset() or next usage of FGraphTrackBuilder for this series/track.
	 */
	UE_API const FGraphSeriesEvent* GetEvent(const float PosX, const float PosY, const FTimingTrackViewport& Viewport, bool bCheckLine, bool bCheckBox) const;

	/** Updates the track's auto-zoom. Does nothing if IsAutoZoomEnabled() is false. */
	UE_API void UpdateAutoZoom(const float InTopY, const float InBottomY, const double InMinEventValue, const double InMaxEventValue, const bool bIsAutoZoomAnimated = true);

	/** Updates the track's auto-zoom. Returns true if viewport was changed. Sets bIsAutoZoomDirty=true if needs another update. */
	UE_API bool UpdateAutoZoomEx(const float InTopY, const float InBottomY, const double InMinEventValue, const double InMaxEventValue, const bool bIsAutoZoomAnimated);

	UE_API virtual FString FormatValue(double Value) const;

private:
	FText Name;
	FText Description;

	bool bIsVisible = true;
	bool bIsDirty = false;

	bool bHasEventDuration = true;

	bool bAutoZoom = false;
	bool bIsAutoZoomDirty = false;

	bool bUseSharedViewport = false;
	FGraphValueViewport ValueViewport;

	FLinearColor Color       = FLinearColor(0.0f, 0.5f, 1.0f, 1.0f);
	FLinearColor FillColor   = FLinearColor(0.0f, 0.5f, 1.0f, 1.0f);
	FLinearColor BorderColor = FLinearColor(0.3f, 0.8f, 1.0f, 1.0f);

protected:
	TArray<FGraphSeriesEvent> Events; // reduced list of events; used to identify an event at a certain screen position (ex.: the event hovered by mouse)
	TArray<FVector2D> Points; // reduced list of points; for drawing points
	TArray<TArray<FVector2D>> LinePoints; // reduced list of points; for drawing the connected line and filled polygon, split into disconnected batches
	TArray<FBox> Boxes; // reduced list of boxes; for drawing boxes
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
