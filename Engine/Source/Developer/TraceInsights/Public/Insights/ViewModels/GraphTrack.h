// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Fonts/SlateFontInfo.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/SharedPointer.h"

// TraceInsights
#include "Insights/ViewModels/BaseTimingTrack.h"
#include "Insights/ViewModels/GraphSeries.h"

#define UE_API TRACEINSIGHTS_API

class FMenuBuilder;
struct FSlateBrush;

class FTimingTrackViewport;

////////////////////////////////////////////////////////////////////////////////////////////////////

// Various available options for display
enum class EGraphOptions
{
	None = 0,

	ShowDebugInfo				= (1 << 0),
	ShowPoints					= (1 << 1),
	ShowPointsWithBorder		= (1 << 2),
	ShowLines					= (1 << 3),
	ShowPolygon					= (1 << 4),
	UseEventDuration			= (1 << 5),
	ShowBars					= (1 << 6),
	ShowBaseline				= (1 << 7),
	ShowThresholds				= (1 << 8),
	ShowVerticalAxisGrid		= (1 << 9),
	ShowHeader					= (1 << 10),

	AutoZoomIncludesBaseline	= (1 << 11),
	AutoZoomIncludesThresholds	= (1 << 12),

	FirstCustomOption			= (1 << 13),

	DefaultEnabledOptions	= None,
	DefaultVisibleOptions	= ShowPoints | ShowPointsWithBorder | ShowLines | ShowPolygon | UseEventDuration | ShowBars,
	DefaultEditableOptions	= ShowPoints | ShowPointsWithBorder | ShowLines | ShowPolygon | UseEventDuration | ShowBars,
};

ENUM_CLASS_FLAGS(EGraphOptions);

////////////////////////////////////////////////////////////////////////////////////////////////////

class FGraphTrack : public FBaseTimingTrack
{
	friend class FGraphTrackBuilder;

	INSIGHTS_DECLARE_RTTI(FGraphTrack, FBaseTimingTrack, UE_API)

private:
	// Visual size of points (in pixels).
	static constexpr float PointVisualSize = 5.5f;

	// Size of points (in pixels) used in reduction algorithm.
	static constexpr double PointSizeX = 3.0f;
	static constexpr float PointSizeY = 3.0f;

public:
	UE_API explicit FGraphTrack();
	UE_API explicit FGraphTrack(const FString& InName);
	UE_API virtual ~FGraphTrack();

	//////////////////////////////////////////////////
	// Options

	EGraphOptions GetEnabledOptions() const { return EnabledOptions; }
	void SetEnabledOptions(EGraphOptions Options) { EnabledOptions = Options; }

	bool AreAllOptionsEnabled(EGraphOptions Options) const { return EnumHasAllFlags(EnabledOptions, Options); }
	bool IsAnyOptionEnabled(EGraphOptions Options) const { return EnumHasAnyFlags(EnabledOptions, Options); }
	void EnableOptions(EGraphOptions Options) { EnabledOptions |= Options; }
	void DisableOptions(EGraphOptions Options) { EnabledOptions &= ~Options; }
	void ToggleOptions(EGraphOptions Options) { EnabledOptions ^= Options; }

	EGraphOptions GetVisibleOptions() const { return VisibleOptions; }
	void SetVisibleOptions(EGraphOptions Options) { VisibleOptions = Options; }

	EGraphOptions GetEditableOptions() const { return EditableOptions; }
	void SetEditableOptions(EGraphOptions Options) { EditableOptions = Options; }

	//////////////////////////////////////////////////
	// FBaseTimingTrack

	UE_API virtual void Reset() override;

	UE_API virtual void PostUpdate(const ITimingTrackUpdateContext& Context) override;

	UE_API virtual void PreDraw(const ITimingTrackDrawContext& Context) const override;
	UE_API virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	UE_API virtual void DrawEvent(const ITimingTrackDrawContext& Context, const ITimingEvent& InTimingEvent, EDrawEventMode InDrawMode) const override;

	UE_API virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;

	UE_API virtual const TSharedPtr<const ITimingEvent> GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const override;

	UE_API virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;

	//////////////////////////////////////////////////

	TArray<TSharedPtr<FGraphSeries>>& GetSeries() { return AllSeries; }

	FGraphValueViewport& GetSharedValueViewport() { return SharedValueViewport; }
	const FGraphValueViewport& GetSharedValueViewport() const { return SharedValueViewport; }

	//TODO: virtual int GetDebugStringLineCount() const override;
	//TODO: virtual void BuildDebugString(FString& OutStr) const override;

	int32 GetNumAddedEvents() const { return NumAddedEvents; }
	int32 GetNumDrawPoints() const { return NumDrawPoints; }
	int32 GetNumDrawLines() const { return NumDrawLines; }
	int32 GetNumDrawBoxes() const { return NumDrawBoxes; }

protected:
	UE_API void UpdateStats();

	UE_API void DrawSeries(const FGraphSeries& Series, UE::Insights::FDrawContext& DrawContext, const FTimingTrackViewport& Viewport) const;

	UE_API virtual void DrawVerticalAxisGrid(const ITimingTrackDrawContext& Context) const;
	UE_API virtual void DrawHeader(const ITimingTrackDrawContext& Context) const;

	// Get the Y value that is used to provide a clipping border between adjacent graph tracks.
	virtual float GetBorderY() const { return 0.0f; }

	UE_API bool ContextMenu_ToggleOption_CanExecute(EGraphOptions Option);
	UE_API virtual void ContextMenu_ToggleOption_Execute(EGraphOptions Option);
	UE_API bool ContextMenu_ToggleOption_IsChecked(EGraphOptions Option);

private:
	UE_API bool ContextMenu_ShowPointsWithBorder_CanExecute();
	UE_API bool ContextMenu_UseEventDuration_CanExecute();

protected:
	TArray<TSharedPtr<FGraphSeries>> AllSeries;

	// Slate resources
	const FSlateBrush* WhiteBrush = nullptr;
	const FSlateBrush* PointBrush = nullptr;
	const FSlateBrush* BorderBrush = nullptr;
	const FSlateFontInfo Font;

	// Flags controlling various Graph options
	EGraphOptions EnabledOptions = EGraphOptions::DefaultEnabledOptions; // currently enabled options
	EGraphOptions VisibleOptions = EGraphOptions::DefaultVisibleOptions; // if the option is visible in the context menu
	EGraphOptions EditableOptions = EGraphOptions::DefaultEditableOptions; // if the option is editable from the context menu (if false, the option can be readonly)

	FGraphValueViewport SharedValueViewport;

	// Time scale (horizontal axis) saved from "graph update" and used later in setting up tooltips.
	double TimeScaleX = 1.0;

	// Stats
	int32 NumAddedEvents = 0; // total event count
	int32 NumDrawPoints = 0;
	int32 NumDrawLines = 0;
	int32 NumDrawBoxes = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FRandomGraphTrack : public FGraphTrack
{
	INSIGHTS_DECLARE_RTTI(FRandomGraphTrack, FGraphTrack, UE_API)

public:
	UE_API FRandomGraphTrack();
	UE_API virtual ~FRandomGraphTrack();

	UE_API virtual void Update(const ITimingTrackUpdateContext& Context) override;

	UE_API void AddDefaultSeries();

protected:
	UE_API void GenerateSeries(FGraphSeries& Series, const FTimingTrackViewport& Viewport, const int32 EventCount, int32 Seed);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
