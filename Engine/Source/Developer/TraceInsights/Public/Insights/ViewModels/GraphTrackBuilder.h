// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API TRACEINSIGHTS_API

class FGraphTrack;
class FGraphSeries;
class FTimingTrackViewport;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FGraphTrackBuilder
{
private:
	struct FPointInfo
	{
		bool bValid;
		float X;
		float Y;

		FPointInfo() : bValid(false) {}
	};

public:
	UE_API explicit FGraphTrackBuilder(FGraphTrack& InTrack, FGraphSeries& InSeries, const FTimingTrackViewport& InViewport);
	UE_API ~FGraphTrackBuilder();

	/**
	 * Non-copyable
	 */
	FGraphTrackBuilder(const FGraphTrackBuilder&) = delete;
	FGraphTrackBuilder& operator=(const FGraphTrackBuilder&) = delete;

	FGraphTrack& GetTrack() const { return Track; }
	FGraphSeries& GetSeries() const { return Series; }
	const FTimingTrackViewport& GetViewport() const { return Viewport; }

	UE_API void AddEvent(double Time, double Duration, double Value, bool bConnected = true);

private:
	UE_API void BeginPoints();
	UE_API bool AddPoint(double Time, double Value);
	UE_API void FlushPoints();
	UE_API void EndPoints();

	UE_API void BeginConnectedLines();
	UE_API void AddConnectedLine(double Time, double Value, bool bNewBatch);
	UE_API void FlushConnectedLine();
	UE_API void AddConnectedLinePoint(float X, float Y);
	UE_API void EndConnectedLines();

	UE_API void BeginBoxes();
	UE_API void AddBox(double Time, double Duration, double Value);
	UE_API void FlushBox();
	UE_API void EndBoxes();

private:
	FGraphTrack& Track;
	FGraphSeries& Series;
	const FTimingTrackViewport& Viewport;

	// Used by the point reduction algorithm.
	double PointsCurrentX;
	TArray<FPointInfo> PointsAtCurrentX;

	// Used by the line reduction algorithm.
	float LinesCurrentX;
	float LinesMinY;
	float LinesMaxY;
	float LinesFirstY;
	float LinesLastY;
	bool bIsLastLineAdded;

	// Used by the box reduction algorithm.
	//...
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
