// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// TraceInsights
#include "Insights/ViewModels/ITimingEvent.h"

#define UE_API TRACEINSIGHTS_API

class FGraphTrack;
class FGraphSeries;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FGraphSeriesEvent
{
	double Time;
	double Duration;
	double Value;

	bool Equals(const FGraphSeriesEvent& Other) const
	{
		return Time == Other.Time && Duration == Other.Duration && Value == Other.Value;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FGraphTrackEvent : public ITimingEvent
{
	INSIGHTS_DECLARE_RTTI(FGraphTrackEvent, ITimingEvent, UE_API)

public:
	explicit FGraphTrackEvent(const TSharedRef<const FGraphTrack> InTrack, const TSharedRef<const FGraphSeries> InSeries, const FGraphSeriesEvent& InSeriesEvent)
		: Track(InTrack), Series(InSeries), SeriesEvent(InSeriesEvent)
	{}

	virtual ~FGraphTrackEvent() = default;

	//////////////////////////////////////////////////
	// ITimingEvent interface

	UE_API virtual const TSharedRef<const FBaseTimingTrack> GetTrack() const override;

	virtual uint32 GetDepth() const override { return 0; }

	virtual double GetStartTime() const override { return SeriesEvent.Time; }
	virtual double GetEndTime() const override { return SeriesEvent.Time + SeriesEvent.Duration; }
	virtual double GetDuration() const override { return SeriesEvent.Duration; }

	//virtual bool IsValidTrack() const override { return Track.IsValid(); }
	//virtual bool IsValid() const override { return Track.IsValid() && Series.IsValid(); }

	virtual bool Equals(const ITimingEvent& Other) const override
	{
		if (GetTypeName() != Other.GetTypeName())
		{
			return false;
		}

		const FGraphTrackEvent& OtherGraphEvent = Other.As<FGraphTrackEvent>();
		return GetTrack() == Other.GetTrack()
			&& Series == OtherGraphEvent.GetSeries()
			&& SeriesEvent.Equals(OtherGraphEvent.GetSeriesEvent());
	}

	//////////////////////////////////////////////////

	const TSharedRef<const FGraphSeries> GetSeries() const { return Series; }
	const FGraphSeriesEvent& GetSeriesEvent() const { return SeriesEvent; }
	double GetValue() const { return SeriesEvent.Value; }

private:
	const TSharedRef<const FGraphTrack> Track;
	const TSharedRef<const FGraphSeries> Series;
	const FGraphSeriesEvent SeriesEvent;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
