// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// TraceInsightsCore
#include "InsightsCore/Filter/ViewModels/IFilterExecutor.h"

// TraceInsights
#include "Insights/Config.h"

#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
namespace Insights { using IFilterExecutor = UE::Insights::IFilterExecutor; }
#endif
