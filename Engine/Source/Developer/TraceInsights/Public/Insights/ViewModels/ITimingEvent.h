// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Common/SimpleRtti.h"

// TraceInsights
#include "Insights/Config.h"

#if UE_INSIGHTS_BACKWARD_COMPATIBILITY_UE54
#include "Insights/ITimingViewSession.h"
#endif

class ITimingViewDrawHelper;
class FBaseTimingTrack;
class FTimingTrackViewport;

namespace UE::Insights { class FDrawContext; }

////////////////////////////////////////////////////////////////////////////////////////////////////

class ITimingEvent
{
	INSIGHTS_DECLARE_RTTI_BASE(ITimingEvent, TRACEINSIGHTS_API)

public:
	virtual const TSharedRef<const FBaseTimingTrack> GetTrack() const = 0;

	bool CheckTrack(const FBaseTimingTrack* TrackPtr) const { return &GetTrack().Get() == TrackPtr; }

	virtual uint32 GetDepth() const = 0;

	virtual double GetStartTime() const = 0;
	virtual double GetEndTime() const = 0;
	virtual double GetDuration() const = 0;

	virtual bool Equals(const ITimingEvent& Other) const = 0;

	static bool AreEquals(const ITimingEvent& A, const ITimingEvent& B)
	{
		return A.Equals(B);
	}

	static bool AreValidAndEquals(const TSharedPtr<const ITimingEvent> A, const TSharedPtr<const ITimingEvent> B)
	{
		return A.IsValid() && B.IsValid() && (*A).Equals(*B);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class ITimingEventFilter
{
	INSIGHTS_DECLARE_RTTI_BASE(ITimingEventFilter, TRACEINSIGHTS_API)

public:
	/**
	 * Returns true if the track passes the filter.
	 */
	virtual bool FilterTrack(const FBaseTimingTrack& InTrack) const = 0;

	/**
	 * Returns true if the timing event passes the filter.
	 */
	virtual bool FilterEvent(const ITimingEvent& InEvent) const = 0;
	virtual bool FilterEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName, uint64 InEventType = 0, uint32 InEventColor = 0) const = 0;

	/**
	 * Returns a number that changes each time an attribute of this filter changes.
	 */
	virtual uint32 GetChangeNumber() const = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class ITimingEventRelation
{
	INSIGHTS_DECLARE_RTTI_BASE(ITimingEventRelation, TRACEINSIGHTS_API)

public:
	enum class EDrawFilter
	{
		BetweenScrollableTracks, //  Only draw relations between 2 scrollable tracks.
		BetweenDockedTracks, // Only draw relation if the source or target track are docked.
	};

	ITimingEventRelation() {}
	virtual ~ITimingEventRelation() = default;

	virtual void Draw(const UE::Insights::FDrawContext& DrawContext, const FTimingTrackViewport& Viewport, const ITimingViewDrawHelper& Helper, const EDrawFilter Filter) = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
