// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FBaseTimingTrack;
struct FSlateBrush;
struct FSlateFontInfo;

/** Helper allowing access to common drawing elements for tracks */
class ITimingViewDrawHelper
{
public:
	virtual const FSlateBrush* GetWhiteBrush() const = 0;
	virtual const FSlateBrush* GetEventBorderBrush() const = 0;
	virtual const FSlateBrush* GetHoveredEventBorderBrush() const = 0;
	virtual const FSlateBrush* GetSelectedEventBorderBrush() const = 0;
	virtual const FSlateFontInfo& GetEventFont() const = 0;
	virtual FLinearColor GetEdgeColor() const = 0;
	virtual FLinearColor GetValidAreaColor() const = 0;
	virtual FLinearColor GetInvalidAreaColor() const = 0;
	virtual FLinearColor GetTrackNameTextColor(const FBaseTimingTrack& Track) const = 0;
	virtual FLinearColor GetTrackNameBackgroundColor(const FBaseTimingTrack& Track) const { return GetEdgeColor(); }
	virtual int32 GetHeaderBackgroundLayerId() const = 0;
	virtual int32 GetHeaderTextLayerId() const = 0;
	virtual int32 GetRelationLayerId() const = 0;
	virtual int32 GetTextLayerId() const = 0;
	virtual int32 GetFirstLayerId() const = 0;
	virtual int32 GetNumLayerIds() const = 0;
};