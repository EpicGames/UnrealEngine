// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// TraceInsights
#include "Insights/ViewModels/TimingEvent.h"

#define UE_API TRACEINSIGHTS_API

////////////////////////////////////////////////////////////////////////////////////////////////////

class FThreadTrackEvent : public FTimingEvent
{
	INSIGHTS_DECLARE_RTTI(FThreadTrackEvent, FTimingEvent, UE_API)

public:
	UE_API FThreadTrackEvent(const TSharedRef<const FBaseTimingTrack> InTrack, double InStartTime, double InEndTime, uint32 InDepth);
	virtual ~FThreadTrackEvent() = default;

	UE_API uint32 GetTimerIndex() const;
	UE_API void SetTimerIndex(uint32 InTimerIndex);

	UE_API uint32 GetTimerId() const;
	UE_API void SetTimerId(uint32 InTimerId);

private:
	uint32 TimerIndex = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
