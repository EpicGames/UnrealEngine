// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

// TraceInsightsCore
#include "InsightsCore/Common/SimpleRtti.h"

// TraceInsights
#include "Insights/ViewModels/ITimingEvent.h"
#include "Insights/ViewModels/TimingEventSearch.h"

#define UE_API TRACEINSIGHTS_API

class FBaseTimingTrack;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingEvent : public ITimingEvent
{
	INSIGHTS_DECLARE_RTTI(FTimingEvent, ITimingEvent, UE_API)

public:
	FTimingEvent(const TSharedRef<const FBaseTimingTrack> InTrack, double InStartTime, double InEndTime, uint32 InDepth, uint64 InType = uint64(-1))
		: Track(InTrack)
		, StartTime(InStartTime)
		, EndTime(InEndTime)
		, ExclusiveTime(0.0)
		, Depth(InDepth)
		, Type(InType)
	{}

	virtual ~FTimingEvent() = default;

	FTimingEvent(const FTimingEvent&) = default;
	FTimingEvent& operator=(const FTimingEvent&) = default;

	FTimingEvent(FTimingEvent&&) = default;
	FTimingEvent& operator=(FTimingEvent&&) = default;

	//////////////////////////////////////////////////
	// ITimingEvent interface

	virtual const TSharedRef<const FBaseTimingTrack> GetTrack() const override { return Track; }

	virtual uint32 GetDepth() const override { return Depth; }

	virtual double GetStartTime() const override { return StartTime; }
	virtual double GetEndTime() const override { return EndTime; }
	virtual double GetDuration() const override { return EndTime - StartTime; }

	virtual bool Equals(const ITimingEvent& Other) const override
	{
		if (GetTypeName() != Other.GetTypeName())
		{
			return false;
		}

		const FTimingEvent& OtherTimingEvent = Other.As<FTimingEvent>();
		return Track == Other.GetTrack()
			&& Depth == OtherTimingEvent.GetDepth()
			&& StartTime == OtherTimingEvent.GetStartTime()
			&& EndTime == OtherTimingEvent.GetEndTime();
	}

	//////////////////////////////////////////////////

	FTimingEventSearchHandle& GetSearchHandle() const { return SearchHandle; }

	double GetExclusiveTime() const { return ExclusiveTime; }
	void SetExclusiveTime(double InExclusiveTime) { ExclusiveTime = InExclusiveTime; }
	bool IsExclusiveTimeComputed() const { return bIsExclusiveTimeComputed; }
	void SetIsExclusiveTimeComputed(bool InIsExclusiveTime) { bIsExclusiveTimeComputed = InIsExclusiveTime; }

	uint64 GetType() const { return Type; }

	static UE_API uint32 ComputeEventColor(uint32 Id);
	static UE_API uint32 ComputeEventColor(const TCHAR* Str);
	static UE_API uint32 ComputeEventColor(const TCHAR* Str, TConstArrayView<uint8> Metadata);

protected:
	void SetType(uint64 InType) { Type = InType; }

private:
	// The track this timing event is contained within
	TSharedRef<const FBaseTimingTrack> Track;

	// Handle to a previous search, used to accelerate access to underlying event data
	mutable FTimingEventSearchHandle SearchHandle;

	// The start time of the event
	double StartTime;

	// The end time of the event
	double EndTime;

	// For hierarchical events, the cached exclusive time
	double ExclusiveTime;

	// A flag used to avoid recomputing the exclusive time
	bool bIsExclusiveTimeComputed = false;

	// The depth of the event
	uint32 Depth;

	// The event type (category, group id, etc.).
	uint64 Type;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FAcceptNoneTimingEventFilter : public ITimingEventFilter
{
	INSIGHTS_DECLARE_RTTI(FAcceptNoneTimingEventFilter, ITimingEventFilter, UE_API)

public:
	FAcceptNoneTimingEventFilter() {}
	virtual ~FAcceptNoneTimingEventFilter() = default;

	virtual bool FilterTrack(const FBaseTimingTrack& InTrack) const override { return false; }
	virtual bool FilterEvent(const ITimingEvent& InEvent) const override { return false; }
	virtual bool FilterEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName, uint64 InEventType = 0, uint32 InEventColor = 0) const override { return false; }
	virtual uint32 GetChangeNumber() const override { return 0; }
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FAcceptAllTimingEventFilter : public ITimingEventFilter
{
	INSIGHTS_DECLARE_RTTI(FAcceptAllTimingEventFilter, ITimingEventFilter, UE_API)

public:
	FAcceptAllTimingEventFilter() {}
	virtual ~FAcceptAllTimingEventFilter() = default;

	virtual bool FilterTrack(const FBaseTimingTrack& InTrack) const override { return true; }
	virtual bool FilterEvent(const ITimingEvent& InEvent) const override { return true; }
	virtual bool FilterEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName, uint64 InEventType = 0, uint32 InEventColor = 0) const override { return true; }
	virtual uint32 GetChangeNumber() const override { return 0; }
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FAggregatedTimingEventFilter : public ITimingEventFilter
{
	INSIGHTS_DECLARE_RTTI(FAggregatedTimingEventFilter, ITimingEventFilter, UE_API)

public:
	FAggregatedTimingEventFilter() : ChangeNumber(0) {}
	virtual ~FAggregatedTimingEventFilter() = default;

	//////////////////////////////////////////////////
	// ITimingEventFilter interface

	//virtual bool FilterTrack(const FBaseTimingTrack& InTrack) const = 0;
	//virtual bool FilterEvent(const ITimingEvent& InEvent) const = 0;
	//virtual bool FilterEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName = nullptr, uint64 InEventType = 0, uint32 InEventColor = 0) const = 0;
	virtual uint32 GetChangeNumber() const override { return ChangeNumber; }

	//////////////////////////////////////////////////

	const TArray<TSharedPtr<ITimingEventFilter>> GetChildren() const { return Children; }
	void AddChild(TSharedPtr<ITimingEventFilter> InFilter) { Children.Add(InFilter); ChangeNumber++; }

protected:
	uint32 ChangeNumber;
	TArray<TSharedPtr<ITimingEventFilter>> Children;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FAllAggregatedTimingEventFilter : public FAggregatedTimingEventFilter
{
	INSIGHTS_DECLARE_RTTI(FAllAggregatedTimingEventFilter, FAggregatedTimingEventFilter, UE_API)

public:
	FAllAggregatedTimingEventFilter() {}
	virtual ~FAllAggregatedTimingEventFilter() = default;

	//////////////////////////////////////////////////
	// ITimingEventFilter interface

	virtual bool FilterTrack(const FBaseTimingTrack& InTrack) const override
	{
		for (TSharedPtr<ITimingEventFilter> Filter : Children)
		{
			if (!FilterTrack(InTrack))
			{
				return false;
			}
		}
		return true;
	}

	virtual bool FilterEvent(const ITimingEvent& InEvent) const override
	{
		for (TSharedPtr<ITimingEventFilter> Filter : Children)
		{
			if (FilterEvent(InEvent))
			{
				return false;
			}
		}
		return true;
	}

	virtual bool FilterEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName, uint64 InEventType = 0, uint32 InEventColor = 0) const override
	{
		for (TSharedPtr<ITimingEventFilter> Filter : Children)
		{
			if (!FilterEvent(InEventStartTime, InEventEndTime, InEventDepth, InEventName, InEventType, InEventColor))
			{
				return false;
			}
		}
		return true;
	}

	//////////////////////////////////////////////////
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FAnyAggregatedTimingEventFilter : public FAggregatedTimingEventFilter
{
	INSIGHTS_DECLARE_RTTI(FAnyAggregatedTimingEventFilter, FAggregatedTimingEventFilter, UE_API)

public:
	FAnyAggregatedTimingEventFilter() {}
	virtual ~FAnyAggregatedTimingEventFilter() = default;

	//////////////////////////////////////////////////
	// ITimingEventFilter interface

	virtual bool FilterTrack(const FBaseTimingTrack& InTrack) const override
	{
		for (TSharedPtr<ITimingEventFilter> Filter : Children)
		{
			if (FilterTrack(InTrack))
			{
				return true;
			}
		}
		return false;
	}

	virtual bool FilterEvent(const ITimingEvent& InEvent) const override
	{
		for (TSharedPtr<ITimingEventFilter> Filter : Children)
		{
			if (FilterEvent(InEvent))
			{
				return true;
			}
		}
		return false;
	}

	virtual bool FilterEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName, uint64 InEventType = 0, uint32 InEventColor = 0) const override
	{
		for (TSharedPtr<ITimingEventFilter> Filter : Children)
		{
			if (FilterEvent(InEventStartTime, InEventEndTime, InEventDepth, InEventName, InEventType, InEventColor))
			{
				return true;
			}
		}
		return false;
	}

	//////////////////////////////////////////////////
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingEventFilter : public ITimingEventFilter
{
	INSIGHTS_DECLARE_RTTI(FTimingEventFilter, ITimingEventFilter, UE_API)

public:
	FTimingEventFilter()
		: bFilterByTrackTypeName(false)
		, TrackTypeName(NAME_None)
		, bFilterByTrackInstance(false)
		, TrackInstance(nullptr)
		, ChangeNumber(0)
	{}

	virtual ~FTimingEventFilter() = default;

	FTimingEventFilter(const FTimingEventFilter&) = default;
	FTimingEventFilter& operator=(const FTimingEventFilter&) = default;

	FTimingEventFilter(FTimingEventFilter&&) = default;
	FTimingEventFilter& operator=(FTimingEventFilter&&) = default;

	//////////////////////////////////////////////////
	// ITimingEventFilter interface

	UE_API virtual bool FilterTrack(const FBaseTimingTrack& InTrack) const override;
	virtual bool FilterEvent(const ITimingEvent& InEvent) const override { return true; }
	virtual bool FilterEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName, uint64 InEventType = 0, uint32 InEventColor = 0) const override { return true; }
	virtual uint32 GetChangeNumber() const override { return ChangeNumber; }

	//////////////////////////////////////////////////

	bool IsFilteringByTrackTypeName() const { return bFilterByTrackTypeName; }
	const FName& GetTrackTypeName() const { return TrackTypeName; }
	void SetFilterByTrackTypeName(bool bInFilterByTrackTypeName) { if (bFilterByTrackTypeName != bInFilterByTrackTypeName) { bFilterByTrackTypeName = bInFilterByTrackTypeName; ChangeNumber++; } }
	void SetTrackTypeName(const FName InTrackTypeName) { if (TrackTypeName != InTrackTypeName) { TrackTypeName = InTrackTypeName; ChangeNumber++; } }

	bool IsFilteringByTrackInstance() const { return bFilterByTrackInstance; }
	TSharedPtr<FBaseTimingTrack> GetTrackInstance() const { return TrackInstance; }
	void SetFilterByTrackInstance(bool bInFilterByTrackInstance) { if (bFilterByTrackInstance != bInFilterByTrackInstance) { bFilterByTrackInstance = bInFilterByTrackInstance; ChangeNumber++; } }
	void SetTrackInstance(TSharedPtr<FBaseTimingTrack> InTrackInstance) { if (TrackInstance != InTrackInstance) { TrackInstance = InTrackInstance; ChangeNumber++; } }

	//////////////////////////////////////////////////

protected:
	bool bFilterByTrackTypeName;
	FName TrackTypeName;

	bool bFilterByTrackInstance;
	TSharedPtr<FBaseTimingTrack> TrackInstance;

	uint32 ChangeNumber;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingEventFilterByMinDuration : public FTimingEventFilter
{
	INSIGHTS_DECLARE_RTTI(FTimingEventFilterByMinDuration, FTimingEventFilter, UE_API)

public:
	FTimingEventFilterByMinDuration(double InMinDuration) : MinDuration(InMinDuration) {}
	virtual ~FTimingEventFilterByMinDuration() = default;

	//////////////////////////////////////////////////
	// ITimingEventFilter interface

	virtual bool FilterEvent(const ITimingEvent& InEvent) const override
	{
		return InEvent.GetDuration() >= MinDuration;
	}

	virtual bool FilterEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName, uint64 InEventType = 0, uint32 InEventColor = 0) const override
	{
		return InEventEndTime - InEventStartTime >= MinDuration;
	}

	//////////////////////////////////////////////////

	double GetMinDuration() const { return MinDuration; }
	void SetMinDuration(double InMinDuration) { if (MinDuration != InMinDuration) { MinDuration = InMinDuration; ChangeNumber++; } }

private:
	double MinDuration;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingEventFilterByMaxDuration : public FTimingEventFilter
{
	INSIGHTS_DECLARE_RTTI(FTimingEventFilterByMaxDuration, FTimingEventFilter, UE_API)

public:
	FTimingEventFilterByMaxDuration(double InMaxDuration) : MaxDuration(InMaxDuration) {}
	virtual ~FTimingEventFilterByMaxDuration() = default;

	//////////////////////////////////////////////////
	// ITimingEventFilter interface

	virtual bool FilterEvent(const ITimingEvent& InEvent) const override
	{
		return InEvent.GetDuration() <= MaxDuration;
	}

	virtual bool FilterEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName, uint64 InEventType = 0, uint32 InEventColor = 0) const override
	{
		return InEventEndTime - InEventStartTime <= MaxDuration;
	}

	//////////////////////////////////////////////////

	double GetMaxDuration() const { return MaxDuration; }
	void SetMaxDuration(double InMaxDuration) { if (MaxDuration != InMaxDuration) { MaxDuration = InMaxDuration; ChangeNumber++; } }

private:
	double MaxDuration;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingEventFilterByEventType : public FTimingEventFilter
{
	INSIGHTS_DECLARE_RTTI(FTimingEventFilterByEventType, FTimingEventFilter, UE_API)

public:
	FTimingEventFilterByEventType(uint64 InEventType) : EventType(InEventType) {}
	virtual ~FTimingEventFilterByEventType() = default;

	//////////////////////////////////////////////////
	// ITimingEventFilter interface

	virtual bool FilterEvent(const ITimingEvent& InEvent) const override
	{
		if (InEvent.Is<FTimingEvent>())
		{
			const FTimingEvent& Event = InEvent.As<FTimingEvent>();
			return Event.GetType() == EventType;
		}
		return false;
	}

	virtual bool FilterEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName, uint64 InEventType = 0, uint32 InEventColor = 0) const override
	{
		return InEventType == EventType;
	}

	//////////////////////////////////////////////////

	uint64 GetEventType() const { return EventType; }
	void SetEventType(uint64 InEventType) { if (EventType != InEventType) { EventType = InEventType; ChangeNumber++; } }

private:
	uint64 EventType;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingEventFilterByFrameIndex : public FTimingEventFilter
{
	INSIGHTS_DECLARE_RTTI(FTimingEventFilterByFrameIndex, FTimingEventFilter, UE_API)

public:
	FTimingEventFilterByFrameIndex(uint64 InFrameIndex) : FrameIndex(InFrameIndex) {}
	virtual ~FTimingEventFilterByFrameIndex() = default;

	//////////////////////////////////////////////////
	// ITimingEventFilter interface

	virtual bool FilterEvent(const ITimingEvent& InEvent) const override
	{
		if (InEvent.Is<FTimingEvent>())
		{
			const FTimingEvent& Event = InEvent.As<FTimingEvent>();
			return Event.GetType() == FrameIndex;
		}
		return false;
	}

	virtual bool FilterEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName, uint64 InFrameIndex = 0, uint32 InEventColor = 0) const override
	{
		return InFrameIndex == FrameIndex;
	}

	//////////////////////////////////////////////////

	uint64 GetFrameIndex() const { return FrameIndex; }
	void SetFrameIndex(uint64 InFrameIndex) { if (FrameIndex != InFrameIndex) { FrameIndex = InFrameIndex; ChangeNumber++; } }

private:
	uint64 FrameIndex;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
