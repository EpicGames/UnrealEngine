// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Map.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/SharedPointer.h"

namespace UE::Insights
{
	class IFilterExecutor;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Search behavior flags
enum class ETimingEventSearchFlags : int32
{
	// Search all matches
	None = 0,
	SearchAll = None,

	// Whether to stop at the first match
	StopAtFirstMatch = (1 << 0),
};
ENUM_CLASS_FLAGS(ETimingEventSearchFlags);

////////////////////////////////////////////////////////////////////////////////////////////////////
// A handle to a timing event that was previously searched
struct FTimingEventSearchHandle
{
	bool IsValid() const { return Id != uint64(-1); }
	void Reset() { Id = uint64(-1); }
	bool operator==(const FTimingEventSearchHandle& InOther) const
	{
		return Id == InOther.Id;
	}

	static FTimingEventSearchHandle GenerateHandle()
	{
		static uint64 GlobalId = 0;

		FTimingEventSearchHandle ReturnValue;
		ReturnValue.Id = GlobalId++;
		return ReturnValue;
	}

private:
	uint64 Id = uint64(-1);
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// Parameters for a timing event search
class FTimingEventSearchParameters
{
public:
	// Predicate called to filter event matches.
	// Note it is assumed that the SearchPredicate (below) will perform basic range checks on StartTime and EndTime,
	// therefore this is for any other filtering that needs to take place (e.g. left of an event, below an event etc.)
	// Returns true to pass the filter.
	typedef TFunctionRef<bool(double /*StartTime*/, double /*EndTime*/, uint32 /*Depth*/)> EventFilterPredicate;

	// Predicate called when we get a match
	typedef TFunctionRef<void(double /*StartTime*/, double /*EndTime*/, uint32 /*Depth*/)> EventMatchedPredicate;

	enum class ESearchDirection : uint32
	{
		Forward = 0,
		Backward = 1,
	};

	FTimingEventSearchParameters(double InStartTime, double InEndTime, ETimingEventSearchFlags Flags, EventFilterPredicate InEventFilter = NoFilter, EventMatchedPredicate InEventMatched = NoMatch)
		: EventFilter(InEventFilter)
		, EventMatched(InEventMatched)
		, SearchHandle(nullptr)
		, StartTime(InStartTime)
		, EndTime(InEndTime)
		, Flags(Flags)
	{}

private:
	// Default predicates
	static bool NoFilter(double, double, uint32) { return true; }
	static void NoMatch(double, double, uint32) {}

public:
	// Predicate for search filtering
	EventFilterPredicate EventFilter;

	// Predicate called when we get a match
	EventMatchedPredicate EventMatched;

	// A handle to a previous search. This will be written during a search if a cache is utilized and a hit was not generated.
	FTimingEventSearchHandle* SearchHandle;

	// Start time of the search
	double StartTime;

	// End time of the search
	double EndTime;

	// Search behavior flags
	ETimingEventSearchFlags Flags;

	TSharedPtr<UE::Insights::IFilterExecutor> FilterExecutor;

	ESearchDirection SearchDirection = ESearchDirection::Forward;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// Simple acceleration structure used to return previously searched results
template <typename PayloadType, int32 Size = 3>
struct TTimingEventSearchCache
{
public:
	TTimingEventSearchCache()
		: WriteIndex(0)
	{}

	// Write to the cache
	// @return a handle to the search
	FTimingEventSearchHandle Write(double InStartTime, double InEndTime, uint32 InDepth, const PayloadType& InPayload)
	{
		TPair<FTimingEventSearchHandle, FResultData>& WritePair = CachedValues[WriteIndex];

		WritePair.Key = FTimingEventSearchHandle::GenerateHandle();
		WritePair.Value.Payload = InPayload;
		WritePair.Value.StartTime = InStartTime;
		WritePair.Value.EndTime = InEndTime;
		WritePair.Value.Depth = InDepth;

		WriteIndex = (WriteIndex + 1) % Size;

		return WritePair.Key;
	}

	// Attempt to read from the cache.
	// @return false if the value was not found
	bool Read(const FTimingEventSearchHandle& InHandle, double& OutStartTime, double& OutEndTime, uint32& OutDepth, PayloadType& OutPayload)
	{
		static_assert(Size > 1, "TTimingEventSearchCache only works with caches that have multiple values");

		if (InHandle.IsValid())
		{
			// Start read at the previous write, to catch most recently written values
			for (int32 ReadIndex = (WriteIndex + (Size - 1)) % Size, ReadCount = 0; ReadCount < Size; ReadCount++, ReadIndex = (ReadIndex + 1) % Size)
			{
				check(ReadIndex >= 0 && ReadIndex < Size);

				const TPair<FTimingEventSearchHandle, FResultData>& CachedPair = CachedValues[ReadIndex];
				if (InHandle == CachedPair.Key)
				{
					OutStartTime = CachedPair.Value.StartTime;
					OutEndTime = CachedPair.Value.EndTime;
					OutDepth = CachedPair.Value.Depth;
					OutPayload = CachedPair.Value.Payload;
					return true;
				}
			}
		}

		return false;
	}

	// Reset the cache. Any reads after this will return false
	void Reset()
	{
		for (TPair<FTimingEventSearchHandle, FResultData>& CachedPair : CachedValues)
		{
			CachedPair.Key.Reset();
		}
	}

private:
	// Current write index
	int32 WriteIndex;

	// A cached result
	struct FResultData
	{
		PayloadType Payload;
		double StartTime;
		double EndTime;
		uint32 Depth;
	};

	// Array of cached searches
	TPair<FTimingEventSearchHandle, FResultData> CachedValues[Size];
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// Helper used to orchestrate a search of a timing event track's events
// Example of usage:
//
//		FMyStruct MatchedPayload;
//
//		TTimingEventSearch<FMyStruct>::Search(
//		Parameters,
//		[](const TTimingEventSearch<FMyStruct>::FContext& InContext)
//		{
//			for (FMyStruct& Payload : Payloads)
//			{
//				InContext.Check(InEventStartTime, InEventEndTime, 0, Payload);
//			}
//		},
//		[&MatchedPayload](double InStartTime, double InEndTime, uint32 InDepth, const FMyStruct& InEvent)
//		{
//			MatchedPayload = InEvent;
//		},
//		[&MatchedPayload](double InStartTime, double InEndTime, uint32 InDepth)
//		{
//			// Do something with MatchedPayload, e.g. call a captured lambda.
//		});
//
template <typename PayloadType>
struct TTimingEventSearch
{
public:
	struct FContext;

	// Predicate called when a match has been made to the search parameters
	typedef TFunctionRef<void(double /*InStartTime*/, double /*InEndTime*/, uint32 /*InDepth*/, const PayloadType& /*InPayload*/)> PayloadMatchedPredicate;

	// Predicate called to filter by payload contents
	// Return true to pass the filer
	typedef TFunctionRef<bool(double /*InStartTime*/, double /*InEndTime*/, uint32 /*InDepth*/, const PayloadType& /*InPayload*/)> PayloadFilterPredicate;

	// Predicate called when a match has been found
	typedef TFunctionRef<void(double /*InStartTime*/, double /*InEndTime*/, uint32 /*InDepth*/, const PayloadType& /*InPayload*/)> FoundPredicate;

	// Predicate called to run the search, e.g. iterate over an array of events
	// It is expected to call FContext::Check on each valid searched event.
	typedef TFunctionRef<void(FContext& /*Context*/)> SearchPredicate;

	// Context used to operate a search.
	struct FContext
	{
	public:
		FContext(const FTimingEventSearchParameters& InParameters, PayloadFilterPredicate InPayloadFilterPredicate, PayloadMatchedPredicate InPayloadMatchedPredicate)
			: Parameters(InParameters)
			, PayloadMatched(InPayloadMatchedPredicate)
			, PayloadFilter(InPayloadFilterPredicate)
			, FoundStartTime(-1.0)
			, FoundEndTime(-1.0)
			, FoundDepth(0)
			, bFound(false)
			, bContinueSearching(true)
		{
		}

		// Function called to check and potentially match an event
		void Check(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const PayloadType& InEvent)
		{
			if (bContinueSearching && Parameters.EventFilter(InEventStartTime, InEventEndTime, InEventDepth) && PayloadFilter(InEventStartTime, InEventEndTime, InEventDepth, InEvent))
			{
				Parameters.EventMatched(InEventStartTime, InEventEndTime, InEventDepth);
				PayloadMatched(InEventStartTime, InEventEndTime, InEventDepth, InEvent);

				FoundPayload = InEvent;
				FoundDepth = InEventDepth;
				FoundStartTime = InEventStartTime;
				FoundEndTime = InEventEndTime;

				bFound = true;
				bContinueSearching = (Parameters.Flags & ETimingEventSearchFlags::StopAtFirstMatch) == ETimingEventSearchFlags::None;
			}
		}

		// Access the search parameters
		const FTimingEventSearchParameters& GetParameters() const { return Parameters; }

		// Accessors for read-only results
		const PayloadType& GetPayloadFound() const { return FoundPayload; }
		double GetStartTimeFound() const { return FoundStartTime; }
		double GetEndTimeFound() const { return FoundEndTime; }
		uint32 GetDepthFound() const { return FoundDepth; }
		bool IsMatchFound() const { return bFound; }
		bool ShouldContinueSearching() const { return bContinueSearching; }

		// Allows search to be aborted by predicates
		void AbortSearch() { bContinueSearching = false; }

	private:
		// Search parameters
		FTimingEventSearchParameters Parameters;

		// Predicate called when an event was matched
		PayloadMatchedPredicate PayloadMatched;

		// Filter applied to payloads
		PayloadFilterPredicate PayloadFilter;

		// The payload we have found
		PayloadType FoundPayload;

		// The start time of the event that was found
		double FoundStartTime;

		// The end time of the event that was found
		double FoundEndTime;

		// The depth of the event that was found
		uint32 FoundDepth;

		// Whether a match was found
		bool bFound;

		// Internal flag to skip work if a match was found
		bool bContinueSearching;
	};

public:
	// Default predicates
	static bool NoFilter(double, double, uint32, const PayloadType&) { return true; }
	static void NoMatch(double, double, uint32, const PayloadType&) {}

	// Search using only the event filter and no match predicate
	static bool Search(const FTimingEventSearchParameters& InParameters, SearchPredicate InSearchPredicate, FoundPredicate InFoundPredicate)
	{
		return Search(InParameters, InSearchPredicate, NoFilter, InFoundPredicate, NoMatch);
	}

	// Search using only the event filter, no match predicate and a cache
	static bool Search(const FTimingEventSearchParameters& InParameters, SearchPredicate InSearchPredicate, FoundPredicate InFoundPredicate, TTimingEventSearchCache<PayloadType>& InCache)
	{
		return Search(InParameters, InSearchPredicate, NoFilter, InFoundPredicate, NoMatch, &InCache);
	}

	// Search using a specific payload filter
	static bool Search(const FTimingEventSearchParameters& InParameters, SearchPredicate InSearchPredicate, PayloadFilterPredicate InFilterPredicate, FoundPredicate InFoundPredicate, PayloadMatchedPredicate InPayloadMatchedPredicate, TTimingEventSearchCache<PayloadType>* InCache = nullptr)
	{
		if (InCache != nullptr && InParameters.SearchHandle != nullptr)
		{
			double StartTime;
			double EndTime;
			uint32 Depth;
			PayloadType Payload;
			if (InCache->Read(*InParameters.SearchHandle, StartTime, EndTime, Depth, Payload))
			{
				InParameters.EventMatched(StartTime, EndTime, Depth);
				InPayloadMatchedPredicate(StartTime, EndTime, Depth, Payload);
				InFoundPredicate(StartTime, EndTime, Depth, Payload);
				return true;
			}
		}

		FContext Context(InParameters, InFilterPredicate, InPayloadMatchedPredicate);

		InSearchPredicate(Context);

		if (Context.IsMatchFound())
		{
			InFoundPredicate(Context.GetStartTimeFound(), Context.GetEndTimeFound(), Context.GetDepthFound(), Context.GetPayloadFound());

			if (InCache != nullptr && InParameters.SearchHandle != nullptr)
			{
				*InParameters.SearchHandle = InCache->Write(Context.GetStartTimeFound(), Context.GetEndTimeFound(), Context.GetDepthFound(), Context.GetPayloadFound());
			}
		}

		return Context.IsMatchFound();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////
