// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Fonts/SlateFontInfo.h"
#include "Templates/SharedPointer.h"

// TraceInsights
#include "Insights/ViewModels/BaseTimingTrack.h"

#define UE_API TRACEINSIGHTS_API

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FSlateBrush;

class ITimingEventsTrackDrawStateBuilder
{
public:
	typedef TFunctionRef<const FString(float /*AvailableWidth*/)> GetEventNameCallback;

public:
	virtual ~ITimingEventsTrackDrawStateBuilder() = default;

	virtual void AddEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName, uint64 InEventType = 0, uint32 InEventColor = 0) = 0;
	virtual void AddEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, uint32 InEventColor, GetEventNameCallback InGetEventNameCallback) = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingEventsTrack : public FBaseTimingTrack
{
	INSIGHTS_DECLARE_RTTI(FTimingEventsTrack, FBaseTimingTrack, UE_API)

public:
	UE_API explicit FTimingEventsTrack();
	UE_API explicit FTimingEventsTrack(const FString& InTrackName);
	UE_API virtual ~FTimingEventsTrack();

	//////////////////////////////////////////////////
	// FBaseTimingTrack

	UE_API virtual void Reset() override;

	UE_API virtual void PreUpdate(const ITimingTrackUpdateContext& Context) override;
	UE_API virtual void PostUpdate(const ITimingTrackUpdateContext& Context) override;

	UE_API virtual void Draw(const ITimingTrackDrawContext& Context) const override;
	UE_API virtual void PostDraw(const ITimingTrackDrawContext& Context) const override;
	UE_API virtual void DrawEvent(const ITimingTrackDrawContext& Context, const ITimingEvent& InTimingEvent, EDrawEventMode InDrawMode) const override;

	UE_API virtual const TSharedPtr<const ITimingEvent> GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const override;

	UE_API virtual TSharedPtr<ITimingEventFilter> GetFilterByEvent(const TSharedPtr<const ITimingEvent> InTimingEvent) const override;

	//////////////////////////////////////////////////

	virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) = 0;
	virtual void BuildFilteredDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) {}

protected:
	int32 GetNumLanes() const { return NumLanes; }
	void SetNumLanes(int32 InNumLanes) { NumLanes = InNumLanes; }

	const struct FTimingEventsTrackDrawState& GetDrawState() const { return *DrawState; }
	const struct FTimingEventsTrackDrawState& GetFilteredDrawState() const { return *FilteredDrawState; }

	float GetFilteredDrawStateOpacity() const { return FilteredDrawStateInfo.Opacity; }
	bool UpdateFilteredDrawStateOpacity() const
	{
		if (FilteredDrawStateInfo.Opacity == 1.0f)
		{
			return true;
		}
		else
		{
			FilteredDrawStateInfo.Opacity = FMath::Min(1.0f, FilteredDrawStateInfo.Opacity + 0.05f);
			return false;
		}
	}

	UE_API void UpdateTrackHeight(const ITimingTrackUpdateContext& Context);

	UE_API void DrawEvents(const ITimingTrackDrawContext& Context, const float OffsetY = 1.0f) const;
	UE_API void DrawHeader(const ITimingTrackDrawContext& Context) const;

	UE_API void DrawMarkers(const ITimingTrackDrawContext& Context, float LineY, float LineH) const;

	UE_API void DrawSelectedEventInfo(const FString& InText, const FTimingTrackViewport& Viewport, const UE::Insights::FDrawContext& DrawContext, const FSlateBrush* WhiteBrush, const FSlateFontInfo& Font) const;
	UE_API void DrawSelectedEventInfoEx(const FString& InText, const FString& InLeftText, const FString& InTopText, const FTimingTrackViewport& Viewport, const UE::Insights::FDrawContext& DrawContext, const FSlateBrush* WhiteBrush, const FSlateFontInfo& Font) const;

	UE_API int32 GetHeaderBackgroundLayerId(const ITimingTrackDrawContext& Context) const;
	UE_API int32 GetHeaderTextLayerId(const ITimingTrackDrawContext& Context) const;

	UE_API virtual const TSharedPtr<const ITimingEvent> GetEvent(double InTime, double SecondsPerPixel, int32 Depth) const;

	virtual bool HasCustomFilter() const { return false; }

	/* Can be overridden to force a max depth for the track. */
	virtual int32 GetMaxDepth() const { return -1; }

private:
	int32 NumLanes; // number of lanes (sub-tracks)
	TSharedRef<struct FTimingEventsTrackDrawState> DrawState;
	TSharedRef<struct FTimingEventsTrackDrawState> FilteredDrawState;

	struct FFilteredDrawStateInfo
	{
		double ViewportStartTime = 0.0;
		double ViewportScaleX = 0.0;
		double LastBuildDuration = 0.0;
		TWeakPtr<ITimingEventFilter> LastEventFilter;
		uint32 LastFilterChangeNumber = 0;
		uint32 Counter = 0;
		mutable float Opacity = 0.0f;
	};
	FFilteredDrawStateInfo FilteredDrawStateInfo;

public:
	static UE_API bool bUseDownSampling; // toggle to enable/disable downsampling, for debugging purposes only
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
