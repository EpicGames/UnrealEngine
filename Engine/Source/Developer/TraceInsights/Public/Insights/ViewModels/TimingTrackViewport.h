// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// TraceInsights
#include "Insights/ViewModels/TimingViewLayout.h"

#define UE_API TRACEINSIGHTS_API

class SScrollBar;

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class ETimingTrackViewportDirtyFlags : uint32
{
	None = 0,

	//////////////////////////////////////////////////
	// Horizontal viewport flags

	/** Width has changed. */
	HSizeChanged = (1 << 0),

	/** StartTime has changed. */
	HPositionChanged = (1 << 1),

	/** ScaleX has changed. */
	HScaleChanged = (1 << 2),

	/** Intersection between valid session time ([StartSessionTime - EndSessionTime) interval) and viewport time ([StartTime - EndTime) interval) has changed. */
	HClippedSessionTimeChanged = (1 << 3),

	/** Misc horizontal viewport invalidation flag. */
	HInvalidated = (1 << 4),

	//////////////////////////////////////////////////
	// Vertical viewport flags

	/** Height has changed. */
	VSizeChanged = (1 << 5),

	/** ScrollPosY has changed. */
	VPositionChanged = (1 << 6),

	/** Layout has changed. */
	VLayoutChanged = (1 << 7),

	/** Misc vertical viewport invalidation flag. */
	VInvalidated = (1 << 8),

	//////////////////////////////////////////////////

	AllHorizontal = HSizeChanged | HPositionChanged | HScaleChanged | HClippedSessionTimeChanged | HInvalidated,
	AllVertical = VSizeChanged | VPositionChanged | VLayoutChanged | VInvalidated,
	All = AllHorizontal | AllVertical
};
ENUM_CLASS_FLAGS(ETimingTrackViewportDirtyFlags);

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingTrackViewport
{
public:
	FTimingTrackViewport()
	{
		Reset();
	}

	void Reset()
	{
		Width = 0.0f;
		Height = 0.0f;

		MinValidTime = 0.0;
		MaxValidTime = 0.0;

		StartTime = 0.0;
		EndTime = 0.0;

		MinScaleX = (5 * 20) / 3600.0; // 1h between major tick marks
		MaxScaleX = 1.0E10; // 10ns between major tick marks
		ScaleX = (5 * 20) / 5.0; // 5s between major tick marks

		PosY = 0.0f;
		TopOffset = 0.0f;
		BottomOffset = 0.0f;
		ScrollHeight = 1.0f;
		ScrollPosY = 0.0f;
		Layout.ForceNormalMode();

		DirtyFlags = ETimingTrackViewportDirtyFlags::All;
	}

	/** Width of viewport, in pixels. [px] */
	float GetWidth() const { return Width; }

	/** Height of viewport, in pixels. [px] */
	float GetHeight() const { return Height; }

	UE_API bool UpdateSize(const float InWidth, const float InHeight);

	/** Minimum session time, in seconds. [s] */
	double GetMinValidTime() const { return MinValidTime; }
	/** Maximum session time, in seconds. [s] */
	double GetMaxValidTime() const { return MaxValidTime; }
	void SetMaxValidTime(const double InMaxValidTime) { MaxValidTime = InMaxValidTime; }

	/**
	  * Time of viewport's left side, in seconds. [s]
	  */
	double GetStartTime() const { return StartTime; }
	/**
	  * Time of viewport's right side, in seconds. [s]
	  * Computed when StartTime, ScaleX or Width changes.
	  */
	double GetEndTime() const { return EndTime; }
	double GetDuration() const { return EndTime - StartTime; }

	double GetMinScaleX() const { return MinScaleX; }
	double GetMaxScaleX() const { return MaxScaleX; }

	/** Current scale factor between seconds and pixels (Slate units). [px/s] */
	double GetScaleX() const { return ScaleX; }

	//////////////////////////////////////////////////
	// Conversions

	float TimeToSlateUnits(const double Time) const
	{
		return static_cast<float>((Time - StartTime) * ScaleX);
	}

	float TimeToSlateUnitsRounded(const double Time) const
	{
		return static_cast<float>(FMath::RoundToDouble((Time - StartTime) * ScaleX));
	}

	float GetViewportDXForDuration(const double DT) const
	{
		return static_cast<float>(DT * ScaleX);
	}

	double GetDurationForViewportDX(const float DX) const
	{
		return static_cast<double>(DX) / ScaleX;
	}

	double SlateUnitsToTime(const float X) const
	{
		return StartTime + static_cast<double>(X) / ScaleX;
	}

	double AlignTimeToPixel(const double InTime, const double InScaleX) const
	{
		return FMath::RoundToDouble(InTime * InScaleX) / InScaleX;
	}

	double AlignTimeToPixel(const double Time) const
	{
		return AlignTimeToPixel(Time, ScaleX);
	}

	UE_API double RestrictEndTime(const double InEndTime) const;
	UE_API double RestrictDuration(const double InStartTime, const double InEndTime) const;

	//////////////////////////////////////////////////
	// Scrolling

	UE_API bool ScrollAtTime(const double Time);
	UE_API bool CenterOnTimeInterval(const double Time, const double Duration);

	UE_API void GetHorizontalScrollLimits(double& OutMinT, double& OutMaxT);
	UE_API bool EnforceHorizontalScrollLimits(const double U);

	//////////////////////////////////////////////////
	// Zooming

	UE_API bool ZoomOnTimeInterval(const double Time, const double Duration);
	UE_API bool RelativeZoomWithFixedX(const float Delta, const float X);
	UE_API bool ZoomWithFixedX(const double NewScaleX, const float X);
	UE_API bool SetScaleX(const double NewScaleX);

	//////////////////////////////////////////////////
	// Vertical axis

	/** The Y position where it starts the top docked tracks (i.e. height of the toolbar), in pixels. [px] */
	float GetPosY() const { return PosY; }
	void SetPosY(float InPosY) { PosY = InPosY; }

	/** Top offset (total height of the visible top docked tracks), in pixels. [px] */
	float GetTopOffset() const { return TopOffset; }
	void SetTopOffset(float InTopOffset) { TopOffset = InTopOffset; }
	
	/** Bottom offset (total height of the visible bottom docked tracks), in pixels. [px] */
	float GetBottomOffset() const { return BottomOffset; }
	void SetBottomOffset(float InBottomOffset) { BottomOffset = InBottomOffset; }

	float GetScrollableAreaHeight() const { return Height - TopOffset - BottomOffset; }

	/** Height of the vertical (virtual) scrollable area, in pixels. [px] */
	float GetScrollHeight() const { return ScrollHeight; }
	void SetScrollHeight(const float InScrollHeight) { ScrollHeight = InScrollHeight; }

	/** Current vertical scroll position, in pixels. [px] */
	float GetScrollPosY() const { return ScrollPosY; }
	void SetScrollPosY(const float InScrollPosY) { ScrollPosY = InScrollPosY; }

	float GetViewportY(const float Y) const { return PosY + TopOffset + Y - ScrollPosY; }

	const FTimingViewLayout& GetLayout() const { return Layout; }
	UE_API void UpdateLayout();

	bool IsLayoutCompactModeEnabled() const { return Layout.bIsCompactMode; }

	void SwitchLayoutCompactMode()
	{
		Layout.bIsCompactMode = !Layout.bIsCompactMode;
		AddDirtyFlags(ETimingTrackViewportDirtyFlags::VLayoutChanged);
	}

	void ToggleLayoutMinTrackHeight()
	{
		if (Layout.TargetMinTimelineH == 0.0f)
		{
			Layout.TargetMinTimelineH = FTimingViewLayout::RealMinTimelineH;
		}
		else
		{
			Layout.TargetMinTimelineH = 0.0f;
		}

		Layout.MinTimelineH = Layout.TargetMinTimelineH; // no layout animation
		AddDirtyFlags(ETimingTrackViewportDirtyFlags::VLayoutChanged);
	}

	//////////////////////////////////////////////////

	UE_API bool OnUserScrolled(TSharedPtr<SScrollBar> ScrollBar, float ScrollOffset);
	UE_API void UpdateScrollBar(TSharedPtr<SScrollBar> ScrollBar) const;

	UE_API bool OnUserScrolledY(TSharedPtr<SScrollBar> ScrollBar, float ScrollOffset);
	UE_API void UpdateScrollBarY(TSharedPtr<SScrollBar> ScrollBar) const;

	//////////////////////////////////////////////////

	ETimingTrackViewportDirtyFlags GetDirtyFlags() const { return DirtyFlags; }
	bool IsDirty() const { return DirtyFlags != ETimingTrackViewportDirtyFlags::None; }
	bool IsDirty(ETimingTrackViewportDirtyFlags InDirtyFlags) const { return EnumHasAnyFlags(DirtyFlags, InDirtyFlags); }
	bool IsHorizontalViewportDirty() const { return EnumHasAnyFlags(DirtyFlags, ETimingTrackViewportDirtyFlags::AllHorizontal); }
	bool IsVerticalViewportDirty() const { return EnumHasAnyFlags(DirtyFlags, ETimingTrackViewportDirtyFlags::AllVertical); }
	void AddDirtyFlags(ETimingTrackViewportDirtyFlags InFlags) { DirtyFlags |= InFlags; }
	void ResetDirtyFlags() { DirtyFlags = ETimingTrackViewportDirtyFlags::None; }

private:
	float Width; // width of viewport, in pixels; [px]
	float Height; // height of viewport, in pixels; [px]

	double MinValidTime; // min session time, in seconds; [s]
	double MaxValidTime; // max session time, in seconds; [s]

	double StartTime; // time of viewport's left side, in seconds; [s]
	double EndTime; // time of viewport's right side, in seconds; [s]; computed when StartTime, Scale or Width changes

	double MinScaleX; // min scale factor; [px/s]
	double MaxScaleX; // max scale factor; [px/s]
	double ScaleX; // scale factor between seconds and pixels; [px/s]

	float PosY; // position of top docked tracks == height of toolbar; [px]
	float TopOffset; // top offset, in pixels; [px]
	float BottomOffset; // bottom offset, in pixels; [px]
	float ScrollHeight; // height of the vertical scrollable area, in pixels; [px]
	float ScrollPosY; // current vertical scroll position, in pixels; [px]
	FTimingViewLayout Layout;

	ETimingTrackViewportDirtyFlags DirtyFlags;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
