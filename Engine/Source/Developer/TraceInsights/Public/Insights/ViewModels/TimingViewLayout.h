// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"

#define UE_API TRACEINSIGHTS_API

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FTimingViewLayout
{
	static constexpr float RealMinTimelineH = 7.0f;

	static constexpr float NormalLayoutEventH = 14.0f;
	static constexpr float NormalLayoutEventDY = 2.0f;
	static constexpr float NormalLayoutTimelineDY = 2.0f;
	static constexpr float NormalLayoutChildTimelineDY = 4.0f;
	static constexpr float NormalLayoutMinTimelineH = 0.0f;

	static constexpr float CompactLayoutEventH = 4.0f;
	static constexpr float CompactLayoutEventDY = 1.0f;
	static constexpr float CompactLayoutTimelineDY = 1.0f;
	static constexpr float CompactLayoutChildTimelineDY = 1.0f;
	static constexpr float CompactLayoutMinTimelineH = 0.0f;

	//////////////////////////////////////////////////

	bool bIsCompactMode;

	float EventH; // height of a timing event, in Slate units
	float EventDY; // vertical space between timing events in two adjacent lanes, in Slate units
	float TimelineDY; // space at top and bottom of each track (i.e. above first lane and below last lane), in Slate units
	float ChildTimelineDY; // space between a child track's lanes and the parent's lanes, in Slate units
	float MinTimelineH; // current minimum height of a track, in Slate units
	float TargetMinTimelineH; // targeted minimum height of a track (for animating min track height), in Slate units

	//////////////////////////////////////////////////

	float GetLaneY(int32 Depth) const
	{
		// 1.0f is for the top horizontal line of each track
		return 1.0f + TimelineDY + (EventDY + EventH) * (float)Depth;
	}

	//////////////////////////////////////////////////

	float GetChildLaneY(int32 Depth) const
	{
		return (EventDY + EventH) * (float)Depth;
	}

	/* The layout of a track:
	*	1.0f // the line between tracks
	*	TimelineDY
	*	[ChildTrack]
	*	[ChildTimelineDY] // if ChildTrack is valid and ChildTrack.Height > 0
	*	TrackLanes
	*	TimelineDY
	*/

	float ComputeTrackHeight(int32 NumLanes) const
	{
		if (NumLanes <= 0)
		{
			return MinTimelineH;
		}
		else //if (NumLanes > 0)
		{
			// 1.0f is for horizontal line between timelines
			float TrackHeight = 1.0f + TimelineDY + (EventH + EventDY) * (float)NumLanes - EventDY + TimelineDY;

			if (TrackHeight < FTimingViewLayout::RealMinTimelineH)
			{
				return FTimingViewLayout::RealMinTimelineH;
			}

			return TrackHeight;
		}
	}

	float ComputeChildTrackHeight(int32 NumLanes) const
	{
		if (NumLanes <= 0)
		{
			return MinTimelineH;
		}
		else //if (NumLanes > 0)
		{
			return (EventH + EventDY) * (float)NumLanes - EventDY;
		}
	}

	UE_API void ForceNormalMode();
	UE_API void ForceCompactMode();
	UE_API bool Update();
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
