// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Fonts/SlateFontInfo.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"

#define UE_API TRACEINSIGHTS_API

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FSlateBrush;

namespace UE::Insights { class FDrawContext; }

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTooltipDrawState
{
public:
	static constexpr float BorderX = 6.0f;
	static constexpr float BorderY = 3.0f;
	static constexpr float MinWidth = 128.0f;
	static constexpr float MinHeight = 0.0f;

	static constexpr float DefaultTitleHeight = 14.0f;
	static constexpr float DefaultLineHeight = 14.0f;
	static constexpr float NameValueDX = 2.0f;

	static UE_API FLinearColor DefaultTitleColor;
	static UE_API FLinearColor DefaultNameColor;
	static UE_API FLinearColor DefaultValueColor;

private:
	enum class FDrawTextType
	{
		Misc,
		Title,
		Name,  // X is computed at Draw (right aligned at ValueOffsetX)
		Value, // X is computed at Draw (left aligned at ValueOffsetX + NameValueDX)
	};

	struct FDrawTextInfo
	{
		float X; // relative X position of text inside tooltip
		float Y; // relative Y position of text inside tooltip
		FVector2D TextSize;
		FString Text;
		FLinearColor Color;
		FDrawTextType Type;
	};

public:
	UE_API FTooltipDrawState();
	UE_API ~FTooltipDrawState();

	UE_API void Reset();

	UE_API void ResetContent();
	UE_API void AddTitle(FStringView Title);
	UE_API void AddTitle(FStringView Title, const FLinearColor& Color);
	UE_API void AddNameValueTextLine(FStringView Name, FStringView Value);
	UE_API void AddTextLine(FStringView Text, const FLinearColor& Color);
	UE_API void AddTextLine(const float X, const float Y, FStringView Text, const FLinearColor& Color);
	UE_API void UpdateLayout(); // updates ValueOffsetX and DesiredSize

	const FLinearColor& GetBackgroundColor() const { return BackgroundColor; }
	void SetBackgroundColor(const FLinearColor& InBackgroundColor) { BackgroundColor = InBackgroundColor; }

	float GetOpacity() const { return Opacity; }
	void SetOpacity(const float InOpacity) { Opacity = InOpacity; }

	float GetDesiredOpacity() const { return DesiredOpacity; }
	void SetDesiredOpacity(const float InDesiredOpacity) { DesiredOpacity = InDesiredOpacity; }

	const FVector2D& GetSize() const { return Size; }
	const FVector2D& GetDesiredSize() const { return DesiredSize; }

	UE_API void Update(); // updates current Opacity and Size

	const FVector2D& GetPosition() const { return Position; }
	UE_API void SetPosition(const FVector2D& MousePosition, const float MinX, const float MaxX, const float MinY, const float MaxY);
	void SetPosition(const float PosX, const float PosY) { Position.X = PosX; Position.Y = PosY; }

	UE_API void Draw(const UE::Insights::FDrawContext& DrawContext) const;

	void SetFontScale(float InFontScale) { FontScale = InFontScale; }
	float GetFontScale() const { return FontScale; }

	void SetImage(TSharedPtr<FSlateBrush> InImageBrush) { ImageBrush = InImageBrush; }

private:
	const FSlateBrush* WhiteBrush;
	const FSlateFontInfo Font;

	FLinearColor BackgroundColor;

	FVector2D Size;
	FVector2D DesiredSize;

	FVector2D Position;

	float ValueOffsetX;
	float NewLineY;

	float Opacity;
	float DesiredOpacity;

	float FontScale;

	TArray<FDrawTextInfo> Texts;
	TSharedPtr<FSlateBrush> ImageBrush;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
