// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

#define UE_API TRACEINSIGHTS_API

class SWidget;
struct FAssetData;

namespace UE::Insights::ObjectProfiler
{

class IAssetInfoProvider;

/** Interface for the Object Profiler Memory View. */
class IObjectProfilerMemoryView
{
public:
	/** Default constructor. */
	IObjectProfilerMemoryView() {}

	/** Virtual destructor. */
	 virtual ~IObjectProfilerMemoryView() {}

	virtual TSharedRef<SWidget> GetWidget() = 0;

	virtual void SetAssetInfoProvider(TSharedPtr<IAssetInfoProvider> InAssetInfoProvider) = 0;
};

} // namespace UE::Insights::ObjectProfiler

#undef UE_API
