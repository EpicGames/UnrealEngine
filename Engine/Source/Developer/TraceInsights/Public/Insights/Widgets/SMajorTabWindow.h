// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Framework/Docking/TabManager.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h" // for FInsightsMajorTabExtender

#define UE_API TRACEINSIGHTS_API

////////////////////////////////////////////////////////////////////////////////////////////////////

class FActiveTimerHandle;
class FMenuBuilder;
class FUICommandList;
class SDockTab;
class SWindow;

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace Insights
{

/** Implements the base class for a major tab window. */
class SMajorTabWindow : public SCompoundWidget
{
public:
	/** Default constructor. */
	UE_API SMajorTabWindow(const FName& InMajorTabId);

	/** Virtual destructor. */
	UE_API virtual ~SMajorTabWindow();

	SLATE_BEGIN_ARGS(SMajorTabWindow) {}
	SLATE_END_ARGS()

	const FName& GetMajorTabId() const { return MajorTabId; }
	TSharedPtr<FTabManager> GetTabManager() const { return TabManager; }
	TSharedPtr<FWorkspaceItem> GetWorkspaceMenuGroup() const { return WorkspaceMenuGroup; }
	const TSharedPtr<FUICommandList> GetCommandList() const { return CommandList; }

	UE_API virtual void Reset();

	/** Constructs this widget. */
	UE_API void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);

	UE_API void ShowTab(const FName& TabId);
	UE_API void HideTab(const FName& TabId);
	void ShowHideTab(const FName& TabId, bool bShow) { bShow ? ShowTab(TabId) : HideTab(TabId); }
	UE_API void CloseAllOpenTabs();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// OnWindowClosedEvent
public:
	/** The event to execute when the window is closed. */
	DECLARE_MULTICAST_DELEGATE(FWindowClosedEvent);
	FWindowClosedEvent& GetWindowClosedEvent() { return WindowClosedEvent; }
private:
	/** The event to execute when the window is closed. */
	FWindowClosedEvent WindowClosedEvent;

protected:
	UE_API virtual const TCHAR* GetAnalyticsEventName() const;
	UE_API virtual TSharedRef<FWorkspaceItem> CreateWorkspaceMenuGroup();
	UE_API virtual void RegisterTabSpawners();
	UE_API virtual TSharedRef<FTabManager::FLayout> CreateDefaultTabLayout() const;
	UE_API virtual TSharedRef<SWidget> CreateToolbar(TSharedPtr<FExtender> Extender);

	void SetCommandList(const TSharedPtr<FUICommandList> InCommandList) { CommandList = InCommandList; }

	UE_API void AddOpenTab(const TSharedRef<SDockTab>& DockTab);
	UE_API void RemoveOpenTab(const TSharedRef<SDockTab>& DockTab);

private:
	/**
	 * Fill the main menu with menu items.
	 *
	 * @param MenuBuilder The multi-box builder that should be filled with content for this pull-down menu.
	 * @param TabManager A Tab Manager from which to populate tab spawner menu items.
	 */
	static UE_API void FillMenu(FMenuBuilder& MenuBuilder, const TSharedPtr<FTabManager> TabManager);

	/** Returns true if the current analysis session is valid. */
	UE_API bool IsValidSession() const;

	/** Updates the amount of time the window has been active. */
	UE_API EActiveTimerReturnType UpdateActiveDuration(double InCurrentTime, float InDeltaTime);

	/**
	 * The system will use this event to notify a widget that the cursor has entered it. This event is NOT bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 */
	UE_API virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/**
	 * The system will use this event to notify a widget that the cursor has left it. This event is NOT bubbled.
	 *
	 * @param MouseEvent Information about the input event
	 */
	UE_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	/**
	 * Called after a key is pressed when this widget has focus
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param  InKeyEvent  Key event
	 *
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/**
	 * Called when the user is dropping something onto a widget; terminates drag and drop.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether this event was handled.
	 */
	UE_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/**
	 * Called during drag and drop when the the mouse is being dragged over a widget.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether this event was handled.
	 */
	UE_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)  override;

private:
	/** The major tab id for this window. */
	FName MajorTabId;

	/** Holds the tab manager that manages the front-end's tabs. */
	TSharedPtr<FTabManager> TabManager;

	/** The menu group. */
	TSharedPtr<FWorkspaceItem> WorkspaceMenuGroup;

	/** All tabs owned by this window */
	TSet<TSharedPtr<SDockTab>> OpenTabs;

	/** Command list used in the window. Maps commands to window specific actions. */
	TSharedPtr<FUICommandList> CommandList;

	/** The handle to the active update duration tick */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	/** The number of seconds the window has been active */
	float DurationActive;

	/** Tab specific slate extender structure, has the lifetime of this widget */
	TSharedPtr<FInsightsMajorTabExtender> Extension;
};

} // namespace Insights

#undef UE_API
