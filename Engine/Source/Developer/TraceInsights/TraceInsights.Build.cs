// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TraceInsights : ModuleRules
{
	public TraceInsights(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"AppFramework", // for SColorPicker, RestoreStarshipSuite()
				"ApplicationCore",
				"AssetRegistry",
				"AutomationDriver",
				"Cbor",
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"InputCore",
				"RenderCore",
				"RHI",
				"SessionServices",
				"Slate",
				"Sockets",
				"SourceCodeAccess",
				"TraceAnalysis",
				"TraceInsightsCore",
				"TraceLog",
				"TraceTools",
				"TraceServices",
				"WorkspaceMenuStructure",
				"XmlParser",
			}
		);

		// Modules required for running automation in stand alone Insights
		if (Target.Configuration != UnrealTargetConfiguration.Shipping && !Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AutomationWorker",
					"AutomationController",
					"AutomationWindow",
					"SessionServices",
				}
			);
		}

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ContentBrowser",
					"AssetDefinition",
					"Engine",
					"UnrealEd",
				}
			);
		}

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ImageCore",
				"Json",
				"SlateCore",
				"ToolWidgets",
				"ToolMenus",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Messaging",
				"MessageLog",
			}
		);

		if (Target.bWithLiveCoding)
		{
			PrivateDependencyModuleNames.Add("LiveCoding");
		}
	}
}
